[CoreRedirects]

; 5.0
+PropertyRedirects=(OldName="MetaSoundSettings.AutoUpdateBlacklist",NewName="AutoUpdateDenylist")
+PropertyRedirects=(OldName="MetaSoundSettings.AutoUpdateAssetBlacklist",NewName="AutoUpdateDenylist")

+StructRedirects=(OldName="MetasoundEditorGraphInputBoolRef",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultBoolRef")
+StructRedirects=(OldName="MetasoundEditorGraphInputIntRef",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultIntRef")
+StructRedirects=(OldName="MetasoundEditorGraphInputObjectRef",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultObjectRef")
 
+ClassRedirects=(OldName="MetasoundEditorGraphInputLiteral",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultLiteral")
+ClassRedirects=(OldName="MetasoundEditorGraphInputBool",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultBool")
+ClassRedirects=(OldName="MetasoundEditorGraphInputBoolArray",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultBoolArray")
+ClassRedirects=(OldName="MetasoundEditorGraphInputInt",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultInt")
+ClassRedirects=(OldName="MetasoundEditorGraphInputIntArray",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultIntArray")
+ClassRedirects=(OldName="MetasoundEditorGraphInputFloat",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultFloat")
+ClassRedirects=(OldName="MetasoundEditorGraphInputFloatArray",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultFloatArray")
+ClassRedirects=(OldName="MetasoundEditorGraphInputString",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultString")
+ClassRedirects=(OldName="MetasoundEditorGraphInputStringArray",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultStringArray")
+ClassRedirects=(OldName="MetasoundEditorGraphInputObject",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultObject")
+ClassRedirects=(OldName="MetasoundEditorGraphInputObjectArray",NewName="/Script/MetasoundEditor.MetasoundEditorGraphMemberDefaultObjectArray")

; 5.1
+ClassRedirects=(OldName="MetaSound",NewName="/Script/MetasoundEngine.MetaSoundPatch")
+PropertyRedirects=(OldName="MetasoundEditorSettings.bPinMetaSoundInAssetMenu",NewName="bPinMetaSoundPatchInAssetMenu")

; 5.3
+EnumRedirects=(OldName="MetasoundSourceAudioFormat",NewName="/Script/MetasoundEngine.MetaSoundOutputAudioFormat")

; 5.8
;Template redirects (renamed from "Configuration")
+StructRedirects=(OldName="/Script/MetasoundFrontend.MetasoundFrontendDocumentConfiguration",NewName="/Script/MetasoundFrontend.MetaSoundFrontendDocumentTemplate")
+StructRedirects=(OldName="/Script/MetasoundFrontend.MetaSoundFrontendDocumentConfigurationVertexMetadata",NewName="/Script/MetasoundFrontend.MetaSoundFrontendDocumentTemplateVertexMetadata")
+StructRedirects=(OldName="/Script/MetasoundFrontend.MetaSoundFrontendDocumentVertexConfiguration",NewName="/Script/MetasoundFrontend.MetaSoundFrontendDocumentVertexTemplate")
+StructRedirects=(OldName="/Script/MetasoundFrontend.MetaSoundFrontendPresetConfigurationVertexMetadata",NewName="/Script/MetasoundFrontend.MetaSoundFrontendPresetVertexMetadata")
+StructRedirects=(OldName="/Script/MetasoundFrontend.MetaSoundFrontendPresetDocumentConfiguration",NewName="/Script/MetasoundFrontend.MetaSoundFrontendPresetTemplate")
+StructRedirects=(OldName="/Script/MetasoundEngine.MetaSoundRandomizerDocumentConfiguration",NewName="/Script/MetasoundEngine.MetaSoundRandomizerTemplate")
+StructRedirects=(OldName="/Script/MetasoundEditor.MetaSoundDocumentConfigurationWidgetTabInfo",NewName="/Script/MetasoundEditor.MetaSoundTemplateWidgetTabInfo")
+ClassRedirects=(OldName="/Script/MetasoundEditor.MetaSoundDocumentConfigurationViewModelBase",NewName="/Script/MetasoundEditor.MetaSoundTemplateViewModelBase")
+ClassRedirects=(OldName="/Script/MetasoundEditor.MetaSoundDocumentConfigurationWidgetInterface",NewName="/Script/MetasoundEditor.MetaSoundTemplateWidgetInterface")
+FunctionRedirects=(OldName="/Script/MetasoundEditor.MetaSoundTemplateWidgetInterface.IsSupportedConfiguration",NewName="/Script/MetasoundEditor.MetaSoundTemplateWidgetInterface.IsSupportedTemplate")
+PropertyRedirects=(OldName="/Script/MetasoundFrontend.MetasoundFrontendDocument.Configuration",NewName="/Script/MetasoundFrontend.MetasoundFrontendDocument.Template")
