{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MetaSound",
	"Description": "A high-performance audio system that enables sound designers to have complete control over audio DSP graph generation of sound sources, via sample-accurate control and modulation of sound using audio parameters and audio events from game data and Blueprints",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MetasoundGraphCore",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundGenerator",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundFrontend",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundStandardNodes",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundEngine",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundEngineTest",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MetasoundEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AudioWidgets",
			"Enabled": true
		},
		{
			"Name": "AudioSynesthesia",
			"Enabled": true
		},
		{
			"Name": "ContentBrowserAssetDataSource",
			"Enabled":  true
		},
		{
			"Name": "WaveTable",
			"Enabled": true
		},
		{
			"Name": "ModelViewViewModel",
			"Enabled": true		
		}
	]
}
