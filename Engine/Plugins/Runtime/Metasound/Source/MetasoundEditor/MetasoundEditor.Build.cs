// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundEditor : ModuleRules
	{
		public MetasoundEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetTools",
					"AppFramework",
					"AudioExtensions",
					"AudioSynesthesia",
					"ContentBrowser",
					"ContentBrowserAssetDataSource",
					"ContentBrowserData",
					"EditorWidgets",
					"Kismet",
					"KismetWidgets",
					"MetasoundEngine",
					"MetasoundFrontend",
					"MetasoundGenerator",
					"MetasoundGraphCore",
					"MetasoundStandardNodes",
					"Projects",
					"SignalProcessing",
					"SubobjectEditor",
					"ToolMenus",
					"ToolWidgets",
					"WaveTable",
					"WaveTableEditor"
				}
			);

			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"ApplicationCore",
					"AudioAnalyzer",
					"AudioEditor",
					"AudioMixer",
					"AudioWidgets",
					"AssetDefinition",
					"ClassViewer",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DetailCustomizations",
					"EditorFramework",
					"EditorStyle",
					"EditorSubsystem",
					"Engine",
					"GraphEditor",
					"InputCore",
					"ModelViewViewModel",
					"PropertyEditor",
					"RenderCore",
					"Slate",
					"SlateCore",
					"StructUtilsEditor",
					"UMG",
					"UnrealEd"
				}
			);
		}
	}
}
