// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInMetasound.h"

#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorModule.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundSettings.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"

namespace Metasound::Editor
{
	namespace FindInMetaSoundPrivate
	{
		/** Get the string for the pin's enum value, empty string if not enum type */
		FString GetEnumPinValueString(const UEdGraphPin& EnumPin)
		{
			using namespace Metasound::Frontend;
			
			if (const UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(EnumPin.GetOwningNode()))
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = MetaSoundNode->GetBuilderChecked().GetConstBuilder();
				if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(DocBuilder, &EnumPin))
				{
					FDataTypeRegistryInfo Info;
					if (IDataTypeRegistry::Get().GetDataTypeInfo(Vertex->TypeName, Info))
					{
						if (Info.bIsEnum)
						{
							TSharedPtr<const Frontend::IEnumDataTypeInterface> EnumInterface = IDataTypeRegistry::Get().GetEnumInterfaceForDataType(Vertex->TypeName);
							if (EnumInterface)
							{
								FMetasoundFrontendLiteral DefaultLiteral;
								FGraphBuilder::GetPinLiteral(EnumPin, DefaultLiteral);
								if (DefaultLiteral.IsValid())
								{
									const int32 EnumValue = FCString::Atoi(*DefaultLiteral.ToString());
									const TOptional<FName> EnumName = EnumInterface->ToName(EnumValue);
									if (EnumName.IsSet())
									{
										return EnumName->ToString();
									}
								}
							}
						}
					}
				}
			}
			return FString();
		}

		/** Get the string describing the member's literal value. */
		FString GetMemberLiteralString(const UMetasoundEditorGraphMember* Member, bool bUseFullValueString)
		{
			using namespace Metasound::Frontend;

			FString LiteralString;
			const bool bIsDefaultPaged = Member->IsDefaultPaged();

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);
			auto GetPageName = [&Settings](const FGuid& PageID)
				{
					const FMetaSoundPageSettings* Page = Settings->FindPageSettings(PageID);
					if (Page)
					{
						return Page->Name.ToString();
					}
					return LOCTEXT("FindMetasound_InvalidPage", "Invalid Page").ToString();
				};

			// Enum interface for converting int value to enum string
			TSharedPtr<const Frontend::IEnumDataTypeInterface> EnumInterface;
			FDataTypeRegistryInfo Info;
			if (IDataTypeRegistry::Get().GetDataTypeInfo(Member->GetDataType(), Info))
			{
				if (Info.bIsEnum)
				{
					EnumInterface = IDataTypeRegistry::Get().GetEnumInterfaceForDataType(Member->GetDataType());
				}
			}

			Member->GetLiteral()->IterateDefaults([&LiteralString, bIsDefaultPaged, &GetPageName, bUseFullValueString, &EnumInterface](const FGuid& PageId, FMetasoundFrontendLiteral Literal)
				{
					if (bIsDefaultPaged)
					{
						LiteralString += FText::Format(LOCTEXT("FindMetasound_PageFormat", "{0} Page: "), FText::FromString(GetPageName(PageId))).ToString();
					}

					if (bUseFullValueString)
					{
						bool bStringSet = false;
						if (EnumInterface)
						{
							int32 Value;
							Literal.TryGet(Value);

							TOptional<FName> Name = EnumInterface->ToName(Value);
							if (Name.IsSet())
							{
								bStringSet = true;
								LiteralString += Name->ToString();
							}
						}

						if (!bStringSet)
						{
							LiteralString += Literal.ToString();
						}
					}
					// Shortened strings for certain types
					else
					{
						if (Literal.IsArray())
						{
							// Don't list array contents (only show on tooltip to save space)
							LiteralString += FText::Format(LOCTEXT("FindMetasound_ArrayNumDescriptionFormat", "({0} element array)"), Literal.GetArrayNum()).ToString();
						}
						else if (Literal.GetType() == EMetasoundFrontendLiteralType::UObject)
						{
							UObject* Object = nullptr;
							Literal.TryGet(Object);
							if (Object)
							{
								// Get a shorter name than ToString (which is LexToString)
								FString ObjectName;
								Object->GetName(ObjectName);
								LiteralString += ObjectName;
							}
							else
							{
								LiteralString += LOCTEXT("FindMetasound_NoneObject", "(None Object)").ToString();
							}
						}
						else
						{
							bool bStringSet = false;
							if (EnumInterface)
							{
								int32 Value;
								Literal.TryGet(Value);

								TOptional<FName> Name = EnumInterface->ToName(Value);
								if (Name.IsSet())
								{
									bStringSet = true;
									LiteralString += Name->ToString();
								}
							}

							if (!bStringSet)
							{
								LiteralString += Literal.ToString();
							}
						}
					}

					LiteralString += "\n";
				});

			// Remove last "\n" if needed
			LiteralString.RemoveFromEnd("\n");

			return LiteralString;
		}
	}

	FFindInMetasoundResult::FFindInMetasoundResult(const FString& InResultName)
		:Value(InResultName), DuplicationIndex(0), Class(nullptr), Pin(), GraphNode(nullptr)
	{
	}

	FFindInMetasoundResult::FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UClass* InClass, int InDuplicationIndex)
	: Parent(InParent), Value(InResultName), DuplicationIndex(InDuplicationIndex), Class(InClass), Pin(), GraphNode(nullptr)
	{
		ValueText = GetValueText();
	}

	FFindInMetasoundResult::FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UEdGraphPin* InPin)
	: Parent(InParent), Value(InResultName), DuplicationIndex(0), Class(nullptr), Pin(InPin), GraphNode(nullptr)
	{
		ValueText = GetValueText();
	}

	FFindInMetasoundResult::FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UEdGraphNode* InNode)
	: Parent(InParent), Value(InResultName), DuplicationIndex(0), Class(InNode->GetClass()), Pin(), GraphNode(InNode)
	{
		if (GraphNode.IsValid())
		{
			CommentText = GraphNode->NodeComment;
		}
		ValueText = GetValueText();
	}

	FReply FFindInMetasoundResult::OnClick(TWeakPtr<class FEditor> MetaSoundEditor)
	{
		if (GraphNode.IsValid())
		{
			MetaSoundEditor.Pin()->GetGraphEditor()->JumpToNode(GraphNode.Get());
		}
		else if (UEdGraphPin* ResolvedPin = Pin.Get())
		{
			MetaSoundEditor.Pin()->GetGraphEditor()->JumpToPin(ResolvedPin);
		}
		return FReply::Handled();
	}

	FText FFindInMetasoundResult::GetCategory() const
	{
		if (Class == nullptr && Pin.Get())
		{
			return LOCTEXT("FindMetasound_PinCategory", "Pin");
		}
		return LOCTEXT("FindMetasound_NodeCategory", "Node");
	}

	TSharedRef<SWidget> FFindInMetasoundResult::CreateIcon() const
	{
		FSlateColor IconColor = FSlateColor::UseForeground();
		const FSlateBrush* Brush = nullptr;
		bool bIsPin = false;
		if (const UEdGraphPin* ResolvedPin = Pin.Get())
		{
			bIsPin = true;
			if (const UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(ResolvedPin->GetOwningNode()))
			{
				// Get data type and constructor pin status
				FName DataTypeName;
				bool bIsConstructorPin = false;
				if (const UMetasoundEditorGraphMember* Member = GetMetaSoundGraphMember(Node))
				{
					DataTypeName = Member->GetDataType();
					if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(Member))
					{
						bIsConstructorPin = Vertex->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Value;
					}
				}
				else if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Node))
				{
					if (ResolvedPin->Direction == EGPD_Input)
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = FGraphBuilder::GetBuilderFromPinChecked(*ResolvedPin).GetConstBuilder();
						FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, ResolvedPin);
						check(InputVertexHandle.IsSet());
						const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
						check(InputVertex);
						DataTypeName = InputVertex->TypeName;
						bIsConstructorPin = DocBuilder.GetNodeInputAccessType(InputVertexHandle.NodeID, InputVertexHandle.VertexID) == EMetasoundFrontendVertexAccessType::Value;
					}
					else if (ResolvedPin->Direction == EGPD_Output)
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = FGraphBuilder::GetBuilderFromPinChecked(*ResolvedPin).GetConstBuilder();
						FMetasoundFrontendVertexHandle OutputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, ResolvedPin);
						check(OutputVertexHandle.IsSet());
						const FMetasoundFrontendVertex* OutputVertex = DocBuilder.FindNodeOutput(OutputVertexHandle.NodeID, OutputVertexHandle.VertexID);
						check(OutputVertex);
						DataTypeName = OutputVertex->TypeName;
						bIsConstructorPin = DocBuilder.GetNodeOutputAccessType(OutputVertexHandle.NodeID, OutputVertexHandle.VertexID) == EMetasoundFrontendVertexAccessType::Value;
					}
				}

				// Get brush 
				const IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
				Brush = EditorModule.GetIconBrush(DataTypeName, bIsConstructorPin);
			}
			else
			{
				Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
			}

			// Get color
			const UEdGraphSchema* Schema = ResolvedPin->GetSchema();
			IconColor = Schema->GetPinTypeColor(ResolvedPin->PinType);
		}
		else if (GraphNode.IsValid())
		{
			if (UMetasoundEditorGraphNode* MetaSoundGraphNode = Cast<UMetasoundEditorGraphNode>(GraphNode))
			{
				// Variable nodes do not have a node title icon or color, 
				// so use generic one and corresponding pin type color
				if (UMetasoundEditorGraphVariableNode* MetaSoundGraphVariableNode = Cast<UMetasoundEditorGraphVariableNode>(MetaSoundGraphNode))
				{
					Brush = FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
					UEdGraphPin* FirstPin = MetaSoundGraphVariableNode->GetPinAt(0);
					if (FirstPin)
					{
						const UEdGraphSchema* Schema = FirstPin->GetSchema();
						IconColor = Schema->GetPinTypeColor(FirstPin->PinType);
					}
				}
				else if (UMetasoundEditorGraphMemberNode* MetaSoundGraphMemberNode = Cast<UMetasoundEditorGraphMemberNode>(MetaSoundGraphNode))
				{
					Brush = MetaSoundGraphNode->GetNodeTitleIconBrush();
					IconColor = GraphNode->GetNodeTitleTextColor();
				}
				else
				{
					Brush = MetaSoundGraphNode->GetNodeTitleIconBrush();
					IconColor = GraphNode->GetNodeTitleColor();
				}
			}
			else
			{
				Brush = FAppStyle::GetBrush(TEXT("GraphEditor.NodeGlyph"));
			}
		}

		static const FVector2D IconSize16 = FVector2D(16.0f, 16.0f);

		return SNew(SImage)
			.Image(Brush)
			.ColorAndOpacity(IconColor)
			.ToolTipText(GetCategory())
			.DesiredSizeOverride(IconSize16);
	}

	FString FFindInMetasoundResult::GetCommentText() const
	{
		return CommentText;
	}

	FText FFindInMetasoundResult::GetValueText()
	{
		using namespace Frontend;

		if (!ValueText.IsEmpty())
		{
			return ValueText;
		}

		bool bIsOverriddenLiteral = false;

		// Try to get corresponding member
		const UEdGraphPin* ResolvedPin = Pin.Get();
		const UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(GraphNode);
		if (!MemberNode && ResolvedPin)
		{
			MemberNode = Cast<UMetasoundEditorGraphMemberNode>(ResolvedPin->GetOwningNode());
		}

		// Get value from member literal
		if (const UMetasoundEditorGraphMember* Member = GetMetaSoundGraphMember(MemberNode))
		{
			// Don't show boolean value for triggers 
			if (Member->GetDataType() != GetMetasoundDataTypeName<FTrigger>())
			{
				// Only list number of items for arrays to avoid string issues 
				FMetaSoundFrontendDocumentBuilder& DocBuilder = Member->GetFrontendBuilderChecked();
				
				// Don't show full object names and array contents
				ValueText = FText::FromString(FindInMetaSoundPrivate::GetMemberLiteralString(Member, /*bUseFullValueString=*/false));

				// Check if input is an overridden default value (for presets) 
				if (const UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(Member))
				{
					if (const FPresetVertexMetadata* VertexMetadata = DocBuilder.FindConstVertexMetadata<FPresetVertexMetadata>(Input->NodeID))
					{
						bIsOverriddenLiteral = VertexMetadata->bOverrideInheritedDefault;
					}
				}

				// Don't show default value if variable mutator node is connected 
				if (const UMetasoundEditorGraphVariable* Variable = Cast<UMetasoundEditorGraphVariable>(Member))
				{
					const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariable(Variable->GetMemberName());
					if (FrontendVariable)
					{
						auto IsMutatorNode = [&FrontendVariable](const UMetasoundEditorGraphMemberNode* Node)
						{
							return Node->GetNodeID() == FrontendVariable->MutatorNodeID;
						};
						TArray<UMetasoundEditorGraphMemberNode*> Nodes = Variable->GetNodes();
						if ( UMetasoundEditorGraphMemberNode** MutatorNode = Nodes.FindByPredicate(IsMutatorNode))
						{
							if (*MutatorNode && (*MutatorNode)->Pins[0]->HasAnyConnections())
							{
								ValueText = LOCTEXT("FindMetasound_VariableSetterConnectedDescription", "(See value from variable setter node connection)");
							}
						}
					}
				}
			}
		}
		// Get value information directly from pin 
		else if (ResolvedPin)
		{
			const bool bIsUnconnectedInputPin = ResolvedPin->Direction == EGPD_Input && ResolvedPin->LinkedTo.Num() == 0;
			if (bIsUnconnectedInputPin)
			{
				if (ResolvedPin->DefaultObject)
				{
					const AActor* DefaultActor = Cast<AActor>(ResolvedPin->DefaultObject);
					ValueText = FText::FromString(DefaultActor ? *DefaultActor->GetActorLabel() : *ResolvedPin->DefaultObject->GetName());
				}
				else if (!ResolvedPin->AutogeneratedDefaultValue.IsEmpty())
				{
					ValueText = FText::FromString(ResolvedPin->AutogeneratedDefaultValue);
				}
				else if (!ResolvedPin->DefaultTextValue.IsEmpty())
				{
					ValueText = FText::FromString(ResolvedPin->DefaultTextValue.ToString());
				}
				// Cached value on ed graph pin or enum as special case to be converted from int32 to enum string
				else if (ResolvedPin->PinType.PinCategory.ToString().Equals(TEXT("Int32")))
				{
					// If non enum int32, case below with default value may apply
					ValueText = FText::FromString(FindInMetaSoundPrivate::GetEnumPinValueString(*ResolvedPin));
				}

				if (ValueText.IsEmpty() && !ResolvedPin->DefaultValue.IsEmpty())
				{
					// Don't show default for trigger type
					if (ResolvedPin->PinType.PinCategory != GetMetasoundDataTypeName<FTrigger>())
					{
						ValueText = FText::FromString(ResolvedPin->DefaultValue);
					}
				}

				// Get value from input literal 
				if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(ResolvedPin->GetOwningNode()))
				{
					if (ResolvedPin->Direction == EGPD_Input)
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = FGraphBuilder::GetBuilderFromPinChecked(*ResolvedPin).GetConstBuilder();
						FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, ResolvedPin);
						check(InputVertexHandle.IsSet());
						const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
						check(InputVertex);
						const FMetasoundFrontendVertexLiteral* DefaultLiteral = DocBuilder.FindNodeInputDefault(InputVertexHandle.NodeID, InputVertex->Name);
						if (DefaultLiteral)
						{
							FMetasoundFrontendLiteral Literal = DefaultLiteral->Value;
							// Get input literal class default override status 
							const bool bIsDefaultConstructed = Literal.GetType() == EMetasoundFrontendLiteralType::None;
							if (!bIsDefaultConstructed)
							{
								bIsOverriddenLiteral = true;
							}

							// Get shorter object names 
							if (Literal.GetType() == EMetasoundFrontendLiteralType::UObject)
							{
								UObject* Object = nullptr;
								Literal.TryGet(Object);
								if (Object)
								{
									FString ObjectName;
									Object->GetName(ObjectName);
									ValueText = FText::FromString(ObjectName);
								}
							}
						}
					}
				}
			}
		}
			
		if (bIsOverriddenLiteral)
		{
			ValueText = FText::Format(LOCTEXT("FindMetasound_OverriddenValueDescriptionFormat", "{0} (overridden)"), ValueText);
		}
		return ValueText;
	}

	FText FFindInMetasoundResult::GetValueTooltipText()
	{
		// Get extended value text for tooltip
			
		// Try to get corresponding member
		const UEdGraphPin* ResolvedPin = Pin.Get();
		UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(GraphNode);
		if (!MemberNode && ResolvedPin)
		{
			MemberNode = Cast<UMetasoundEditorGraphMemberNode>(ResolvedPin->GetOwningNode());
		}

		// Get value from member literal
		if (const UMetasoundEditorGraphMember* Member = GetMetaSoundGraphMember(MemberNode))
		{
			// Get full object names and array contents (GetValueText shortens these)
			return FText::FromString(FindInMetaSoundPrivate::GetMemberLiteralString(Member, /*bUseFullValueString=*/true));
		}
		// Get value from pin's external node 
		else if (ResolvedPin)
		{
			// Get full object name (GetValueText shortens these)
			if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(ResolvedPin->GetOwningNode()))
			{
				if (ResolvedPin->Direction == EGPD_Input)
				{
					const FMetaSoundFrontendDocumentBuilder& DocBuilder = FGraphBuilder::GetBuilderFromPinChecked(*ResolvedPin).GetConstBuilder();
					FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, ResolvedPin);
					check(InputVertexHandle.IsSet());
					const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
					check(InputVertex);

					if (const FMetasoundFrontendVertexLiteral* DefaultLiteral = DocBuilder.FindNodeInputDefault(InputVertexHandle.NodeID, InputVertex->Name))
					{
						return FText::FromString(DefaultLiteral->Value.ToString());
					}
				}
			}
		}

		// Default to same as value text 
		return GetValueText();
	}

	const UMetasoundEditorGraphMember* FFindInMetasoundResult::GetMetaSoundGraphMember(const UEdGraphNode* EdGraphNode)
	{
		if (const UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(EdGraphNode))
		{
			return Cast<UMetasoundEditorGraphMember>(MemberNode->GetMember());
		}
		return nullptr;
	}

	void SFindInMetasound::Construct(const FArguments& InArgs, TSharedPtr<FEditor> InMetaSoundEditor)
	{
		MetaSoundEditorPtr = InMetaSoundEditor;

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("FindMetasound_GraphSearchHint", "Search"))
					.OnTextChanged(this, &SFindInMetasound::OnSearchTextChanged)
					.OnTextCommitted(this, &SFindInMetasound::OnSearchTextCommitted)
					.DelayChangeNotificationsWhileTyping(false)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SAssignNew(TreeView, STreeViewType)
					.TreeItemsSource(&ItemsFound)
					.OnGenerateRow(this, &SFindInMetasound::OnGenerateRow)
					.OnGetChildren(this, &SFindInMetasound::OnGetChildren)
					.OnSelectionChanged(this, &SFindInMetasound::OnTreeSelectionChanged)
					.OnMouseButtonDoubleClick(this, &SFindInMetasound::OnTreeSelectionDoubleClick)
					.SelectionMode(ESelectionMode::Single)
				]
			]
		];
	}

	void SFindInMetasound::FocusForUse()
	{
		// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
		FWidgetPath FilterTextBoxWidgetPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

		// Set keyboard focus directly
		FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
	}

	void SFindInMetasound::FocusForUse(const FString& NewSearchTerms)
	{
		FocusForUse();

		if (!NewSearchTerms.IsEmpty())
		{
			SearchTextField->SetText(FText::FromString(NewSearchTerms));
			InitiateSearch();
		}
	}

	void SFindInMetasound::OnSearchTextChanged(const FText& Text)
	{
		SearchValue = Text.ToString();
	}

	void SFindInMetasound::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
	{
		if (CommitType == ETextCommit::OnEnter)
		{
			InitiateSearch();
		}
	}

	void SFindInMetasound::InitiateSearch()
	{
		TArray<FString> Tokens;
		if (SearchValue.Contains("\"") && SearchValue.ParseIntoArray(Tokens, TEXT("\""), true) > 0)
		{
			for (auto &TokenIt : Tokens)
			{
				// we have the token, we don't need the quotes anymore, they'll just confused the comparison later on
				TokenIt = TokenIt.TrimQuotes();
				// We remove the spaces as all later comparison strings will also be de-spaced
				TokenIt = TokenIt.Replace(TEXT(" "), TEXT(""));
			}

			// due to being able to handle multiple quoted blocks like ("Make Epic" "Game Now") we can end up with
			// and empty string between (" ") blocks so this simply removes them
			struct FRemoveMatchingStrings
			{
				bool operator()(const FString& RemovalCandidate) const
				{
					return RemovalCandidate.IsEmpty();
				}
			};
			Tokens.RemoveAll(FRemoveMatchingStrings());
		}
		else
		{
			// unquoted search equivalent to a match-any-of search
			SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);
		}

		for (auto It(ItemsFound.CreateIterator()); It; ++It)
		{
			TreeView->SetItemExpansion(*It, false);
		}
		ItemsFound.Empty();
		if (Tokens.Num() > 0)
		{
			HighlightText = FText::FromString(SearchValue);
			MatchTokens(Tokens);
		}

		// Insert a fake result to inform user if none found
		if (ItemsFound.Num() == 0)
		{
			ItemsFound.Add(FSearchResult(MakeShared<FFindInMetasoundResult>(LOCTEXT("FindMetaSound_NoResults", "No Results found").ToString())));
		}
		else
		{
			// Insert a fake result for stat tracking
			FText ResultsStats = FText::Format(LOCTEXT("FindMetaSound_NumResultsFmt", "{0} Result(s): {1} Matching Node(s), {2} Matching Pin(s)"), FoundNodeCount + FoundPinCount, FoundNodeCount, FoundPinCount);
			ItemsFound.Insert(FSearchResult(MakeShared<FFindInMetasoundResult>(ResultsStats.ToString())), 0);
		}

		TreeView->RequestTreeRefresh();

		for (auto It(ItemsFound.CreateIterator()); It; ++It)
		{
			TreeView->SetItemExpansion(*It, true);
		}
	}

	void SFindInMetasound::MatchTokens(const TArray<FString> &Tokens)
	{
		RootSearchResult.Reset();

		UEdGraph* Graph = MetaSoundEditorPtr.Pin()->GetGraphEditor()->GetCurrentGraph();
		MatchTokensInGraph(Graph, Tokens);
	}

	void SFindInMetasound::MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens)
	{
		using namespace Metasound::Frontend;
		if (!Graph)
		{
			return;
		}
		RootSearchResult = MakeShared<FFindInMetasoundResult>(FString("MetasoundRootResult"));
		FoundNodeCount = 0;
		FoundPinCount = 0;

		for (auto It(Graph->Nodes.CreateConstIterator()); It; ++It)
		{
			UEdGraphNode* Node = *It;

			FString DisplayName;
			FString NodeName = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
			FString NodeType = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
			FString DataTypeName;
			FString MetadataString;
			bool bIsMemberNode = false;
			if (const UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(Node))
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = MetaSoundNode->GetBuilderChecked().GetConstBuilder();
				DisplayName = MetaSoundNode->GetDisplayName().ToString();

				// Additional information for member nodes
				if (const UMetasoundEditorGraphMember* MetaSoundMember = FFindInMetasoundResult::GetMetaSoundGraphMember(Node))
				{
					bIsMemberNode = true;
					DataTypeName = MetaSoundMember->GetDataType().ToString();
					NodeName = MetaSoundMember->GetMemberName().ToString();
					// Get specific node type and display name for variables
					if (const UMetasoundEditorGraphVariableNode* VariableNode = Cast<UMetasoundEditorGraphVariableNode>(MetaSoundNode))
					{
						NodeName = VariableNode->GetBreadcrumb().ClassName.ToString();
						switch (VariableNode->GetClassType())
						{
						case EMetasoundFrontendClassType::VariableMutator:
						{
							NodeType = TEXT("Variable (Set)");
							break;
						}
						case EMetasoundFrontendClassType::VariableAccessor:
						{
							NodeType = TEXT("Variable (Get)");
							break;
						}
						case EMetasoundFrontendClassType::VariableDeferredAccessor:
						{
							NodeType = TEXT("Variable (Get Delayed)");
							break;
						}

						default:
							break;
						}
						if (const UMetasoundEditorGraphVariable* MetaSoundVariable = Cast<UMetasoundEditorGraphVariable>(MetaSoundMember))
						{
							DisplayName = MetaSoundVariable->GetDisplayName().ToString();
						}
					}
				}
				// Add external node keyword and category to search string
				else if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Node))
				{
					if (const FMetasoundFrontendNode* FrontendNode = DocBuilder.FindNode(ExternalNode->GetNodeID()))
					{
						if (const FMetasoundFrontendClass* Class = DocBuilder.FindDependency(FrontendNode->ClassID))
						{
							for (const FText& Keyword : Class->Metadata.GetKeywords())
							{
								MetadataString += Keyword.ToString();
							}
							for (const FText& CategoryHierarchy : Class->Metadata.GetCategoryHierarchy())
							{
								MetadataString += CategoryHierarchy.ToString();
							}
						}
					}
				}
			}

			FString NodeResultName;
			if (!DisplayName.IsEmpty())
			{
				// Show node type (input/output/variable) for members, and only display name for others
				if (bIsMemberNode)
				{
					NodeResultName = DisplayName + " - " + NodeType;
				}
				else
				{
					NodeResultName = DisplayName;
				}
			}
			else
			{
				NodeResultName = NodeName + " - " + NodeType;
			}

			FSearchResult NodeResult;

			FString NodeSearchString = NodeName + NodeType + Node->NodeComment + DisplayName + DataTypeName + MetadataString;
			NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

			bool bNodeMatchesSearch = StringMatchesSearchTokens(Tokens, NodeSearchString);
			if (bNodeMatchesSearch)
			{
				NodeResult = MakeShared<FFindInMetasoundResult>(NodeResultName, RootSearchResult, Node);
				FoundNodeCount++;
			}

			for (TArray<UEdGraphPin*>::TIterator PinIt(Node->Pins); PinIt; ++PinIt)
			{
				UEdGraphPin* Pin = *PinIt;
				if (Pin && Pin->PinFriendlyName.CompareTo(FText::FromString(TEXT(" "))) != 0)
				{
					FText PinDisplayName = Pin->GetSchema()->GetPinDisplayName(Pin);
					FString PinCategory = Pin->PinType.PinCategory.ToString();
					// String for values that may not be directly retrievable from the pin default value
					FString PinValueString;

					// Member data type name and value
					FString PinDataTypeName;
					if (const UMetasoundEditorGraphMember* MetaSoundMember = FFindInMetasoundResult::GetMetaSoundGraphMember(Node))
					{
						PinDataTypeName = MetaSoundMember->GetDataType().ToString();
						PinValueString = FindInMetaSoundPrivate::GetMemberLiteralString(MetaSoundMember, /*bUseFullValueString=*/true);
					}
					else if (PinCategory.Equals(TEXT("Int32")))
					{
						// Enum value string (needs conversion from pin DefaultValue int)
						const FString EnumValueString = FindInMetaSoundPrivate::GetEnumPinValueString(*Pin);
						if (!EnumValueString.IsEmpty())
						{
							PinValueString = EnumValueString;
						}
					}
					
					FString PinSearchString = PinDisplayName.ToString() + Pin->PinName.ToString() + Pin->PinFriendlyName.ToString() + Pin->DefaultValue + PinCategory + Pin->PinType.PinSubCategory.ToString() + PinDataTypeName + (Pin->PinType.PinSubCategoryObject.IsValid() ? Pin->PinType.PinSubCategoryObject.Get()->GetFullName() : TEXT("")) + PinValueString;

					PinSearchString = PinSearchString.Replace(TEXT(" "), TEXT(""));
					if (StringMatchesSearchTokens(Tokens, PinSearchString))
					{
						if (!NodeResult.IsValid())
						{
							NodeResult = MakeShared<FFindInMetasoundResult>(NodeResultName, RootSearchResult, Node);
						}
						FSearchResult PinResult(MakeShared<FFindInMetasoundResult>(PinDisplayName.ToString(), NodeResult, Pin));
						NodeResult->Children.Add(PinResult);
						FoundPinCount++;
					}
				}
			}

			// Node or pin matches search
			if (bNodeMatchesSearch || (NodeResult.IsValid() && NodeResult->Children.Num() > 0))
			{
				ItemsFound.Add(NodeResult);
			}
		}

		for (const UEdGraph* Subgraph : Graph->SubGraphs)
		{
			MatchTokensInGraph(Subgraph, Tokens);
		}
	}

	TSharedRef<ITableRow> SFindInMetasound::OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef< SHorizontalBox> TableRowBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				InItem->CreateIcon()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(STextBlock)
					.Text(FText::FromString(InItem->Value))
					.HighlightText(HighlightText)
					.ToolTipText(FText::Format(LOCTEXT("FindMetasound_ResultSearchToolTipFmt", "{0}: {1}"), InItem->GetCategory(), FText::FromString(InItem->Value)))
			];
		
		// To avoid showing value on both pin and node,
		// only show value on pin result or on node result if no pin  
		if (InItem->Pin.Get() || InItem->Children.Num() == 0)
		{
			const FText ValueTooltip = InItem->GetCommentText().IsEmpty() ? 
				InItem->GetValueTooltipText() : 
				FText::Format(LOCTEXT("FindMetasound_NodeValueWithCommentFmt", "[Comment: {0}]\n{1}"), FText::FromString(InItem->GetCommentText()), InItem->GetValueTooltipText());
			TableRowBox->AddSlot()
				.FillWidth(1)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(2)
				[
					SNew(STextBlock)
					.Text(InItem->GetValueText())
					.HighlightText(HighlightText)
					.ToolTipText(ValueTooltip)
				];
		}
		return SNew(STableRow<TSharedPtr<FFindInMetasoundResult>>, OwnerTable)
		[
			TableRowBox
		];
	}

	void SFindInMetasound::OnGetChildren(FSearchResult InItem, TArray< FSearchResult >& OutChildren)
	{
		OutChildren += InItem->Children;
	}

	void SFindInMetasound::OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type)
	{
		if (Item.IsValid())
		{
			Item->OnClick(MetaSoundEditorPtr);
		}
	}

	void SFindInMetasound::OnTreeSelectionDoubleClick(FSearchResult Item)
	{
		if (Item.IsValid())
		{
			Item->OnClick(MetaSoundEditorPtr);
		}
	}

	bool SFindInMetasound::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
	{
		bool bFoundAllTokens = true;
		//search the entry for each token, it must have all of them to pass
		for (auto TokIT(Tokens.CreateConstIterator()); TokIT; ++TokIT)
		{
			const FString& Token = *TokIT;
			if (!ComparisonString.Contains(Token))
			{
				bFoundAllTokens = false;
				break;
			}
		}
		return bFoundAllTokens;
	}
} // namespace Metasound::Editor
#undef LOCTEXT_NAMESPACE
