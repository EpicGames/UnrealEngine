// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetaSoundGraphPanelPinFactory.h"

#include "EdGraph/EdGraphPin.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundWaveTable.h"
#include "SGraphPin.h"
#include "SMetasoundGraphEnumPin.h"
#include "SMetasoundGraphPin.h"
#include "WaveTable.h"


namespace Metasound::Editor
{
	// Pin category constants
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryAudio = GetMetasoundDataTypeName<FAudioBuffer>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryBoolean = GetMetasoundDataTypeName<bool>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryFloat = GetMetasoundDataTypeName<float>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryInt32 = GetMetasoundDataTypeName<int32>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryObject = "object";
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryString = GetMetasoundDataTypeName<FString>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryTime = GetMetasoundDataTypeName<FTime>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryTimeArray = GetMetasoundDataTypeName<TArray<FTime>>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryTrigger = GetMetasoundDataTypeName<FTrigger>();
	const FName FMetaSoundGraphPanelPinFactory::PinCategoryWaveTable = GetMetasoundDataTypeName<WaveTable::FWaveTable>();

	bool FMetaSoundGraphPanelPinFactory::IsInternalMetaSoundPinCategory(FName InPinCategoryName)
	{
		return InPinCategoryName == PinCategoryAudio
			|| InPinCategoryName == PinCategoryTime
			|| InPinCategoryName == PinCategoryTimeArray
			|| InPinCategoryName == PinCategoryTrigger
			|| InPinCategoryName == PinCategoryWaveTable;
	}

	FMetaSoundGraphPanelPinFactory::FMetaSoundGraphPanelPinFactory()
	{
		RegisterCorePinTypes();
	}

	TSharedPtr<SGraphPin> FMetaSoundGraphPanelPinFactory::CreatePin(UEdGraphPin* InPin) const
	{
		if (!InPin)
		{
			return nullptr;
		}

		const UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(InPin->GetOwningNode());
		if (!MetaSoundNode)
		{
			return nullptr;
		}

		// Data type lookup takes priority to allow external plugins to override core types
		const Frontend::FDataTypeRegistryInfo RegistryInfo = MetaSoundNode->GetPinDataTypeInfo(*InPin);
		if (const FOnCreateMetaSoundPinWidget* DataTypeDelegate = DataTypeToWidgetDelegateMap.Find(RegistryInfo.DataTypeName))
		{
			if (DataTypeDelegate->IsBound())
			{
				TSharedPtr<SGraphPin> PinWidget = DataTypeDelegate->Execute(InPin);
				if (PinWidget.IsValid())
				{
					return PinWidget;
				}
			}
		}

		const FName& PinCategory = InPin->PinType.PinCategory;
		if (const FOnCreateMetaSoundPinWidget* CategoryDelegate = CategoryToWidgetDelegateMap.Find(PinCategory))
		{
			if (CategoryDelegate->IsBound())
			{
				TSharedPtr<SGraphPin> PinWidget = CategoryDelegate->Execute(InPin);
				if (PinWidget.IsValid())
				{
					return PinWidget;
				}
			}
		}

		return nullptr;
	}

	const FEdGraphPinType* FMetaSoundGraphPanelPinFactory::FindPinType(FName InDataTypeName) const
	{
		if (const FPinConfiguration* PinConfiguration = PinTypes.Find(InDataTypeName))
		{
			return &PinConfiguration->PinType;
		}

		return nullptr;
	}

	TSharedRef<FMetaSoundGraphPanelPinFactory> FMetaSoundGraphPanelPinFactory::GetChecked()
	{
		IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
		return StaticCastSharedRef<FMetaSoundGraphPanelPinFactory>(EditorModule.GetGraphPanelPinFactory());
	}

	bool FMetaSoundGraphPanelPinFactory::GetCustomPinIcons(UEdGraphPin* InPin, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const
	{
		if (InPin)
		{
			if (const UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
			{
				Frontend::FDataTypeRegistryInfo RegistryInfo = MetaSoundNode->GetPinDataTypeInfo(*InPin);
				return GetCustomPinIcons(RegistryInfo.DataTypeName, OutConnectedIcon, OutDisconnectedIcon);
			}
		}
		return false;
	}

	bool FMetaSoundGraphPanelPinFactory::GetCustomPinIcons(FName InDataType, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const
	{
		const FPinConfiguration* PinConfiguration = PinTypes.Find(InDataType);
		if (!PinConfiguration || (!PinConfiguration->PinConnectedIcon && !PinConfiguration->PinDisconnectedIcon))
		{
			return false;
		}

		OutConnectedIcon = PinConfiguration->PinConnectedIcon;
		OutDisconnectedIcon = PinConfiguration->PinDisconnectedIcon ? PinConfiguration->PinDisconnectedIcon : PinConfiguration->PinConnectedIcon;
		return true;
	}

	FLinearColor FMetaSoundGraphPanelPinFactory::GetPinColor(const FEdGraphPinType& InPinType) const
	{
		const UMetasoundEditorSettings* Settings = GetDefault<UMetasoundEditorSettings>();
		check(Settings);

		if (InPinType.PinCategory == PinCategoryAudio)
		{
			return Settings->AudioPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryBoolean)
		{
			return Settings->BooleanPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryFloat)
		{
			return Settings->FloatPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryInt32)
		{
			return Settings->IntPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryObject)
		{
			return Settings->ObjectPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryString)
		{
			return Settings->StringPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryTime || InPinType.PinCategory == PinCategoryTimeArray)
		{
			return Settings->TimePinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryTrigger)
		{
			return Settings->TriggerPinTypeColor;
		}

		if (InPinType.PinCategory == PinCategoryWaveTable)
		{
			return Settings->WaveTablePinTypeColor;
		}

		// Check registered custom pin colors
		if (const FLinearColor* CustomColor = CustomCategoryColors.Find(InPinType.PinCategory))
		{
			return *CustomColor;
		}

		return Settings->DefaultPinTypeColor;
	}

	void FMetaSoundGraphPanelPinFactory::RegisterCorePinTypes()
	{
		using namespace Frontend;

		RegisterCategoryPin(PinCategoryBoolean, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPinBool, Pin);
		}));

		RegisterCategoryPin(PinCategoryFloat, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPinFloat, Pin);
		}));

		RegisterCategoryPin(PinCategoryTime, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPinFloat, Pin);
		}));

		RegisterCategoryPin(PinCategoryInt32, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin) -> TSharedPtr<SGraphPin>
		{
			if (SMetasoundGraphEnumPin::FindEnumInterfaceFromPin(Pin))
			{
				return SNew(SMetasoundGraphEnumPin, Pin);
			}
			return SNew(SMetasoundGraphPinInteger, Pin);
		}));

		RegisterCategoryPin(PinCategoryObject, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPinObject, Pin);
		}));

		RegisterCategoryPin(PinCategoryString, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPinString, Pin);
		}));

		RegisterCategoryPin(PinCategoryAudio, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			return SNew(SMetasoundGraphPin, Pin);
		}));

		RegisterCategoryPin(PinCategoryTrigger, FOnCreateMetaSoundPinWidget::CreateLambda([](UEdGraphPin* Pin)
		{
			TSharedPtr<SGraphPin> PinWidget = SNew(SMetasoundGraphPin, Pin);
			const FSlateBrush& PinConnectedBrush = Style::GetSlateBrushSafe("MetasoundEditor.Graph.TriggerPin.Connected");
			const FSlateBrush& PinDisconnectedBrush = Style::GetSlateBrushSafe("MetasoundEditor.Graph.TriggerPin.Disconnected");
			PinWidget->SetCustomPinIcon(&PinConnectedBrush, &PinDisconnectedBrush);
			return PinWidget;
		}));

		// Register all known data types from the MetaSound data type registry
		const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

		TArray<FName> DataTypeNames;
		DataTypeRegistry.GetRegisteredDataTypeNames(DataTypeNames);

		for (FName DataTypeName : DataTypeNames)
		{
			FDataTypeRegistryInfo RegistryInfo;
			if (ensure(DataTypeRegistry.GetDataTypeInfo(DataTypeName, RegistryInfo)))
			{
				FName PinCategory = DataTypeName;
				FName PinSubCategory;

				if (!IsInternalMetaSoundPinCategory(PinCategory))
				{
					switch (RegistryInfo.PreferredLiteralType)
					{
						case ELiteralType::Boolean:
						case ELiteralType::BooleanArray:
						{
							PinCategory = PinCategoryBoolean;
						}
						break;

						case ELiteralType::Float:
						{
							PinCategory = PinCategoryFloat;
						}
						break;

						case ELiteralType::FloatArray:
						{
							if (RegistryInfo.bIsArrayType)
							{
								PinCategory = PinCategoryFloat;
							}
						}
						break;

						case ELiteralType::Integer:
						{
							PinCategory = PinCategoryInt32;
						}
						break;

						case ELiteralType::IntegerArray:
						{
							if (RegistryInfo.bIsArrayType)
							{
								PinCategory = PinCategoryInt32;
							}
						}
						break;

						case ELiteralType::String:
						{
							PinCategory = PinCategoryString;
						}
						break;

						case ELiteralType::StringArray:
						{
							if (RegistryInfo.bIsArrayType)
							{
								PinCategory = PinCategoryString;
							}
						}
						break;

						case ELiteralType::UObjectProxy:
						case ELiteralType::UObjectProxyArray:
						{
							PinCategory = PinCategoryObject;
						}
						break;

						case ELiteralType::None:
						case ELiteralType::NoneArray:
						case ELiteralType::Invalid:
						default:
						{
							static_assert(static_cast<int32>(ELiteralType::Invalid) == 12, "Possible missing binding of pin category to primitive type");
						}
						break;
					}
				}

				RegisterPin(DataTypeName, FGraphPinParams { .PinCategory = PinCategory, .PinSubCategory = PinSubCategory });
			}
		}
	}

	void FMetaSoundGraphPanelPinFactory::RegisterPin(FName InDataTypeName, const FGraphPinParams& Params)
	{
		using namespace Frontend;
		FDataTypeRegistryInfo DataTypeInfo;
		if (!ensureMsgf(IDataTypeRegistry::Get().GetDataTypeInfo(InDataTypeName, DataTypeInfo), TEXT("Failed to find data type with name '%s'. Pin type not registered"), *InDataTypeName.ToString()))
		{
			return;
		}

		const FName PinCategory = Params.PinCategory.IsNone() ? PinCategoryObject : Params.PinCategory;

		FEdGraphPinType PinType;
		PinType.PinCategory = PinCategory;
		PinType.PinSubCategory = Params.PinSubCategory;
		PinType.ContainerType = DataTypeInfo.bIsArrayType ? EPinContainerType::Array : EPinContainerType::None;
		PinType.PinSubCategoryObject = IDataTypeRegistry::Get().GetUClassForDataType(InDataTypeName);

		FPinConfiguration PinConfiguration;
		PinConfiguration.PinType = PinType;
		PinConfiguration.PinColor = Params.PinColor ? TOptional<FLinearColor>(*Params.PinColor) : TOptional<FLinearColor>();
		PinConfiguration.PinConnectedIcon = Params.PinConnectedIcon;
		PinConfiguration.PinDisconnectedIcon = Params.PinDisconnectedIcon;
		PinTypes.Emplace(InDataTypeName, MoveTemp(PinConfiguration));

		if (Params.PinColor && !Params.PinCategory.IsNone())
		{
			CustomCategoryColors.Add(Params.PinCategory, *Params.PinColor);
		}
	}

	void FMetaSoundGraphPanelPinFactory::UnregisterPin(FName InDataTypeName)
	{
		FPinConfiguration RemovedConfig;
		if (PinTypes.RemoveAndCopyValue(InDataTypeName, RemovedConfig))
		{
			const FName& RemovedCategory = RemovedConfig.PinType.PinCategory;
			if (!RemovedCategory.IsNone() && RemovedConfig.PinColor.IsSet())
			{
				// Only remove the cached color if no other pin type still uses this category
				bool bCategoryStillInUse = false;
				for (const TPair<FName, FPinConfiguration>& Pair : PinTypes)
				{
					if (Pair.Value.PinType.PinCategory == RemovedCategory && Pair.Value.PinColor.IsSet())
					{
						bCategoryStillInUse = true;
						break;
					}
				}

				if (!bCategoryStillInUse)
				{
					CustomCategoryColors.Remove(RemovedCategory);
				}
			}
		}
	}

	void FMetaSoundGraphPanelPinFactory::RegisterCategoryPin(FName InPinCategory, FOnCreateMetaSoundPinWidget InDelegate)
	{
		check(InDelegate.IsBound());
		CategoryToWidgetDelegateMap.Add(InPinCategory, MoveTemp(InDelegate));
	}

	void FMetaSoundGraphPanelPinFactory::RegisterDataTypePin(FName InDataTypeName, FOnCreateMetaSoundPinWidget InDelegate)
	{
		check(InDelegate.IsBound());
		DataTypeToWidgetDelegateMap.Add(InDataTypeName, MoveTemp(InDelegate));
	}

	void FMetaSoundGraphPanelPinFactory::UnregisterCategoryPin(FName InPinCategory)
	{
		CategoryToWidgetDelegateMap.Remove(InPinCategory);
	}

	void FMetaSoundGraphPanelPinFactory::UnregisterDataTypePin(FName InDataTypeName)
	{
		DataTypeToWidgetDelegateMap.Remove(InDataTypeName);
	}
} // namespace Metasound::Editor
