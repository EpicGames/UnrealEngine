// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IMetaSoundGraphPanelPinFactory.h"


namespace Metasound::Editor
{
	class FMetaSoundGraphPanelPinFactory : public IMetaSoundGraphPanelPinFactory
	{
	public:
		FMetaSoundGraphPanelPinFactory();
		virtual ~FMetaSoundGraphPanelPinFactory() = default;

		virtual TSharedPtr<SGraphPin> CreatePin(UEdGraphPin* InPin) const override;

		virtual const FEdGraphPinType* FindPinType(FName InDataTypeName) const override;

		// Convenience getter for returning concrete private implementation
		// from owning editor module and casting accordingly.
		static TSharedRef<FMetaSoundGraphPanelPinFactory> GetChecked();

		virtual bool GetCustomPinIcons(UEdGraphPin* InPin, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const override;
		virtual bool GetCustomPinIcons(FName InDataType, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const override;

		virtual FLinearColor GetPinColor(const FEdGraphPinType& InPinType) const override;

		virtual void RegisterCategoryPin(FName InPinCategory, FOnCreateMetaSoundPinWidget InDelegate) override;
		virtual void UnregisterCategoryPin(FName InPinCategory) override;

		virtual void RegisterDataTypePin(FName InDataTypeName, FOnCreateMetaSoundPinWidget InDelegate) override;
		virtual void UnregisterDataTypePin(FName InDataTypeName) override;

		virtual void RegisterPin(FName InDataTypeName, const FGraphPinParams& Params = FGraphPinParams { }) override;
		virtual void UnregisterPin(FName InDataTypeName) override;

		// Pin category constants
		static const FName PinCategoryAudio;
		static const FName PinCategoryBoolean;
		static const FName PinCategoryFloat;
		static const FName PinCategoryInt32;
		static const FName PinCategoryObject;
		static const FName PinCategoryString;
		static const FName PinCategoryTime;
		static const FName PinCategoryTimeArray;
		static const FName PinCategoryTrigger;
		static const FName PinCategoryWaveTable;

		// Returns whether the given name is an internally-managed MetaSound pin category
		static bool IsInternalMetaSoundPinCategory(FName InPinCategoryName);

	private:
		void RegisterCorePinTypes();

		/** Map from pin category to widget creation delegate */
		TMap<FName, FOnCreateMetaSoundPinWidget> CategoryToWidgetDelegateMap;

		/** Map from data type name to widget creation delegate (higher priority than category) */
		TMap<FName, FOnCreateMetaSoundPinWidget> DataTypeToWidgetDelegateMap;

		struct FPinConfiguration
		{
			FEdGraphPinType PinType;
			TOptional<FLinearColor> PinColor;
			const FSlateBrush* PinConnectedIcon = nullptr;
			const FSlateBrush* PinDisconnectedIcon = nullptr;
		};
		TMap<FName, FPinConfiguration> PinTypes;

		/** Cached custom pin category colors for O(1) lookup in GetPinColor */
		TMap<FName, FLinearColor> CustomCategoryColors;
	};
} // namespace Metasound::Editor
