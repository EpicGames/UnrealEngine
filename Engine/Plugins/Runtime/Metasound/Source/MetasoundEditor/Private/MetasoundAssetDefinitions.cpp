// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAssetDefinitions.h"

#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioEditorSettings.h"
#include "Components/AudioComponent.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "FileHelpers.h"
#include "Framework/Docking/TabManager.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ISourceControlModule.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFactory.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundSource.h"
#include "MetasoundUObjectRegistry.h"
#include "Misc/AssetRegistryInterface.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "SMetasoundInputBrowser.h"
#include "Sound/SoundWave.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SOverlay.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundAssetDefinitions)


#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound::Editor
{
	namespace AssetDefinitionsPrivate
	{
		const FSlateBrush* GetClassBrush(const FAssetData& InAssetData, FName InClassName, bool bIsThumbnail = false)
		{
			using namespace Frontend;

			const FMetaSoundAssetClassInfo ClassInfo(InAssetData);
			if (!ClassInfo.bIsValid)
			{
				UE_LOGF(LogMetaSound, VeryVerbose,
					"ClassBrush for asset '%ls' may return incorrect preset icon. Asset requires reserialization.",
					*InAssetData.GetObjectPathString());
			}

			FString BrushName = FString::Printf(TEXT("MetasoundEditor.%s"), *InClassName.ToString());
			if (ClassInfo.DocInfo.bIsPreset)
			{
				BrushName += TEXT(".Preset");
			}
			BrushName += bIsThumbnail ? TEXT(".Thumbnail") : TEXT(".Icon");

			return &Metasound::Editor::Style::GetSlateBrushSafe(FName(*BrushName));
		}

		void ExecuteBrowseToPresetParentAsset(const FToolMenuContext& InContext)
		{
			using namespace Frontend;

			if (UObject* MetaSound = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UObject>(InContext))
			{
				if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Frontend::IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
					if (const FPresetTemplate* PresetTemplate = DocumentBuilder.GetConstTemplateAs<FPresetTemplate>())
					{
						if (UObject* ParentObject = PresetTemplate->Parent.GetObject(); GEditor && ParentObject)
						{
							GEditor->SyncBrowserToObjects(TArray<UObject*>{ ParentObject });
						}
					}
				}
			}
		}

		void ExecuteOpenPresetParentAsset(const FToolMenuContext& InContext)
		{
			using namespace Frontend;

			if (UObject* MetaSound = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UObject>(InContext))
			{
				if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Frontend::IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
					if (const FPresetTemplate* PresetTemplate = DocumentBuilder.GetConstTemplateAs<FPresetTemplate>())
					{
						if (UObject* ParentObject = PresetTemplate->Parent.GetObject())
						{
							AssetSubsystem->OpenEditorForAsset(ParentObject);
						}
					}
				}
			}
		}

		template <typename TClass, typename TFactoryClass>
		void ExecuteCreateMetaSoundWithTemplate(const FToolMenuContext& MenuContext, TInstancedStruct<FMetaSoundFrontendDocumentTemplate> DocTemplate)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
			{
				const UScriptStruct* DocTemplateStruct = DocTemplate.GetScriptStruct();
				check(DocTemplateStruct);

				const FText DisplayName = DocTemplateStruct->GetDisplayNameText();

				UMetaSoundBaseFactory* Factory = NewObject<TFactoryClass>();
				check(Factory);
				Factory->Template = MoveTemp(DocTemplate);
				Factory->SelectedObjects = Context->LoadSelectedObjects<UObject>();

				if (Factory->SelectedObjects.IsEmpty())
				{
					return;
				}

				check(Factory->SelectedObjects.Last());
				UObject* Outermost = Factory->SelectedObjects.Last()->GetOutermost();
				check(Outermost);

				FString PackagePath;
				FString AssetName;
				const FString Suffix = TEXT("_") + ObjectTools::SanitizeObjectName(DisplayName.ToString());
				IAssetTools::Get().CreateUniqueAssetName(Outermost->GetName(), Suffix, PackagePath, AssetName);

				if (UObject* NewMetaSound = IAssetTools::Get().CreateAssetWithDialog(AssetName, FPackageName::GetLongPackagePath(PackagePath), Factory->GetSupportedClass(), Factory))
				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().SyncBrowserToAssets(TArray<UObject*>({ NewMetaSound }));

					// Save happened in Create call above, but need to save again after
					// Template executes to capture initial procedural changes.
					if (ISourceControlModule::Get().IsEnabled())
					{
						constexpr bool bCheckDirty = false;
						constexpr bool bPromptToSave = false;
						FEditorFileUtils::PromptForCheckoutAndSave({ NewMetaSound->GetPackage() }, bCheckDirty, bPromptToSave);
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Display, "Error creating new asset when creating '%ls' or creation of MetaSound with template was canceled by user.", *AssetName);
				}
			}
		}
		
		void ExecuteReassignClassName(const FToolMenuContext& MenuContext)
		{
			using namespace Frontend;
			using namespace Engine;
			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
			{
				FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
				UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				check(AssetEditorSubsystem);
				const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

				TArray<UPackage*> PackagesToCheckout;
				
				// MetaSound asset filter
				TArray<FTopLevelAssetPath> ClassNames;
				IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&ClassNames](UClass& InClass)
				{
					ClassNames.Add(InClass.GetClassPathName());
				});

				FARFilter MetaSoundFilter;
				MetaSoundFilter.ClassPaths = ClassNames;
				MetaSoundFilter.bRecursiveClasses = true;

				// Get selected objects
				const TArray<UObject*> Objects = Context->LoadSelectedObjects<UObject>();
				for (UObject* SelectedObject : Objects)
				{
					if (!SelectedObject)
					{
						continue;
					}
					
					// Get all assets that reference this one, filtered to MetaSounds
					const FTopLevelAssetPath MetaSoundPath(SelectedObject);
					TArray<FName> OnDiskReferencers;
					AssetRegistry.GetReferencers(MetaSoundPath.GetPackageName(), OnDiskReferencers);
					
					TArray<FAssetData> MetaSoundReferencers;
					if (!OnDiskReferencers.IsEmpty())
					{
						MetaSoundFilter.PackageNames = OnDiskReferencers;
						AssetRegistry.GetAssets(MetaSoundFilter, MetaSoundReferencers);
					}

					TArray<UObject*> ReferencingObjects;
					Algo::TransformIf(MetaSoundReferencers, ReferencingObjects,
					[](const FAssetData& InAssetData)
					{
						return InAssetData.GetAsset() != nullptr;
					},	
					[](const FAssetData& InAssetData)
					{
						return InAssetData.GetAsset();
					});

					// Close open editors to prevent editor graph errors before reference fixup 
					for (UObject* ReferencingObject : ReferencingObjects)
					{
						AssetEditorSubsystem->CloseAllEditorsForAsset(ReferencingObject);
					}

					// Reassign class name
					const TScriptInterface<IMetaSoundDocumentInterface> DocInterface(SelectedObject);
					if (const IMetaSoundDocumentInterface* Interface = DocInterface.GetInterface())
					{
						const FMetasoundFrontendClassMetadata& RootGraphMetadata = Interface->GetConstDocument().RootGraph.Metadata;
						const FMetasoundFrontendVersionNumber& Version = RootGraphMetadata.GetVersion();
						const FNodeRegistryKey OldRegistryKey(EMetasoundFrontendClassType::External, RootGraphMetadata.GetClassName(), Version);
						
						if (AssetManager.ReassignClassName(DocInterface))
						{
							SelectedObject->MarkPackageDirty();
							PackagesToCheckout.Emplace(SelectedObject->GetPackage());

							// Replace references in objects referencing the asset
							const FNodeRegistryKey NewRegistryKey(EMetasoundFrontendClassType::External, RootGraphMetadata.GetClassName(), Version);
							for (UObject* ReferencingObject : ReferencingObjects)
							{
								if (AssetManager.ReplaceReferencesInAsset(ReferencingObject, OldRegistryKey, NewRegistryKey))
								{
									ReferencingObject->MarkPackageDirty();
									PackagesToCheckout.Emplace(ReferencingObject->GetPackage());
								}
								else
								{
									UE_LOGF(LogMetaSound, Warning, "Could not replace references in MetaSound '%ls' after reassigning class name of MetaSound '%ls'", *ReferencingObject->GetFullName(), *SelectedObject->GetFullName());								
								}
							}
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Could not reassign class name of MetaSound '%ls'", *SelectedObject->GetFullName());								
						}
					}
				}

				// Save/checkout
				if (!PackagesToCheckout.IsEmpty())
				{
					FEditorFileUtils::PromptToCheckoutPackages(false, PackagesToCheckout);
				}
			}
		}
		
		void SetObjectSettings(UMetaSoundEditorSubsystem&, UMetaSoundPatch&, UMetaSoundPatch&)
		{
		}

		void SetObjectSettings(UMetaSoundEditorSubsystem& EditorSubsystem, UMetaSoundSource& NewMetaSound, UMetaSoundSource& ParentMetaSound)
		{
			EditorSubsystem.SetSoundWaveSettingsFromTemplate(NewMetaSound, ParentMetaSound);
		}

		// Returns false when any selected asset's class lacks the Referenceable access flag, which
		// disables the "Create Preset" entry for non-Referenceable parents. Uses FAssetData so the
		// parent does not have to be loaded to make this decision.
		bool CanCreateMetaSoundPreset(const FToolMenuContext& InContext)
		{
			using namespace Metasound::Frontend;

			const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
			if (!CBContext)
			{
				return false;
			}

			for (const FAssetData& Asset : CBContext->SelectedAssets)
			{
				const FMetaSoundAssetClassInfo ClassInfo(Asset);
				if (!EnumHasAnyFlags(ClassInfo.AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
				{
					return false;
				}
			}
			return true;
		}

		template <typename TClass, typename TFactoryClass>
		void ExecuteCreateMetaSoundPreset(const FToolMenuContext& MenuContext)
		{
			using namespace Frontend;

			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
			{
				TArray<UObject*> ObjectsToSync;

				for (TClass* ParentMetaSound : Context->LoadSelectedObjects<TClass>())
				{
					FString PackagePath;
					FString AssetName;
					UObject* NewMetaSound = nullptr;

					const UPackage* ParentPackage = ParentMetaSound->GetPackage();
					check(ParentPackage);
					IAssetTools::Get().CreateUniqueAssetName(ParentPackage->GetName(), TEXT("_Preset"), PackagePath, AssetName);

					TInstancedStruct<FPresetTemplate> PresetTemplate = TInstancedStruct<FPresetTemplate>::Make();
					PresetTemplate.GetMutable().Parent = ParentMetaSound;

					IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
					if (ParentPackage->bIsCookedForEditor)
					{
						// Cannot duplicate cooked assets or when underlying asset is cooked, so create new object and copy over properties
						// for sources in InitAsset below. Since copying properties is done manually, 
						// SetSoundWaveSettingsFromTemplate may need to be updated with properties to be copied. 
						UMetaSoundBaseFactory* Factory = NewObject<TFactoryClass>();
						check(Factory);

						NewMetaSound = IAssetTools::Get().CreateAssetWithDialog(AssetName, FPackageName::GetLongPackagePath(PackagePath), Factory->GetSupportedClass(), Factory);
					}
					else
					{
						// Duplicate asset to preserve properties from parent asset (ex. quality settings, soundwave properties)
						NewMetaSound = IAssetTools::Get().DuplicateAssetWithDialogAndTitle(AssetName, FPackageName::GetLongPackagePath(PackagePath), ParentMetaSound, LOCTEXT("CreateMetaSoundPresetTitle", "Create MetaSound Preset"));
					}
				
					if (NewMetaSound)
					{
						UMetaSoundEditorSubsystem& EditorSubsystem = UMetaSoundEditorSubsystem::GetChecked();
						EditorSubsystem.InitAsset(*NewMetaSound, UMetaSoundEditorSubsystem::FInitAssetArgs
						{
							.Template = MoveTemp(PresetTemplate),
							.SelectedObjects = { ParentMetaSound }
						});

						FGraphBuilder::RegisterGraphWithFrontend(*NewMetaSound);

						// Have to set wave settings on new asset when unable to duplicate manually (but after asset is initialized)
						if (ParentPackage->bIsCookedForEditor)
						{
							SetObjectSettings(EditorSubsystem, *CastChecked<TClass>(NewMetaSound), *ParentMetaSound);
						}

						ObjectsToSync.Add(NewMetaSound);
					}
					else
					{
						UE_LOGF(LogMetaSound, Display, "Error creating new asset when creating preset '%ls' or asset creation was canceled by user.", *AssetName);
					}
				}

				// Sync content browser to newly created valid assets
				// Assets can be invalid if multiple assets are created with the same name 
				// then force overwritten within the same operation
				ObjectsToSync.RemoveAllSwap([](const UObject* InObject)
				{
					return !InObject || !InObject->IsValidLowLevelFast();
				});

				if (ObjectsToSync.Num() > 0)
				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
					
					// Save happened in Create/Duplicate asset calls above but
					// need to save again after register graph with frontend (which applies preset transform)
					for (UObject* InObject : ObjectsToSync)
					{
						if (ISourceControlModule::Get().IsEnabled())
						{
							constexpr bool bCheckDirty = false;
							constexpr bool bPromptToSave = false;
							TArray<UPackage*> OutermostPackagesToCheckout;
							OutermostPackagesToCheckout.Add(InObject->GetOutermost());
							FEditorFileUtils::PromptForCheckoutAndSave(OutermostPackagesToCheckout, bCheckDirty, bPromptToSave);
						}
					};
				}
			}
		}

		bool IsPreset(const FToolMenuContext& InContext)
		{
			using namespace Metasound::Frontend;

			if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
			{
				if (CBContext->SelectedAssets.Num() == 1)
				{
					return FMetaSoundAssetClassInfo(CBContext->SelectedAssets.Last()).DocInfo.bIsPreset;
				}
			}

			return false;
		}

		void AddPresetActions(const FSlateIcon& AssetIcon, const UClass& MetaSoundClass, const UClass& Class, FToolUIAction CreateFromTemplateAction, const FText& TemplateDisplayName)
		{
			const FText ClassName = Class.GetDisplayNameText();

			const FName MetaSoundClassName = MetaSoundClass.GetFName();
			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(&Class);
			FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection(MetaSoundClassName);
			check(Section);

			const FText MetaSoundClassDisplayName = MetaSoundClass.GetDisplayNameText();
			Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([CreateAction = MoveTemp(CreateFromTemplateAction), MetaSoundClassDisplayName, MetaSoundClassName, TemplateName = TemplateDisplayName, ClassName, AssetIcon](FToolMenuSection& InSection)
			{
				using namespace Metasound::Editor;

				{
					const TAttribute<FText> Label = FText::Format(LOCTEXT("MetaSound_CreateFromPreset", "Create {0}..."), TemplateName);
					const TAttribute<FText> ToolTip = FText::Format(LOCTEXT("MetaSound_CreateFromPresetToolTipFormat", "Creates a {0} Preset using the selected MetaSound as a parent."), MetaSoundClassDisplayName);

					const FName MenuEntryName = *FString::Format(TEXT("{0}_CreateFromTemplate_{1}"), { MetaSoundClassName.ToString(), TemplateName.ToString()});
					InSection.AddMenuEntry(MenuEntryName, Label, ToolTip, AssetIcon, CreateAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("MetaSound_BrowseToPresetParentFormat", "Browse To Preset Parent");
					const TAttribute<FText> ToolTip = FText::Format(LOCTEXT("MetaSound_BrowseToPresetParentToolTipFormat", "Browses to the selected {0} preset's parent asset in the content browser."), ClassName);
					const FSlateIcon FindInContentBrowserIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&AssetDefinitionsPrivate::ExecuteBrowseToPresetParentAsset);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&AssetDefinitionsPrivate::IsPreset);
					UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&AssetDefinitionsPrivate::IsPreset);
					InSection.AddMenuEntry("MetaSoundSource_BrowseToPresetParent", Label, ToolTip, FindInContentBrowserIcon, UIAction);
				}

				IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<Metasound::Editor::IMetasoundEditorModule>(Metasound::Editor::IMetasoundEditorModule::ModuleName);
				if (!EditorModule.IsRestrictedMode())
				{
					const TAttribute<FText> Label = LOCTEXT("MetaSound_EditPresetParentFormat", "Edit Preset Parent...");
					const TAttribute<FText> ToolTip = FText::Format(LOCTEXT("MetaSound_OpenPresetParentToolTipFormat", "Opens the selected {0} preset's parent MetaSound asset."), ClassName);

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&AssetDefinitionsPrivate::ExecuteOpenPresetParentAsset);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&AssetDefinitionsPrivate::IsPreset);
					UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&AssetDefinitionsPrivate::IsPreset);
					InSection.AddMenuEntry("MetaSoundSource_OpenToPresetParent", Label, ToolTip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"), UIAction);
				}
			}));
		}

		FToolMenuSection* AddMetaSoundContextMenuAction(const UClass& MetaSoundClass, const UClass& ActionClass, TArray<FToolMenuSection*>& OutSections)
		{
			const UAudioEditorSettings* EdSettings = GetDefault<UAudioEditorSettings>();
			check(EdSettings);

			auto GetSection = [&](FName SectionName) -> FToolMenuSection*
			{
				auto MatchesName = [&SectionName](const FToolMenuSection* Section)
				{
					return Section->Name == SectionName;
				};
				if (FToolMenuSection** SectionPtr = OutSections.FindByPredicate(MatchesName))
				{
					check(*SectionPtr);
					return *SectionPtr;
				}

				return nullptr;
			};

			FToolMenuSection* PlaybackSection = GetSection("Playback");
			FToolMenuSection* SoundSection = GetSection("Sound");

			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(&ActionClass);
			check(Menu);

			const UEnum* EnumClass = StaticEnum<EToolMenuInsertType>();
			check(EnumClass);
			const EToolMenuInsertType InsertType = static_cast<EToolMenuInsertType>(EnumClass->GetValueByName(EdSettings->MenuPosition));

			const FName MetaSoundClassName = MetaSoundClass.GetFName();
			FToolMenuSection& MetaSoundSection = Menu->FindOrAddSection(MetaSoundClassName);
			MetaSoundSection.Label = MetaSoundClass.GetDisplayNameText();

			if (SoundSection)
			{
				if (PlaybackSection)
				{
					MetaSoundSection.InsertPosition = FToolMenuInsert(
						PlaybackSection->Name,
						InsertType == EToolMenuInsertType::Last
						? EToolMenuInsertType::Before
						: EToolMenuInsertType::After);

					SoundSection->InsertPosition = FToolMenuInsert(
						MetaSoundSection.Name,
						InsertType == EToolMenuInsertType::Last
						? EToolMenuInsertType::Before
						: EToolMenuInsertType::After);
				}
				else
				{
					MetaSoundSection.InsertPosition = FToolMenuInsert({ }, InsertType);
					SoundSection->InsertPosition = FToolMenuInsert(
						MetaSoundSection.Name,
						InsertType == EToolMenuInsertType::Last
						? EToolMenuInsertType::Before
						: EToolMenuInsertType::After);
				}
			}
			else
			{
				MetaSoundSection.InsertPosition = FToolMenuInsert({ }, InsertType);
			}

			return OutSections.Add_GetRef(&MetaSoundSection);
		}

		void AddTemplateActions(
			const FSlateIcon& AssetIcon,
			const UClass& Class,
			const UClass& MetaSoundClass,
			FToolUIAction CreateFromTemplateAction,
			TConstStructView<FMetaSoundFrontendDocumentTemplate> Template)
		{
			const FText ClassName = Class.GetDisplayNameText();
			const UScriptStruct* ScriptStruct = Template.GetScriptStruct();
			check(ScriptStruct);
			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(&Class);

			FToolMenuSection* Section = nullptr;
			if (const UAssetDefinition_SoundAssetBase* AudioAssetDef = Cast<const UAssetDefinition_SoundAssetBase>(AssetDefinition))
			{
				Section = AudioAssetDef->FindSoundContextMenuSection(MetaSoundClass.GetFName());
			}

			if (!Section)
			{
				TArray<FToolMenuSection*> Sections;
				Section = AddMetaSoundContextMenuAction(MetaSoundClass, Class, Sections);
			}

			if (Section)
			{
				const FText MetaSoundClassDisplayName = MetaSoundClass.GetDisplayNameText();
				const FText TemplateToolTip = Template.GetPtr()->GetEditorOptions().ToolTip;
				auto AddEntryLambda = [CreateAction = MoveTemp(CreateFromTemplateAction), ScriptStruct, TemplateToolTip, ClassName, MetaSoundClassDisplayName, AssetIcon](FToolMenuSection& InSection)
				{
					using namespace Metasound::Editor;
			
					const FText TemplateName = ScriptStruct->GetDisplayNameText();
					const TAttribute<FText> Label = FText::Format(LOCTEXT("MetaSound_CreateFromTemplate", "Create {0}..."), TemplateName);
					const TAttribute<FText> ToolTip = FText::Format(
						LOCTEXT("MetaSound_CreateFromTemplateToolTipFormat", "Creates a {0} with {1} Template, initialized using the selected assets. {2}"),
						MetaSoundClassDisplayName, TemplateName, TemplateToolTip);
			
					const FName MenuEntryName = *FString::Format(TEXT("MetaSound_CreateFromTemplate_{0}"), { TemplateName.ToString() });
					InSection.AddMenuEntry(MenuEntryName, Label, ToolTip, AssetIcon, CreateAction);
				};

				Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda(AddEntryLambda));
			}
		}
	} // namespace AssetDefinitionsPrivate
} // namespace Metasound::Editor


FLinearColor UAssetDefinition_MetaSoundPatch::GetAssetColor() const
{
	if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
	{
		return MetasoundStyle->GetColor("MetaSoundPatch.Color").ToFColorSRGB();
	}

	return FColor::White;
}

TSoftClassPtr<UObject> UAssetDefinition_MetaSoundPatch::GetAssetClass() const
{
	return UMetaSoundPatch::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MetaSoundPatch::GetAssetCategories() const
{
	static const auto Pinned_Categories = { EAssetCategoryPaths::Audio };
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
				LOCTEXT("AssetSoundMetaSoundsSubMenu", "Advanced"), 
				FCategoryPath(LOCTEXT("AssetSoundMetaSoundsSubMenuSection", "MetaSounds"), ECategoryMenuType::Section))
		};

	if (GetDefault<UMetasoundEditorSettings>()->bPinMetaSoundPatchInAssetMenu)
	{
		return Pinned_Categories;
	}

	return Categories;
}

EAssetCommandResult UAssetDefinition_MetaSoundPatch::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	Metasound::Editor::IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<Metasound::Editor::IMetasoundEditorModule>(Metasound::Editor::IMetasoundEditorModule::ModuleName);
	if (!MetaSoundEditorModule.IsRestrictedMode())
	{
		for (UMetaSoundPatch* Metasound : OpenArgs.LoadObjects<UMetaSoundPatch>())
		{
			TSharedRef<Metasound::Editor::FEditor> NewEditor = MakeShared<Metasound::Editor::FEditor>();
			NewEditor->InitMetasoundEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Metasound);
		}
	}
	return EAssetCommandResult::Handled;
}

const FSlateBrush* UAssetDefinition_MetaSoundPatch::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace Metasound::Editor;
	constexpr bool bIsThumbnail = true;
	return AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName, bIsThumbnail);
}

const FSlateBrush* UAssetDefinition_MetaSoundPatch::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace Metasound::Editor;
	return AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName);
}

TArray<FToolMenuSection*> UAssetDefinition_MetaSoundPatch::RebuildSoundContextMenuSections() const
{
	using namespace Metasound::Editor;

	TArray<FToolMenuSection*> Sections = Super::RebuildSoundContextMenuSections();
	if (const UClass* MetaSoundClass = GetAssetClass().Get())
	{
		AssetDefinitionsPrivate::AddMetaSoundContextMenuAction(*MetaSoundClass, *MetaSoundClass, Sections);
	}
	return Sections;
}

FLinearColor UAssetDefinition_MetaSoundSource::GetAssetColor() const
{
 	if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
 	{
 		return MetasoundStyle->GetColor("MetaSoundSource.Color").ToFColorSRGB();
 	}
 
 	return FColor::White;
}

TSoftClassPtr<UObject> UAssetDefinition_MetaSoundSource::GetAssetClass() const
{
	return UMetaSoundSource::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MetaSoundSource::GetAssetCategories() const
{
	static const auto Pinned_Categories = { EAssetCategoryPaths::Audio };
	static const auto Categories = { EAssetCategoryPaths::Audio / LOCTEXT("AssetSoundMetaSoundSourceSubMenu", "MetaSounds") };

	if (GetDefault<UMetasoundEditorSettings>()->bPinMetaSoundSourceInAssetMenu)
	{
		return Pinned_Categories;
	}

	return Categories;
}

EAssetCommandResult UAssetDefinition_MetaSoundSource::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace Metasound;

	for (const FAssetData& AssetData : OpenArgs.Assets)
	{
		const UClass* AssetClass = AssetData.GetClass();
		if (AssetClass && IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass))
		{
			Engine::FMetaSoundAssetManager& AssetManager = Engine::FMetaSoundAssetManager::GetChecked();
			TWeakPtr<IToolkitHost> ToolkitHost = OpenArgs.ToolkitHost;
			const bool bHostNull = !OpenArgs.ToolkitHost.IsValid();
			AssetManager.AddOrLoadAndUpdateFromObjectAsync(AssetData, [ToolkitMode = OpenArgs.GetToolkitMode(), bHostNull, ToolkitHost](FMetaSoundAssetKey, UObject& MetaSoundObject)
			{
				TSharedPtr<IToolkitHost> HostPtr = ToolkitHost.Pin();
				if (bHostNull || HostPtr)
				{
					Editor::IMetasoundEditorModule* EditorModule = FModuleManager::GetModulePtr<Editor::IMetasoundEditorModule>(Editor::IMetasoundEditorModule::ModuleName);
					if (EditorModule)
					{
						if (EditorModule->IsRestrictedMode())
						{
							TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(&MetaSoundObject);
							check(DocInterface.GetObject());
							const Frontend::FMetaSoundAssetClassInfo ClassInfo(*DocInterface.GetInterface());
							if (!ClassInfo.bIsValid || !ClassInfo.DocInfo.bIsPreset)
							{
								return;
							}
						}

						TSharedRef<Editor::FEditor> NewEditor = MakeShared<Editor::FEditor>();
						NewEditor->InitMetasoundEditor(ToolkitMode, HostPtr, &MetaSoundObject);
					}
				}
			});
		}
	}

	return EAssetCommandResult::Handled;
}

const FSlateBrush* UAssetDefinition_MetaSoundSource::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace Metasound::Editor;
	constexpr bool bIsThumbnail = true;
	return AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName, bIsThumbnail);
}

const FSlateBrush* UAssetDefinition_MetaSoundSource::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace Metasound::Editor;
	return AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName);
}

void UAssetDefinition_MetaSoundSource::ExecutePlaySound(const FToolMenuContext& InContext)
{
	if (UMetaSoundSource* MetaSoundSource = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UMetaSoundSource>(InContext))
	{
		// If editor is open, call into it to play to start all visualization requirements therein
		// specific to auditioning MetaSounds (ex. priming audio bus used for volume metering, playtime
		// widget, etc.)
		TSharedPtr<Metasound::Editor::FEditor> Editor = Metasound::Editor::FGraphBuilder::GetEditorForMetasound(*MetaSoundSource);
		if (Editor.IsValid())
		{
			Editor->Play();
			return;
		}

		Metasound::Editor::FGraphBuilder::FGraphBuilder::RegisterGraphWithFrontend(*MetaSoundSource);
		UAssetDefinition_SoundBase::ExecutePlaySound(InContext);
	}
}

void UAssetDefinition_MetaSoundSource::ExecuteStopSound(const FToolMenuContext& InContext)
{
	if (UMetaSoundSource* MetaSoundSource = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UMetaSoundSource>(InContext))
	{
		TSharedPtr<Metasound::Editor::FEditor> Editor = Metasound::Editor::FGraphBuilder::GetEditorForMetasound(*MetaSoundSource);
		if (Editor.IsValid())
		{
			Editor->Stop();
			return;
		}

		UAssetDefinition_SoundBase::ExecuteStopSound(InContext);
	}
}

bool UAssetDefinition_MetaSoundSource::CanExecutePlayCommand(const FToolMenuContext& InContext)
{
	return UAssetDefinition_SoundBase::CanExecutePlayCommand(InContext);
}

ECheckBoxState UAssetDefinition_MetaSoundSource::IsActionCheckedMute(const FToolMenuContext& InContext)
{
	return UAssetDefinition_SoundBase::IsActionCheckedMute(InContext);
}

ECheckBoxState UAssetDefinition_MetaSoundSource::IsActionCheckedSolo(const FToolMenuContext& InContext)
{
	return UAssetDefinition_SoundBase::IsActionCheckedSolo(InContext);
}

void UAssetDefinition_MetaSoundSource::ExecuteMuteSound(const FToolMenuContext& InContext)
{
	UAssetDefinition_SoundBase::ExecuteMuteSound(InContext);
}

void UAssetDefinition_MetaSoundSource::ExecuteSoloSound(const FToolMenuContext& InContext)
{
	UAssetDefinition_SoundBase::ExecuteSoloSound(InContext);
}

bool UAssetDefinition_MetaSoundSource::CanExecuteMuteCommand(const FToolMenuContext& InContext)
{
	return UAssetDefinition_SoundBase::CanExecuteMuteCommand(InContext);
}

bool UAssetDefinition_MetaSoundSource::CanExecuteSoloCommand(const FToolMenuContext& InContext)
{
	return UAssetDefinition_SoundBase::CanExecuteSoloCommand(InContext);
}

TSharedPtr<SWidget> UAssetDefinition_MetaSoundSource::GetThumbnailOverlay(const FAssetData& InAssetData) const
{
	auto OnClickedLambdaOverride = [InAssetData]() -> FReply
	{
		if (UObject* Object = InAssetData.GetAsset())
		{
			TSharedPtr<Metasound::Editor::FEditor> Editor = Metasound::Editor::FGraphBuilder::GetEditorForMetasound(*Object);
			if (UE::AudioEditor::IsSoundPlaying(InAssetData))
			{
				if (Editor.IsValid())
				{
					Editor->Stop();
				}
				else
				{
					UE::AudioEditor::StopSound();
				}
			}
			else
			{
				if (Editor.IsValid())
				{
					Editor->Play();
				}
				else
				{
					// Load and play sound
					UE::AudioEditor::PlaySound(Cast<USoundBase>(Object));
				}
			}
		}
		return FReply::Handled();
	};
	return UAssetDefinition_SoundBase::GetSoundBaseThumbnailOverlay(InAssetData, MoveTemp(OnClickedLambdaOverride));
}

bool UAssetDefinition_MetaSoundSource::GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
{
	OutActionOverlayInfo.IsActionPlayingDelegate = FIsActionPlaying::CreateLambda([InAssetData] ()
	{
		return UE::AudioEditor::IsSoundPlaying(InAssetData);
	});

	auto OnToolTipTextLambda = [InAssetData]() -> FText
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			return LOCTEXT("Thumbnail_StopSoundToolTip", "Stop selected sound");
		}

		return LOCTEXT("Thumbnail_PlaySoundToolTip", "Play selected sound");
	};

	auto OnClickedLambda = [InAssetData]() -> FReply
	{
		if (UObject* Object = InAssetData.GetAsset())
		{
			TSharedPtr<Metasound::Editor::FEditor> Editor = Metasound::Editor::FGraphBuilder::GetEditorForMetasound(*Object);
			if (UE::AudioEditor::IsSoundPlaying(InAssetData))
			{
				if (Editor.IsValid())
				{
					Editor->Stop();
				}
				else
				{
					UE::AudioEditor::StopSound();
				}
			}
			else
			{
				if (Editor.IsValid())
				{
					Editor->Play();
				}
				else
				{
					// Load and play sound
					UE::AudioEditor::PlaySound(Cast<USoundBase>(Object));
				}
			}
		}
		return FReply::Handled();
	};

	OutActionOverlayInfo.ActionButtonArgs = SButton::FArguments()
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.OnClicked_Lambda(OnClickedLambda);

	return true;
}

EAssetCommandResult UAssetDefinition_MetaSoundSource::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	if (ActivateArgs.ActivationMethod == EAssetActivationMethod::Previewed)
	{
		if (UMetaSoundSource* MetaSoundSource = ActivateArgs.LoadFirstValid<UMetaSoundSource>())
		{
			TSharedPtr<Metasound::Editor::FEditor> Editor = Metasound::Editor::FGraphBuilder::GetEditorForMetasound(*MetaSoundSource);
			UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
	
			// If the editor is open, we need to stop or start the editor so it can light up while previewing it in the CB
			if (Editor.IsValid())
			{
				if (PreviewComp && PreviewComp->IsPlaying())
				{
					if (!MetaSoundSource || PreviewComp->Sound == MetaSoundSource)
					{
						Editor->Stop();
					}
				}
				else
				{
					Editor->Play();
				}

				return EAssetCommandResult::Handled;
			}
			else
			{
				return UAssetDefinition_SoundBase::ActivateSoundBase(ActivateArgs);
			}
		}
	}
	return EAssetCommandResult::Unhandled;
}

void UAssetDefinition_MetaSoundSource::GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const
{
	UAssetDefinition_SoundBase::GetSoundBaseAssetActionButtonExtensions(InAssetData, OutExtensions);
}

TArray<FToolMenuSection*> UAssetDefinition_MetaSoundSource::RebuildSoundContextMenuSections() const
{
	using namespace Metasound::Editor;

	TArray<FToolMenuSection*> Sections = Super::RebuildSoundContextMenuSections();
	if (const UClass* MetaSoundClass = GetAssetClass().Get())
	{
		AssetDefinitionsPrivate::AddMetaSoundContextMenuAction(*MetaSoundClass, *MetaSoundClass, Sections);
	}
	return Sections;
}


namespace MenuExtension_MetaSoundTemplate
{
	template <typename TClass, typename TFactoryClass>
	void RegisterTemplateActions(const FSlateIcon& AssetIcon)
	{
		using namespace Metasound;
		using namespace Metasound::Editor;
		using namespace Metasound::Frontend;

		const UClass* MetaSoundClass = TClass::StaticClass();
		check(MetaSoundClass);

		const TSoftClassPtr<UClass> MetaSoundSoftClass(MetaSoundClass);

		for (TObjectIterator<UStruct> It; It; ++It)
		{
			UScriptStruct* Struct = Cast<UScriptStruct>(*It);
			if (!Struct)
			{
				continue;
			}

			if (!Struct->IsChildOf(FMetaSoundFrontendDocumentTemplate::StaticStruct()))
			{
				continue;
			}

			// Presets are special snowflakes for now, as they require a bit more functionality for back
			// compat like copying over SoundWave fields which still needs to be migrated to templates.
			if (Struct == FPresetTemplate::StaticStruct())
			{
				continue;
			}

			// Don't show structs marked as hidden
			if (Struct->HasMetaData("Hidden"))
			{
				continue;
			}

			TInstancedStruct<FMetaSoundFrontendDocumentTemplate> TemplateInstance;
			TemplateInstance.InitializeAsScriptStruct(Struct);
			TArray<TSoftClassPtr<UObject>> SupportedMetaSoundClasses = TemplateInstance.Get().GetEditorOptions().MetaSoundClasses;
			if (SupportedMetaSoundClasses.IsEmpty())
			{
				IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&SupportedMetaSoundClasses](UClass& RegClass)
				{
					SupportedMetaSoundClasses.Add(&RegClass);
				});
			}

			if (SupportedMetaSoundClasses.Contains(MetaSoundSoftClass))
			{
				FToolUIAction DocTemplateCreateAction;
				DocTemplateCreateAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([Struct](const FToolMenuContext& MenuContext)
				{
					TInstancedStruct<FMetaSoundFrontendDocumentTemplate> NewInstance;
					NewInstance.InitializeAsScriptStruct(Struct);
					return AssetDefinitionsPrivate::ExecuteCreateMetaSoundWithTemplate<TClass, TFactoryClass>(MenuContext, MoveTemp(NewInstance));
				});

				const TArray<TSoftClassPtr<UObject>>& AssetActionClasses = TemplateInstance.GetPtr()->GetEditorOptions().AssetActionClasses;
				if (AssetActionClasses.IsEmpty())
				{
					AssetDefinitionsPrivate::AddTemplateActions(AssetIcon, *MetaSoundClass, *MetaSoundClass, MoveTemp(DocTemplateCreateAction), TemplateInstance);
				}
				else
				{
					for (const TSoftClassPtr<UObject>& Class : AssetActionClasses)
					{
						const UClass* ClassPtr = Class.Get();
						if (!ClassPtr)
						{
							ClassPtr = Class.LoadSynchronous();
						}
						check(ClassPtr);

						AssetDefinitionsPrivate::AddTemplateActions(AssetIcon, *ClassPtr, *MetaSoundClass, DocTemplateCreateAction, TemplateInstance);
					}
				}
			}
		}
	}

 	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		if (!UToolMenus::IsToolMenuUIEnabled())
		{
			return;
		}

 		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
 		{
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

 			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			// MetaSound Source Action Registration
			{
				const FSlateIcon AssetIcon = Metasound::Editor::Style::CreateSlateIcon("ClassIcon.MetasoundSource");

				RegisterTemplateActions<UMetaSoundSource, UMetaSoundSourceFactory>(AssetIcon);

				const UClass* MetaSoundClass = UMetaSoundSource::StaticClass();
				check(MetaSoundClass);

				FToolUIAction PresetCreateAction;
				PresetCreateAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&AssetDefinitionsPrivate::ExecuteCreateMetaSoundPreset<UMetaSoundSource, UMetaSoundSourceFactory>);
				PresetCreateAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&AssetDefinitionsPrivate::CanCreateMetaSoundPreset);
				AssetDefinitionsPrivate::AddPresetActions(AssetIcon, *MetaSoundClass, *MetaSoundClass, MoveTemp(PresetCreateAction), FPresetTemplate::StaticStruct()->GetDisplayNameText());

				{
					const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(MetaSoundClass);
					FToolMenuSection* PlaybackSection = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Playback");
					check(PlaybackSection);

					PlaybackSection->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([AssetIcon](FToolMenuSection& InSection)
					{
						auto IsPlayingThis = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };
						auto IsNotPlayingThis = [](const FToolMenuContext& InContext) { return !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };
						auto IsNotPlayingAny = [](const FToolMenuContext& InContext) { return !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */); };
						auto IsPlayingAny = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */); };
						auto IsPlayingOther = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */) && !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };

						{
							const TAttribute<FText> Label = LOCTEXT("Sound_PlaySound" , "Play");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_PlaySoundTooltip", "Plays the selected sound.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Play.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecutePlaySound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::CanExecutePlayCommand);
							UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsNotPlayingThis);
							InSection.AddMenuEntry("Sound_PlaySound", Label, ToolTip, Icon, UIAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_RestartSound", "Restart");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_RestartSoundTooltip", "Restarts the selected sound.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cascade.RestartInLevel.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecutePlaySound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::CanExecutePlayCommand);
							UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsPlayingThis);
							InSection.AddMenuEntry("Sound_RestartSound", Label, ToolTip, Icon, UIAction);
						}

						{ // Stop
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Stop.Small");
							{ // Selected
								const TAttribute<FText> StopSelectedToolTip = LOCTEXT("Sound_StopSoundTooltip", "Stops the selected sound.");
								{
									const TAttribute<FText> Label = LOCTEXT("Sound_StopSoundDisabled", "Stop");

									FToolUIAction UIAction;
									UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(IsPlayingThis);
									UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecuteStopSound);
									UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([IsNotPlayingAny, IsPlayingThis](const FToolMenuContext& Context) { return IsNotPlayingAny(Context) || IsPlayingThis(Context); });
									InSection.AddMenuEntry("Sound_StopSound", Label, StopSelectedToolTip, Icon, UIAction);
								}
							}
							{ // Other
								const TAttribute<FText> Label = TAttribute<FText>::CreateLambda([]()
								{
									if (const USoundBase* OtherSound = UAssetDefinition_SoundBase::GetPlayingSound())
									{
										return FText::Format(LOCTEXT("Sound_StopSoundOtherFormat", "Stop ({0})"), FText::FromName(OtherSound->GetFName()));
									}
									return LOCTEXT("Sound_StopSoundOther", "Stop (Other)");
								});
								const TAttribute<FText> ToolTip = LOCTEXT("Sound_StopOtherSoundTooltip", "Stops the currently previewing (other) sound.");
								FToolUIAction UIAction;
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecuteStopSound);
								UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsPlayingOther);
								InSection.AddMenuEntry("Sound_StopOtherSound", Label, ToolTip, Icon, UIAction);
							}
						}

						{
							const TAttribute<FText> Label = LOCTEXT("Sound_MuteSound", "Mute");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_MuteSoundTooltip", "Mutes the selected sounds.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Mute.Small");
				
							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecuteMuteSound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::CanExecuteMuteCommand);
							UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&UAssetDefinition_MetaSoundSource::IsActionCheckedMute);
							InSection.AddMenuEntry("Sound_SoundMute", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_SoloSound", "Solo");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_SoloSoundTooltip", "Solos the selected sounds.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Solo.Small");
				
							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::ExecuteSoloSound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_MetaSoundSource::CanExecuteSoloCommand);
							UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&UAssetDefinition_MetaSoundSource::IsActionCheckedSolo);
							InSection.AddMenuEntry("Sound_StopSolo", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_ClearMutedSoloed", "Clear Muted/Soloed");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_ClearMutedSoloedTooltip", "Clear all flags to mute/solo specific assets.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.DiffersFromDefault");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteClearMutesAndSolos);
							UIAction.IsActionVisibleDelegate = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecuteClearMutesAndSolos);
							InSection.AddMenuEntry("Sound_ClearMuteSoloSettings", Label, ToolTip, Icon, UIAction);
						}
					}));
				}
 			}

			// MetaSound Patch Action Registration
			{
				FToolUIAction CreateAction;
				CreateAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&AssetDefinitionsPrivate::ExecuteCreateMetaSoundPreset<UMetaSoundPatch, UMetaSoundFactory>);
				CreateAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&AssetDefinitionsPrivate::CanCreateMetaSoundPreset);
				const FSlateIcon AssetIcon = Metasound::Editor::Style::CreateSlateIcon("ClassIcon.MetasoundPatch");

				RegisterTemplateActions<UMetaSoundPatch, UMetaSoundFactory>(AssetIcon);

				const UClass* MetaSoundClass = UMetaSoundPatch::StaticClass();
				check(MetaSoundClass);
				AssetDefinitionsPrivate::AddPresetActions(AssetIcon, *MetaSoundClass, *MetaSoundClass, MoveTemp(CreateAction), FPresetTemplate::StaticStruct()->GetDisplayNameText());
			}

			// Asset actions
			{
				const FNewToolMenuSectionDelegate AssetActionsSectionDelegate = FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					InSection.AddMenuEntry(
						"ReassignClassName",
						LOCTEXT("ReassignClassNameTitle", "Reassign Asset Class Guid"),
						LOCTEXT("ReassignClassNameTooltipText", "Reassigns class name (asset class guid) on selected assets and fixes up asset references.\n" \
									"Useful for fixing up asset key errors caused by MetaSound assets with duplicate class names, \n " \
									"which can occur when assets are duplicated outside of the editor."),
						FSlateIcon(),
						FToolMenuExecuteAction::CreateStatic(&AssetDefinitionsPrivate::ExecuteReassignClassName)
					);
				});

				Metasound::IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&AssetActionsSectionDelegate](UClass& InClass)
				{
					const FString MenuName = TEXT("ContentBrowser.AssetContextMenu.") + InClass.GetName() + TEXT(".AssetActionsSubMenu");
					UToolMenu* AssetActionsMenu = UToolMenus::Get()->ExtendMenu(FName(MenuName));
					FToolMenuSection& MetaSoundAssetActionsMenu = AssetActionsMenu->AddSection("MetaSoundAssetContextActions", LOCTEXT("MetaSoundAssetContextActionsMenuHeading", "MetaSound"));

					MetaSoundAssetActionsMenu.AddDynamicEntry(NAME_None, AssetActionsSectionDelegate);
				});
			}

			// "Browse MetaSound Inputs..." action - opens the MetaSound Input Browser with selected assets.
			{
				auto ExecuteBrowseInputs = [](const FToolMenuContext& InContext)
				{
					const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
					if (!CBContext)
					{
						return;
					}

					TArray<TWeakObjectPtr<UObject>> SelectedAssets;
					for (const FAssetData& AssetData : CBContext->SelectedAssets)
					{
						const UClass* AssetClass = AssetData.GetClass();
						if (!AssetClass || !Metasound::IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass))
						{
							continue;
						}
						if (UObject* Asset = AssetData.GetAsset())
						{
							SelectedAssets.Add(Asset);
						}
					}

					if (SelectedAssets.IsEmpty())
					{
						return;
					}

					TSharedPtr<SDockTab> ExistingTab = FGlobalTabmanager::Get()->TryInvokeTab(SMetasoundInputBrowser::TabId);
					if (ExistingTab.IsValid())
					{
						TSharedRef<SMetasoundInputBrowser> Browser = StaticCastSharedRef<SMetasoundInputBrowser>(ExistingTab->GetContent());
						Browser->SetAssets(SelectedAssets);
					}
				};

				// Register on the base context menu for mixed-type selections
				{
					UToolMenu* BaseMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu");
					FToolMenuSection& Section = BaseMenu->FindOrAddSection("MetaSoundBrowseInputs", LOCTEXT("MetaSoundBrowseInputsHeading", "MetaSound"));

					const FNewToolMenuSectionDelegate BaseDelegate = FNewToolMenuSectionDelegate::CreateLambda(
						[ExecuteBrowseInputs](FToolMenuSection& InSection)
					{
						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(ExecuteBrowseInputs);
						UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([](const FToolMenuContext& InContext)
						{
							const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
							if (!CBContext)
							{
								return false;
							}
							for (const FAssetData& AssetData : CBContext->SelectedAssets)
							{
								const UClass* AssetClass = AssetData.GetClass();
								if (AssetClass && Metasound::IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass))
								{
									return true;
								}
							}
							return false;
						});

						InSection.AddMenuEntry(
							"MetaSound_BrowseInputs",
							LOCTEXT("BrowseInputsTitle", "Browse MetaSound Inputs..."),
							LOCTEXT("BrowseInputsTooltip", "Open the MetaSound Input Browser to view and edit input values across selected assets in bulk."),
							Metasound::Editor::Style::CreateSlateIcon("MetasoundEditor.Metasound.Icon"),
							UIAction
						);
					});

					Section.AddDynamicEntry(NAME_None, BaseDelegate);
				}
			}
 		}));
	});
} // namespace MenuExtension_MetaSoundTemplate
#undef LOCTEXT_NAMESPACE //MetaSoundEditor
