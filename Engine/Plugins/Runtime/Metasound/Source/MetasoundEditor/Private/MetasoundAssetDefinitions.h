// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "MetasoundAssetDefinitions.generated.h"

// Forward Declarations
struct FToolMenuSection;


UCLASS()
class UAssetDefinition_MetaSoundPatch : public UAssetDefinition_SoundAssetBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MetaSoundPatch", "MetaSound Patch"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	// UAssetDefinition End

	virtual TArray<FToolMenuSection*> RebuildSoundContextMenuSections() const override;
};

UCLASS()
class UAssetDefinition_MetaSoundSource : public UAssetDefinition_SoundBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MetaSoundSource", "MetaSound Source"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const override;
	virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override;
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual void GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const override;

 	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
 	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	// UAssetDefinition End

	virtual TArray<FToolMenuSection*> RebuildSoundContextMenuSections() const override;

	// Menu Extension statics
	static void ExecutePlaySound(const FToolMenuContext& InContext);
	static void ExecuteStopSound(const FToolMenuContext& InContext);
	static bool CanExecutePlayCommand(const FToolMenuContext& InContext);
	static ECheckBoxState IsActionCheckedMute(const FToolMenuContext& InContext);
	static ECheckBoxState IsActionCheckedSolo(const FToolMenuContext& InContext);
	static void ExecuteMuteSound(const FToolMenuContext& InContext);
	static void ExecuteSoloSound(const FToolMenuContext& InContext);
	static bool CanExecuteMuteCommand(const FToolMenuContext& InContext);
	static bool CanExecuteSoloCommand(const FToolMenuContext& InContext);
};