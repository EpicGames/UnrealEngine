// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDefaultLiteralCustomization.h"

#include "DetailLayoutBuilder.h"
#include "MetasoundDetailCustomization.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundNodeDetailCustomization.h"
#include "MetasoundSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound::Editor
{
	namespace LiteralCustomization
	{
		FGuid GetGuidPropertyValue(TSharedPtr<IPropertyHandle> GuidProperty)
		{
			// Randomized so if property be invalid for whatever reason,
			// the page default entry isn't mistakenly removed.
			FGuid Guid = FGuid::NewGuid();

			if (!ensure(GuidProperty.IsValid()))
			{
				return Guid;
			}

			auto GetGuidUInt = [&GuidProperty](FName Section)
			{
				int32 Value = 0;
				TSharedPtr<IPropertyHandle> SectionHandle = GuidProperty->GetChildHandle(Section);
				if (ensure(SectionHandle.IsValid()))
				{
					SectionHandle->GetValue(Value);
				}
				return Value;
			};
			return FGuid(
				GetGuidUInt(GET_MEMBER_NAME_CHECKED(FGuid, A)),
				GetGuidUInt(GET_MEMBER_NAME_CHECKED(FGuid, B)),
				GetGuidUInt(GET_MEMBER_NAME_CHECKED(FGuid, C)),
				GetGuidUInt(GET_MEMBER_NAME_CHECKED(FGuid, D))
			);
		}
	} // namespace LiteralCustomization

	FMetasoundDefaultLiteralCustomizationBase::FMetasoundDefaultLiteralCustomizationBase(IDetailCategoryBuilder& InDefaultCategoryBuilder)
		: DefaultCategoryBuilder(&InDefaultCategoryBuilder)
	{
	}

	FMetasoundDefaultLiteralCustomizationBase::~FMetasoundDefaultLiteralCustomizationBase()
	{
		if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
		{
			if (OnPageSettingsUpdatedHandle.IsValid())
			{
				Settings->GetOnPageSettingsUpdatedDelegate().Remove(OnPageSettingsUpdatedHandle);
			}
		}
	}

	void FMetasoundDefaultLiteralCustomizationBase::BuildPageDefaultComboBox(UMetasoundEditorGraphMemberDefaultLiteral& Literal, FText RowName)
	{
		using namespace Engine;

		check(DefaultCategoryBuilder);

		TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> LiteralPtr(&Literal);
		if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
		{
			FOnPageSettingsUpdated& UpdatedDelegate = Settings->GetOnPageSettingsUpdatedDelegate();
			if (OnPageSettingsUpdatedHandle.IsValid())
			{
				UpdatedDelegate.Remove(OnPageSettingsUpdatedHandle);
			}
			OnPageSettingsUpdatedHandle = UpdatedDelegate.AddLambda([this, LiteralPtr]()
			{
				UpdatePagePickerNames(LiteralPtr);
				if (PageDefaultComboBox.IsValid())
				{
					PageDefaultComboBox->RefreshOptions();
				}
			});
		}
		else
		{
			return;
		}

		SAssignNew(PageDefaultComboBox, SSearchableComboBox)
		.OptionsSource(&AddablePageStringNames)
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
		{
			return SNew(STextBlock).Text(FText::FromString(*InItem));
		})
		.OnSelectionChanged_Lambda([this, LiteralPtr](TSharedPtr<FString> NameToAdd, ESelectInfo::Type InSelectInfo)
		{
			using namespace Engine;
			using namespace Frontend;

			if (InSelectInfo != ESelectInfo::OnNavigation)
			{
				if (!LiteralPtr.IsValid())
				{
					return;
				}

				UMetasoundEditorGraphMember* Member = LiteralPtr->FindMember();
				if (!Member)
				{
					return;
				}

				if (!NameToAdd.IsValid())
				{
					return;
				}

				const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
				check(Settings);
				if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(FName(*NameToAdd)))
				{
					const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddPageDefaultValueTransactionFormat", "Add '{0}' Page '{1}' Default Value"),
						FText::FromName(Member->GetMemberName()),
						FText::FromString(*NameToAdd)));

					FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
					UObject& MetaSound = Builder.CastDocumentObjectChecked<UObject>();
					MetaSound.Modify();
					LiteralPtr->Modify();

					const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
					check(EditorSettings);

					TArray<FGuid> ImplementedGuids;
					LiteralPtr->IterateDefaults([&ImplementedGuids](const FGuid& PageID, FMetasoundFrontendLiteral)
					{
						ImplementedGuids.Add(PageID);
					});

					const FGuid FallbackPageID = EditorSettings->ResolveAuditionPage(ImplementedGuids, PageSettings->UniqueId);

					LiteralPtr->InitDefault(PageSettings->UniqueId);

					FMetasoundFrontendLiteral InitValue;
					if (LiteralPtr->TryFindDefault(InitValue, &FallbackPageID))
					{
						LiteralPtr->SetFromLiteral(InitValue, PageSettings->UniqueId);
					}
					constexpr bool bPostTransaction = false;
					Member->UpdateFrontendDefaultLiteral(bPostTransaction, &PageSettings->UniqueId);
					FMetasoundAssetBase& MetasoundAsset = Editor::FGraphBuilder::GetOutermostMetaSoundChecked(*LiteralPtr);
					MetasoundAsset.GetModifyContext().AddMemberIDsModified({ Member->GetMemberID() });

					UpdatePagePickerNames(LiteralPtr);
					PageDefaultComboBox->RefreshOptions();
					FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
				}
			}
		})
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AddPageDefaultValuePrompt", "Add Page Default Value..."))
		];

		DefaultCategoryBuilder->AddCustomRow(RowName)
		.IsEnabled(GetEnabled())
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.Visibility(Visibility)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PageDefaultComboBox->AsShared()
			]
		]
		.ResetToDefaultContent()
		[
			PropertyCustomizationHelpers::MakeResetButton(FSimpleDelegate::CreateLambda([this, LiteralPtr]()
			{
				using namespace Frontend;

				if (!LiteralPtr.IsValid())
				{
					return;
				}

				UMetasoundEditorGraphMember* Member = LiteralPtr->FindMember();
				if (!Member)
				{
					return;
				}

				const FScopedTransaction Transaction(LOCTEXT("ResetPageDefaultsTransaction", "Reset Paged Defaults"));

				const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
				check(Settings);

				FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
				UObject& MetaSound = Builder.CastDocumentObjectChecked<UObject>();
				MetaSound.Modify();
				LiteralPtr->Modify();

				LiteralPtr->ResetDefaults();

				constexpr bool bPostTransaction = false;
				Member->UpdateFrontendDefaultLiteral(bPostTransaction);

				FMetasoundAssetBase& MetasoundAsset = Editor::FGraphBuilder::GetOutermostMetaSoundChecked(*LiteralPtr);
				MetasoundAsset.GetModifyContext().AddMemberIDsModified({ Member->GetMemberID() });
				UpdatePagePickerNames(LiteralPtr);
				PageDefaultComboBox->RefreshOptions();
				FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
			}), LOCTEXT("ResetPageDefaultsTooltip", "Resets page defaults for the given member, leaving just the initial value for the required 'Default' page."))
		];
	}

	TSharedRef<SWidget> FMetasoundDefaultLiteralCustomizationBase::BuildPageDefaultNameWidget(UMetasoundEditorGraphMemberDefaultLiteral& Literal, TSharedRef<IPropertyHandle> ElementProperty)
	{
		TSharedPtr<IPropertyHandle> PageNameProperty = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorMemberPageDefault, PageName));
		if (!ensure(PageNameProperty.IsValid()))
		{
			return SNullWidget::NullWidget;
		}

		FText PageNameText;
		PageNameProperty->GetValueAsFormattedText(PageNameText);

		TSharedRef<SHorizontalBox> NameBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 0)
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(PageNameText)
		];

		const UMetasoundEditorGraphMember* Member = Literal.FindMember();
		if (!ensure(Member))
		{
			return SNullWidget::NullWidget;
		}

		// Can't delete the default page, so only show remove page field if non-default/project defined page.
		const FGuid PageID = LiteralCustomization::GetGuidPropertyValue(ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorMemberPageDefault, PageID)));
		if (PageID != Frontend::DefaultPageID)
		{
			TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> LiteralPtr(&Literal);

			FName PageName;
			PageNameProperty->GetValue(PageName);
			const FText RemoveDescription = FText::Format(LOCTEXT("RemovePageDefaultTransactionFormat", "Remove '{0}' Page '{1}' Default Value"), FText::FromName(Member->GetMemberName()), FText::FromName(PageName));
			TSharedRef<SWidget> RemovePageDefaultButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateLambda([this, RemoveDescription, PageID, LiteralPtr]()
			{
				using namespace Frontend;

				if (!LiteralPtr.IsValid())
				{
					return;
				}

				UMetasoundEditorGraphMember* Member = LiteralPtr->FindMember();
				if (!Member)
				{
					return;
				}

				const FScopedTransaction Transaction(RemoveDescription);

				const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
				check(Settings);

				FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
				UObject& MetaSound = Builder.CastDocumentObjectChecked<UObject>();
				MetaSound.Modify();
				LiteralPtr->Modify();

				LiteralPtr->RemoveDefault(PageID);

				constexpr bool bPostTransaction = false;
				Member->UpdateFrontendDefaultLiteral(bPostTransaction, /*InPageID=*/nullptr, EMetaSoundLiteralChangeType::Remove);

				FMetasoundAssetBase& MetasoundAsset = Editor::FGraphBuilder::GetOutermostMetaSoundChecked(*LiteralPtr);
				MetasoundAsset.GetModifyContext().AddMemberIDsModified({ Member->GetMemberID() });
				UpdatePagePickerNames(LiteralPtr);
				PageDefaultComboBox->RefreshOptions();
				FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
			}),
			RemoveDescription);

			NameBox->AddSlot()
			.FillWidth(1)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2, 0)
			[
				RemovePageDefaultButton
			];
		}

		if (Member->IsDefaultPaged())
		{
			TAttribute<EVisibility> ExecVisibility = TAttribute<EVisibility>::CreateLambda([MemberPtr = TWeakObjectPtr<const UMetasoundEditorGraphMember>(Member), PageID]()
			{
				if (const UMetasoundEditorGraphMember* GraphMember = MemberPtr.Get())
				{
					const FMetaSoundFrontendDocumentBuilder& Builder = GraphMember->GetFrontendBuilderChecked();
					if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(GraphMember->GetMemberName()))
					{
						const bool bIsPreviewing = Editor::IsPreviewingPageInputDefault(Builder, *ClassInput, PageID);
						return bIsPreviewing ? EVisibility::Visible : EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				}

				return EVisibility::Collapsed;
			});

			TSharedRef<SWidget> ExecImageWidget = SNew(SImage)
			.Image(Style::CreateSlateIcon("MetasoundEditor.Page.Executing").GetIcon())
			.ColorAndOpacity(Style::GetPageExecutingColor())
			.Visibility(MoveTemp(ExecVisibility));

			NameBox->AddSlot()
			.FillWidth(1)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2, 0)
			[
				ExecImageWidget
			];
		}

		return NameBox;
	}

	void FMetasoundDefaultLiteralCustomizationBase::CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
	{
		CustomizePageDefaultRows(InLiteral, InDetailLayout);
	}

	void FMetasoundDefaultLiteralCustomizationBase::CustomizePageDefaultRows(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
	{
		DefaultProperties.Reset();

		const UMetasoundEditorGraphMember* Member = InLiteral.FindMember();
		if (!Member)
		{
			return;
		}

		const bool bIsPagedDefault = Member->IsDefaultPaged();

		if (bIsPagedDefault)
		{
			TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> LiteralPtr(&InLiteral);
			UpdatePagePickerNames(LiteralPtr);
		}

		TSharedPtr<IPropertyHandle> DefaultPageArrayHandle = InDetailLayout.AddObjectPropertyData(TArray<UObject*>({ &InLiteral }), UMetasoundEditorGraphMemberDefaultLiteral::GetDefaultsPropertyName());
		if (DefaultPageArrayHandle.IsValid())
		{
			uint32 NumElements = 0;
			TSharedPtr<IPropertyHandleArray> DefaultValueArray = DefaultPageArrayHandle->AsArray();
			if (DefaultValueArray.IsValid())
			{
				DefaultValueArray->GetNumElements(NumElements);
				const bool bHasLocalPageValues = NumElements > 1; // 1 value is only the default
				const bool bShowPageModifiers = Editor::IsInputPageEditingEnabled(Member->GetFrontendBuilderChecked(), bHasLocalPageValues);
				if (bIsPagedDefault && bShowPageModifiers)
				{
					BuildPageDefaultComboBox(InLiteral, DefaultPageArrayHandle->GetPropertyDisplayName());
				}

				for (uint32 Index = 0; Index < NumElements; ++Index)
				{
					TSharedRef<IPropertyHandle> ElementProperty = DefaultValueArray->GetElement(Index);
					TSharedPtr<IPropertyHandle> ValueProperty = ElementProperty->GetChildHandle("Value");
					if (!ValueProperty.IsValid())
					{
						continue;
					}

					DefaultProperties.Add(ValueProperty);

					IDetailPropertyRow& ValueRow = InDetailLayout.AddPropertyToCategory(ValueProperty);
					ValueRow.CustomWidget(true /*bShowChilden */);

					// Name widget
					if (bIsPagedDefault && bShowPageModifiers)
					{
						(*ValueRow.CustomNameWidget())
						[
							BuildPageDefaultNameWidget(InLiteral, ElementProperty)
						];
					}
					else
					{
						TSharedRef<IPropertyHandle> PagedDefaultProperty = ElementProperty;
						uint32 NumChildren = 0;
						if (ElementProperty->GetNumChildren(NumChildren) == FPropertyAccess::Result::Success && NumChildren > 0)
						{
							PagedDefaultProperty = ElementProperty->GetChildHandle(0).ToSharedRef();
						}
						(*ValueRow.CustomNameWidget())
						[
							PagedDefaultProperty->CreatePropertyNameWidget()
						];
					}
					ValueRow.ShowPropertyButtons(false);

					// Value widget
					BuildDefaultValueWidget(ValueRow, ValueProperty);
					ValueRow.IsEnabled(GetEnabled());
					if (ResetOverride.IsSet())
					{
						ValueRow.OverrideResetToDefault(ResetOverride.GetValue());
					}
				}
			}
		}
	}

	void FMetasoundDefaultLiteralCustomizationBase::BuildDefaultValueWidget(IDetailPropertyRow& ValueRow, TSharedPtr<IPropertyHandle> ValueProperty)
	{
		if (!ValueProperty.IsValid())
		{
			return;
		}

		(*ValueRow.CustomValueWidget())
		[
			ValueProperty->CreatePropertyValueWidget()
		];
	}

	TArray<IDetailPropertyRow*> FMetasoundDefaultLiteralCustomizationBase::CustomizeLiteral(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
	{
		CustomizeDefaults(InLiteral, InDetailLayout);
		return { };
	}

	TAttribute<EVisibility> FMetasoundDefaultLiteralCustomizationBase::GetDefaultVisibility() const
	{
		return Visibility;
	}

	TAttribute<bool> FMetasoundDefaultLiteralCustomizationBase::GetEnabled() const
	{
		return Enabled;
	}

	void FMetasoundDefaultLiteralCustomizationBase::SetDefaultVisibility(TAttribute<EVisibility> InVisibility)
	{
		Visibility = InVisibility;
	}

	void FMetasoundDefaultLiteralCustomizationBase::SetEnabled(TAttribute<bool> InEnabled)
	{
		Enabled = InEnabled;
	}

	void FMetasoundDefaultLiteralCustomizationBase::SetResetOverride(const TOptional<FResetToDefaultOverride>& InResetOverride)
	{
		ResetOverride = InResetOverride;
	}

	void FMetasoundDefaultLiteralCustomizationBase::UpdatePagePickerNames(TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> LiteralPtr)
	{
		using namespace Frontend;

		AddablePageStringNames.Reset();
		ImplementedPageNames.Reset();

		if (!LiteralPtr.IsValid())
		{
			return;
		}

		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		check(Settings);

		TSet<FGuid> ImplementedGuids;
		LiteralPtr->IterateDefaults([&ImplementedGuids](const FGuid& PageID, FMetasoundFrontendLiteral)
		{
			ImplementedGuids.Add(PageID);
		});

		Settings->IteratePageSettings([this, &ImplementedGuids](const FMetaSoundPageSettings& PageSettings)
		{
			if (!ImplementedGuids.Contains(PageSettings.UniqueId))
			{
				AddablePageStringNames.Add(MakeShared<FString>(PageSettings.Name.ToString()));
			}
		});

		auto GetPageName = [&Settings](const FGuid& PageID)
		{
			const FMetaSoundPageSettings* Page = Settings->FindPageSettings(PageID);
			if (Page)
			{
				return Page->Name;
			}

			return Metasound::Editor::GetMissingPageName(PageID);
		};

		Algo::Transform(ImplementedGuids, ImplementedPageNames, GetPageName);
	}
} // namespace Metasound::Editor
#undef LOCTEXT_NAMESPACE
