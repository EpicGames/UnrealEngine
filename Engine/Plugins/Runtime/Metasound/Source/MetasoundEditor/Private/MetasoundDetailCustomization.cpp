// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundDetailCustomization.h"

#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "IAudioParameterTransmitter.h"
#include "IDetailGroup.h"
#include "Input/Events.h"
#include "InstancedStructDetails.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundAssetBase.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundUObjectRegistry.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "SGraphPalette.h"
#include "ScopedTransaction.h"
#include "Interfaces/IPluginManager.h"
#include "Sound/SoundWave.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound::Editor
{
	const FName GetMissingPageName(const FGuid& InPageID)
	{
		return FName(*FString::Printf(TEXT("Invalid (...%s)"), *InPageID.ToString().Mid(28, 8)));
	}

	FName BuildChildPath(const FString& InBasePath, FName InPropertyName)
	{
		return FName(InBasePath + TEXT(".") + InPropertyName.ToString());
	}

	FName BuildChildPath(const FName& InBasePath, FName InPropertyName)
	{
		return FName(InBasePath.ToString() + TEXT(".") + InPropertyName.ToString());
	}

	UObject* FMetaSoundDetailCustomizationBase::GetMetaSound() const
	{
		if (Builder.IsValid())
		{
			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
			if (DocBuilder.IsValid())
			{
				return &DocBuilder.CastDocumentObjectChecked<UObject>();
			}
		}

		return nullptr;
	}

	void FMetaSoundDetailCustomizationBase::InitBuilder(UObject& MetaSound)
	{
		using namespace Engine;
		Builder.Reset(&FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound));
	}

	bool FMetaSoundDetailCustomizationBase::IsGraphEditable() const
	{
		return IsEditable();
	}

	bool FMetaSoundDetailCustomizationBase::IsEditable() const
	{
		using namespace Engine;

		if (Builder.IsValid())
		{
			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
			if (DocBuilder.IsValid())
			{
				return DocBuilder.FindConstBuildGraphChecked().Style.bIsGraphEditable;
			}
		}

		return false;
	}

	FMetasoundDetailCustomization::FMetasoundDetailCustomization(FName InDocumentPropertyName)
		: DocumentPropertyName(InDocumentPropertyName)
	{
	}

	FName FMetasoundDetailCustomization::GetTemplatePropertyPath() const
	{
		return BuildChildPath(DocumentPropertyName, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendDocument, Template));
	}

	FName FMetasoundDetailCustomization::GetInterfaceVersionsPropertyPath() const
	{
		return BuildChildPath(DocumentPropertyName, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendDocument, Interfaces));
	}

	FName FMetasoundDetailCustomization::GetRootClassPropertyPath() const
	{
		return BuildChildPath(DocumentPropertyName, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendDocument, RootGraph));
	}

	FName FMetasoundDetailCustomization::GetMetadataPropertyPath() const
	{
		const FName RootClass = FName(GetRootClassPropertyPath());
		return BuildChildPath(RootClass, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClass, Metadata));
	}
	
	static bool IsExperimentalPluginEnabled()
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindEnabledPlugin(TEXT("MetasoundExperimental"));
		return Plugin.IsValid();
	}

	void FMetasoundDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
	{
		using namespace Frontend;

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);

		// Only support modifying a single MetaSound at a time (Multiple
		// MetaSound editing will be covered most likely by separate tool).
		if (Objects.Num() != 1 || !Objects.Last().IsValid())
		{
			return;
		}

		UObject& MetaSound = *Objects.Last().Get();
		InitBuilder(MetaSound);
		TWeakObjectPtr<UMetaSoundSource> MetaSoundSource = Cast<UMetaSoundSource>(&MetaSound);

		// MetaSound patches don't have source settings, so view MetaSound settings by default 
		EMetasoundActiveDetailView DetailsView = EMetasoundActiveDetailView::Metasound;
		if (MetaSoundSource.IsValid())
		{
			// Show source settings by default unless previously set
			DetailsView = EMetasoundActiveDetailView::General;
			if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
			{
				DetailsView = EditorSettings->DetailView;
			}
		}

		switch (DetailsView)
		{
			case EMetasoundActiveDetailView::Metasound:
			{
				IDetailCategoryBuilder& GeneralCategoryBuilder = DetailLayout.EditCategory("MetaSound");
				const FName AccessFlagsPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetAccessFlagsPropertyName());
				const FName AuthorPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetAuthorPropertyName());
				const FName CategoryHierarchyPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetCategoryHierarchyPropertyName());
				const FName ClassNamePropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetClassNamePropertyName());
				const FName DescPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetDescriptionPropertyName());
				const FName DisplayNamePropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetDisplayNamePropertyName());
				const FName KeywordsPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetKeywordsPropertyName());
				const FName VersionPropertyPath = BuildChildPath(GetMetadataPropertyPath(), FMetasoundFrontendClassMetadata::GetVersionPropertyName());

				const FName ClassNameNamePropertyPath = BuildChildPath(ClassNamePropertyPath, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassName, Name));

				const FName MajorVersionPropertyPath = BuildChildPath(VersionPropertyPath, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendVersionNumber, Major));
				const FName MinorVersionPropertyPath = BuildChildPath(VersionPropertyPath, GET_MEMBER_NAME_CHECKED(FMetasoundFrontendVersionNumber, Minor));

				TSharedPtr<IPropertyHandle> AccessFlagsHandle = DetailLayout.GetProperty(AccessFlagsPropertyPath);
				TSharedPtr<IPropertyHandle> AuthorHandle = DetailLayout.GetProperty(AuthorPropertyPath);
				TSharedPtr<IPropertyHandle> CategoryHierarchyHandle = DetailLayout.GetProperty(CategoryHierarchyPropertyPath);
				TSharedPtr<IPropertyHandle> ClassNameHandle = DetailLayout.GetProperty(ClassNameNamePropertyPath);
				TSharedPtr<IPropertyHandle> DisplayNameHandle = DetailLayout.GetProperty(DisplayNamePropertyPath);
				TSharedPtr<IPropertyHandle> DescHandle = DetailLayout.GetProperty(DescPropertyPath);
				TSharedPtr<IPropertyHandle> KeywordsHandle = DetailLayout.GetProperty(KeywordsPropertyPath);
				TSharedPtr<IPropertyHandle> InterfaceVersionsHandle = DetailLayout.GetProperty(GetInterfaceVersionsPropertyPath());
				TSharedPtr<IPropertyHandle> MajorVersionHandle = DetailLayout.GetProperty(MajorVersionPropertyPath);
				TSharedPtr<IPropertyHandle> MinorVersionHandle = DetailLayout.GetProperty(MinorVersionPropertyPath);

				{
					TSharedPtr<IPropertyHandle> TemplateHandle = DetailLayout.GetProperty(GetTemplatePropertyPath());
					if (TemplateHandle.IsValid())
					{
						FSimpleDelegate OnConfigChanged = FSimpleDelegate::CreateSPLambda(AsShared(), [this]()
						{
							if (Builder.IsValid())
							{
								FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
								if (DocBuilder.IsValid())
								{
									DocBuilder.ConfigureDocument();
									DocBuilder.GetDocumentDelegates().OnDocumentTemplateChanged.Broadcast({ EDocumentTemplateChangeType::Struct });
								}
							}
						});
						TemplateHandle->SetOnPropertyValueChanged(OnConfigChanged);

						const auto OnConfigPropChanged = TDelegate<void(const FPropertyChangedEvent&)>::CreateSPLambda(AsShared(), [this](const FPropertyChangedEvent& InEvent)
						{
							if (Builder.IsValid())
							{
								FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
								if (DocBuilder.IsValid())
								{
									if (FMetaSoundTemplate* Template = DocBuilder.GetDocumentTemplate().GetPtr())
									{
										Template->OnPropertyChanged(InEvent, DocBuilder);
										DocBuilder.ConfigureDocument();
										DocBuilder.GetDocumentDelegates().OnDocumentTemplateChanged.Broadcast({ EDocumentTemplateChangeType::Property });
									}
								}
							}
						});
						TemplateHandle->SetOnChildPropertyValueChangedWithData(OnConfigPropChanged);
						GeneralCategoryBuilder.AddProperty(TemplateHandle)
							.Visibility(TAttribute<EVisibility>::CreateSPLambda(this, [this]()
							{
								if (Builder.IsValid() && !FEditor::IgnoreDocTemplateVisibilitySettings())
								{
									FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
									if (DocBuilder.IsValid())
									{
										if (const FMetaSoundTemplate* Template = DocBuilder.GetConstDocumentTemplate().GetPtr())
										{
											return Template->GetEditorOptions().bTemplatePropertiesVisible
												? EVisibility::Visible
												: EVisibility::Collapsed;
										}
									}
								}

								return EVisibility::Visible;
							}
						));
					}
				}
				

				// Invalid for UMetaSounds
				TSharedPtr<IPropertyHandle> OutputFormat = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMetaSoundSource, OutputFormat));
				if (OutputFormat.IsValid() && OutputFormat->IsValidHandle())
				{
					const UEnum* OutputFormatEnum = StaticEnum<EMetaSoundOutputAudioFormat>();
					check(OutputFormatEnum);

					const bool bAllowExperimental = IsExperimentalPluginEnabled();

					// Build list of selectable values
					OutputFormatOptions.Reset();
					OutputFormatOptions.Reserve(OutputFormatEnum->NumEnums());

					for (int32 Index = 0; Index < OutputFormatEnum->NumEnums(); ++Index)
					{
						const bool bIsHidden = OutputFormatEnum->HasMetaData(TEXT("Hidden"), Index);
						const FName Name = OutputFormatEnum->GetNameByIndex(Index);
						// UEnum::GetNameByIndex returns the C++ identifier (e.g. "COUNT", "MAX") — use case-insensitive
					// comparison since UMETA names don't always match the case of the suffix check.
					const FString NameStr = Name.ToString();
					const bool bIsCount = NameStr.EndsWith(TEXT("Count"), ESearchCase::IgnoreCase) || NameStr.EndsWith(TEXT("MAX"), ESearchCase::IgnoreCase);
						
						if (bIsCount)
						{
							continue;
						}

						if (bIsHidden && !bAllowExperimental)
						{
							continue;
						}

						const int64 Value = OutputFormatEnum->GetValueByIndex(Index);
						OutputFormatOptions.Add(MakeShared<int64>(Value));
					}

					auto GetCurrentValue = [OutputFormat]() -> int64
					{
						uint8 Raw = 0;
						if (OutputFormat->GetValue(Raw) == FPropertyAccess::Success)
						{
							return (int64)Raw;
						}
						return 0;
					};

					TSharedPtr<int64> InitiallySelected;
					{
						const int64 Current = GetCurrentValue();
						for (const TSharedPtr<int64>& Opt : OutputFormatOptions)
						{
							if (Opt.IsValid() && *Opt == Current)
							{
								InitiallySelected = Opt;
								break;
							}
						}
						if (!InitiallySelected.IsValid() && OutputFormatOptions.Num() > 0)
						{
							InitiallySelected = OutputFormatOptions[0];
						}
					}

					TSharedRef<SWidget> OutputFormatValueWidget =
						SNew(SComboBox<TSharedPtr<int64>>)
						.OptionsSource(&OutputFormatOptions)
						.InitiallySelectedItem(InitiallySelected)
						.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this,
							&FMetaSoundDetailCustomizationBase::IsFormatEditable)))
						.OnGenerateWidget_Lambda([OutputFormatEnum](TSharedPtr<int64> InItem)
							{
								if (!InItem.IsValid())
								{
									return SNew(STextBlock).Text(FText::GetEmpty());
								}

								const int32 EnumIndex = OutputFormatEnum->GetIndexByValue(*InItem);
								const FText Display = (EnumIndex != INDEX_NONE)
									? OutputFormatEnum->GetDisplayNameTextByIndex(EnumIndex)
									: FText::FromString(TEXT("<Invalid>"));

								return SNew(STextBlock).Text(Display);
							})
						.OnSelectionChanged_Lambda([OutputFormat](TSharedPtr<int64> NewValue, ESelectInfo::Type)
							{
								if (NewValue.IsValid())
								{
									OutputFormat->SetValue((uint8)*NewValue);
								}
							})
						[
							SNew(STextBlock)
							.Text_Lambda([OutputFormatEnum, OutputFormat]()
								{
									uint8 Raw = 0;
									if (OutputFormat->GetValue(Raw) != FPropertyAccess::Success)
									{
										return FText::GetEmpty();
									}

									const int32 EnumIndex = OutputFormatEnum->GetIndexByValue((int64)Raw);
									return (EnumIndex != INDEX_NONE)
										? OutputFormatEnum->GetDisplayNameTextByIndex(EnumIndex)
										: FText::FromString(TEXT("<Invalid>"));
								})
						];
					if (MetaSoundSource.IsValid())
					{
						OutputFormat->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->Stop();
								};
							}
						}));

						OutputFormat->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->CreateAnalyzers(*Source.Get());
								};
							}
						}));
					}

					OutputFormatValueWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetaSoundDetailCustomizationBase::IsEditable)));
					static const FText OutputFormatName = LOCTEXT("MetasoundOutputFormatPropertyName", "Output Format");
					GeneralCategoryBuilder.AddCustomRow(OutputFormatName)
					.NameContent()
					[
						OutputFormat->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						OutputFormatValueWidget
					]
					.RowTag("OutputFormat");
					
					OutputFormat->MarkHiddenByCustomization();
				}

				TSharedPtr<IPropertyHandle> OutputFormatChooser = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMetaSoundSource, ChannelAgnosticFormatChooser));
				if (OutputFormatChooser.IsValid() && OutputFormatChooser->IsValidHandle() && IsExperimentalPluginEnabled())
				{
					if (MetaSoundSource.IsValid())
					{
						OutputFormatChooser->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->Stop();
								};
							}
						}));

						OutputFormatChooser->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->CreateAnalyzers(*Source.Get());
								};
							}
						}));
					}

					TSharedRef<SWidget> OutputFormatValueWidget = OutputFormatChooser->CreatePropertyValueWidget();
					OutputFormatValueWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetaSoundDetailCustomizationBase::IsFormatEditable)));

					static const FText OutputFormatName = LOCTEXT("MetasoundOutputFormatChooserPropertyName", "Output Format Chooser");
					GeneralCategoryBuilder.AddCustomRow(OutputFormatName)
					.NameContent()
					[
						OutputFormatChooser->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						OutputFormatValueWidget
					]
					.RowTag("OutputFormatChooser");
					
					OutputFormatChooser->MarkHiddenByCustomization();
				}

				// Custom format name — visible only when ChannelAgnostic + Custom chooser.
				TSharedPtr<IPropertyHandle> CustomFormatName = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMetaSoundSource, ChannelAgnosticCustomFormat));
				if (CustomFormatName.IsValid() && CustomFormatName->IsValidHandle() && IsExperimentalPluginEnabled())
				{
					// Stop playback before format change and recreate analyzers after,
					// matching the pattern used by OutputFormat and OutputFormatChooser.
					if (MetaSoundSource.IsValid())
					{
						CustomFormatName->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->Stop();
								};
							}
						}));

						CustomFormatName->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([Source = MetaSoundSource]()
						{
							if (Source.IsValid())
							{
								TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(*Source.Get());
								if (ParentEditor.IsValid())
								{
									ParentEditor->CreateAnalyzers(*Source.Get());
								};
							}
						}));
					}

					TSharedRef<SWidget> CustomFormatWidget = CustomFormatName->CreatePropertyValueWidget();
					CustomFormatWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetaSoundDetailCustomizationBase::IsEditable)));

					static const FText CustomFormatLabel = LOCTEXT("MetasoundCustomFormatPropertyName", "Custom Format Name");
					GeneralCategoryBuilder.AddCustomRow(CustomFormatLabel)
					.NameContent()
					[
						CustomFormatName->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						CustomFormatWidget
					]
					.Visibility(TAttribute<EVisibility>::CreateLambda([MetaSoundSource]()
					{
						if (MetaSoundSource.IsValid()
							&& MetaSoundSource->OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic
							&& MetaSoundSource->ChannelAgnosticFormatChooser == EMetasoundChannelAgnosticSourceFormatChooser::Custom)
						{
							return EVisibility::Visible;
						}
						return EVisibility::Collapsed;
					}))
					.RowTag("CustomFormatName");

					CustomFormatName->MarkHiddenByCustomization();
				}

				// Updates FText properties on open editors if required
				{
					FSimpleDelegate RegisterOnChange = FSimpleDelegate::CreateLambda([this]()
					{
						if (Builder.IsValid())
						{
							FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
							if (DocBuilder.IsValid())
							{
								DocBuilder.GetConstDocumentChecked().RootGraph.Style.UpdateChangeID();
								const FDocumentModifyDelegates& DocumentDelegates = DocBuilder.GetDocumentDelegates();
								DocumentDelegates.OnDocumentMetadataChanged.Broadcast();
							}

							{
								FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
								RegOptions.bForceViewSynchronization = true;
								FGraphBuilder::RegisterGraphWithFrontend(DocBuilder.CastDocumentObjectChecked<UObject>(), MoveTemp(RegOptions));
							}
						}
					});

					AccessFlagsHandle->SetOnPropertyValueChanged(RegisterOnChange);
					AuthorHandle->SetOnPropertyValueChanged(RegisterOnChange);
					DescHandle->SetOnPropertyValueChanged(RegisterOnChange);
					DisplayNameHandle->SetOnPropertyValueChanged(RegisterOnChange);
					KeywordsHandle->SetOnPropertyValueChanged(RegisterOnChange);
					KeywordsHandle->SetOnChildPropertyValueChanged(RegisterOnChange);
					CategoryHierarchyHandle->SetOnPropertyValueChanged(RegisterOnChange);
					CategoryHierarchyHandle->SetOnChildPropertyValueChanged(RegisterOnChange);
				}

				GeneralCategoryBuilder.AddProperty(DisplayNameHandle);
				GeneralCategoryBuilder.AddProperty(DescHandle);
				GeneralCategoryBuilder.AddProperty(AuthorHandle);
				GeneralCategoryBuilder.AddProperty(AccessFlagsHandle);
				GeneralCategoryBuilder.AddProperty(MajorVersionHandle);
				GeneralCategoryBuilder.AddProperty(MinorVersionHandle);

				static const FText ClassGuidName = LOCTEXT("MetasoundClassGuidPropertyName", "Class Guid");
				GeneralCategoryBuilder.AddCustomRow(ClassGuidName).NameContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ClassGuidName)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]
				]
				.ValueContent()
				[
					ClassNameHandle->CreatePropertyValueWidget()
				];
				GeneralCategoryBuilder.AddProperty(CategoryHierarchyHandle);
				GeneralCategoryBuilder.AddProperty(KeywordsHandle);

				DetailLayout.HideCategory("Sound");
				DetailLayout.HideCategory("Voice Management");
				DetailLayout.HideCategory("Routing");
				DetailLayout.HideCategory("Modulation");
				DetailLayout.HideCategory("Memory");
				DetailLayout.HideCategory("Developer");
				DetailLayout.HideCategory("Advanced");
			}
			break;

			case EMetasoundActiveDetailView::General:
			default:
				DetailLayout.HideCategory("MetaSound");

				TArray<TSharedRef<IPropertyHandle>>DeveloperProperties;
				TArray<TSharedRef<IPropertyHandle>>SoundProperties;

				DetailLayout.EditCategory("Sound")
					.GetDefaultProperties(SoundProperties);
				DetailLayout.EditCategory("Developer")
					.GetDefaultProperties(DeveloperProperties);

				auto HideProperties = [](const TSet<FName>& PropsToHide, const TArray<TSharedRef<IPropertyHandle>>& Properties)
				{
					for (TSharedRef<IPropertyHandle> Property : Properties)
					{
						if (PropsToHide.Contains(Property->GetProperty()->GetFName()))
						{
							Property->MarkHiddenByCustomization();
						}
					}
				};

				static const TSet<FName> SoundPropsToHide =
				{
					GET_MEMBER_NAME_CHECKED(USoundWave, bLooping),
					GET_MEMBER_NAME_CHECKED(USoundWave, SoundGroup)
				};
				HideProperties(SoundPropsToHide, SoundProperties);

				static const TSet<FName> DeveloperPropsToHide =
				{
					GET_MEMBER_NAME_CHECKED(USoundBase, Duration),
					GET_MEMBER_NAME_CHECKED(USoundBase, MaxDistance),
					GET_MEMBER_NAME_CHECKED(USoundBase, TotalSamples)
				};
				HideProperties(DeveloperPropsToHide, DeveloperProperties);

				break;
		}

		// Hack to hide parent structs for nested metadata properties
		DetailLayout.HideCategory("CustomView");
		
		DetailLayout.HideCategory("Curves");
		DetailLayout.HideCategory("File Path");
		DetailLayout.HideCategory("Info");
		DetailLayout.HideCategory("Waveform Processing");
	}

	FMetasoundPagesDetailCustomization::FMetasoundPagesDetailCustomization()
	{
	}

	bool FMetasoundPagesDetailCustomization::IsEditable() const
	{
		using namespace Metasound::Frontend;

		if (Builder.IsValid())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
			if (DocBuilder.IsValid())
			{
				if (const FMetaSoundTemplate* Template = DocBuilder.GetDocumentTemplate().GetPtr())
				{
					return Template->GetEditorOptions().bPageGraphEditingEnabled;
				}
			}
		}

		return FMetaSoundDetailCustomizationBase::IsEditable();
	}

	void FMetasoundPagesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
	{
		using namespace Engine;
		using namespace Frontend;

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);

		// Only support modifying a single MetaSound at a time (Multiple
		// MetaSound editing will be covered most likely by separate tool).
		if (Objects.Num() != 1 || !Objects.Last().IsValid())
		{
			return;
		}

		if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
		{
			Settings->GetOnPageSettingsUpdatedDelegate().AddSPLambda(this, [this]()
			{
				UpdateItemNames();
				if (ComboBox.IsValid())
				{
					ComboBox->RefreshOptions();
				}
			});
		}

		SAssignNew(ComboBox, SSearchableComboBox)
			.OptionsSource(&AddableItems)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock).Text(FText::FromString(*InItem));
			})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NameToAdd, ESelectInfo::Type InSelectInfo)
			{
				using namespace Engine;
				using namespace Frontend;

				if (InSelectInfo != ESelectInfo::OnNavigation && IsBuilderValid())
				{
					UObject& MetaSound = GetMetaSoundChecked();

					const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddPageTransactionFormat", "Add MetaSound Page Graph '{0}'"), FText::FromString(*NameToAdd)));
					MetaSound.Modify();

					// Underlying DocBuilder's pageID is a property that is tracked by transaction stack, so signal as modifying behavior
					Builder->Modify();

					constexpr bool bDuplicateLastGraph = true;
					constexpr bool bSetAsBuildGraph = true;

					EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
					Builder->AddGraphPage(FName(*NameToAdd.Get()), bDuplicateLastGraph, bSetAsBuildGraph, Result);

					if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
					{
						IAssetEditorInstance* AssetEditor = AssetSubsystem->FindEditorForAsset(&MetaSound, /*bFocusIfOpen=*/false);
						if (FEditor* MetaSoundEditor = static_cast<FEditor*>(AssetEditor))
						{
							MetaSoundEditor->RefreshDetails();
						}
					}
				}
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddPageGraphAction", "Add Page Graph..."))
				.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetasoundPagesDetailCustomization::IsEditable)))
			];

		TSharedRef<SWidget> Utilities = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				ComboBox->AsShared()
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateLambda([this]()
				{
					using namespace Frontend;
					UObject& MetaSound = GetMetaSoundChecked();

					const FScopedTransaction Transaction(LOCTEXT("ResetGraphPagesTransaction", "Reset MetaSound Graph Pages"));
					MetaSound.Modify();

					constexpr bool bClearDefaultGraph = false;

					// Underlying DocBuilder's pageID is a property that is tracked by transaction stack, so signal as modifying behavior
					Builder->Modify();
					Builder->ResetGraphPages(bClearDefaultGraph);

					UpdateItemNames();
					ComboBox->RefreshOptions();
					FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
				}), LOCTEXT("ResetGraphPagesTooltip", "Removes all page graphs from the given MetaSound defined in the MetaSound project settings (does not remove the required 'Default' graph)."))
			];

		Utilities->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetaSoundDetailCustomizationBase::IsEditable)));

		{
			const FText HeaderName = LOCTEXT("PageGraphsDisplayName", "Graphs");
			IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Graphs", HeaderName);
			Category.AddCustomRow(HeaderName)[Utilities];
			Category.AddCustomRow(LOCTEXT("ImplementedPagesLabel", "Graphs")) [ SAssignNew(EntryWidgets, SVerticalBox) ];
		}

		// Registration of page listener instance calls OnReload which in turn causes RefreshView, so no need to call directly
		if (UMetasoundEditorViewBase* View = CastChecked<UMetasoundEditorViewBase>(Objects.Last()))
		{
			if (UObject* MetaSound = View->GetMetasound())
			{
				InitBuilder(*MetaSound);
				PageListener = MakeShared<FPageListener>(StaticCastSharedRef<FMetasoundPagesDetailCustomization>(AsShared()));
				Builder->AddTransactionListener(PageListener->AsShared());
			}
		}
	}

	const FMetasoundFrontendDocument& FMetasoundPagesDetailCustomization::GetConstDocumentChecked() const
	{
		check(IsBuilderValid());
		return Builder->GetBuilder().GetConstDocumentChecked();
	}

	UObject& FMetasoundPagesDetailCustomization::GetMetaSoundChecked() const
	{
		check(IsBuilderValid());
		return Builder->GetBuilder().CastDocumentObjectChecked<UObject>();
	}

	bool FMetasoundPagesDetailCustomization::IsBuilderValid() const
	{
		return Builder.IsValid() && Builder->GetBuilder().IsValid();
	}

	void FMetasoundPagesDetailCustomization::RebuildImplemented()
	{
		EntryWidgets->ClearChildren();

		if (!IsBuilderValid())
		{
			return;
		}

		TSet<FGuid> ImplementedGuids;
		GetConstDocumentChecked().RootGraph.IterateGraphPages([&ImplementedGuids](const FMetasoundFrontendGraph& Graph)
		{
			ImplementedGuids.Add(Graph.PageID);
		});

		auto CreateEntryWidget = [this](bool bIsDefault, const FGuid& InEntryPageID, FName InName) -> TSharedRef<SWidget>
		{
			using namespace Frontend;

			TSharedRef<SHorizontalBox> EntryWidget = SNew(SHorizontalBox);

			// Page Focus
			{
				TSharedRef<SWidget> SelectButtonWidget = PropertyCustomizationHelpers::MakeUseSelectedButton(
					FSimpleDelegate::CreateLambda([this, PageID = InEntryPageID, InName]()
					{
						constexpr bool bOpenEditor = false; // Already focused by user action
						UMetaSoundEditorSubsystem::GetConstChecked().SetFocusedPage(*Builder.Get(), PageID, bOpenEditor);
						IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(&GetMetaSoundChecked(), /*bFocusIfOpen=*/false);
						FEditor* MetaSoundEditor = static_cast<FEditor*>(AssetEditor);
						if (MetaSoundEditor)
						{
							MetaSoundEditor->RefreshDetails();
						}
						BuildPageName = InName;
					}),
					TAttribute<FText>::Create([this, InName]()
					{
						return BuildPageName == InName
							? LOCTEXT("FocusedPageTooltip", "Currently focused page.")
							: LOCTEXT("SetFocusedPageTooltip", "Sets the actively focused graph page of the MetaSound.");
					}),
					TAttribute<bool>::Create([this, InName]()
					{
						return BuildPageName != InName;
					}));

				EntryWidget->AddSlot()
					.Padding(2.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.AutoWidth()[ SelectButtonWidget ];
			}

			// Page Name
			{
				EntryWidget->AddSlot()
				.Padding(2.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromName(InName))
				];
			}

			// Page Remove
			if (!bIsDefault)
			{
				TSharedRef<SWidget> RemoveButtonWidget = PropertyCustomizationHelpers::MakeDeleteButton(
					FSimpleDelegate::CreateLambda([this, InName, InEntryPageID]()
					{
						using namespace Frontend;
						const FScopedTransaction Transaction(FText::Format(LOCTEXT("RemovePageTransactionFormat", "Remove MetaSound Page '{0}'"), FText::FromName(InName)));
						UObject& MetaSound = GetMetaSoundChecked();
						MetaSound.Modify();

						// Removal may modify the builder's build page ID if it is the currently set value
						Builder->Modify();

						const bool bGraphRemoved = Builder->GetBuilder().RemoveGraphPage(InEntryPageID);
						if (bGraphRemoved)
						{
							UpdateItemNames();
							ComboBox->RefreshOptions();
							RebuildImplemented();
						}
					}), LOCTEXT("RemovePageGraphTooltip", "Removes the associated page graph from the MetaSound."));
				EntryWidget->AddSlot()
					.Padding(2.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.AutoWidth()[ RemoveButtonWidget ];
			}

			// Page Playing Icon
			{
				TAttribute<FText> ToolTip = LOCTEXT("MetaSound_ExecutingPageGraphTooltip", "Currently executing graph.");
				TAttribute<EVisibility> Visibility = TAttribute<EVisibility>::CreateSPLambda(AsShared(), [this, PageID = InEntryPageID]()
				{
					if (Builder.IsValid())
					{
						const bool bIsPreviewing = IsPreviewingPageGraph(Builder->GetConstBuilder(), PageID);
						return bIsPreviewing ? EVisibility::Visible : EVisibility::Collapsed;
					}

					return EVisibility::Collapsed;
				});
				TSharedRef<SWidget> ExecImageWidget = SNew(SImage)
					.Image(Style::CreateSlateIcon("MetasoundEditor.Page.Executing").GetIcon())
					.ColorAndOpacity(Style::GetPageExecutingColor())
					.Visibility(MoveTemp(Visibility));

				EntryWidget->AddSlot()
					.Padding(2.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.AutoWidth()[ ExecImageWidget ];
			}

			EntryWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMetasoundPagesDetailCustomization::IsEditable)));
			return EntryWidget;
		};

		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		check(Settings);
		Settings->IteratePageSettings([&](const FMetaSoundPageSettings& PageSettings)
		{
			if (ImplementedGuids.Remove(PageSettings.UniqueId) > 0)
			{
				const bool bIsDefault = PageSettings.UniqueId == Frontend::DefaultPageID;
				EntryWidgets->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					CreateEntryWidget(bIsDefault, PageSettings.UniqueId, PageSettings.Name)
				];
			}
		});

		for (const FGuid& MissingPageID : ImplementedGuids)
		{
			constexpr bool bIsDefault = false;
			const FName MissingName = Editor::GetMissingPageName(MissingPageID);
			EntryWidgets->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					CreateEntryWidget(bIsDefault, MissingPageID, MissingName)
				];
		}
	}

	void FMetasoundPagesDetailCustomization::RefreshView()
	{
		if (Builder.IsValid())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
			const FGuid& PageID = DocBuilder.GetBuildPageID();

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);
			if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageID))
			{
				BuildPageName = PageSettings->Name;
			}
			else
			{
				constexpr bool bOpenEditor = false; // Already open/focused by user action
				UMetaSoundEditorSubsystem::GetConstChecked().SetFocusedPage(*Builder.Get(), PageID, bOpenEditor);
				BuildPageName = GetMissingPageName(PageID);
			}
		}
		else
		{
			BuildPageName = Frontend::DefaultPageName;
		}

		UpdateItemNames();
		ComboBox->RefreshOptions();
		RebuildImplemented();
	}

	void FMetasoundPagesDetailCustomization::UpdateItemNames()
	{
		AddableItems.Reset();
		ImplementedNames.Reset();

		if (!IsBuilderValid())
		{
			return;
		}

		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		check(Settings);

		TSet<FGuid> ImplementedGuids;
		const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
		Document.RootGraph.IterateGraphPages([&ImplementedGuids](const FMetasoundFrontendGraph& Graph)
		{
			ImplementedGuids.Add(Graph.PageID);
		});

		Settings->IteratePageSettings([this, &ImplementedGuids](const FMetaSoundPageSettings& Page)
		{
			if (!ImplementedGuids.Contains(Page.UniqueId))
			{
				AddableItems.Add(MakeShared<FString>(Page.Name.ToString()));
			}
		});

		auto GetPageName = [&Settings](const FGuid& PageID)
		{
			const FMetaSoundPageSettings* Page = Settings->FindPageSettings(PageID);
			if (Page)
			{
				return Page->Name;
			}

			return GetMissingPageName(PageID);
		};

		Algo::Transform(ImplementedGuids, ImplementedNames, GetPageName);
	}

	void FMetasoundPagesDetailCustomization::FPageListener::OnBuilderReloaded(Frontend::FDocumentModifyDelegates& OutDelegates)
	{
		if (TSharedPtr<FMetasoundPagesDetailCustomization> ParentPtr = Parent.Pin())
		{
			ParentPtr->RefreshView();
		}

		OutDelegates.PageDelegates.OnPageAdded.AddSP(this, &FPageListener::OnPageAdded);
		OutDelegates.PageDelegates.OnPageSet.AddSP(this, &FPageListener::OnPageSet);
		OutDelegates.PageDelegates.OnRemovingPage.AddSP(this, &FPageListener::OnRemovingPage);
	}

	void FMetasoundPagesDetailCustomization::FPageListener::OnPageAdded(const Frontend::FDocumentMutatePageArgs& Args)
	{
		if (TSharedPtr<FMetasoundPagesDetailCustomization> ParentPtr = Parent.Pin())
		{
			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);
			if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(Args.PageID))
			{
				if (PageSettings->Name != ParentPtr->BuildPageName)
				{
					ParentPtr->BuildPageName = PageSettings->Name;
					FGraphBuilder::RegisterGraphWithFrontend(ParentPtr->GetMetaSoundChecked());
				}

				ParentPtr->AddableItems.RemoveAll([&PageSettings](const TSharedPtr<FString>& Item) { return Item->Compare(PageSettings->Name.ToString()) == 0; });
				ParentPtr->ImplementedNames.Add(PageSettings->Name);
				ParentPtr->ComboBox->RefreshOptions();
				ParentPtr->RebuildImplemented();
			}
		}
	}

	void FMetasoundPagesDetailCustomization::FPageListener::OnPageSet(const Frontend::FDocumentMutatePageArgs& Args)
	{
		if (TSharedPtr<FMetasoundPagesDetailCustomization> ParentPtr = Parent.Pin())
		{
			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);
			if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(Args.PageID))
			{
				ParentPtr->BuildPageName = PageSettings->Name;
				ParentPtr->ComboBox->RefreshOptions();
				ParentPtr->RebuildImplemented();
			}
		}
	}

	void FMetasoundPagesDetailCustomization::FPageListener::OnRemovingPage(const Frontend::FDocumentMutatePageArgs& Args)
	{
		if (TSharedPtr<FMetasoundPagesDetailCustomization> ParentPtr = Parent.Pin())
		{
			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);
			if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(Args.PageID))
			{
				if (PageSettings->Name != ParentPtr->BuildPageName)
				{
					ParentPtr->BuildPageName = PageSettings->Name;
					FGraphBuilder::RegisterGraphWithFrontend(ParentPtr->GetMetaSoundChecked());
				}

				ParentPtr->AddableItems.Add(MakeShared<FString>(PageSettings->Name.ToString()));
				ParentPtr->ImplementedNames.Remove(PageSettings->Name);
				ParentPtr->ComboBox->RefreshOptions();
				ParentPtr->RebuildImplemented();
			}
		}
	}

	bool FMetasoundInterfacesDetailCustomization::IsEditable() const
	{
		using namespace Metasound::Frontend;

		if (Builder.IsValid())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
			if (DocBuilder.IsValid())
			{
				if (const FMetaSoundTemplate* Template = DocBuilder.GetDocumentTemplate().GetPtr())
				{
					return Template->GetEditorOptions().bInterfaceEditingEnabled;
				}
			}
		}

		return FMetaSoundDetailCustomizationBase::IsEditable();
	}

	void FMetasoundInterfacesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);

		// Only support modifying a single MetaSound at a time (Multiple
		// MetaSound editing will be covered most likely by separate tool).
		if (Objects.Num() != 1 || !Objects.Last().IsValid())
		{
			return;
		}

		if (UMetasoundInterfacesView* InterfacesView = CastChecked<UMetasoundInterfacesView>(Objects.Last()))
		{
			if (UObject* MetaSound = InterfacesView->GetMetasound())
			{
				InitBuilder(*MetaSound);
			}
		}

		TAttribute<bool> IsEditableAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP
		(
			this, &FMetasoundInterfacesDetailCustomization::IsEditable
		));

		UpdateInterfaceNames();

		const UMetasoundEditorSettings* MetasoundEditorSettings = GetDefault<UMetasoundEditorSettings>();
		SAssignNew(InterfaceComboBox, SSearchableComboBox)
			.OptionsSource(&AddableInterfaceNames)
			.OnGenerateWidget_Lambda([&InputColor = MetasoundEditorSettings->InputNodeTitleColor, &OutputColor = MetasoundEditorSettings->OutputNodeTitleColor](TSharedPtr<FString> InItem)
			{
				using namespace Metasound::Frontend;

				FText* InterfaceDescription = nullptr;
				FMetasoundFrontendVersion InterfaceVersion;
				FMetasoundFrontendInterface InterfaceToAdd;
				if (ensure(ISearchEngine::Get().FindHighestInterfaceVersion(FName{ *InItem.Get() }, InterfaceVersion)))
				{
					if (ensure(IInterfaceRegistry::Get().FindInterface(InterfaceVersion, InterfaceToAdd)))
					{
						InterfaceDescription = InterfaceToAdd.Metadata.Description.IsEmpty() ? nullptr : &InterfaceToAdd.Metadata.Description;
					}
				}

				return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(*InItem))
						.ToolTipText(InterfaceDescription ? *InterfaceDescription : FText::FromStringView(*InItem))
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(8.f, 0.f))
					]
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SBorder)
						.BorderImage(&Style::GetSlateBrushSafe("MetasoundEditor.InterfacesInOut.Background"))
						.ToolTipText(FText::Format(LOCTEXT("InterfaceInputsOutputsTooltip", "{0} {0}|plural(one=input,other=inputs), {1} {1}|plural(one=output,other=outputs)"), TArray<FFormatArgumentValue>{ {InterfaceToAdd.Inputs.Num()}, {InterfaceToAdd.Outputs.Num()} }))
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.ColorAndOpacity(InputColor)
									.Font(FCoreStyle::GetDefaultFontStyle(FName("Bold"), 10))
									.Text(FText::AsNumber(InterfaceToAdd.Inputs.Num()))
								]
							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.Font(FCoreStyle::GetDefaultFontStyle(FName("Bold"), 10))
									.Text(FText::AsCultureInvariant(" | "))
								]
							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.ColorAndOpacity(OutputColor)
									.Font(FCoreStyle::GetDefaultFontStyle(FName("Bold"), 10))
									.Text(FText::AsNumber(InterfaceToAdd.Outputs.Num()))
								]
						] // SBorder
					]; // SHorizontalBox
			})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NameToAdd, ESelectInfo::Type InSelectInfo)
			{
				using namespace Metasound;
				using namespace Metasound::Frontend;

				if (Builder.IsValid() && InSelectInfo != ESelectInfo::OnNavigation)
				{
					FMetasoundFrontendVersion InterfaceVersion;
					if (ensure(ISearchEngine::Get().FindHighestInterfaceVersion(FName{ *NameToAdd.Get() }, InterfaceVersion)))
					{
						FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
						UObject& MetaSound = DocBuilder.CastDocumentObjectChecked<UObject>();

						FMetasoundFrontendInterface InterfaceToAdd;
						if (ensure(IInterfaceRegistry::Get().FindInterface(InterfaceVersion, InterfaceToAdd)))
						{
							const FScopedTransaction Transaction(FText::Format(
								LOCTEXT("AddInterfaceTransactionFormat", "Add MetaSound Interface '{0}'"),
								FText::FromString(InterfaceToAdd.Metadata.Version.ToString())));

							MetaSound.Modify();
							FModifyInterfaceOptions Options({ }, { InterfaceToAdd });
							Options.bSetDefaultNodeLocations = false; // Don't automatically add nodes to ed graph
							DocBuilder.ModifyInterfaces(MoveTemp(Options));
						}

						UpdateInterfaceNames();
						InterfaceComboBox->RefreshOptions();

						FMetaSoundAssetRegistrationOptions Options = FGraphBuilder::GetDefaultRegistrationOptions();
						Options.bIgnoreIfLiveAuditioning = true;
						FGraphBuilder::RegisterGraphWithFrontend(MetaSound, MoveTemp(Options));
					}
				}
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UpdateInterfaceAction", "Add Interface..."))
				.IsEnabled(IsEditableAttribute)
			];

		TSharedRef<SWidget> InterfaceUtilities = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				InterfaceComboBox->AsShared()
			]
		+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateLambda([this]()
				{
					using namespace Frontend;

					if (!Builder.IsValid())
					{
						return;
					}

					FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
					if (!DocBuilder.IsValid())
					{
						return;
					}

					FMetasoundFrontendVersion OutVersion;
					TArray<FMetasoundFrontendInterface> InheritedInterfaces;
					Algo::TransformIf(ImplementedInterfaceNames, InheritedInterfaces,
						[&](const FName& Name)
						{
							return ISearchEngine::Get().FindHighestInterfaceVersion(Name, OutVersion);
						},
						[&](const FName& Name)
						{
							FMetasoundFrontendInterface Interface;
							const bool bFoundInterface = IInterfaceRegistry::Get().FindInterface(OutVersion, Interface);
							checkf(bFoundInterface, TEXT("Interface not found in registry, indicating SearchEngine has entry and unregistration was not evaluated."));
							return Interface;
						});

					UObject& MetaSound = DocBuilder.CastDocumentObjectChecked<UObject>();
					{
						const FScopedTransaction Transaction(LOCTEXT("RemoveAllInterfacesTransaction", "Remove All MetaSound Interfaces"));
						MetaSound.Modify();
						FModifyInterfaceOptions Options(InheritedInterfaces, { });
						Options.bSetDefaultNodeLocations = false; // Don't automatically add nodes to ed graph
						DocBuilder.ModifyInterfaces(MoveTemp(Options));
					}

					UpdateInterfaceNames();
					InterfaceComboBox->RefreshOptions();
					FGraphBuilder::RegisterGraphWithFrontend(MetaSound);

				}), LOCTEXT("RemoveInterfaceTooltip1", "Removes all interfaces from the given MetaSound."))
			];
		InterfaceUtilities->SetEnabled(IsEditableAttribute);

		const FText HeaderName = LOCTEXT("InterfacesGroupDisplayName", "Interfaces");
		IDetailCategoryBuilder& InterfaceCategory = DetailLayout.EditCategory("Interfaces", HeaderName);

		InterfaceCategory.AddCustomRow(HeaderName)
		[
			InterfaceUtilities
		];

		auto CreateInterfaceEntryWidget = [&](FName InInterfaceName) -> TSharedRef<SWidget>
		{
			using namespace Frontend;

			FMetasoundFrontendVersion InterfaceVersion;
			if (!ensure(ISearchEngine::Get().FindHighestInterfaceVersion(InInterfaceName, InterfaceVersion)))
			{
				return SNullWidget::NullWidget;
			}

			FMetasoundFrontendInterface InterfaceToAdd;
			const bool bFoundInterface = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, InterfaceToAdd);

			TSharedRef<SWidget> RemoveButtonWidget = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateLambda([this, InInterfaceName, InterfaceVersion, bFoundInterface, InterfaceToAdd]()
			{
				using namespace Frontend;

				if (!Builder.IsValid())
				{
					return;
				}

				FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
				if (!DocBuilder.IsValid())
				{
					return;
				}

				UObject& MetaSound = DocBuilder.CastDocumentObjectChecked<UObject>();
				{
					if (ensure(bFoundInterface))
					{
						const FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveInterfaceTransactionFormat", "Remove MetaSound Interface '{0}'"), FText::FromString(InterfaceVersion.ToString())));
						MetaSound.Modify();
						FModifyInterfaceOptions Options({ InterfaceToAdd }, { });
						Options.bSetDefaultNodeLocations = false; // Don't automatically add nodes to ed graph
						DocBuilder.ModifyInterfaces(MoveTemp(Options));
					}
				}

				UpdateInterfaceNames();
				InterfaceComboBox->RefreshOptions();
				FGraphBuilder::RegisterGraphWithFrontend(MetaSound);

			}), LOCTEXT("RemoveInterfaceTooltip2", "Removes the associated interface from the MetaSound."));

			TSharedRef<SWidget> EntryWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromName(InterfaceVersion.Name))
					.ToolTipText(bFoundInterface && !InterfaceToAdd.Metadata.Description.IsEmpty() ? InterfaceToAdd.Metadata.Description : FText::FromName(InterfaceVersion.Name))
				]
			+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					RemoveButtonWidget
				];

			EntryWidget->SetEnabled(IsEditableAttribute);
			return EntryWidget;
		};

		TArray<FName> InterfaceNames = ImplementedInterfaceNames.Array();
		InterfaceNames.Sort([](const FName& A, const FName& B) { return A.LexicalLess(B); });
		for (const FName& InterfaceName : InterfaceNames)
		{
			InterfaceCategory.AddCustomRow(FText::FromName(InterfaceName))
			[
				CreateInterfaceEntryWidget(InterfaceName)
			];
		}
	}

	void FMetasoundInterfacesDetailCustomization::UpdateInterfaceNames()
	{
		AddableInterfaceNames.Reset();
		ImplementedInterfaceNames.Reset();

		if (const UObject* MetaSoundObject = GetMetaSound())
		{
			auto GetVersionName = [](const FMetasoundFrontendVersion& Version) { return Version.Name; };
			const UClass* MetaSoundClass = MetaSoundObject->GetClass();
			auto CanAddOrRemoveInterface = [ClassName = MetaSoundClass->GetClassPathName()](const FMetasoundFrontendVersion& Version)
			{
				using namespace Frontend;

				FMetasoundFrontendInterface Interface;
				if (IInterfaceRegistry::Get().FindInterface(Version, Interface))
				{
					auto FindClassOptionsPredicate = [&ClassName](const FMetasoundFrontendInterfaceUClassOptions& Options) { return Options.ClassPath == ClassName; };
					if (const FMetasoundFrontendInterfaceUClassOptions* Options = Interface.Metadata.UClassOptions.FindByPredicate(FindClassOptionsPredicate))
					{
						return Options->bIsModifiable;
					}

					// If no options are found for the given class, interface is modifiable by default.
					return true;
				}

				return false;
			};

			const TSet<FMetasoundFrontendVersion>& InheritedInterfaces = Builder->GetBuilder().GetConstDocumentChecked().Interfaces;
			Algo::TransformIf(InheritedInterfaces, ImplementedInterfaceNames, CanAddOrRemoveInterface, GetVersionName);

			TArray<FMetasoundFrontendVersion> InterfaceVersions = Frontend::ISearchEngine::Get().FindAllInterfaceVersions();
			for (const FMetasoundFrontendVersion& Version : InterfaceVersions)
			{
				if (!ImplementedInterfaceNames.Contains(Version.Name))
				{
					if (CanAddOrRemoveInterface(Version))
					{
						FString Name = Version.Name.ToString();
						AddableInterfaceNames.Add(MakeShared<FString>(MoveTemp(Name)));
					}
				}
			}

			AddableInterfaceNames.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B) { return A->Compare(*B) < 0; });
		}
	}
} // namespace Metasound::Editor
#undef LOCTEXT_NAMESPACE
