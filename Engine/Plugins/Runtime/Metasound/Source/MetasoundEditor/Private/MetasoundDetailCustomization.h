// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IDetailCustomization.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "SGraphActionMenu.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"


// Forward Declarations
class FPropertyRestriction;
class IDetailLayoutBuilder;
class UMetaSoundBuilderBase;
struct FPointerEvent;

namespace Metasound
{
	namespace Editor
	{
		const FName GetMissingPageName(const FGuid& InPageID);

		class FMetaSoundDetailCustomizationBase : public IDetailCustomization
		{
		public:
			virtual ~FMetaSoundDetailCustomizationBase() = default;

			UE_DEPRECATED(5.8, "Use virtual IsEditable instead")
			bool IsGraphEditable() const;
			bool IsFormatEditable() const { return IsEditable(); }

			virtual bool IsEditable() const;

		protected:
			UObject* GetMetaSound() const;
			void InitBuilder(UObject& MetaSound);

			TStrongObjectPtr<UMetaSoundBuilderBase> Builder;
		};

		class FMetasoundDetailCustomization : public FMetaSoundDetailCustomizationBase
		{
		public:
			FMetasoundDetailCustomization(FName InDocumentPropertyName);
			virtual ~FMetasoundDetailCustomization() = default;

			// IDetailCustomization interface
			virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
			// End of IDetailCustomization interface

		private:
			FName GetTemplatePropertyPath() const;
			FName GetInterfaceVersionsPropertyPath() const;
			FName GetRootClassPropertyPath() const;
			FName GetMetadataPropertyPath() const;

			FName DocumentPropertyName;
			TArray<TSharedPtr<int64>> OutputFormatOptions;
			TSharedPtr<SComboBox<TSharedPtr<int64>>> OutputFormatCombo;
		};

		class FMetasoundPagesDetailCustomization : public FMetaSoundDetailCustomizationBase
		{
		public:
			FMetasoundPagesDetailCustomization();

			// FMetaSoundDetailCustomizationBase interface
			virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
			virtual bool IsEditable() const override;
			// End of FMetaSoundDetailCustomizationBase interface

		private:
			const FMetasoundFrontendDocument& GetConstDocumentChecked() const;
			UObject& GetMetaSoundChecked() const;
			bool IsBuilderValid() const;
			void RebuildImplemented();
			void RefreshView();
			void UpdateItemNames();

			TArray<TSharedPtr<FString>> AddableItems;

			TSet<FName> ImplementedNames;
			TSharedPtr<SSearchableComboBox> ComboBox;
			TSharedPtr<SVerticalBox> EntryWidgets;
			FName BuildPageName;

			class FPageListener : public Frontend::IDocumentBuilderTransactionListener
			{
				TWeakPtr<FMetasoundPagesDetailCustomization> Parent;

			public:
				FPageListener() = default;
				FPageListener(TSharedRef<FMetasoundPagesDetailCustomization> InParent)
					: Parent(InParent)
				{
				}

				virtual ~FPageListener() = default;

			private:
				virtual void OnBuilderReloaded(Frontend::FDocumentModifyDelegates& OutDelegates) override;
				void OnPageAdded(const Frontend::FDocumentMutatePageArgs& Args);
				void OnPageSet(const Frontend::FDocumentMutatePageArgs& Args);
				void OnRemovingPage(const Frontend::FDocumentMutatePageArgs& Args);
			};
			TSharedPtr<FPageListener> PageListener;
		};

		class FMetasoundInterfacesDetailCustomization : public FMetaSoundDetailCustomizationBase
		{
		public:
			// IDetailCustomization interface
			virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
			// End of IDetailCustomization interface

			virtual bool IsEditable() const override;

		private:
			void UpdateInterfaceNames();

			TArray<TSharedPtr<FString>> AddableInterfaceNames;

			TSet<FName> ImplementedInterfaceNames;
			TSharedPtr<SSearchableComboBox> InterfaceComboBox;
		};
	} // namespace Editor
} // namespace Metasound