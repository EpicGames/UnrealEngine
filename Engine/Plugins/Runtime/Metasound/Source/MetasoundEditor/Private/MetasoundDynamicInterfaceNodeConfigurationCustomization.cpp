// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicInterfaceNodeConfigurationCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendRegistryKey.h"
#include "ScopedTransaction.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"

namespace Metasound::Editor
{
	FDynamicInterfaceNodeConfigurationCustomization::FDynamicInterfaceNodeConfigurationCustomization(
		TSharedPtr<IPropertyHandle> InStructProperty,
		TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
		: FMetaSoundNodeConfigurationCustomization(InStructProperty, InNode)
	{
		CacheClassInterfaceDescriptions();
	}

	bool FDynamicInterfaceNodeConfigurationCustomization::CacheClassInterfaceDescriptions()
	{
		if (!GraphNode.IsValid())
		{
			return false;
		}

		const FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetConstBuilder();
		const FGuid& NodeID = GraphNode->GetNodeID();
		const FMetasoundFrontendNode* Node = DocBuilder.FindNode(NodeID);
		if (!Node)
		{
			return false;
		}

		const FMetasoundFrontendClass* NodeClass = DocBuilder.FindDependency(Node->ClassID);
		if (!NodeClass)
		{
			return false;
		}

		const Frontend::FNodeClassRegistryKey RegistryKey(NodeClass->Metadata);
		FClassInterface ClassInterface;
		if (!Frontend::INodeClassRegistry::GetChecked().FindClassInterface(RegistryKey, ClassInterface))
		{
			return false;
		}

		SubInterfaceDescriptions = ClassInterface.GetSubInterfaceDescriptions();
		VariantDescriptions = ClassInterface.GetVariantDescriptions();
		return SubInterfaceDescriptions.Num() > 0 || VariantDescriptions.Num() > 0;
	}

	uint32 FDynamicInterfaceNodeConfigurationCustomization::GetSubInterfaceCount(FName SubInterfaceName, uint32 InDefault) const
	{
		if (!StructProperty || !StructProperty->IsValidHandle())
		{
			return InDefault;
		}

		uint32 Result = InDefault;
		StructProperty->EnumerateRawData([&Result, SubInterfaceName](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
			{
				if (const FMetaSoundDynamicInterfaceNodeConfiguration* Config = InstancedStruct->GetPtr<FMetaSoundDynamicInterfaceNodeConfiguration>())
				{
					if (const uint32* Count = Config->SubInterfaceCounts.Find(SubInterfaceName))
					{
						Result = *Count;
					}
				}
			}
			return true;
		});
		return Result;
	}

	FName FDynamicInterfaceNodeConfigurationCustomization::GetVariantSelection(FName VariantName) const
	{
		if (!StructProperty || !StructProperty->IsValidHandle())
		{
			return NAME_None;
		}

		FName Result = NAME_None;
		StructProperty->EnumerateRawData([&Result, VariantName](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
			{
				if (const FMetaSoundDynamicInterfaceNodeConfiguration* Config = InstancedStruct->GetPtr<FMetaSoundDynamicInterfaceNodeConfiguration>())
				{
					if (const FName* Selection = Config->VariantSelections.Find(VariantName))
					{
						Result = *Selection;
					}
				}
			}
			return true;
		});
		return Result;
	}

	void FDynamicInterfaceNodeConfigurationCustomization::SetSubInterfaceCount(FName SubInterfaceName, uint32 NewCount)
	{
		if (!StructProperty || !StructProperty->IsValidHandle())
		{
			return;
		}

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("ChangeSubInterfaceCount", "Change {0} Count"),
			FText::FromName(SubInterfaceName)));

		StructProperty->NotifyPreChange();

		StructProperty->EnumerateRawData([SubInterfaceName, NewCount](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
			{
				if (FMetaSoundDynamicInterfaceNodeConfiguration* Config = InstancedStruct->GetMutablePtr<FMetaSoundDynamicInterfaceNodeConfiguration>())
				{
					Config->SubInterfaceCounts.FindOrAdd(SubInterfaceName) = NewCount;
				}
			}
			return true;
		});

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);

		// NOTE: This mutates the configuration in-place via the property handle and calls
		// UpdateNodeInterfaceFromConfiguration directly, bypassing SetNodeConfiguration's
		// type-compatibility validation. The in-place mutation path is used because the
		// property handle's NotifyPreChange/NotifyPostChange calls integrate with the
		// undo system. Routing through SetNodeConfiguration would require rebuilding the
		// full config and re-setting the property, which may conflict with the property
		// handle's undo integration. Consider routing through SetNodeConfiguration in a
		// follow-up if validation gaps become an issue.
		if (GraphNode.IsValid())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
			DocBuilder.UpdateNodeInterfaceFromConfiguration(GraphNode->GetNodeID());
		}
	}

	void FDynamicInterfaceNodeConfigurationCustomization::SetVariantSelection(FName VariantName, FName NewDataType)
	{
		if (!StructProperty || !StructProperty->IsValidHandle())
		{
			return;
		}

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("ChangeVariantSelection", "Change {0} Type"),
			FText::FromName(VariantName)));

		StructProperty->NotifyPreChange();

		StructProperty->EnumerateRawData([VariantName, NewDataType](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
			{
				if (FMetaSoundDynamicInterfaceNodeConfiguration* Config = InstancedStruct->GetMutablePtr<FMetaSoundDynamicInterfaceNodeConfiguration>())
				{
					Config->VariantSelections.FindOrAdd(VariantName) = NewDataType;
				}
			}
			return true;
		});

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);

		// NOTE: See comment in SetSubInterfaceCount regarding in-place mutation vs
		// routing through SetNodeConfiguration.
		if (GraphNode.IsValid())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
			DocBuilder.UpdateNodeInterfaceFromConfiguration(GraphNode->GetNodeID());
		}
	}

	void FDynamicInterfaceNodeConfigurationCustomization::GenerateChildContent(IDetailChildrenBuilder& ChildBuilder)
	{
		// If we have no sub-interfaces or variants, fall back to default behavior
		if (SubInterfaceDescriptions.IsEmpty() && VariantDescriptions.IsEmpty())
		{
			FInstancedStructDataDetails::GenerateChildContent(ChildBuilder);
			return;
		}

		// Add sub-interface count spinboxes
		for (const FSubInterfaceDescription& Desc : SubInterfaceDescriptions)
		{
			const FName SubInterfaceName = Desc.SubInterfaceName;

			// Widget IDs use the pattern "SubInterface.<Name>" for AutomationDriver testability.
			const FName SpinBoxDriverId = *FString::Printf(TEXT("SubInterface.%s"), *SubInterfaceName.ToString());

			TSharedPtr<SSpinBox<uint32>> SpinBoxWidget;
			ChildBuilder.AddCustomRow(FText::FromName(SubInterfaceName))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromName(SubInterfaceName))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SAssignNew(SpinBoxWidget, SSpinBox<uint32>)
					.MinValue(Desc.Min)
					.MaxValue(Desc.Max)
					.Value_Lambda([this, SubInterfaceName, DefaultCount = Desc.NumDefault]() -> uint32
					{
						return GetSubInterfaceCount(SubInterfaceName, DefaultCount);
					})
					.OnValueCommitted_Lambda([this, SubInterfaceName](uint32 NewValue, ETextCommit::Type /*CommitType*/)
					{
						SetSubInterfaceCount(SubInterfaceName, NewValue);
					})
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.AddMetaData<FTagMetaData>(FTagMetaData(SpinBoxDriverId))
				];
			if (SpinBoxWidget)
			{
				SpinBoxWidget->SetTag(SpinBoxDriverId);
			}
		}

		// Add variant type combo boxes.
		// Use SetNum + Empty rather than Reset to avoid freeing inner TArrays
		// while prior SComboBox widgets may still hold OptionsSource pointers
		// during Slate's deferred widget teardown.
		VariantOptionArrays.SetNum(VariantDescriptions.Num());
		for (auto& Arr : VariantOptionArrays)
		{
			Arr.Empty();
		}

		for (int32 VariantIndex = 0; VariantIndex < VariantDescriptions.Num(); ++VariantIndex)
		{
			const FVariantDescription& Desc = VariantDescriptions[VariantIndex];
			const FName VariantName = Desc.VariantName;

			// Build the options array for the combo box
			TArray<TSharedPtr<FName>>& Options = VariantOptionArrays[VariantIndex];
			for (const FName& DataType : Desc.DataTypes)
			{
				Options.Add(MakeShared<FName>(DataType));
			}

			// Find the currently selected option
			const FName CurrentSelection = GetVariantSelection(VariantName);
			TSharedPtr<FName> CurrentOption;
			for (const TSharedPtr<FName>& Option : Options)
			{
				if (*Option == CurrentSelection)
				{
					CurrentOption = Option;
					break;
				}
			}

			// Widget IDs use the pattern "Variant.<Name>" for AutomationDriver testability.
			const FName ComboBoxDriverId = *FString::Printf(TEXT("Variant.%s"), *VariantName.ToString());

			TSharedPtr<SComboBox<TSharedPtr<FName>>> ComboBoxWidget;
			ChildBuilder.AddCustomRow(FText::FromName(VariantName))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromName(VariantName))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SAssignNew(ComboBoxWidget, SComboBox<TSharedPtr<FName>>)
					.OptionsSource(&Options)
					.InitiallySelectedItem(CurrentOption)
					.AddMetaData<FTagMetaData>(FTagMetaData(ComboBoxDriverId))
					.OnGenerateWidget_Lambda([](TSharedPtr<FName> InItem)
					{
						return SNew(STextBlock)
							.Text(FText::FromName(*InItem))
							.Font(IDetailLayoutBuilder::GetDetailFont());
					})
					.OnSelectionChanged_Lambda([this, VariantName](TSharedPtr<FName> InItem, ESelectInfo::Type /*SelectInfo*/)
					{
						if (InItem.IsValid())
						{
							SetVariantSelection(VariantName, *InItem);
						}
					})
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this, VariantName]()
						{
							const FName Selection = GetVariantSelection(VariantName);
							return Selection.IsNone() ? LOCTEXT("NoTypeSelected", "None") : FText::FromName(Selection);
						})
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				];
			if (ComboBoxWidget)
			{
				ComboBoxWidget->SetTag(ComboBoxDriverId);
			}
		}
	}
} // namespace Metasound::Editor

#undef LOCTEXT_NAMESPACE
