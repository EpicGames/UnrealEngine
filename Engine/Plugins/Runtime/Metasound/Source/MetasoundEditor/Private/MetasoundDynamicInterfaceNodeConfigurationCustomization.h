// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundNodeConfigurationCustomization.h"
#include "MetasoundVertex.h"

namespace Metasound::Editor
{
	/** Details customization for FMetaSoundDynamicInterfaceNodeConfiguration.
	 *  Displays sub-interface instance counts as clamped spinboxes
	 *  and variant type selections as combo box dropdowns.
	 *  Sub-interface names and variant names are shown as read-only labels. */
	class FDynamicInterfaceNodeConfigurationCustomization : public FMetaSoundNodeConfigurationCustomization
	{
	public:
		FDynamicInterfaceNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode);

		//~ Begin IDetailCustomNodeBuilder interface
		virtual void GenerateChildContent(IDetailChildrenBuilder& ChildBuilder) override;
		// Disable tick-based regeneration. The base class (FInstancedStructDataDetails)
		// ticks to detect struct type changes via CachedInstanceTypes, but since this
		// override of GenerateChildContent does not call the base class implementation,
		// CachedInstanceTypes is never set, causing infinite per-frame regeneration.
		// Node configuration type changes are driven by explicit user interaction
		// (spinbox commits, combo selection) rather than external struct type swaps.
		virtual bool RequiresTick() const override { return false; }
		//~ End IDetailCustomNodeBuilder interface

	private:
		/** Look up the FClassInterface from the node registry for the graph node's underlying class. */
		bool CacheClassInterfaceDescriptions();

		/** Modify the configuration's TMap and trigger a node interface update. */
		void SetSubInterfaceCount(FName SubInterfaceName, uint32 NewCount);
		void SetVariantSelection(FName VariantName, FName NewDataType);

		/** Read current value from the configuration's TMap. */
		uint32 GetSubInterfaceCount(FName SubInterfaceName, uint32 InDefault) const;
		FName GetVariantSelection(FName VariantName) const;

		/** Cached descriptions from the FClassInterface. */
		TArray<FSubInterfaceDescription> SubInterfaceDescriptions;
		TArray<FVariantDescription> VariantDescriptions;

		/** Variant option sources for combo boxes (kept alive for SComboBox). */
		TArray<TArray<TSharedPtr<FName>>> VariantOptionArrays;
	};
} // namespace Metasound::Editor
