// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditor.h"

#include "Algo/AnyOf.h"
#include "AudioAnalyzerRack.h"
#include "AudioDevice.h"
#include "AudioWidgetsEnums.h"
#include "Audio/AudioWidgetSubsystem.h"
#include "Blueprint/UserWidget.h"
#include "Components/AudioComponent.h"
#include "DetailLayoutBuilder.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "EdGraphHandleTypes.h"
#include "Editor.h"
#include "Engine/World.h"
#include "FindInMetasound.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "GraphEditorDragDropAction.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAudioExtensionPlugin.h"
#include "IDetailsView.h"
#include "IMetasoundEngineModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelEditor.h"
#include "Logging/TokenizedMessage.h"
#include "Metasound.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditorCommands.h"
#include "MetasoundEditorDocumentClipboardUtils.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundGenerator.h"
#include "MetasoundLog.h"
#include "MetasoundNodeDetailCustomization.h"
#include "MetasoundPresetWidgetInterface.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTemplateWidgetInterface.h"
#include "MetasoundUObjectRegistry.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SMetasoundActionMenu.h"
#include "SMetasoundPalette.h"
#include "SMetasoundStats.h"
#include "SNodePanel.h"
#include "Stats/Stats.h"
#include "Styling/AppStyle.h"
#include "Templates/Function.h"
#include "DocumentTemplates/MetasoundDocumentConfigurator.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "Templates/SharedPointer.h"
#include "TemplateViewModels/MetasoundTemplateViewModelBase.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "UObject/ScriptInterface.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Styling/StyleColors.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectIterator.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditor)

struct FGraphActionNode;

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound
{
	namespace Editor
	{
		namespace AssetEditorPrivate
		{
			static bool bEnableLiveAudition = false;
			static bool bIgnoreDocTemplateVisibilitySettings = false;
			static bool bShowGraphOverride = false;
			static const FText LoadingAssetsFormat = LOCTEXT("MetaSoundScanInProgressNotificationText", "Loading {0} MetaSound Assets...");

			FAutoConsoleVariableRef CVarMetaSoundEditorEnableLiveAudition(
				TEXT("au.MetaSound.Editor.Experimental.EnableLiveAudition"),
				bEnableLiveAudition,
				TEXT("If true, enable live audition in MetaSound Editor. This allows immediate audible feedback from all builder actions supported"
					"by the MetaSound Dynamic Transactor (ex. connecting, disconnecting, creating and removing nodes. Experimental).\n")
				TEXT("Default: false"),
				ECVF_Default);

			FAutoConsoleVariableRef CVarMetaSoundEditorIgnoreTemplateVisibilitySettings(
				TEXT("au.MetaSound.Editor.IgnoreTemplateVisibilitySettings"),
				bIgnoreDocTemplateVisibilitySettings,
				TEXT("If true, all settings on a MetaSound Template for what is and is not visible in editor are ignored and all tabs are invokable.")
				TEXT("Default: false"),
				ECVF_Default);

			TFunction<bool(UUserWidget*)> GetPresetWidgetFilterFunction(const FMetaSoundFrontendDocumentBuilder& DocBuilder)
			{
				using namespace Frontend;

				if (!DocBuilder.IsValid() || DocBuilder.GetIsGraphEditable())
				{
					return { };
				}

				UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
				UAudioWidgetSubsystem* WidgetSubsystem = GEngine ? GEngine->GetEngineSubsystem<UAudioWidgetSubsystem>() : nullptr;
				if (!World || !WidgetSubsystem)
				{
					return { };
				}

				// Referenced assets
				FMetasoundAssetBase& ThisAsset = DocBuilder.GetMetasoundAsset();
				TArray<FMetasoundAssetBase*> ReferencedAssets;
				const bool bIsValidPreset = IMetaSoundAssetManager::GetChecked().GetReferencedPresetHierarchy(ThisAsset, ReferencedAssets);
				if (!bIsValidPreset)
				{
					return { };
				}

				TSet<FNodeClassRegistryKey> ReferencedRegistryKeys;
				Algo::Transform(ReferencedAssets, ReferencedRegistryKeys, [](const FMetasoundAssetBase* ReferencedAsset)
				{
					if (ensure(ReferencedAsset))
					{
						TScriptInterface<const IMetaSoundDocumentInterface> RefDocInterface = ReferencedAsset->GetOwningAsset();
						const FMetasoundFrontendDocument& ReferencedDocument = RefDocInterface->GetConstDocument();
						const FMetasoundFrontendGraphClass& ReferencedGraphClass = ReferencedDocument.RootGraph;
						return FNodeClassRegistryKey(ReferencedGraphClass);
					}
					return FNodeClassRegistryKey();
				});

				// Filter whether the widget supports this MetaSound
				return [ThisRegistryKey = FNodeRegistryKey(DocBuilder.GetConstDocumentChecked().RootGraph), ReferencedRegistryKeys](UUserWidget* UserWidget)
				{
					bool bSupportAllPresets = false;
					TArray<TScriptInterface<IMetaSoundDocumentInterface>> ExcludedMetaSounds;
					TArray<TScriptInterface<IMetaSoundDocumentInterface>> IncludedMetaSounds;
					IMetaSoundPresetWidgetInterface::Execute_GetSupportedMetaSounds(UserWidget, bSupportAllPresets, ExcludedMetaSounds, IncludedMetaSounds);

					// Show if all presets supported and this MetaSound is not excluded 
					if (bSupportAllPresets)
					{
						if (!ExcludedMetaSounds.IsEmpty())
						{
							for (TScriptInterface<IMetaSoundDocumentInterface> ExcludedMetaSound : ExcludedMetaSounds)
							{
								if (!ExcludedMetaSound)
								{
									continue;
								}

								// This MetaSound or the referenced MetaSound is excluded 
								const FMetasoundFrontendGraphClass& ExcludedGraphClass = ExcludedMetaSound->GetConstDocument().RootGraph;
								const FNodeRegistryKey ExcludedRegistryKey(ExcludedGraphClass);

								if (ThisRegistryKey == ExcludedRegistryKey || ReferencedRegistryKeys.Contains(ExcludedRegistryKey))
								{
									return false;
								}
							}
						}
						return true;
					}

					// This MetaSound is in the inclusion list 
					for (TScriptInterface<IMetaSoundDocumentInterface> IncludedMetaSound : IncludedMetaSounds)
					{
						if (!IncludedMetaSound)
						{
							continue;
						}

						// This MetaSound or the referenced MetaSound is included 
						const FMetasoundFrontendGraphClass& IncludedGraphClass = IncludedMetaSound->GetConstDocument().RootGraph;
						const FNodeRegistryKey IncludedRegistryKey(IncludedGraphClass);

						if (ThisRegistryKey == IncludedRegistryKey || ReferencedRegistryKeys.Contains(IncludedRegistryKey))
						{
							return true;
						}
					}
					return false;
				};
			}

			void SanitizeTabInfo(const UScriptStruct& TemplateStruct, FMetaSoundTemplateWidgetTabInfo& InOutTabInfo)
			{
				if (InOutTabInfo.TabName.IsNone())
				{
					FNameBuilder NameBuilder;
					NameBuilder.Appendf(TEXT("MetasoundEditor_%s"), *TemplateStruct.GetName());
					InOutTabInfo.TabName = *NameBuilder;
					if (InOutTabInfo.TabDisplayName.IsEmpty())
					{
						InOutTabInfo.TabDisplayName = TemplateStruct.GetDisplayNameText();
					}
				}

				if (InOutTabInfo.TabIcon.IsNone())
				{
					InOutTabInfo.TabIcon = "MetasoundEditor.MetasoundPatch.Thumbnail";
				}

				if (InOutTabInfo.ToolTip.IsEmpty())
				{
					InOutTabInfo.ToolTip = FText::Format(
						LOCTEXT("TemplateWidgetTab_DefaultToolTipFormat", "Custom control for MetaSound '{0}' configuration"),
						TemplateStruct.GetDisplayNameText());
				}

				if (InOutTabInfo.StyleName.IsNone())
				{
					InOutTabInfo.StyleName = "MetaSoundStyle";
				}
			};
		} // namespace AssetEditorPrivate

		bool IsPreviewingPageInputDefault(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassInput& InClassInput, const FGuid& InPageID)
		{
			using namespace Engine;
			using namespace Frontend;

			const UObject& MetaSound = Builder.CastDocumentObjectChecked<const UObject>();
			if (FGraphBuilder::IsPreviewingMetaSound(MetaSound))
			{
				if (const FMetasoundFrontendClassInputDefault* InputDefault = FindPreferredPage(InClassInput.GetDefaults(), UMetaSoundSettings::GetPageOrder()))
				{
					return InputDefault->PageID == InPageID;
				}
			}

			return false;
		}

		bool IsPreviewingPageGraph(const FMetaSoundFrontendDocumentBuilder& Builder, const FGuid& InPageID)
		{
			using namespace Engine;
			using namespace Frontend;

			const UObject& MetaSound = Builder.CastDocumentObjectChecked<const UObject>();
			if (FGraphBuilder::IsPreviewingMetaSound(MetaSound))
			{
				const FMetasoundFrontendGraphClass& GraphClass = Builder.GetConstDocumentChecked().RootGraph;
				if (const FMetasoundFrontendGraph* GraphPage = FindPreferredPage(GraphClass.GetConstGraphPages(), UMetaSoundSettings::GetPageOrder()))
				{
					return GraphPage->PageID == InPageID;
				}
			}

			return false;
		}

		bool IsPageAuditioningEnabled()
		{
			IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			return !MetaSoundEditorModule.IsRestrictedMode();
		}

		bool IsPageGraphEditingEnabled(const FMetaSoundFrontendDocumentBuilder& Builder, bool bHasLocalPageValues)
		{
			IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			if (MetaSoundEditorModule.IsRestrictedMode())
			{
				return false;
			}

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			if (!Settings)
			{
				return false;
			}

			if (Settings->GetProjectPageSettings().IsEmpty())
			{
				if (!bHasLocalPageValues)
				{
					return false;
				}
			}

			if (!Builder.IsValid())
			{
				return false;
			}

			TConstStructView<FMetaSoundFrontendDocumentTemplate> TemplateView = Builder.GetConstDocumentTemplate();
			if (TemplateView.IsValid())
			{
				return TemplateView.GetPtr()->GetEditorOptions().bPageGraphEditingEnabled;
			}

			return true;
		}

		bool IsInputPageEditingEnabled(const FMetaSoundFrontendDocumentBuilder& Builder, bool bHasProjectPageValues)
		{
			if (!Builder.IsValid())
			{
				return false;
			}

			IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			if (MetaSoundEditorModule.IsRestrictedMode())
			{
				return false;
			}

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			if (!Settings)
			{
				return false;
			}

			if (Settings->GetProjectPageSettings().IsEmpty())
			{
				if (!bHasProjectPageValues)
				{
					return false;
				}
			}

			return true;
		}

		namespace TabNamesPrivate
		{
			const FName Analyzers = "MetasoundEditor_Analyzers";
			const FName Details = "MetasoundEditor_Details";
			const FName GraphCanvas = "MetasoundEditor_GraphCanvas";
			const FName Members = "MetasoundEditor_Members";
			const FName Palette = "MetasoundEditor_Palette";
			const FName Interfaces = "MetasoundEditor_Interfaces";
			const FName Pages = "MetasoundEditor_Pages";
			const FName Find = "MetasoundEditor_Find";

			const FName DocTemplateWidgets_Factory = "MetasoundEditor_DocTemplateWidgetFactory";
		} // namespace TabNamesPrivate

		static const TArray<FText> NodeSectionNames
		{
			LOCTEXT("NodeSectionName_Invalid", "INVALID"),
			LOCTEXT("NodeSectionName_Inputs", "Inputs"),
			LOCTEXT("NodeSectionName_Outputs", "Outputs"),
			LOCTEXT("NodeSectionName_Variables", "Variables")
		};

		class FMetasoundGraphMemberSchemaAction : public FEdGraphSchemaAction
		{
			FGuid MemberID;

		public:
			UEdGraph* Graph = nullptr;
			TWeakObjectPtr<UMetaSoundBuilderBase> Builder;

			void SetMemberID(const FGuid& InID)
			{
				MemberID = InID;
			}

			void SetBuilder(UMetaSoundBuilderBase& InBuilder)
			{
				Builder = &InBuilder;
			}

			FMetasoundGraphMemberSchemaAction()
				: FEdGraphSchemaAction()
			{
			}

			FMetasoundGraphMemberSchemaAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const ENodeSection InSectionID)
				: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, FText(), static_cast<int32>(InSectionID))
			{
			}

			UMetasoundEditorGraphMember* GetGraphMember() const
			{
				UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(Graph);
				return MetasoundGraph->FindMember(MemberID);
			}

			FName GetMemberName() const
			{
				if (UMetasoundEditorGraphMember* Member = GetGraphMember())
				{
					return Member->GetMemberName();
				}
				return NAME_None;
			}

			// FEdGraphSchemaAction interface
			virtual bool IsParentable() const override
			{
				return true;
			}

			virtual void MovePersistentItemToCategory(const FText& NewCategoryName) override
			{
				checkNoEntry();
			}

			virtual int32 GetReorderIndexInContainer() const override
			{
				if (Builder.IsValid())
				{
					if (UMetasoundEditorGraphMember* Member = GetGraphMember())
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
						if (Member->IsA<UMetasoundEditorGraphVertex>())
						{
							auto FindVertexWithID = ([this](const FMetasoundFrontendClassVertex& Vertex)
							{
								return Vertex.NodeID == MemberID;
							});
							const FMetasoundFrontendDocument& Document = DocBuilder.GetConstDocumentChecked();
							const FMetasoundFrontendClassInterface& Interface = Document.RootGraph.GetDefaultInterface();
							if (Member->IsA<UMetasoundEditorGraphInput>())
							{
								return Interface.Inputs.IndexOfByPredicate(FindVertexWithID);
							}

							if (Member->IsA<UMetasoundEditorGraphOutput>())
							{
								return Interface.Outputs.IndexOfByPredicate(FindVertexWithID);
							}
						}
						else if (Member->IsA<UMetasoundEditorGraphVariable>())
						{
							auto FindVariableWithID = [this](const FMetasoundFrontendVariable& Variable)
							{
								return Variable.ID == MemberID;
							};
							return DocBuilder.FindConstBuildGraphChecked().Variables.IndexOfByPredicate(FindVariableWithID);
						}
					}
				}
				return INDEX_NONE;
			}

			virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) override
			{
				// TODO: Implement reordering
				checkNoEntry();

				return false;
			}
		};

		class FMetaSoundDragDropMemberAction : public FGraphSchemaActionDragDropAction
		{
			TSharedPtr<FEditor> Editor;
			TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMember;

		public:
			FMetaSoundDragDropMemberAction(TSharedPtr<FEditor> InEditor, UMetasoundEditorGraphMember* InGraphMember)
				: Editor(InEditor)
				, GraphMember(InGraphMember)
			{
				CursorDecoratorWindow = SWindow::MakeCursorDecorator();
				constexpr bool bShowImmediately = false;
				FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), bShowImmediately);
			}

			DRAG_DROP_OPERATOR_TYPE(FMetaSoundDragDropMemberAction, FGraphSchemaActionDragDropAction)

			virtual FReply DroppedOnPanel(const TSharedRef<SWidget>& InPanel, const FVector2f& InScreenPosition, const FVector2f& InGraphPosition, UEdGraph& InGraph) override
			{
				if (!GraphMember.IsValid() || &InGraph != GraphMember->GetOwningGraph())
				{
					return FReply::Unhandled();
				}

				return DroppedOnPin(InScreenPosition, InGraphPosition);
			}
			virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override { return FReply::Unhandled(); }
			virtual FReply DroppedOnPin(const FVector2f& InScreenPosition, const FVector2f& InGraphPosition) override
			{
				using namespace Engine;
				using namespace Frontend;

				if (!GraphMember.IsValid())
				{
					return FReply::Unhandled();
				}

				if (!Editor.IsValid())
				{
					return FReply::Unhandled();
				}

				UMetasoundEditorGraph* MetasoundGraph = GraphMember->GetOwningGraph();
				check(MetasoundGraph);
				UObject& ParentMetaSound = MetasoundGraph->GetMetasoundChecked();
				FMetaSoundFrontendDocumentBuilder& DocBuilder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetaSound);

				if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(GraphMember.Get()))
				{
					GetMutableDefault<UMetasoundEditorSettings>()->AddRecentlyUsedMetaSoundsGraphAction(GraphMember->GetMemberID().ToString());
					
					const FScopedTransaction Transaction(LOCTEXT("DropAddNewInputNode", "Drop New MetaSound Input Node"));
					ParentMetaSound.Modify();
					MetasoundGraph->Modify();
					Input->Modify();

					const FMetasoundFrontendNode* TemplateNode = FInputNodeTemplate::CreateNode(DocBuilder, Input->GetMemberName());
					if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetaSound, TemplateNode->GetID()))
					{
						NewGraphNode->Modify();
						NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(InGraphPosition));
						NewGraphNode->SyncLocationFromFrontendNode();

						TryConnectToHoveredPin(NewGraphNode);

						FGraphBuilder::RegisterGraphWithFrontend(ParentMetaSound);
						Editor->ClearSelectionAndSelectNode(NewGraphNode);
						return FReply::Handled();
					}
				}

				if (UMetasoundEditorGraphOutput* Output = Cast<UMetasoundEditorGraphOutput>(GraphMember.Get()))
				{
					// Note: Purposedly not registering Output nodes usage into Recently Used nodes.
					// Indeed, once placed in a graph, they don't appear in the graph action menu anymore, so they would be hidden in the
					// Recently Used list too anyway.
					
					TArray<UMetasoundEditorGraphMemberNode*> Nodes = Output->GetNodes();
					if (Nodes.IsEmpty())
					{
						const FScopedTransaction Transaction(LOCTEXT("DropAddNewOutputNode", "Drop New MetaSound Output Node"));
						ParentMetaSound.Modify();
						MetasoundGraph->Modify();
						Output->Modify();

						if (UMetasoundEditorGraphOutputNode* NewGraphNode = FGraphBuilder::AddOutputNode(ParentMetaSound, Output->NodeID))
						{
							NewGraphNode->Modify();
							NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(InGraphPosition));
							NewGraphNode->SyncLocationFromFrontendNode();

							TryConnectToHoveredPin(NewGraphNode);

							FGraphBuilder::RegisterGraphWithFrontend(ParentMetaSound);
							TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*MetasoundGraph);
							if (MetasoundEditor.IsValid())
							{
								MetasoundEditor->ClearSelectionAndSelectNode(NewGraphNode);
							}
							return FReply::Handled();
						}
					}
					else
					{
						if (Editor.IsValid())
						{
							Editor->JumpToNodes(Nodes);
							return FReply::Handled();
						}
					}
				}

				if (UMetasoundEditorGraphVariable* Variable = Cast<UMetasoundEditorGraphVariable>(GraphMember.Get()))
				{
					GetMutableDefault<UMetasoundEditorSettings>()->AddRecentlyUsedMetaSoundsGraphAction(GraphMember->GetMemberID().ToString());
					
					const FScopedTransaction Transaction(LOCTEXT("DropAddNewVariableNode", "Drop New MetaSound Variable Node"));
					ParentMetaSound.Modify();
					MetasoundGraph->Modify();
					Variable->Modify();

					FMetasoundFrontendClass VariableClass;
					const bool bMakeOrJumpToMutator = FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
					if (bMakeOrJumpToMutator)
					{
						if (const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariable(Variable->GetMemberName()); FrontendVariable && FrontendVariable->MutatorNodeID.IsValid())
						{
							auto IsMutatorNode = [&FrontendVariable](const UMetasoundEditorGraphMemberNode* Node)
							{
								return Node->GetNodeID() == FrontendVariable->MutatorNodeID;
							};
							TArray<UMetasoundEditorGraphMemberNode*> Nodes = Variable->GetNodes();
							if (UMetasoundEditorGraphMemberNode** MutatorNode = Nodes.FindByPredicate(IsMutatorNode))
							{
								check(*MutatorNode);
								Editor->JumpToNodes<UMetasoundEditorGraphMemberNode>({ *MutatorNode });
								return FReply::Handled();
							}
						}
						else
						{
							ensure(IDataTypeRegistry::Get().GetFrontendVariableMutatorClass(Variable->GetDataType(), VariableClass));
						}
					}
					else
					{
						const bool bJumpToGetters = FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Control);
						if (bJumpToGetters)
						{
							TArray<UMetasoundEditorGraphMemberNode*> Nodes = Variable->GetNodes();
							for (int32 i = Nodes.Num() - 1; i >= 0; --i)
							{
								const UMetasoundEditorGraphVariableNode* VariableNode = CastChecked<UMetasoundEditorGraphVariableNode>(Nodes[i]);
								const EMetasoundFrontendClassType ClassType = VariableNode->GetClassType();
								if (ClassType != EMetasoundFrontendClassType::VariableAccessor
									&& ClassType != EMetasoundFrontendClassType::VariableDeferredAccessor)
								{
									Nodes.RemoveAtSwap(i, EAllowShrinking::No);
								}
							}
							Editor->JumpToNodes(Nodes);
							return FReply::Handled();
						}
						else
						{
							const bool bMakeGetDeferred = FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Alt);
							if (bMakeGetDeferred)
							{
								ensure(IDataTypeRegistry::Get().GetFrontendVariableDeferredAccessorClass(Variable->GetDataType(), VariableClass));
							}
							else
							{
								ensure(IDataTypeRegistry::Get().GetFrontendVariableAccessorClass(Variable->GetDataType(), VariableClass));
							}
						}
					}

					const FMetasoundFrontendNode* NewVariableNode = DocBuilder.AddGraphVariableNode(Variable->GetMemberName(), VariableClass.Metadata.GetType());
					if (ensure(NewVariableNode))
					{
						if (UMetasoundEditorGraphVariableNode* NewGraphNode = FGraphBuilder::AddVariableNode(ParentMetaSound, NewVariableNode->GetID()))
						{
							NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(InGraphPosition));
							NewGraphNode->SyncLocationFromFrontendNode();

							TryConnectToHoveredPin(NewGraphNode);

							FGraphBuilder::RegisterGraphWithFrontend(ParentMetaSound);
							Editor->ClearSelectionAndSelectNode(NewGraphNode);
							return FReply::Handled();
						}
					}
				}

				return FReply::Unhandled();
			}
			virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action) { return FReply::Unhandled(); }
			virtual FReply DroppedOnCategory(FText Category) override { return FReply::Unhandled(); }

			bool TryConnectToHoveredPin(UMetasoundEditorGraphNode* InNewGraphNode)
			{
				if (!GetHoveredPin())
				{
					return false;
				}

				FEdGraphPinHandle FromPin = InNewGraphNode->GetPinAt(0);
				FEdGraphPinHandle ToPin = GetHoveredPin();

				if ((FromPin.GetPin() != nullptr) && (ToPin.GetPin() != nullptr))
				{
					const UEdGraph* MyGraphObj = FromPin.GetGraph();

					// the pin may change during the creation of the link
					if (const UEdGraphSchema* GraphSchema = MyGraphObj->GetSchema())
					{
						return GraphSchema->TryCreateConnection(FromPin.GetPin(), ToPin.GetPin());
					}
				}

				return false;
			}

			Frontend::FConnectability CanBeConnected(const FName& DataType0, const FName& DataType1)
			{
				using namespace Frontend;

				FConnectability OutConnectability;
				OutConnectability.Connectable = FConnectability::EConnectable::No;
				OutConnectability.Reason = FConnectability::EReason::None;

				if (DataType0 == FName())
				{
					OutConnectability.Connectable = FConnectability::EConnectable::No;
					OutConnectability.Reason = FConnectability::EReason::IncompatibleDataTypes;
				}
				else if (IsCastable(DataType0, DataType1))
				{
					OutConnectability.Connectable = FConnectability::EConnectable::Yes;
					OutConnectability.Reason = FConnectability::EReason::None;
				}
				else
				{
					OutConnectability.PossibleConverterNodeClasses = FMetasoundFrontendRegistryContainer::Get()->GetPossibleConverterNodes(DataType0, DataType1);

					if (OutConnectability.PossibleConverterNodeClasses.Num() > 0)
					{
						OutConnectability.Connectable = FConnectability::EConnectable::YesWithConverterNode;
					}
				}

				return OutConnectability;
			}

			virtual void HoverTargetChanged() override
			{
				using namespace Frontend;

				bDropTargetValid = false;

				const FSlateBrush* PrimarySymbol = nullptr;
				const FSlateBrush* SecondarySymbol = nullptr;
				FSlateColor PrimaryColor;
				FSlateColor SecondaryColor;
				GetDefaultStatusSymbol(PrimarySymbol, PrimaryColor, SecondarySymbol, SecondaryColor);

				const FText IncompatibleText = LOCTEXT("MetasoundHoverNotCompatibleText", "'{0}' is not compatible with '{1}'");
				const FText CompatibleText = LOCTEXT("MetasoundHoverCompatibleText", "Convert {0} to {1}.");

				FText Message;
				if (GraphMember.IsValid())
				{
					const UMetasoundEditorGraph* OwningGraph = GraphMember->GetOwningGraph();
					Message = GraphMember->GetDisplayName();
					if (GetHoveredGraph() && OwningGraph)
					{
						if (GetHoveredGraph() == OwningGraph)
						{
							const UMetaSoundBuilderBase& Builder = OwningGraph->GetBuilderChecked();
							const bool bIsEditable = Builder.GetConstBuilder().GetIsGraphEditable();
							if (!bIsEditable)
							{
								Message = FText::Format(LOCTEXT("DropTargetFailIsPreset", "'{0}': Graph is not editable"), GraphMember->GetDisplayName());
							}
							else if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(GraphMember.Get()))
							{
								bDropTargetValid = true;

								PrimarySymbol = &Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Input");
								SecondarySymbol = nullptr;

								UEdGraphPin* PinUnderCursor = GetHoveredPin();
								
								if (PinUnderCursor != nullptr && PinUnderCursor->Direction == EGPD_Input)
								{
									Frontend::FConstInputHandle InputHandle = Editor::FGraphBuilder::GetConstInputHandleFromPin(PinUnderCursor);
									const FName& DataType = InputHandle->GetDataType();
									const FName& OtherDataType = GraphMember->GetDataType();

									FConnectability Connectability = CanBeConnected(OtherDataType, DataType);

									PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
									Message = FText();
									if (Connectability.Connectable == Frontend::FConnectability::EConnectable::No)
									{
										PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
										Message = FText::Format(IncompatibleText, FText::FromName(OtherDataType), FText::FromName(DataType));
									}
									else if (Connectability.Connectable == Frontend::FConnectability::EConnectable::YesWithConverterNode)
									{
										PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ViaCast"));
										Message = FText::Format(CompatibleText, FText::FromName(OtherDataType), FText::FromName(DataType));
									}
								}
							}
							else if (UMetasoundEditorGraphOutput* Output = Cast<UMetasoundEditorGraphOutput>(GraphMember.Get()))
							{
								bDropTargetValid = true;

								if (!Output->GetNodes().IsEmpty())
								{
									PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ShowNode"));
									SecondarySymbol = nullptr;
									Message = FText::Format(LOCTEXT("DropTargetShowOutput", "Show '{0}' (One per graph)"), GraphMember->GetDisplayName());
								}
								else
								{
									if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
									{
										PrimarySymbol = MetasoundStyle->GetBrush("MetasoundEditor.Graph.Node.Class.Output");
										SecondarySymbol = nullptr;
									}

									UEdGraphPin* PinUnderCursor = GetHoveredPin();

									if (PinUnderCursor != nullptr && PinUnderCursor->Direction == EGPD_Output)
									{
										Frontend::FConstOutputHandle OutputHandle = Editor::FGraphBuilder::GetConstOutputHandleFromPin(PinUnderCursor);
										const FName& DataType = OutputHandle->GetDataType();
										const FName& OtherDataType = GraphMember->GetDataType();

										FConnectability Connectability = CanBeConnected(OtherDataType, DataType);

										PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
										Message = FText();
										if (Connectability.Connectable == Frontend::FConnectability::EConnectable::No)
										{
											PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
											Message = FText::Format(IncompatibleText, FText::FromName(DataType), FText::FromName(OtherDataType));
										}
										else if (Connectability.Connectable == Frontend::FConnectability::EConnectable::YesWithConverterNode)
										{
											PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ViaCast"));
											Message = FText::Format(CompatibleText, FText::FromName(DataType), FText::FromName(OtherDataType));
										}
									}
								}
							}
							else if (UMetasoundEditorGraphVariable* Variable = Cast<UMetasoundEditorGraphVariable>(GraphMember.Get()))
							{
								bDropTargetValid = true;

								PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ShowNode"));

								if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
								{
									PrimarySymbol = MetasoundStyle->GetBrush("MetasoundEditor.Graph.Node.Class.Variable");
									SecondarySymbol = nullptr;
								}

								const FText DisplayName = GraphMember->GetDisplayName();
								const FText GetterToolTip = FText::Format(LOCTEXT("DropTargetGetterVariableToolTipFormat", "{0}\nAdd:\n* Get (Drop)\n* Get Delayed (Alt+Drop)\n"), DisplayName);
								static const FText GetJumpToToolTip = LOCTEXT("JumpToGettersToolTip", "Get (Ctrl+Drop)");
								static const FText AddOrJumpToSetToolTip;
								FConstNodeHandle MutatorNodeHandle = Variable->GetConstVariableHandle()->FindMutatorNode();
								if (MutatorNodeHandle->IsValid())
								{
									Message = FText::Format(LOCTEXT("DropTargetVariableJumpToFormat", "{0}\nJump To:\n* {1}\n* Set (Shift+Drop, One per graph)"), GetterToolTip, GetJumpToToolTip);
								}
								else
								{
									TArray<FConstNodeHandle> AccessorNodeHandles = Variable->GetConstVariableHandle()->FindAccessorNodes();

									if (AccessorNodeHandles.IsEmpty())
									{
										Message = FText::Format(LOCTEXT("DropTargetVariableAddSetGetFormat", "{0}* Set (Shift+Drop)"), GetterToolTip);
									}
									else
									{
										Message = FText::Format(LOCTEXT("DropTargetVariableAddSetJumpToGetFormat", "{0}* Set (Shift+Drop)\n\nJump To:\n* {1}"), GetterToolTip, GetJumpToToolTip);
									}
								}

								UEdGraphPin* PinUnderCursor = GetHoveredPin();

								if (PinUnderCursor != nullptr && PinUnderCursor->Direction == EGPD_Input)
								{
									Frontend::FConstInputHandle InputHandle = Editor::FGraphBuilder::GetConstInputHandleFromPin(PinUnderCursor);
									const FName& DataType = InputHandle->GetDataType();
									const FName& OtherDataType = GraphMember->GetDataType();

									FConnectability Connectability = CanBeConnected(OtherDataType, DataType);

									PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
									Message = FText();
									if (Connectability.Connectable == Frontend::FConnectability::EConnectable::No)
									{
										PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
										Message = FText::Format(IncompatibleText, FText::FromName(OtherDataType), FText::FromName(DataType));
									}
									else if (Connectability.Connectable == Frontend::FConnectability::EConnectable::YesWithConverterNode)
									{
										PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ViaCast"));
										Message = FText::Format(CompatibleText, FText::FromName(OtherDataType), FText::FromName(DataType));
									}
								}
							}
						}
						else
						{
							Message = FText::Format(LOCTEXT("DropTargetFailNotParentGraph", "'{0}': Graph is not parent of member."), GraphMember->GetDisplayName());
						}
					}
				}

				SetSimpleFeedbackMessage(PrimarySymbol, PrimaryColor, Message, SecondarySymbol, SecondaryColor);
			}
		};

		class SMetaSoundGraphPaletteItem : public SGraphPaletteItem
		{
		private:
			TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction;
			FMetasoundFrontendVersion InterfaceVersion;

		public:
			SLATE_BEGIN_ARGS(SMetaSoundGraphPaletteItem)
			{
			}

			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
			{
				TSharedPtr<FEdGraphSchemaAction> Action = InCreateData->Action;
				MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);

				if (UMetasoundEditorGraphVertex* GraphVertex = Cast<UMetasoundEditorGraphVertex>(MetasoundAction->GetGraphMember()))
				{
					InterfaceVersion = GraphVertex->GetInterfaceVersion();
				}

				SGraphPaletteItem::Construct(SGraphPaletteItem::FArguments(), InCreateData);
			}

		protected:
			virtual void OnNameTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit) override
			{
				using namespace Frontend;

				if (InterfaceVersion.IsValid())
				{
					return;
				}

				if (MetasoundAction.IsValid())
				{
					if (UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
					{
						// Check if new name has changed
						// Check against the non namespaced member name because
						// this text box is only for the non namespaced part of the name
						// (namespace is in parent menu items)
						FName Namespace;
						FName Name;
						Audio::FParameterPath::SplitName(GraphMember->GetMemberName(), Namespace, Name);

						if (Name == InNewText.ToString())
						{
							return;
						}

						const FText TransactionLabel = FText::Format(LOCTEXT("Rename Graph Member", "Set MetaSound {0}'s Name"), GraphMember->GetGraphMemberLabel());
						const FScopedTransaction Transaction(TransactionLabel);

						constexpr bool bPostTransaction = false;
						GraphMember->SetDisplayName(FText::GetEmpty(), bPostTransaction);

						// Add back namespace if needed
						FString NewName = InNewText.ToString();
						if (!Namespace.IsNone())
						{
							NewName = Namespace.ToString() + Audio::FParameterPath::NamespaceDelimiter + NewName;
						}
						GraphMember->SetMemberName(FName(NewName), bPostTransaction);
					}
				}


			}

			virtual TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly) override
			{
				TSharedRef<SWidget> TextWidget = SGraphPaletteItem::CreateTextSlotWidget(InCreateData, bIsReadOnly);

				bool bIsConstructorPin = false;

				const FSlateBrush* IconBrush = nullptr;
				const FVector2D IconSize16 = FVector2D(16.0f, 16.0f);
				FSlateColor IconColor = FSlateColor::UseForeground();

				const bool bIsInterfaceMember = InterfaceVersion.IsValid();
				const FSlateBrush* InterfaceIconBrush = bIsInterfaceMember ? FAppStyle::GetBrush("Icons.Lock") : FStyleDefaults::GetNoBrush();

				if (TSharedPtr<FMetasoundGraphMemberSchemaAction> GraphMemberAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(InCreateData->Action))
				{

					if (UMetasoundEditorGraphMember* GraphMember = GraphMemberAction->GetGraphMember())
					{
						if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember))
						{
							EMetasoundFrontendVertexAccessType AccessType = Vertex->GetVertexAccessType();
							bIsConstructorPin = AccessType == EMetasoundFrontendVertexAccessType::Value;
						}
						FName DataTypeName = GraphMember->GetDataType();

						if (const FEdGraphPinType* PinType = FMetaSoundGraphPanelPinFactory::GetChecked()->FindPinType(DataTypeName))
						{
							if (const UMetasoundEditorGraphSchema* Schema = GetDefault<UMetasoundEditorGraphSchema>())
							{
								IconColor = Schema->GetPinTypeColor(*PinType);
							}
						}

						const IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
						IconBrush = EditorModule.GetIconBrush(DataTypeName, bIsConstructorPin);
					}
				}

				TSharedRef<SHorizontalBox> LayoutWidget = SNew(SHorizontalBox);
				LayoutWidget->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(IconBrush)
					.ColorAndOpacity(IconColor)
					.DesiredSizeOverride(IconSize16)
				];

				if (bIsInterfaceMember)
				{
					LayoutWidget->AddSlot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(InterfaceIconBrush)
						.ToolTipText(bIsInterfaceMember ? FText::Format(LOCTEXT("InterfaceMemberToolTipFormat", "Cannot Add/Remove: Member of interface '{0}'"), FText::FromName(InterfaceVersion.Name)) : FText())
						.ColorAndOpacity(FSlateColor::UseForeground())
						.DesiredSizeOverride(IconSize16)
					];
				}
				
				LayoutWidget->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(2, 0, 0, 0)
				[
					TextWidget
				];

				return LayoutWidget;
			}

			virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override
			{
				if (MetasoundAction.IsValid())
				{
					if (UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
					{
						return GraphMember->CanRename(InNewText, OutErrorMessage);
					}
				}

				return false;
			}
		};

		FEditor::FTemplateWidgetSummoner::FTemplateWidgetSummoner(FName FactoryName, FEditor::FBlueprintId InBlueprintId, TSharedPtr<class FAssetEditorToolkit> InHostingApp)
			: FWorkflowTabFactory(FactoryName, InHostingApp)
			, BlueprintId(InBlueprintId)
		{
			using namespace Frontend;

			bIsSingleton = false;

			TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

			TabLabel = LOCTEXT("MetaSoundTemplateView_MenuTitle", "Template");
			TabIdentifier = "Template";
			ViewMenuTooltip = LOCTEXT("MetaSoundTemplateView_ToolTip", "Custom view for the current MetaSound Template");

			TSharedPtr<FEditor> Parent = GetParentEditor();
			if (ensure(Parent.IsValid()))
			{
				if (FEditor::FUserWidgetData* WidgetData = Parent->DocTemplateWidgetData.Find(BlueprintId))
				{
					TabLabel = WidgetData->TabInfo.TabDisplayName;
					TabIdentifier = WidgetData->TabInfo.TabName;
					TabIcon = FSlateIcon(WidgetData->TabInfo.StyleName, WidgetData->TabInfo.TabIcon);
					if (const FMetaSoundTemplate* Template = Parent->GetDocumentTemplate())
					{
						ViewMenuTooltip = Template->GetEditorOptions().ToolTip;
					}
				}
			}

			ViewMenuDescription = TabLabel;
		}

		TSharedRef<SWidget> FEditor::FTemplateWidgetSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
		{
			if (const FUserWidgetData* WidgetData = GetWidgetData())
			{
				if (UUserWidget* UserWidget = WidgetData->Widget.Get())
				{
					return UserWidget->TakeWidget();
				}
			}

			return SNullWidget::NullWidget;
		}

		TSharedRef<SDockTab> FEditor::FTemplateWidgetSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
		{
			TSharedRef<SDockTab> DockTab = SNew(SDockTab);

			DockTab->SetLabel(TabLabel);

			if (const FUserWidgetData* WidgetData = GetWidgetData())
			{
				if (UUserWidget* UserWidget = WidgetData->Widget.Get())
				{
					DockTab->SetContent(UserWidget->TakeWidget());
				}
			}

			return DockTab;
		}

		TSharedPtr<FEditor> FEditor::FTemplateWidgetSummoner::GetParentEditor() const
		{
			return StaticCastWeakPtr<FEditor>(HostingApp).Pin();
		}

		const FEditor::FUserWidgetData* FEditor::FTemplateWidgetSummoner::GetWidgetData() const
		{
			TSharedPtr<FEditor> Parent = GetParentEditor();
			if (ensure(Parent.IsValid()))
			{
				return Parent->DocTemplateWidgetData.Find(BlueprintId);
			}

			return nullptr;
		}

		FText FEditor::FTemplateWidgetSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
		{
			using namespace Frontend;

			TSharedPtr<FEditor> Parent = GetParentEditor();
			if (Parent.IsValid())
			{
				if (const FMetaSoundTemplate* Template = Parent->GetDocumentTemplate())
				{
					return Template->GetEditorOptions().ToolTip;
				}
			}
			return { };
		}

		const FLazyName FEditor::EditorName = IMetasoundEditorModule::ModuleName;

		FEditor::FEditor()
		: GraphConnectionManager(MakeUnique<FGraphConnectionManager>())
		{
		}

		void FEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
		{
			WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MetasoundEditor", "MetaSound Editor"));
			auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

			FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

			InTabManager->RegisterTabSpawner(TabNamesPrivate::GraphCanvas, FOnSpawnTab::CreateLambda(
				[
					InPageStatsWidget = PageStatsWidget,
					InMetasoundGraphEditor = MetasoundGraphEditor,
					InRenderStatsWidget = RenderStatsWidget
				] (const FSpawnTabArgs& Args)
			{
				TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(
					FText::Format(LOCTEXT("GraphCanvasTabLabel", "Graph Page: {0}"), InPageStatsWidget.IsValid() ?
						FText::FromName(InPageStatsWidget->GetDisplayedPageName()) : LOCTEXT("MetasoundGraphCanvasTitle", "MetaSound Graph")));

				TSharedRef<SOverlay> Overlay = SNew(SOverlay)
					+ SOverlay::Slot()
					[
						InMetasoundGraphEditor.ToSharedRef()
					]
					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					[
						InRenderStatsWidget.ToSharedRef()
					]
					.Padding(5.0f, 5.0f);

				if (InPageStatsWidget.IsValid())
				{
					TSharedRef<SVerticalBox> GraphStatsWidget = SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.AutoHeight()
						[
							InPageStatsWidget.ToSharedRef()
						];
					Overlay->AddSlot()
					.VAlign(VAlign_Bottom)
					[
						GraphStatsWidget
					];
				}

				SpawnedTab->SetContent(Overlay);
				return SpawnedTab;
			}))
			.SetDisplayName(LOCTEXT("GraphCanvasTab", "Viewport"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Details, FOnSpawnTab::CreateLambda([InMetasoundDetails = MetasoundDetails](const FSpawnTabArgs& Args)
			{
				return SNew(SDockTab).Label(LOCTEXT("MetaSoundDetailsTitle", "Details"))[ InMetasoundDetails.ToSharedRef() ];
			}))
			.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Members, FOnSpawnTab::CreateLambda([InGraphMembersMenu = GraphMembersMenu](const FSpawnTabArgs& Args)
			{
				TSharedRef<SDockTab> NewTab = SNew(SDockTab)
				.Label(LOCTEXT("GraphMembersMenulTitle", "Members"))
				[
					InGraphMembersMenu.ToSharedRef()
				];

				if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
				{
					NewTab->SetTabIcon(MetasoundStyle->GetBrush("MetasoundEditor.Metasound.Icon"));
				}

				return NewTab;
			}))
			.SetDisplayName(LOCTEXT("MembersTab", "Members"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon("MetaSoundStyle", "MetasoundEditor.Metasound.Icon"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Analyzers, FOnSpawnTab::CreateLambda([InAnalyzerRack = AnalyzerRack.ToWeakPtr()](const FSpawnTabArgs& Args)
			{
				TSharedRef<SDockTab> DockTab = SNew(SDockTab).Label(LOCTEXT("MetasoundAnalyzersTitle", "Analyzers"));
				if (TSharedPtr<AudioWidgets::FAudioAnalyzerRack> PinnedAnalyzerRack = InAnalyzerRack.Pin()) // AnalyzerRack only created for MetaSoundSource, not MetaSoundPatch.
				{
					TSharedRef<SWidget> AnalyzerRackWidget = PinnedAnalyzerRack->CreateWidget(DockTab, Args);
					DockTab->SetContent(AnalyzerRackWidget);
				}
				return DockTab;
			}))
			.SetDisplayName(LOCTEXT("AnalyzersTab", "Analyzers"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Pages, FOnSpawnTab::CreateLambda([this, InPagesDetails = PagesDetails](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
				.Label(LOCTEXT("MetasoundPagesDetailsTitle", "Pages"))
				[
					InPagesDetails.ToSharedRef()
				];
			}))
			.SetDisplayName(LOCTEXT("PagesTab", "Pages"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Interfaces, FOnSpawnTab::CreateLambda([InInterfacesDetails = InterfacesDetails](const FSpawnTabArgs&)
			{
				return SNew(SDockTab).Label(LOCTEXT("MetasoundInterfacesDetailsTitle", "Interfaces")) [ InInterfacesDetails.ToSharedRef() ];
			}))
			.SetDisplayName(LOCTEXT("InterfacesTab", "Interfaces"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Interface"));

			InTabManager->RegisterTabSpawner(TabNamesPrivate::Find, FOnSpawnTab::CreateLambda([InFindWidget = FindWidget](const FSpawnTabArgs&)
			{
				return SNew(SDockTab).Label(LOCTEXT("MetasoundFindTitle", "Find Results")) [ InFindWidget.ToSharedRef() ];
			}))
			.SetDisplayName(LOCTEXT("FindTab", "Find in MetaSound"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults"));
		}

		void FEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
		{
			using namespace Metasound::Editor;

			FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Analyzers);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::GraphCanvas);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Details);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Members);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Pages);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Interfaces);
			InTabManager->UnregisterTabSpawner(TabNamesPrivate::Find);

			for (auto FactoryIt = DocTemplateTabSet.CreateConstIterator(); FactoryIt; ++FactoryIt)
			{
				const FName TabId = FactoryIt.Value()->GetIdentifier();
				InTabManager->UnregisterTabSpawner(TabId);
			}
			DocTemplateTabSet.Clear();
		}

		bool FEditor::IgnoreDocTemplateVisibilitySettings()
		{
			return AssetEditorPrivate::bIgnoreDocTemplateVisibilitySettings;
		}

		bool FEditor::IsLiveAuditionEnabled()
		{
			return AssetEditorPrivate::bEnableLiveAudition;
		}

		bool FEditor::IsPlaying() const
		{
			if (UObject* MetaSound = GetMetasoundObject())
			{
				if (const UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent())
				{
					if (PreviewComponent->IsPlaying())
					{
						if (const USoundBase* Sound = PreviewComponent->Sound)
						{
							return Sound->GetUniqueID() == MetaSound->GetUniqueID();
						}
					}
				}
			}

			return false;
		}

		FEditor::~FEditor()
		{
			if (IsPlaying())
			{
				Stop();
			}

			GraphConnectionManager.Reset();
			PagesView.Reset();
			InterfacesView.Reset();
			DestroyAnalyzers();

			FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
			LevelEditor.OnMapChanged().RemoveAll(this);

			for (FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
			{
				if (UUserWidget* UserWidget = Pair.Value.Widget.Get())
				{
					if (UBlueprint* OwningBlueprint = UBlueprint::GetBlueprintFromClass(UserWidget->GetClass()))
					{
						OwningBlueprint->OnCompiled().RemoveAll(this);
					}
				}
			}
			DocTemplateWidgetData.Reset();

			check(GEditor);
			GEditor->UnregisterForUndo(this);
		}

		void FEditor::InitMetasoundEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit)
		{
			using namespace Metasound::Frontend;
			using namespace Metasound::Engine;

			check(ObjectToEdit);
			checkf(IMetasoundUObjectRegistry::Get().IsRegisteredClass(ObjectToEdit), TEXT("Object passed in was not registered as a valid metasound interface!"));

			IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			if (!MetaSoundEditorModule.IsRestrictedMode())
			{
				IMetasoundEngineModule& MetaSoundEngineModule = FModuleManager::GetModuleChecked<IMetasoundEngineModule>("MetaSoundEngine");
				MetaSoundEngineModule.PrimeAssetManager();
			}

			// Support undo/redo
			ObjectToEdit->SetFlags(RF_Transactional);

			// Typically sounds are versioned on load of the asset. There are certain instances where an asset is not versioned on reload.
			// This forces versioning the document on load prior to the editor synchronizing and building the editor graph if an asset is
			// reloaded while the asset editor was open.
			Builder.Reset(&Engine::FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*ObjectToEdit));
			DocListener = MakeShared<FDocumentListener>(StaticCastSharedRef<FEditor>(AsShared()));
			Builder->AddTransactionListener(DocListener->AsShared());

			// Stat widgets are potentially intractable with transaction listener, so create then here
			SAssignNew(PageStatsWidget, SPageStats)
				.Visibility(EVisibility::HitTestInvisible);

			SAssignNew(RenderStatsWidget, SRenderStats)
				.Visibility(EVisibility::HitTestInvisible);

			if (FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(ObjectToEdit))
			{
				constexpr bool bForceNodeCreation = false;
				FInputNodeTemplate::GetChecked().Inject(Builder->GetBuilder(), bForceNodeCreation);

				// Ensures validation is re-run on re-opening of the editor.
				// This is needed to refresh errors potentially caused by unloading of
				// references (ex. if a referenced asset is force deleted in the editor).
				MetaSoundAsset->GetModifyContext().SetForceRefreshViews();
			}

			GEditor->RegisterForUndo(this);

			FGraphEditorCommands::Register();
			FEditorCommands::Register();
			BindGraphCommands();

			// If sound was already playing in the editor (ex. from ContentBrowser),
			// restart to synchronize visual state of editor (ex. volume meter analysis
			// via transient AudioBus, PlayTime, etc.). If playing, registration is not
			// required here as it will be handled in play call below after UI is initialized
			const bool bRestartSound = IsPlaying();
			if (!bRestartSound)
			{
				FGraphBuilder::RegisterGraphWithFrontend(*ObjectToEdit);
			}

			RefreshEditorContext(*ObjectToEdit);
			CreateInternalWidgets(*ObjectToEdit);

			if (UMetaSoundSource* MetaSoundSource = Cast<UMetaSoundSource>(ObjectToEdit))
			{
				CreateAnalyzers(*MetaSoundSource);
			}

			TSharedRef<FTabManager::FStack> DetailsStack = FTabManager::NewStack()
				->SetSizeCoefficient(0.50f)
				->SetHideTabWell(false)
				->AddTab(TabNamesPrivate::Details, ETabState::OpenedTab)
				->AddTab(TabNamesPrivate::Pages, ETabState::OpenedTab);

			GraphCanvasTabStack = FTabManager::NewStack()
				->SetSizeCoefficient(0.8f)
				->SetHideTabWell(true)
				->SetExtensionId(TabNamesPrivate::GraphCanvas)
				->AddTab(TabNamesPrivate::GraphCanvas, ETabState::OpenedTab);

			for (const FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
			{
				GraphCanvasTabStack->AddTab(Pair.Value.TabInfo.TabName, ETabState::ClosedTab);
			}

			const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_MetasoundEditor_Layout_v17")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split(FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.15f)
						->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.25f)
							->SetHideTabWell(false)
							->AddTab(TabNamesPrivate::Members, ETabState::OpenedTab)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)
							->SetHideTabWell(true)
							->AddTab(TabNamesPrivate::Interfaces, ETabState::OpenedTab)
						)
						->Split
						(
							DetailsStack
						)
					)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.77f)
						->SetOrientation(Orient_Vertical)
						->Split
						(
							GraphCanvasTabStack.ToSharedRef()
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->SetHideTabWell(true)
							->AddTab(TabNamesPrivate::Find, ETabState::OpenedTab)
						)
					)

					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.08f)
						->SetHideTabWell(true)
						->AddTab(TabNamesPrivate::Analyzers, ETabState::OpenedTab)
					)
				)
			);

			constexpr bool bCreateDefaultStandaloneMenu = true;
			constexpr bool bCreateDefaultToolbar = true;
			constexpr bool bToolbarFocusable = false;
			constexpr bool bUseSmallToolbarIcons = true;

			FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TEXT("MetasoundEditorApp"), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit, bToolbarFocusable, bUseSmallToolbarIcons);

			ExtendToolbarInternal();
			RegenerateMenusAndToolbars();

			ReloadAllDocTemplateTabs();

			NotifyDocumentVersioned();

			if (bRestartSound)
			{
				Play();
			}
			else
			{
				constexpr bool bIsPlaying = false;
				UpdatePageInfo(bIsPlaying);
				UpdateRenderInfo(bIsPlaying);
			}

			RefreshExecVisibility(Builder->GetConstBuilder().GetBuildPageID());

			FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
			const bool bAssetScanComplete = AssetManager.IsInitialAssetScanComplete();
			const int32 NumProcessingAssets = AssetManager.GetActiveAsyncLoadRequestCount();
			if (!bAssetScanComplete || NumProcessingAssets > 0)
			{
				NotifyAssetLoadingInProgress(NumProcessingAssets);
			}

			UpdateEnabledTabs();
		}

		UObject* FEditor::GetMetasoundObject() const
		{
			if (HasEditingObject())
			{
				return GetEditingObject();
			}

			// During init, editing object isn't yet set by underlying EditorToolkit::Init.
			// If it hasn't been cached off, use the builder's pointer which is set
			// early in editor initialization.
			if (Builder.IsValid())
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
				if (DocBuilder.IsValid())
				{
					return &DocBuilder.CastDocumentObjectChecked<UObject>();
				}
			}

			return nullptr;
		}

		void FEditor::SetSelection(const TArray<UObject*>& SelectedObjects, bool bInvokeTabOnSelectionSet)
		{
			if (GraphMembersMenu.IsValid())
			{
				// Only support menu selection of a single object until multiselect functionality is added 
				if (SelectedObjects.Num() == 1)
				{
					if (UMetasoundEditorGraphMember* Member = Cast<UMetasoundEditorGraphMember>(SelectedObjects[0]))
					{
						const FName ActionName = Member->GetMemberName();
						GraphMembersMenu->SelectItemByName(ActionName, ESelectInfo::Direct, static_cast<int32>(Member->GetSectionID()));
					}
				}
			}

			if (MetasoundDetails.IsValid())
			{
				if (SelectedObjects.IsEmpty())
				{
					if (bInvokeTabOnSelectionSet && TabManager.IsValid())
					{
						if (ShowPageGraphDetails())
						{
							TabManager->TryInvokeTab(TabNamesPrivate::Pages);
						}
					}
				}
				else
				{
					MetasoundDetails->SetObjects(SelectedObjects);
					MetasoundDetails->HideFilterArea(false);
					if (bInvokeTabOnSelectionSet && TabManager.IsValid())
					{
						TabManager->TryInvokeTab(TabNamesPrivate::Details);
					}
				}
			}
		}

		bool FEditor::ShowPageGraphDetails() const
		{
			if (Builder.IsValid())
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
				if (DocBuilder.IsValid())
				{
					const FMetasoundFrontendDocument& Document = DocBuilder.GetConstDocumentChecked();
					if (const FMetaSoundFrontendDocumentTemplate* DocTemplate = Document.Template.GetPtr())
					{
						if (!DocTemplate->GetEditorOptions().bPageGraphEditingEnabled)
						{
							return false;
						}
					}

					const bool bLastGraph = Document.RootGraph.GetConstGraphPages().Num() == 1;
					const bool bHasLocalPageValues = !bLastGraph && Document.RootGraph.FindConstGraph(Frontend::DefaultPageID) != nullptr;
					return Editor::IsPageGraphEditingEnabled(DocBuilder, bHasLocalPageValues);
				}
			}

			return false;
		}

		bool FEditor::GetBoundsForSelectedNodes(FSlateRect& Rect, float Padding)
		{
			return MetasoundGraphEditor->GetBoundsForSelectedNodes(Rect, Padding);
		}

		FName FEditor::GetToolkitFName() const
		{
			return FEditor::EditorName;
		}

		FText FEditor::GetBaseToolkitName() const
		{
			return LOCTEXT("AppLabel", "MetaSound Editor");
		}

		FString FEditor::GetWorldCentricTabPrefix() const
		{
			return LOCTEXT("WorldCentricTabPrefix", "MetaSound ").ToString();
		}

		FLinearColor FEditor::GetWorldCentricTabColorScale() const
		{
			if (const ISlateStyle* MetaSoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
			{
				UObject* MetaSound = GetMetasoundObject();
				if (UMetaSoundSource* MetaSoundSource = Cast<UMetaSoundSource>(MetaSound))
				{
					return MetaSoundStyle->GetColor("MetaSoundSource.Color");
				}

				if (UMetaSoundPatch* MetaSoundPatch = Cast<UMetaSoundPatch>(MetaSound))
				{
					return MetaSoundStyle->GetColor("MetaSoundPatch.Color");
				}
			}

			return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
		}

		const FSlateBrush* FEditor::GetDefaultTabIcon() const
		{
			using namespace Frontend;

			FString IconName = TEXT("MetasoundEditor");
			if (IsPlaying())
			{
				IconName += TEXT(".Play");
			}
			else
			{
				UObject* MetaSound = GetMetasoundObject();
				if (UMetaSoundSource* MetaSoundSource = Cast<UMetaSoundSource>(MetaSound))
				{
					IconName += TEXT(".MetasoundSource");
				}
				else if (UMetaSoundPatch* MetaSoundPatch = Cast<UMetaSoundPatch>(MetaSound))
				{
					IconName += TEXT(".MetasoundPatch");
				}

				TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = MetaSound;
				if (DocInterface.GetObject() != nullptr)
				{
					const UStruct* Struct = DocInterface->GetConstDocument().Template.GetScriptStruct();
					if (Struct && Struct->IsChildOf(FPresetTemplate::StaticStruct()))
					{
						IconName += TEXT(".Preset");
					}
				}

				IconName += TEXT(".Icon");
			}

			return &Style::GetSlateBrushSafe(FName(*IconName));
		}

		FLinearColor FEditor::GetDefaultTabColor() const
		{
			if (UObject* MetaSound = GetMetasoundObject())
			{
				if (IsPlaying())
				{
					if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
					{
						if (UMetaSoundSource* MetaSoundSource = Cast<UMetaSoundSource>(MetaSound))
						{
							return MetasoundStyle->GetColor("MetaSoundSource.Color");
						}

						if (UMetaSoundPatch* MetaSoundPatch = Cast<UMetaSoundPatch>(MetaSound))
						{
							return MetasoundStyle->GetColor("MetaSoundPatch.Color");
						}
					}
				}
			}	

			return FAssetEditorToolkit::GetDefaultTabColor();
		}

		FName FEditor::GetEditorName() const 
		{
			return FEditor::EditorName;
		}

		void FEditor::PostUndo(bool bSuccess)
		{
			if (MetasoundGraphEditor.IsValid())
			{
				MetasoundGraphEditor->ClearSelectionSet();
				MetasoundGraphEditor->NotifyGraphChanged();
			}

			FSlateApplication::Get().DismissAllMenus();

			// In case of undoing 'clear document config' refresh toolbar to include button
			if (UToolMenus* ToolMenus = UToolMenus::Get())
			{
				ToolMenus->RefreshAllWidgets();
			}

			// Playback must be stopped if undoing a page change transaction
			bool bStopPlayback = !Builder.IsValid() || !PageStatsWidget.IsValid();
			if (!bStopPlayback)
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
				bStopPlayback = DocBuilder.GetBuildPageID() != PageStatsWidget->GetDisplayedPageID();
			}

			SyncAuditionState();

			if (bStopPlayback)
			{
				Stop();
			}

			UpdatePageInfo(IsPlaying());
			bRefreshGraph = true;
		}

		void FEditor::NotifyAssetLoadingInProgress(int32 NumProcessing)
		{
			if (MetasoundGraphEditor.IsValid())
			{
				const FText CloseNotificationText = LOCTEXT("MetaSoundScanInProgressNotificationButtonText", "Close");

				FSimpleDelegate OnCloseNotification = FSimpleDelegate::CreateLambda([this]()
				{
					if (LoadingNotificationPtr)
					{
						LoadingNotificationPtr->Fadeout();
						LoadingNotificationPtr.Reset();
					}
				});

				const FText InitialText = NumProcessing > 0
					? FText::Format(AssetEditorPrivate::LoadingAssetsFormat, NumProcessing)
					: LOCTEXT("AssetScanOrTagRetrievelInProgress", "Waiting for Asset Scan/AssetTag Registration to Finish...");

				FNotificationInfo Info(InitialText);
				Info.SubText = LOCTEXT("MetaSoundScanInProgressNotificationSubText", "Class selector results may be incomplete");
				Info.bUseThrobber = true;
				Info.bFireAndForget = false;
				Info.bUseSuccessFailIcons = false;
				Info.FadeOutDuration = 1.0f;
				Info.ButtonDetails.Add(FNotificationButtonInfo(CloseNotificationText, FText(), OnCloseNotification));

				LoadingNotificationPtr = MetasoundGraphEditor->AddNotification(Info);
				if (LoadingNotificationPtr.IsValid())
				{
					LoadingNotificationPtr->SetVisibility(EVisibility::Visible);
					LoadingNotificationPtr->SetCompletionState(SNotificationItem::CS_Pending);
				}
			}
		}

		void FEditor::NotifyAssetLoadingComplete()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				if (LoadingNotificationPtr.IsValid())
				{
					LoadingNotificationPtr->Fadeout();
					LoadingNotificationPtr.Reset();
				}

				FNotificationInfo Info(LOCTEXT("MetaSoundScanInProgressNotification", "MetaSound Asset Registration Complete"));
				Info.bFireAndForget = true;
				Info.bUseSuccessFailIcons = true;
				Info.ExpireDuration = 3.0f;
				Info.FadeOutDuration = 1.0f;

				MetasoundGraphEditor->AddNotification(Info, true /* bSuccess */);
			}
		}

		void FEditor::NotifyDocumentVersioned()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& MetaSoundGraph = GetMetaSoundGraphChecked();
				UObject* MetaSoundObject = GetMetasoundObject();
				if (FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSoundObject))
				{
					if (MetaSoundAsset->GetVersionedOnLoad())
					{
						MetaSoundAsset->ClearVersionedOnLoad();

						TScriptInterface<IMetaSoundDocumentInterface> DocInterface = MetaSoundAsset->GetOwningAsset();
						check(DocInterface.GetObject());

						const FString VersionString = DocInterface->GetConstDocument().Metadata.Version.Number.ToString();
						FText Msg = FText::Format(LOCTEXT("MetaSoundDocumentVersioned",
							"Document data migrated, interfaces updated, and/or properties versioned on "
							"load. (Current document version: '{0}'). See log for more details."), FText::FromString(VersionString));
						FNotificationInfo Info(Msg);
						Info.bFireAndForget = true;
						Info.bUseSuccessFailIcons = false;
						Info.ExpireDuration = 5.0f;

						MetasoundGraphEditor->AddNotification(Info, false /* bSuccess */);
						MetaSoundObject->MarkPackageDirty();
					}
				}
			}
		}

		void FEditor::NotifyNodePasteFailure_MultipleVariableSetters()
		{
			FNotificationInfo Info(LOCTEXT("NodePasteFailed_MultipleVariableSetters", "Node(s) not pasted: Only one variable setter node possible per graph."));
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = false;
			Info.ExpireDuration = 5.0f;

			MetasoundGraphEditor->AddNotification(Info, false /* bSuccess */);
		}
		
		void FEditor::NotifyNodePasteFailure_MultipleOutputs()
		{
			FNotificationInfo Info(LOCTEXT("NodePasteFailed_MultipleOutputs", "Node(s) not pasted: Only one output node possible per graph."));
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = false;
			Info.ExpireDuration = 5.0f;

			MetasoundGraphEditor->AddNotification(Info, false /* bSuccess */);
		}

		void FEditor::NotifyNodePasteFailure_ReferenceLoop()
		{
			FNotificationInfo Info(LOCTEXT("NodePasteFailed_ReferenceLoop", "Node(s) not pasted: Nodes would create asset reference cycle."));
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = false;
			Info.ExpireDuration = 5.0f;

			MetasoundGraphEditor->AddNotification(Info, false /* bSuccess */);
		}

		void FEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
		{
			if (MetasoundGraphEditor.IsValid() && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				// If a property change event occurs outside of the metasound UEdGraph and results in the metasound document changing,
				// then the document and the UEdGraph need to be synchronized. There may be a better trigger for this call to reduce
				// the number of times the graph is synchronized.
				if (UObject* MetaSound = GetMetasoundObject())
				{
					if (FMetasoundAssetBase* Asset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSound))
					{
						Asset->GetModifyContext().SetDocumentModified();
					}
				}
			}
		}

		void FEditor::CreateInternalWidgets(UObject& MetaSound)
		{
			CreateGraphEditorWidget(MetaSound);

			FDetailsViewArgs Args;
			Args.bHideSelectionTip = true;
			Args.NotifyHook = this;

			SAssignNew(GraphMembersMenu, SGraphActionMenu, false)
				.AlphaSortItems(true)
				.AutoExpandActionMenu(true)
				.OnActionDoubleClicked(this, &FEditor::OnMemberActionDoubleClicked)
				.OnActionDragged(this, &FEditor::OnActionDragged)
				.OnActionMatchesName(this, &FEditor::HandleActionMatchesName)
				.OnActionSelected(this, &FEditor::OnActionSelected)
				.OnCollectAllActions(this, &FEditor::CollectAllActions)
				.OnCollectStaticSections(this, &FEditor::CollectStaticSections)
				.OnContextMenuOpening(this, &FEditor::OnContextMenuOpening)
				.OnCreateWidgetForAction(this, &FEditor::OnCreateWidgetForAction)
				.OnCanRenameSelectedAction(this, &FEditor::CanRenameOnActionNode)
				.OnGetFilterText(this, &FEditor::GetFilterText)
				.OnGetSectionTitle(this, &FEditor::OnGetSectionTitle)
				.OnGetSectionWidget(this, &FEditor::OnGetMenuSectionWidget)
				.OnCreateCustomRowExpander_Lambda([](const FCustomExpanderData& InCustomExpanderData)
				{
					return SNew(SMetasoundActionMenuExpanderArrow, InCustomExpanderData);
				})
				.UseSectionStyling(true);

			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			MetasoundDetails = PropertyModule.CreateDetailView(Args);
			MetasoundDetails->SetExtensionHandler(MakeShared<FMetaSoundNodeExtensionHandler>());

			// Set details selection to the MetaSound's source settings 
			// Don't invoke tab as this can be called in response
			// to opening multiple assets, and the higher level
			// request handles tab invocation/focus
			constexpr bool bInvokeTabOnSelectionSet = false;
			SetSelection({ &MetaSound }, bInvokeTabOnSelectionSet);
			InterfacesDetails = PropertyModule.CreateDetailView(Args);
			if (InterfacesDetails.IsValid())
			{
				InterfacesView = TStrongObjectPtr(NewObject<UMetasoundInterfacesView>());
				InterfacesView->SetMetasound(&MetaSound);
				const TArray<UObject*> InterfacesViewObj{ InterfacesView.Get() };

				InterfacesDetails->SetObjects(InterfacesViewObj);
				InterfacesDetails->HideFilterArea(true);
			}

			PagesDetails = PropertyModule.CreateDetailView(Args);
			if (PagesDetails.IsValid())
			{
				PagesView = TStrongObjectPtr(NewObject<UMetasoundPagesView>());
				PagesView->SetMetasound(&MetaSound);
				const TArray<UObject*> PagesViewObj{ PagesView.Get() };

				PagesDetails->SetObjects(PagesViewObj);
				PagesDetails->HideFilterArea(true);

				TAttribute<bool> EnabledAttr = TAttribute<bool>::CreateSPLambda(AsShared(), [this]()
				{
					return ShowPageGraphDetails();
				});
				TAttribute<EVisibility> VisibilityAttr = TAttribute<EVisibility>::CreateSPLambda(AsShared(), [this]()
				{
					return ShowPageGraphDetails() ? EVisibility::Visible : EVisibility::Hidden;
				});
				PagesDetails->SetEnabled(EnabledAttr);
				PagesDetails->SetVisibility(VisibilityAttr);
			}

			if (Builder.IsValid() && !Builder->GetConstBuilder().GetIsGraphEditable())
			{
				FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
				LevelEditor.OnMapChanged().AddSP(this, &FEditor::ChangeDocTemplateWidgetWorld);
			}

			Palette = SNew(SMetasoundPalette);

			FindWidget = SNew(SFindInMetasound, SharedThis(this));
		}

		// TODO: Tie in rename on GraphActionMenu.  For now, just renameable via field in details
		bool FEditor::CanRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const
		{
			return false;
		}

		void FEditor::CreateAnalyzers(UMetaSoundSource& MetaSoundSource)
		{
			if (!AnalyzerRack.IsValid())
			{
				using namespace AudioWidgets;

				// Set params so that rack layout is stored specific to Metasound Editor and the Metasound Editor styles are used for analyzer widgets:
				const FAudioAnalyzerRack::FRackConstructParams Params
				{
					.TabManagerLayoutName = TEXT("MetasoundEditor_FAudioAnalyzerRack_v0"),
					.StyleSet = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"),
					.EditorSettingsClass = UMetasoundEditorSettings::StaticClass(),
				};
				AnalyzerRack = MakeShared<FAudioAnalyzerRack>(Params);
			}

			if (ensure(GEditor))
			{
				// Init (or reinit) the analyzer rack with the correct num channels:
				AnalyzerRack->Init(MetaSoundSource.NumChannels, GEditor->GetMainAudioDeviceID());
			}
			else
			{
				AnalyzerRack->DestroyAnalyzers();
			}
		}

		void FEditor::ReloadAllDocTemplateTabs()
		{
			using namespace Frontend;

			if (TabManager.IsValid())
			{
				for (auto FactoryIt = DocTemplateTabSet.CreateConstIterator(); FactoryIt; ++FactoryIt)
				{
					const FName TabId = FactoryIt.Value()->GetIdentifier();
					if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(TabId))
					{
						Tab->RequestCloseTab();
					}
					TabManager->UnregisterTabSpawner(TabId);
				}
			}

			DocTemplateTabSet.Clear();

			for (TPair<FBlueprintId, FUserWidgetData>& WidgetData : DocTemplateWidgetData)
			{
				if (UUserWidget* Widget = WidgetData.Value.Widget.Get())
				{
					if (UBlueprint* OwningBlueprint = UBlueprint::GetBlueprintFromClass(Widget->GetClass()))
					{
						OwningBlueprint->OnCompiled().RemoveAll(this);
					}
				}
			}
			DocTemplateWidgetData.Reset();

			RebuildDocTemplateUserWidgets();

			TMap<FName, FSlateIcon> TabIcons;
			for (const FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
			{
				const FName TabName = Pair.Value.TabInfo.TabName;
				FNameBuilder FactoryName(TabNamesPrivate::DocTemplateWidgets_Factory);
				if (!TabName.IsNone())
				{
					FactoryName.Appendf(TEXT("_%s"), *TabName.ToString());
				}
				DocTemplateTabSet.RegisterFactory(MakeShared<FTemplateWidgetSummoner>(*FactoryName, Pair.Key, AsShared()));
				TabIcons.Add(TabName, FSlateIcon(Pair.Value.TabInfo.StyleName, Pair.Value.TabInfo.TabIcon));
			}

			if (TabManager.IsValid())
			{
				for (auto FactoryIt = DocTemplateTabSet.CreateConstIterator(); FactoryIt; ++FactoryIt)
				{
					FTabSpawnerEntry& SpawnerEntry = FactoryIt.Value()->RegisterTabSpawner(TabManager.ToSharedRef(), nullptr)
					.SetGroup(WorkspaceMenuCategory.ToSharedRef());

					const FName TabId = FactoryIt.Value()->GetIdentifier();
					if (const FSlateIcon* TabIcon = TabIcons.Find(TabId))
					{
						SpawnerEntry.SetIcon(*TabIcon);
					}

				}
			}
		}

		void FEditor::RebuildDocTemplateUserWidgets(const FGuid& TargetBlueprintId)
		{
			using namespace AssetEditorPrivate;
			using namespace Frontend;

			UMetaSoundBuilderBase* BuilderBase = Builder.Get();
			if (!BuilderBase)
			{
				return;
			}

			const FMetaSoundFrontendDocumentBuilder& DocBuilder = BuilderBase->GetConstBuilder();
			if (!DocBuilder.IsValid())
			{
				return;
			}

			TConstStructView<FMetaSoundTemplate> DocTemplate = DocBuilder.GetConstDocumentTemplate();
			if (!DocTemplate.IsValid() || !DocTemplate.GetPtr()->GetEditorOptions().bCustomTabEnabled)
			{
				return;
			}

			if (!GEditor || !GEngine)
			{
				return;
			}

			UWorld* World = GEditor->GetEditorWorldContext().World();
			UAudioWidgetSubsystem* WidgetSubsystem = GEngine->GetEngineSubsystem<UAudioWidgetSubsystem>();
			if (!World || !WidgetSubsystem)
			{
				return;
			}

			// Preset separated out for back compat
			if (TFunction<bool(UUserWidget*)> PresetFilterFunction = GetPresetWidgetFilterFunction(DocBuilder); PresetFilterFunction.IsSet())
			{
				TFunction<FMetaSoundTemplateWidgetTabInfo(UUserWidget*)> OnPresetConstruct = [&DocTemplate, &BuilderBase, &TargetBlueprintId](UUserWidget* UserWidget)
				{
					IMetaSoundPresetWidgetInterface::Execute_OnConstructed(UserWidget, BuilderBase);

					FMetaSoundTemplateWidgetTabInfo TabInfo;
					IMetaSoundPresetWidgetInterface::Execute_OnInitializingTab(UserWidget, TabInfo);
					return TabInfo;
				};
				RebuildUserWidgets(*World, *WidgetSubsystem,
					UMetaSoundPresetWidgetInterface::StaticClass(),
					MoveTemp(PresetFilterFunction),
					MoveTemp(OnPresetConstruct),
					TargetBlueprintId);
				return;
			}

			TFunction<bool(UUserWidget*)> DocTemplateFilterFunction = [&DocTemplate, BuilderBase](UUserWidget* UserWidget)
			{
				bool bIsSupportedTemplate = true;
				if (const UScriptStruct* ScriptStruct = DocTemplate.GetScriptStruct())
				{
					IMetaSoundTemplateWidgetInterface::Execute_IsSupportedTemplate(
						UserWidget,
						ScriptStruct->GetFName(),
						BuilderBase,
						bIsSupportedTemplate);
				}
				return bIsSupportedTemplate;
			};

			TFunction<FMetaSoundTemplateWidgetTabInfo(UUserWidget*)> OnDocTemplateTabConstruct = [&DocTemplate, BuilderBase](UUserWidget* UserWidget)
			{
				TScriptInterface<IMetaSoundTemplateWidgetInterface> WidgetInterface(UserWidget);
				if (ensure(WidgetInterface.GetObject()))
				{
					IMetaSoundTemplateWidgetInterface::Execute_OnBuilderSet(UserWidget, BuilderBase);
				}

				TSubclassOf<UMetaSoundTemplateViewModelBase> VMClass;
				IMetaSoundTemplateWidgetInterface::Execute_OnInitializingViewModel(UserWidget, VMClass);
				UMetaSoundTemplateViewModelBase::ConstructWidgetViewModel(UserWidget, *BuilderBase, DocTemplate, VMClass);

				FMetaSoundTemplateWidgetTabInfo TabInfo;
				IMetaSoundTemplateWidgetInterface::Execute_OnInitializingTab(UserWidget, TabInfo);
				return TabInfo;
			};

			RebuildUserWidgets(*World, *WidgetSubsystem,
				UMetaSoundTemplateWidgetInterface::StaticClass(),
				MoveTemp(DocTemplateFilterFunction),
				MoveTemp(OnDocTemplateTabConstruct),
				TargetBlueprintId);
		}

		void FEditor::OnDocTemplateWidgetBlueprintCompiled(UBlueprint* Blueprint)
		{
			using namespace AssetEditorPrivate;

			if (Blueprint)
			{
				const FGuid BlueprintId = Blueprint->GetBlueprintGuid();
				if (const FUserWidgetData* WidgetData = DocTemplateWidgetData.Find(BlueprintId))
				{
					if (TabManager.IsValid())
					{
						if (TSharedPtr<SDockTab> DockTab = TabManager->FindExistingLiveTab(WidgetData->TabInfo.TabName))
						{
							DockTab->SetContent(SNullWidget::NullWidget);
						}
					}
				}

				RebuildDocTemplateUserWidgets(BlueprintId);

				if (const FUserWidgetData* WidgetData = DocTemplateWidgetData.Find(BlueprintId))
				{
					if (TabManager.IsValid())
					{
						if (TSharedPtr<SDockTab> DockTab = TabManager->FindExistingLiveTab(WidgetData->TabInfo.TabName))
						{
							if (UUserWidget* UserWidget = WidgetData->Widget.Get())
							{
								DockTab->SetContent(UserWidget->TakeWidget());
							}
						}
					}
				}
			}
		}
		
		void FEditor::ChangeDocTemplateWidgetWorld(UWorld* World, EMapChangeType MapChangeType)
		{
			switch (MapChangeType)
			{
				case EMapChangeType::TearDownWorld:
				{
					// Moves the UMG widgets to the transient package if the
					// World they were built with is being torn down.
					for (FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
					{
						UUserWidget* DocTemplateWidget = Pair.Value.Widget.Get();
						if (DocTemplateWidget && World == DocTemplateWidget->GetWorld())
						{
							DocTemplateWidget->Rename(nullptr, GetTransientPackage());
						}
					}
				}
				break;

				case EMapChangeType::LoadMap:
				case EMapChangeType::NewMap:
				{
					ReloadAllDocTemplateTabs();
				}
				break;

				case EMapChangeType::SaveMap:
				default:
				break;
			}
		}

		TSharedRef<SWidget> FEditor::CreateAuditionMenuOptions()
		{
			TSharedPtr<FUICommandList> Commands = MakeShared<FUICommandList>();
			constexpr bool bShouldCloseWindowAfterMenuSelection = false;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Commands);
			CreateAuditionPageSubMenuOptions(MenuBuilder);
			TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
			TWeakObjectPtr<UMetaSoundBuilderBase> WeakBuilderPtr(Builder.Get());
			MenuWidget->SetVisibility(TAttribute<EVisibility>::Create([WeakBuilderPtr]()
			{
				return IsPageAuditioningEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
			}));
			return MenuWidget;
		}

		TSharedRef<SWidget> FEditor::CreateTemplateMenuOptions()
		{
			TSharedPtr<FUICommandList> Commands = MakeShared<FUICommandList>();
			constexpr bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Commands);
			CreateTemplateSubMenuOptions(MenuBuilder);
			TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
			TWeakObjectPtr<UMetaSoundBuilderBase> WeakBuilderPtr(Builder.Get());
			return MenuWidget;
		}

		void FEditor::CreateAuditionPageSubMenuOptions(FMenuBuilder& MenuBuilder)
		{
			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			if (!Settings)
			{
				return;
			}

			MenuBuilder.BeginSection("SetAuditionPlatformSectionHeader", LOCTEXT("AuditionPlatformSectionName", "Audition Platform"));
			{
				auto CreatePlatformEntry = [this, &MenuBuilder](FName PlatformName, const FText& PlatformText, const FText& Tooltip)
				{
					FUIAction SetPlatformAction;
					SetPlatformAction.ExecuteAction = FExecuteAction::CreateLambda([this, PlatformName]()
					{
						if (UMetasoundEditorSettings* EditorSettings = GetMutableDefault<UMetasoundEditorSettings>())
						{
							EditorSettings->SetAuditionPlatform(PlatformName);
							Stop();
							SyncAuditionState();
						}
					});
					SetPlatformAction.GetActionCheckState = FGetActionCheckState::CreateLambda([this, PlatformName]()
					{
						if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
						{
							if (EditorSettings->GetAuditionPlatform() == PlatformName)
							{
								return ECheckBoxState::Checked;
							}
						}

						return ECheckBoxState::Unchecked;
					});

					MenuBuilder.AddMenuEntry(PlatformText, Tooltip, FSlateIcon(), SetPlatformAction, { }, EUserInterfaceActionType::RadioButton);
				};

				TArray<FName> AuditionPlatforms = UMetasoundEditorSettings::GetAuditionPlatformNames();

				// Protects against stale setting not showing after platform values are manipulated just for visibility
				if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
				{
					AuditionPlatforms.AddUnique(EditorSettings->GetAuditionPlatform());
				}

				for (const FName& PlatformName : AuditionPlatforms)
				{
					const FText PlatformText = FText::FromName(PlatformName);
					FText Tooltip;
					if (PlatformName == UMetasoundEditorSettings::DefaultAuditionPlatform)
					{
						Tooltip = LOCTEXT("SetDefaultPlatformToolTip", "Sets the page audition platform to 'Default', which follows target/cook settings for unspecified platforms.");
					}
					else if (PlatformName == UMetasoundEditorSettings::EditorAuditionPlatform)
					{
						Tooltip = LOCTEXT("SetEditorPlatformToolTip", "Sets the page audition platform to 'Editor', which ignores any explicit target/cook settings.");
					}
					else
					{
						Tooltip = FText::Format(LOCTEXT("SetAuditionPlatformToolTip", "Sets the page audition platform to '{0}'."), PlatformText);
					}

					CreatePlatformEntry(PlatformName, PlatformText, Tooltip);
				}
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("SetAuditionPageSectionHeader", LOCTEXT("SetAuditionPageDescription", "Audition Page"));
			{
				TSharedRef<FEditor> ThisShared = StaticCastSharedRef<FEditor>(AsShared());
				const FText FocusPageTooltip = LOCTEXT("EnableAuditionFocusPageTooltip",
					"Synchronizes audition page to currently focused graph page.\r\n"
					"If focused graph page is non-targetable for the selected audition\r\n"
					"platform, will issue warning behavior is not reflected at runtime\r\n"
					"(see 'MetaSound Editor' user settings).");
				MenuBuilder.AddWidget(
					SNew(SCheckBox)
					.OnCheckStateChanged_Lambda([EditorPtr = TWeakPtr<FEditor>(ThisShared)](ECheckBoxState State)
					{
						TSharedPtr<FEditor> ThisEditor = EditorPtr.Pin();
						if (!ThisEditor.IsValid())
						{
							return;
						}
						if (UMetasoundEditorSettings* EdSettings = GetMutableDefault<UMetasoundEditorSettings>())
						{
							switch (State)
							{
								case ECheckBoxState::Checked:
								{
									EdSettings->SetAuditionPageMode(EAuditionPageMode::Focused);
									ThisEditor->Stop();
									ThisEditor->SyncAuditionState();
									break;
								}

								case ECheckBoxState::Unchecked:
								case ECheckBoxState::Undetermined:
								default:
								{
									EdSettings->SetAuditionPageMode(EAuditionPageMode::User);
								}
								break;
							}
						}
					})
					.IsChecked_Lambda([]()
					{
						if (const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>())
						{
							if (EdSettings->GetAuditionPageMode() == EAuditionPageMode::Focused)
							{
								return ECheckBoxState::Checked;
							}
						}

						return ECheckBoxState::Unchecked;
					})
					.ToolTipText(FocusPageTooltip),
					LOCTEXT("EnableAuditionAndFocusGraphPageSync", "Sync With Graph Page"),
					true,
					true,
					FocusPageTooltip
				);

				auto TryAddPageEntry = [EditorPtr = TWeakPtr<FEditor>(ThisShared), &MenuBuilder](const FMetaSoundPageSettings& PageSettings)
				{
					const FName AuditionPage = PageSettings.Name;
					const FText PageText = FText::FromName(PageSettings.Name);
					FUIAction SetTargetPageAction;

					SetTargetPageAction.ExecuteAction = FExecuteAction::CreateLambda([EditorPtr, AuditionPage]()
					{
						if (UMetasoundEditorSettings* EditorSettings = GetMutableDefault<UMetasoundEditorSettings>())
						{
							if (EditorSettings->GetAuditionPage() != AuditionPage)
							{
								EditorSettings->SetAuditionPage(AuditionPage);
								if (TSharedPtr<FEditor> ThisEditor = EditorPtr.Pin())
								{
									ThisEditor->Stop();
									ThisEditor->SyncAuditionState();
								}
							}
						}
					});

					SetTargetPageAction.GetActionCheckState = FGetActionCheckState::CreateLambda([AuditionPage]()
					{
						if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
						{
							if (EditorSettings->GetAuditionPage() == AuditionPage)
							{
								return ECheckBoxState::Checked;
							}
						}

						return ECheckBoxState::Unchecked;
					});
					SetTargetPageAction.CanExecuteAction = FCanExecuteAction::CreateLambda([AuditionPage]()
					{
						if (const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>())
						{
							return EdSettings->GetAuditionPageMode() == EAuditionPageMode::User;
						}

						if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
						{
							return Settings->FindPageSettings(AuditionPage) != nullptr;
						}
						return false;
					});

					TAttribute<FText> TooltipAttribute = TAttribute<FText>::CreateLambda([AuditionPage, PageText]()
					{
						if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
						{
							if (EditorSettings->GetAuditionPlatform() != UMetasoundEditorSettings::EditorAuditionPlatform)
							{
								if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
								{
									if (const FMetaSoundPageSettings* PageSetting = Settings->FindPageSettings(AuditionPage))
									{
										if (!PageSetting->PlatformCanTargetPage(EditorSettings->GetAuditionPlatform()))
										{
											return FText::Format(LOCTEXT("AuditionPageInvalidForPlatformToolTip", "Platform '{0}' does not target page '{1}'. See 'MetaSound' Project Settings"), FText::FromName(EditorSettings->GetAuditionPlatform()), PageText);
										}
									}
								}
							}

							const bool bUserAuditionMode = EditorSettings->GetAuditionPageMode() == EAuditionPageMode::User;
							if (bUserAuditionMode)
							{
								if (EditorSettings->GetAuditionPage() != AuditionPage)
								{
									return FText::Format(LOCTEXT("SetAuditionPageToolTip", "Sets the user's editor AuditionPage setting to '{0}'."), PageText);
								}
							}
						}

						return FText();
					});

					MenuBuilder.AddMenuEntry(PageText,
						MoveTemp(TooltipAttribute),
						FSlateIcon(),
						SetTargetPageAction,
						{ },
						EUserInterfaceActionType::RadioButton);
				};

				Settings->IteratePageSettings(TryAddPageEntry);
			}
			MenuBuilder.EndSection();
		}

		void FEditor::CreateTemplateSubMenuOptions(FMenuBuilder& MenuBuilder)
		{
			TAttribute<EVisibility> ClearVisibility = TAttribute<EVisibility>::CreateSPLambda(this, [this]()
			{
				const IMetasoundEditorModule & MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
				if (!MetaSoundEditorModule.IsRestrictedMode())
				{
					return GetDocumentTemplate() ? EVisibility::Visible : EVisibility::Collapsed;
				}

				return EVisibility::Collapsed;
			});


			IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			if (!MetaSoundEditorModule.IsRestrictedMode())
			{
				MenuBuilder.BeginSection("ConvertTemplateHeader", LOCTEXT("ConvertTemplateDisplayName", "Convert Template"), ClearVisibility);
				{
					if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
					{
						if (const UScriptStruct* ConfigStruct = DocBuilder->GetConstDocumentTemplate().GetScriptStruct())
						{
							MenuBuilder.AddMenuEntry(
								FText::Format(LOCTEXT("ClearTemplateButtonText", "Convert From '{0}' ..."), ConfigStruct->GetDisplayNameText()),
								LOCTEXT("ConvertFromTemplateTooltip", "Removes the MetaSound's configuration, leaving the MetaSound state intact and enabling editing of configuration-managed MetaSound functionality."),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateSP(this, &FEditor::ShowConvertFromConfigDialog)),
								{ },
								EUserInterfaceActionType::Button
							);
						}
					}
				}
				MenuBuilder.EndSection();
			}

			MenuBuilder.BeginSection("SetTemplateSectionHeader", LOCTEXT("SelectTemplateSectionName", "Set Template"));
			{
				auto CreateConfigEntry = [this, &MenuBuilder](UScriptStruct* ConfigStruct)
				{
					FUIAction SetConfigAction;
					SetConfigAction.ExecuteAction = FExecuteAction::CreateSPLambda(this, [this, ConfigStruct]()
					{
						if (FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
						{
							TInstancedStruct<FMetaSoundFrontendDocumentTemplate> ConfigInstance;

							check(ConfigStruct);
							ConfigInstance.InitializeAsScriptStruct(ConfigStruct);

							DocBuilder->SetTemplate(MoveTemp(ConfigInstance));
							DocBuilder->ConfigureDocument();

							ReloadAllDocTemplateTabs();
							UpdateEnabledTabs();
						}
					});
					SetConfigAction.GetActionCheckState = FGetActionCheckState::CreateSPLambda(this, [this, ConfigStruct]()
					{
						if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
						{
							if (ConfigStruct == DocBuilder->GetConstDocumentTemplate().GetScriptStruct())
							{
								return ECheckBoxState::Checked;
							}
						}

						return ECheckBoxState::Unchecked;
					});

					MenuBuilder.AddMenuEntry(
						ConfigStruct->GetDisplayNameText(),
						ConfigStruct->GetToolTipText(),
						FSlateIcon(),
						SetConfigAction,
						{ },
						EUserInterfaceActionType::RadioButton
					);
				};

				for (TObjectIterator<UStruct> It; It; ++It)
				{
					if (UScriptStruct* Struct = Cast<UScriptStruct>(*It))
					{
						if (Struct->IsChildOf(FMetaSoundFrontendDocumentTemplate::StaticStruct())
							&& !Struct->HasMetaData("Hidden"))
						{
							CreateConfigEntry(Struct);
						}
					}
				}
			}
			MenuBuilder.EndSection();
		}

		void FEditor::DestroyAnalyzers()
		{
			if (AnalyzerRack.IsValid())
			{
				AnalyzerRack->DestroyAnalyzers();
			}
		}

		void FEditor::ExtendToolbarInternal()
		{
			TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();
			ToolbarExtender->AddToolBarExtension
			(
				"Asset",
				EExtensionHook::After,
				GetToolkitCommands(),
				FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
				{
 					ToolbarBuilder.BeginSection("Utilities");
 					{
						ToolbarBuilder.AddComboButton(
							FUIAction(),
							FOnGetContent::CreateSP(this, &FEditor::CreateTemplateMenuOptions),
							LOCTEXT("TemplateSettingsMenu", "Template"),
							LOCTEXT("TemplateSettingsMenu_Tooltip", "Settings related to procedurally-generated MetaSound Templates"),
							GetExportStatusImage()
						);
 					}
 					ToolbarBuilder.EndSection();

					ToolbarBuilder.BeginSection("Settings");
					{
						if (IsAuditionable())
						{
							ToolbarBuilder.AddToolBarButton(
								FEditorCommands::Get().EditSourceSettings,
								NAME_None,
								TAttribute<FText>(),
								TAttribute<FText>(),
								Style::CreateSlateIcon("MetasoundEditor.Settings"),
								"EditSourceSettings"
							);
						}

						ToolbarBuilder.AddToolBarButton(
							FEditorCommands::Get().EditMetasoundSettings,
							NAME_None,
							TAttribute<FText>(),
							TAttribute<FText>(),
							Style::CreateSlateIcon("MetasoundEditor.MetasoundSource.Thumbnail"),
							"EditMetasoundSettings"
						);
					}
					ToolbarBuilder.EndSection();

					if (IsAuditionable())
					{
						ToolbarBuilder.BeginSection("Audition");
						{
							ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateLeft");
							{
								ToolbarBuilder.AddToolBarButton(
									FEditorCommands::Get().Play,
									NAME_None,
									TAttribute<FText>(),
									TAttribute<FText>::Create([this] { return GetGraphStatusDescription(); }),
									TAttribute<FSlateIcon>::Create([this]() { return GetPlayIcon(); })
								);
							}
							ToolbarBuilder.EndStyleOverride();

							ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateRight");
							{
								ToolbarBuilder.AddToolBarButton(
									FEditorCommands::Get().Stop,
									NAME_None,
									TAttribute<FText>(),
									TAttribute<FText>(),
									TAttribute<FSlateIcon>::Create([this]() { return GetStopIcon(); })
								);
							}
							ToolbarBuilder.EndStyleOverride();

							const bool bShowAuditionSettings = IsPageAuditioningEnabled();
							if (bShowAuditionSettings)
							{
								ToolbarBuilder.AddComboButton(
									FUIAction(),
									FOnGetContent::CreateSP(this, &FEditor::CreateAuditionMenuOptions),
									LOCTEXT("AuditionSettingsMenu", "Audition"),
									LOCTEXT("AuditionSettingsMenu_Tooltip", "Settings related to auditioning MetaSound (Target page, platform etc.)"),
									Style::CreateSlateIcon("MetasoundEditor.Audition"),
									false
								);
							}

							ToolbarBuilder.AddToolBarButton(
								FEditorCommands::Get().ToggleShowGraph,
								NAME_None,
								TAttribute<FText>::CreateSPLambda(this, [this]() { return AssetEditorPrivate::bShowGraphOverride
									? LOCTEXT("HideGraphLabel", "Hide Graph")
									: LOCTEXT("ShowGraphLabel", "Show Graph");
								}),
								TAttribute<FText>(),
								Style::CreateSlateIcon("MetasoundEditor.Settings")
							);
						}
						ToolbarBuilder.EndSection();
					}
				})
			);

			AddToolbarExtender(ToolbarExtender);

			if (GEditor)
			{
				if (UMetaSoundEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>())
				{
					for (const TSharedRef<FExtender>& Extender : Subsystem->GetToolbarExtenders())
					{
						AddToolbarExtender(Extender);
					}
				}
			}
		}

		FSlateIcon FEditor::GetImportStatusImage() const
		{
			const FName IconName = "MetasoundEditor.Import";
			return FSlateIcon("MetaSoundStyle", IconName);
		}

		FSlateIcon FEditor::GetExportStatusImage() const
		{
			FName IconName = "MetasoundEditor.Export";
			if (!bPassedValidation)
			{
				IconName = "MetasoundEditor.ExportError";
			}

			return FSlateIcon("MetaSoundStyle", IconName);
		}

		void FEditor::BindGraphCommands()
		{
			const FEditorCommands& Commands = FEditorCommands::Get();

			ToolkitCommands->MapAction(
				Commands.Play,
				FExecuteAction::CreateSP(this, &FEditor::Play));

			ToolkitCommands->MapAction(
				Commands.Stop,
				FExecuteAction::CreateSP(this, &FEditor::Stop));

			ToolkitCommands->MapAction(
				Commands.Import,
				FExecuteAction::CreateSP(this, &FEditor::Import));

			ToolkitCommands->MapAction(
				Commands.Export,
				FExecuteAction::CreateSP(this, &FEditor::Export));

			ToolkitCommands->MapAction(
				Commands.TogglePlayback,
				FExecuteAction::CreateSP(this, &FEditor::TogglePlayback));

			ToolkitCommands->MapAction(
				FGenericCommands::Get().Undo,
				FExecuteAction::CreateSP(this, &FEditor::UndoGraphAction));

			ToolkitCommands->MapAction(
				FGenericCommands::Get().Redo,
				FExecuteAction::CreateSP(this, &FEditor::RedoGraphAction));

			ToolkitCommands->MapAction(
				Commands.EditMetasoundSettings,
				FExecuteAction::CreateSP(this, &FEditor::EditMetasoundSettings));

			ToolkitCommands->MapAction(
				Commands.EditSourceSettings,
				FExecuteAction::CreateSP(this, &FEditor::EditSourceSettings));

			ToolkitCommands->MapAction(FGenericCommands::Get().Delete,
				FExecuteAction::CreateSP(this, &FEditor::DeleteSelectedInterfaceItems),
				FCanExecuteAction::CreateSP(this, &FEditor::CanDeleteInterfaceItems));

			ToolkitCommands->MapAction(FGenericCommands::Get().Rename,
				FExecuteAction::CreateSP(this, &FEditor::RenameSelectedInterfaceItem),
				FCanExecuteAction::CreateSP(this, &FEditor::CanRenameSelectedInterfaceItems));
			
			ToolkitCommands->MapAction(FGenericCommands::Get().Duplicate,
				FExecuteAction::CreateSP(this, &FEditor::DuplicateSelectedMemberItems),
				FCanExecuteAction::CreateSP(this, &FEditor::CanDuplicateSelectedMemberItems));

			ToolkitCommands->MapAction(FGenericCommands::Get().Copy,
				FExecuteAction::CreateSP(this, &FEditor::CopySelectedMemberItems),
				FCanExecuteAction::CreateSP(this, &FEditor::CanCopySelectedMemberItems));

			ToolkitCommands->MapAction(FGenericCommands::Get().Cut,
				FExecuteAction::CreateSP(this, &FEditor::CutSelectedMemberItems),
				FCanExecuteAction::CreateSP(this, &FEditor::CanCutSelectedMemberItems));

			ToolkitCommands->MapAction(FGenericCommands::Get().Paste,
				FExecuteAction::CreateSP(this, &FEditor::PasteSelectedMemberItems),
				FCanExecuteAction::CreateSP(this, &FEditor::CanPasteSelectedMemberItems));

			ToolkitCommands->MapAction(
				FEditorCommands::Get().UpdateNodeClass,
				FExecuteAction::CreateSP(this, &FEditor::UpdateSelectedNodeClasses));

			ToolkitCommands->MapAction(
				FEditorCommands::Get().FindInMetaSound,
				FExecuteAction::CreateSP(this, &FEditor::ShowFindInMetaSound));

			ToolkitCommands->MapAction(
				FEditorCommands::Get().CopySelectionAsConfiguratorCodeToClipboard,
				FExecuteAction::CreateSP(this, &FEditor::CopySelectionAsConfiguratorCodeToClipboard));

			ToolkitCommands->MapAction(
				FEditorCommands::Get().ToggleShowGraph,
				FExecuteAction::CreateSP(this, &FEditor::ToggleShowHideGraph),
				FCanExecuteAction::CreateSP(this, &FEditor::CanToggleShowHideGraph));
		}

		void FEditor::Import()
		{
			// TODO: Prompt OFD and provide path from user
			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return;
			}

			const FString InputPath = FPaths::ProjectIntermediateDir() / TEXT("MetaSounds") + FPaths::ChangeExtension(MetaSound->GetPathName(), FMetasoundAssetBase::FileExtension);
			
			// TODO: use the same directory as the currently open MetaSound
			const FString OutputPath = FString("/Game/ImportedMetaSound/GeneratedMetaSound");

			FMetasoundFrontendDocument MetasoundDoc;

			if (Frontend::ImportJSONAssetToMetasound(InputPath, MetasoundDoc))
			{
				//TSet<UClass*> ImportClasses;

				// TODO: Update importing to support interfaces

				//if (ImportClasses.Num() < 1)
				{
					TArray<FString> InterfaceNames;
					Algo::Transform(MetasoundDoc.Interfaces, InterfaceNames, [] (const FMetasoundFrontendVersion& InterfaceVersion) { return InterfaceVersion.ToString(); });
					UE_LOGF(LogMetaSound, Warning, "Cannot create UObject from MetaSound document. No UClass supports interface(s) \"%ls\"", *FString::Join(InterfaceNames, TEXT(",")));
				}
#if 0
				else
				{
					UClass* AnyClass = nullptr;
					for (UClass* ImportClass : ImportClasses)
					{
						AnyClass = ImportClass;
						if (ImportClasses.Num() > 1)
						{
							// TODO: Modal dialog to give user choice of import type.
							TArray<FString> InterfaceNames;
							Algo::Transform(MetasoundDoc.Interfaces, InterfaceNames, [](const FMetasoundFrontendVersion& InterfaceVersion) { return InterfaceVersion.ToString(); });
							UE_LOGF(LogMetaSound, Warning, "Duplicate UClass support interface(s) \"%ls\" with UClass \"%ls\"", *FString::Join(InterfaceNames, TEXT(",")), *ImportClass->GetName());
						}
					}

					// TODO: Update to just use simple UObject NewObject
				}
#endif
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Could not import MetaSound at path: %ls", *InputPath);
			}
		}

		void FEditor::Export()
		{
			if (UObject* MetaSound = GetMetasoundObject())
			{
				FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSound);
				check(MetaSoundAsset);

				static const FString MetasoundExtension(TEXT(".metasound"));

				// TODO: We could just make this an object.
				const FString Path = FPaths::ProjectSavedDir() / TEXT("MetaSounds") + FPaths::ChangeExtension(MetaSound->GetPathName(), MetasoundExtension);
				MetaSoundAsset->GetDocumentHandle()->ExportToJSONAsset(Path);
			}	
		}

		FText FEditor::GetGraphStatusDescription() const
		{
			if (!GraphStatusDescriptionOverride.IsEmpty())
			{
				return GraphStatusDescriptionOverride;
			}

			switch (HighestMessageSeverity)
			{
				case EMessageSeverity::Error:
				{
					return LOCTEXT("MetaSoundPlayStateTooltip_Error", "MetaSound contains errors and cannot be played.");
				}

				case EMessageSeverity::PerformanceWarning:
				case EMessageSeverity::Warning:
				{
					return LOCTEXT("MetaSoundPlayStateTooltip_Warning", "MetaSound contains warnings and playback behavior may be undesired.");
				}

				case EMessageSeverity::Info:
				default:
				{
					return FEditorCommands::Get().Play->GetDescription();
				}
			}
		}

		const FSlateIcon& FEditor::GetPlayIcon() const
		{
			switch (HighestMessageSeverity)
			{
				case EMessageSeverity::Error:
				{
					static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Play.Error");
					return Icon;
				}

				case EMessageSeverity::PerformanceWarning:
				case EMessageSeverity::Warning:
				{
					if (IsPlaying())
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Play.Active.Warning");
						return Icon;
					}
					else
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Play.Inactive.Warning");
						return Icon;
					}
				}

				case EMessageSeverity::Info:
				default:
				{
					if (IsPlaying())
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Play.Active.Valid");
						return Icon;
					}
					else
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Play.Inactive.Valid");
						return Icon;
					}
				}
			}
		}

		const FSlateIcon& FEditor::GetStopIcon() const
		{
			switch (HighestMessageSeverity)
			{
				case EMessageSeverity::Error:
				{
					static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Stop.Disabled");
					return Icon;
				}

				case EMessageSeverity::PerformanceWarning:
				case EMessageSeverity::Warning:
				case EMessageSeverity::Info:
				default:
				{
					if (IsPlaying())
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Stop.Active");
						return Icon;
					}
					else
					{
						static const FSlateIcon Icon = Style::CreateSlateIcon("MetasoundEditor.Stop.Inactive");
						return Icon;
					}
				}
			}
		}

		void FEditor::FUserWidgetData::SetAuditionState(UAudioComponent* AudioComponent, bool bIsAuditioning) const
		{
			if (UUserWidget* UserWidget = Widget.Get())
			{
				if (UMetaSoundPresetWidgetInterface::StaticClass() == Class)
				{
					IMetaSoundPresetWidgetInterface::Execute_OnAuditionStateChanged(UserWidget, AudioComponent, bIsAuditioning);
				}
				else if (ensure(UMetaSoundTemplateWidgetInterface::StaticClass() == Class))
				{
					IMetaSoundTemplateWidgetInterface::Execute_OnAuditionStateChanged(UserWidget, AudioComponent, bIsAuditioning);
				}
			}
		}

		void FEditor::Play()
		{
			using namespace Engine;

			if (USoundBase* MetaSoundToPlay = Cast<USoundBase>(GetMetasoundObject()))
			{
				SyncAuditionState();

				if (HighestMessageSeverity == EMessageSeverity::Error)
				{
					return;
				}

				// Even though the MetaSoundSource will attempt to register via InitResources
				// later in this execution (and deeper in the stack), this call forces
				// re-registering to make sure everything is up-to-date.
				FGraphBuilder::RegisterGraphWithFrontend(*MetaSoundToPlay);

				UpdateRenderInfo(true /* bIsPlaying */);
				UpdatePageInfo(true);

				if (UMetaSoundSource* Source = Cast<UMetaSoundSource>(GetMetasoundObject()))
				{
					check(GEditor);
					if (UAudioComponent* PreviewComp = GEditor->ResetPreviewAudioComponent())
					{
						if (Builder.IsValid())
						{
							FOnCreateAuditionGeneratorHandleDelegate GeneratorHandleDelegate;
							CastChecked<UMetaSoundSourceBuilder>(Builder.Get())->Audition(nullptr, PreviewComp, GeneratorHandleDelegate, AssetEditorPrivate::bEnableLiveAudition);
						}

						if (IsPlaying())
						{
							constexpr bool bIsAuditioning = true;
							for (FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
							{
								Pair.Value.SetAuditionState(PreviewComp, bIsAuditioning);
							}
						}

						SetPreviewID(PreviewComp->GetUniqueID());

						if (UAudioBus* AudioBus = AnalyzerRack->GetAudioBus())
						{
							PreviewComp->SetAudioBusSendPostEffect(AudioBus, 1.0f);
						}

						GraphConnectionManager = RebuildConnectionManager(PreviewComp);
					}
				}

				MetasoundGraphEditor->RegisterActiveTimer(0.0f,
					FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
					{
						const bool bIsPlaying = IsPlaying();
						UpdateRenderInfo(bIsPlaying, InDeltaTime);

						if (bIsPlaying)
						{
							return EActiveTimerReturnType::Continue;
						}
						else
						{
							UpdatePageInfo(bIsPlaying);
							GraphConnectionManager = RebuildConnectionManager();

							constexpr bool bIsAuditioning = false;
							for (FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
							{
								Pair.Value.SetAuditionState(nullptr, bIsAuditioning);
							}

							return EActiveTimerReturnType::Stop;
						}
					})
				);

				AnalyzerRack->StartProcessing();
			}
		}

		void FEditor::SetPreviewID(uint32 InPreviewID)
		{
			if (HasEditingObject())
			{
				GetMetaSoundGraphChecked().SetPreviewID(InPreviewID);
			}
		}

		UMetasoundEditorGraph* FEditor::GetMetaSoundGraph()
		{
			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(GetMetasoundObject());
			if (MetaSoundAsset)
			{
				UEdGraph* Graph = MetaSoundAsset->GetGraph();
				return Cast<UMetasoundEditorGraph>(Graph);
			}

			return nullptr;
		}

		UMetasoundEditorGraph& FEditor::GetMetaSoundGraphChecked()
		{
			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(GetMetasoundObject());
			check(MetaSoundAsset);

			UEdGraph* Graph = MetaSoundAsset->GetGraph();
			check(Graph);

			return *CastChecked<UMetasoundEditorGraph>(Graph);
		}

		void FEditor::ExecuteNode()
		{
			const FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				ExecuteNode(CastChecked<UEdGraphNode>(*NodeIt));
			}
		}

		bool FEditor::CanExecuteNode() const
		{
			return true;
		}

		TSharedPtr<SGraphEditor> FEditor::GetGraphEditor() const
		{
			return MetasoundGraphEditor;
		}

		void FEditor::Stop()
		{
			check(GEditor);
			GEditor->ResetPreviewAudioComponent();
			SetPreviewID(INDEX_NONE);
		}

		void FEditor::SyncAuditionState(bool bSetAuditionFocus)
		{
			GraphStatusDescriptionOverride = { };
			HighestMessageSeverity = GetMetaSoundGraphChecked().GetHighestMessageSeverity();

			if (Builder.IsValid())
			{
				if (bSetAuditionFocus)
				{
					constexpr bool bOpenEditor = false; // Already Focused
					constexpr bool bPostTransaction = false;
					const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
					const FGuid BuildPageID = DocBuilder.GetBuildPageID();
					UMetaSoundEditorSubsystem::GetChecked().SetFocusedPage(*Builder.Get(), BuildPageID, bOpenEditor, bPostTransaction);
				}

				if (const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>())
				{
					if (EdSettings->GetAuditionPlatform() != UMetasoundEditorSettings::EditorAuditionPlatform)
					{
						if (!UMetaSoundEditorSubsystem::GetChecked().IsPageAuditionPlatformCookTarget(EdSettings->GetAuditionPage()))
						{
							GraphStatusDescriptionOverride = LOCTEXT("InvalidAuditionPageWarning",
								"Selected Audition Page in MetaSound Editor Settings is not a target page for the selected 'Audition Platform'. "
								"Execution may result in behavior that does not exhibit runtime behavior.");
							if (HighestMessageSeverity > EMessageSeverity::Warning)
							{
								HighestMessageSeverity = EMessageSeverity::Warning;
							}
						}
					}
				}
			}
		}

		void FEditor::TogglePlayback()
		{
			check(GEditor);

			if (IsPlaying())
			{
				Stop();
			}
			else
			{
				Play();
			}
		}

		bool FEditor::CanToggleShowHideGraph() const
		{
			if (Builder.IsValid())
			{
				if (const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder(); DocBuilder.IsValid())
				{
					TConstStructView<FMetaSoundFrontendDocumentTemplate> Template = DocBuilder.GetConstDocumentTemplate();
					return Template.IsValid() && !Template.GetPtr()->GetEditorOptions().bGraphEditorVisible;
				}
			}

			return false;
		}

		void FEditor::ToggleShowHideGraph()
		{
			AssetEditorPrivate::bShowGraphOverride = !AssetEditorPrivate::bShowGraphOverride;
			UpdateEnabledTabs();
		}

		void FEditor::ExecuteNode(UEdGraphNode* InNode)
		{
			using namespace Metasound;
			using namespace Metasound::Frontend;

			if (!GEditor)
			{
				return;
			}

			if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(InNode))
				{
					if (const FMetasoundFrontendClass* Class = ExternalNode->GetFrontendClass())
					{
						// Editor external nodes can represent frontend template nodes, so check
						// to make sure underlying frontend node is of type 'External' to avoid
						// ensure when generating asset key.
						if (Class->Metadata.GetType() == EMetasoundFrontendClassType::External)
						{
							IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
							if (!MetaSoundEditorModule.IsRestrictedMode())
							{
								const FMetaSoundAssetKey AssetKey(Class->Metadata);
								if (const FMetasoundAssetBase* Asset = IMetaSoundAssetManager::GetChecked().FindAsset(AssetKey))
								{
									AssetSubsystem->OpenEditorForAsset(Asset->GetOwningAsset());
								}
							}
						}
					}
				}
			}
		}

		void FEditor::EditObjectSettings()
		{
			if (GraphMembersMenu.IsValid())
			{
				GraphMembersMenu->SelectItemByName(FName());
			}

			if (MetasoundGraphEditor.IsValid())
			{
				bManuallyClearingGraphSelection = true;
				MetasoundGraphEditor->ClearSelectionSet();
				bManuallyClearingGraphSelection = false;
			}

			// Clear selection first to force refresh of customization
			// if swapping from one object-level edit mode to the other
			// (ex. Metasound Settings to General Settings)
			SetSelection({ });
			SetSelection({ GetMetasoundObject() });
		}

		void FEditor::ShowConvertFromConfigDialog()
		{
			using namespace Frontend;

			check(GEditor);

			if (Builder.IsValid())
			{
				TSharedPtr<SWindow> DialogWindow =
					SNew(SWindow)
					.Title(LOCTEXT("ConvertMetaSoundTemplatePrompt", "Convert From MetaSound Template?"))
					.SupportsMinimize(false)
					.SupportsMaximize(false)
					.SizingRule(ESizingRule::Autosized)
					.AutoCenter(EAutoCenter::PreferredWorkArea);
				
				TSharedPtr<SBox> DialogContent =
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Bottom)
						[
							SNew(SButton)
							.Text(LOCTEXT("MetasoundConvertFromConfigDialogAccept", "Accept"))
							.OnClicked_Lambda([this, &DialogWindow]()
							{
								if (FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder(); DocBuilder.IsValid())
								{
									const UScriptStruct* ScriptStruct = DocBuilder.GetConstDocumentTemplate().GetScriptStruct();
									const FText DisplayName = ScriptStruct ? ScriptStruct->GetDisplayNameText() : LOCTEXT("EmptyConfigName", "Unset");
									const FScopedTransaction Transaction(FText::Format(LOCTEXT("ConvertFromTemplateFormat", "Convert From '{0}' Template"), DisplayName));

									GetMetasoundObject()->Modify();
									EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
									DocBuilder.SetTemplate({ });
									DocBuilder.ConfigureDocument();
									ReloadAllDocTemplateTabs();
									UpdateEnabledTabs();
								}
								
								if (UToolMenus* ToolMenus = UToolMenus::Get())
								{
									ToolMenus->RefreshAllWidgets();
								}
								
								RefreshGraphMemberMenu();
								RefreshDetails();

								DialogWindow->RequestDestroyWindow();
								
								return FReply::Handled();
							})
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						[
							SNew(SButton)
							.Text(LOCTEXT("MetasoundConvertFromConfigDialogCancel", "Cancel"))
							.OnClicked_Lambda([&DialogWindow]()
							{
								DialogWindow->RequestDestroyWindow();
								
								return FReply::Handled();
							})
						]
				];
				
				DialogWindow->SetContent(DialogContent.ToSharedRef());
				
				FSlateApplication::Get().AddModalWindow(DialogWindow.ToSharedRef(), GetGraphEditor());
			}
		}

		void FEditor::EditSourceSettings()
		{
			if (UMetasoundEditorSettings* EditorSettings = GetMutableDefault<UMetasoundEditorSettings>())
			{
				EditorSettings->DetailView = EMetasoundActiveDetailView::General;
			}

			EditObjectSettings();
			RefreshDetails();
		}

		void FEditor::EditMetasoundSettings()
		{
			if (UMetasoundEditorSettings* EditorSettings = GetMutableDefault<UMetasoundEditorSettings>())
			{
				EditorSettings->DetailView = EMetasoundActiveDetailView::Metasound;
			}

			EditObjectSettings();
			RefreshDetails();
		}

		void FEditor::SyncInBrowser()
		{
			TArray<UObject*> ObjectsToSync;

			const FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				// TODO: Implement sync to referenced Metasound if selected node is a reference to another metasound
			}

			if (!ObjectsToSync.Num())
			{
				ObjectsToSync.Add(GetMetasoundObject());
			}

			check(GEditor);
			GEditor->SyncBrowserToObjects(ObjectsToSync);
		}

		void FEditor::AddInput()
		{
		}

		bool FEditor::CanAddInput() const
		{
			return MetasoundGraphEditor->GetSelectedNodes().Num() == 1;
		}

		void FEditor::OnCreateComment()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				UEdGraph* Graph = MetasoundGraphEditor->GetCurrentGraph();
				if (Graph && IsGraphEditable())
				{
					FMetasoundGraphSchemaAction_NewComment CommentAction;
					CommentAction.PerformAction(Graph, nullptr, MetasoundGraphEditor->GetPasteLocation2f());
				}
			}
		}

		void FEditor::CreateGraphEditorWidget(UObject& MetaSound)
		{
			if (!GraphEditorCommands.IsValid())
			{
				GraphEditorCommands = MakeShared<FUICommandList>();

				GraphEditorCommands->MapAction(FEditorCommands::Get().BrowserSync,
					FExecuteAction::CreateSP(this, &FEditor::SyncInBrowser));

				GraphEditorCommands->MapAction(FEditorCommands::Get().EditMetasoundSettings,
					FExecuteAction::CreateSP(this, &FEditor::EditMetasoundSettings));

				if (MetaSound.IsA<UMetaSoundSource>())
				{
					GraphEditorCommands->MapAction(FEditorCommands::Get().EditSourceSettings,
						FExecuteAction::CreateSP(this, &FEditor::EditSourceSettings));
				}

				GraphEditorCommands->MapAction(FEditorCommands::Get().AddInput,
					FExecuteAction::CreateSP(this, &FEditor::AddInput),
					FCanExecuteAction::CreateSP(this, &FEditor::CanAddInput));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteAllToCommonInputs,
					FExecuteAction::CreateSP(this, &FEditor::PromoteAllToCommonInputs),
					FCanExecuteAction::CreateSP(this, &FEditor::CanPromoteAllToCommonInputs));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteAllToInput,
					FExecuteAction::CreateSP(this, &FEditor::PromoteAllToInputs),
					FCanExecuteAction::CreateSP(this, &FEditor::CanPromoteAllToInputs));

				// Editing Commands
				GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->SelectAllNodes(); }));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
					FExecuteAction::CreateSP(this, &FEditor::CopySelectedNodes));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
					FExecuteAction::CreateSP(this, &FEditor::CutSelectedNodes),
					FCanExecuteAction::CreateLambda([this]() { return CanDeleteNodes(); }));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
					FExecuteAction::CreateLambda([this]() { PasteNodes(); }),
					FCanExecuteAction::CreateSP(this, &FEditor::CanPasteNodes));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
					FExecuteAction::CreateSP(this, &FEditor::DeleteSelectedNodes),
					FCanExecuteAction::CreateLambda([this]() { return CanDeleteNodes(); }));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
					FExecuteAction::CreateLambda([this] { DuplicateNodes(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanDuplicateNodes(); }));

				GraphEditorCommands->MapAction(FGenericCommands::Get().Rename,
					FExecuteAction::CreateLambda([this] { RenameSelectedNode(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanRenameSelectedNodes(); }));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteToInput,
					FExecuteAction::CreateLambda([this] { PromoteToInput(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanPromoteToInput(); }));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteToOutput,
					FExecuteAction::CreateLambda([this] { PromoteToOutput(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanPromoteToOutput(); }));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteToVariable,
					FExecuteAction::CreateLambda([this] { PromoteToVariable(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanPromoteToVariable(); }));

				GraphEditorCommands->MapAction(FEditorCommands::Get().PromoteToDeferredVariable,
					FExecuteAction::CreateLambda([this] { PromoteToDeferredVariable(); }),
					FCanExecuteAction::CreateLambda([this]() { return CanPromoteToDeferredVariable(); }));

				GraphEditorCommands->MapAction(FEditorCommands::Get().CopySelectionAsConfiguratorCodeToClipboard,
					FExecuteAction::CreateSP(this, &FEditor::CopySelectionAsConfiguratorCodeToClipboard));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().HideNoConnectionPins,
					FExecuteAction::CreateSP(this, &FEditor::HideUnconnectedPins));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().ShowAllPins,
					FExecuteAction::CreateSP(this, &FEditor::ShowUnconnectedPins));

				// Alignment Commands
				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignTop(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignMiddle(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignBottom(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignLeft(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignCenter(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnAlignRight(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnStraightenConnections(); }));

				// Distribution Commands
				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnDistributeNodesH(); }));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
					FExecuteAction::CreateLambda([this]() { MetasoundGraphEditor->OnDistributeNodesV(); }));

				// Node Commands
				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().CreateComment,
					FExecuteAction::CreateSP(this, &FEditor::OnCreateComment));

				GraphEditorCommands->MapAction(FGraphEditorCommands::Get().FindReferences,
					FExecuteAction::CreateSP(this, &FEditor::FindSelectedNodeInGraph));

				GraphEditorCommands->MapAction(FEditorCommands::Get().UpdateNodeClass,
					FExecuteAction::CreateSP(this, &FEditor::UpdateSelectedNodeClasses));

				GraphEditorCommands->MapAction(FEditorCommands::Get().CopySelectionAsConfiguratorCodeToClipboard,
					FExecuteAction::CreateSP(this, &FEditor::CopySelectionAsConfiguratorCodeToClipboard));
			}

			SGraphEditor::FGraphEditorEvents GraphEvents;
			GraphEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FEditor::OnCreateGraphActionMenu);
			GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FEditor::ExecuteNode);
			GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FEditor::OnSelectedNodesChanged);
			GraphEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FEditor::OnNodeTitleCommitted);

			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
			check(MetaSoundAsset);

			SAssignNew(MetasoundGraphEditor, SGraphEditor)
				.AdditionalCommands(GraphEditorCommands)
				.Appearance(this, &FEditor::GetGraphAppearance)
				.AutoExpandActionMenu(true)
				.GraphEvents(GraphEvents)
				.GraphToEdit(MetaSoundAsset->GetGraph())
				.IsEditable(this, &FEditor::IsGraphEditable)
				.ShowGraphStateOverlay(false);
		}

		FGraphAppearanceInfo FEditor::GetGraphAppearance() const
		{
			FGraphAppearanceInfo AppearanceInfo;

			if (UObject* MetaSound = GetMetasoundObject())
			{
				const FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSound);
				check(MetaSoundAsset);
				AppearanceInfo.CornerText = MetaSoundAsset->GetDisplayName();
			}

			return AppearanceInfo;
		}

		void FEditor::OnSelectedNodesChanged(const TSet<UObject*>& InSelectedNodes)
		{
			TArray<UObject*> Selection;
			for (UObject* NodeObject : InSelectedNodes)
			{
				if (UMetasoundEditorGraphInputNode* InputNode = Cast<UMetasoundEditorGraphInputNode>(NodeObject))
				{
					Selection.Add(InputNode->Input);
				}
				else if (UMetasoundEditorGraphOutputNode* OutputNode = Cast<UMetasoundEditorGraphOutputNode>(NodeObject))
				{
					Selection.Add(OutputNode->Output);
				}
				else if (UMetasoundEditorGraphVariableNode* VariableNode = Cast<UMetasoundEditorGraphVariableNode>(NodeObject))
				{
					Selection.Add(VariableNode->Variable);
				}
				else
				{
					Selection.Add(NodeObject);
				}
			}

			if (GraphMembersMenu.IsValid() && !bManuallyClearingGraphSelection)
			{
				GraphMembersMenu->SelectItemByName(FName());
			}
			SetSelection(Selection);
		}

		void FEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
		{
			if (NodeBeingChanged)
			{
				const FScopedTransaction Transaction(TEXT(""), LOCTEXT("RenameNode", "Rename Node"), NodeBeingChanged);
				NodeBeingChanged->Modify();
				NodeBeingChanged->OnRenameNode(NewText.ToString());
			}
		}

		void FEditor::DeleteInterfaceItem(TSharedPtr<FMetasoundGraphMemberSchemaAction> ActionToDelete)
		{
			using namespace Metasound::Frontend;
			if (!Builder.IsValid())
			{
				return;
			}

			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return;
			}	

			UMetasoundEditorGraphMember* GraphMember = ActionToDelete->GetGraphMember();
			if (ensure(GraphMember))
			{
				const FGuid MemberID = GraphMember->GetMemberID();
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
				UMetasoundEditorGraphMember* NextToSelect = Graph.FindAdjacentMember(*GraphMember);

				{
					const FScopedTransaction Transaction(LOCTEXT("MetaSoundEditorDeleteSelectedMember", "Delete MetaSound Graph Member"));
					MetaSound->Modify();
					Graph.Modify();
					GraphMember->Modify();

					const bool bRemovedMetadata = Builder->ClearMemberMetadata(GraphMember->GetMemberID());
					if (bRemovedMetadata)
					{
						const FName MemberName = GraphMember->GetMemberName();
						EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
						if (GraphMember->IsA<UMetasoundEditorGraphInput>())
						{
							Builder->RemoveGraphInput(MemberName, Result);
						}
						else if (GraphMember->IsA<UMetasoundEditorGraphOutput>())
						{
							Builder->RemoveGraphOutput(MemberName, Result);
						}
						else if (UMetasoundEditorGraphVariable* Variable = Cast<UMetasoundEditorGraphVariable>(GraphMember))
						{
							Builder->RemoveGraphVariable(MemberName, Result);
						}
						ensure(Result == EMetaSoundBuilderResult::Succeeded);
					}
				}

				if (NextToSelect && GraphMembersMenu.IsValid())
				{
					if (GraphMembersMenu->SelectItemByName(NextToSelect->GetMemberName(), ESelectInfo::Direct, static_cast<int32>(NextToSelect->GetSectionID())))
					{
						const TArray<UObject*> GraphMembersToSelect { NextToSelect };
						SetSelection(GraphMembersToSelect);
					}
				}
			}

			FGraphBuilder::RegisterGraphWithFrontend(*MetaSound);
		}

		void FEditor::DeleteSelected()
		{
			using namespace Frontend;

			if (!IsGraphEditable())
			{
				return;
			}

			if (CanDeleteNodes())
			{
				DeleteSelectedNodes();
			}
			DeleteSelectedInterfaceItems();
		}

		void FEditor::DeleteSelectedNodes()
		{
			using namespace Metasound::Frontend;

			const FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			MetasoundGraphEditor->ClearSelectionSet();

			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return;
			}

			const FScopedTransaction Transaction(LOCTEXT("MetaSoundEditorDeleteSelectedNode2", "Delete Selected MetaSound Node(s)"));
			check(MetaSound);
			MetaSound->Modify();
			UEdGraph* Graph = MetasoundGraphEditor->GetCurrentGraph();
			check(Graph);
			Graph->Modify();
			for (UObject* NodeObj : SelectedNodes)
			{
				if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(NodeObj))
				{
					if (Node->CanUserDeleteNode())
					{
						Node->RemoveFromDocument();
					}
				}
				else if (UMetasoundEditorGraphCommentNode* CommentNode = Cast<UMetasoundEditorGraphCommentNode>(NodeObj))
				{
					CommentNode->RemoveFromDocument();
				}
			}
		}

		void FEditor::DeleteSelectedInterfaceItems()
		{
			if (!IsGraphEditable() || !GraphMembersMenu.IsValid())
			{
				return;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);
			if (Actions.IsEmpty())
			{
				return;
			}

			TSharedPtr<FMetasoundGraphMemberSchemaAction> ActionToDelete;
			for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
			{
				TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (MetasoundAction.IsValid())
				{
					const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember();
					if (ensure(nullptr != GraphMember))
					{
						const FMetasoundFrontendVersion* InterfaceVersion = nullptr;
						if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember))
						{
							InterfaceVersion = &Vertex->GetInterfaceVersion();
						}

						if (InterfaceVersion && InterfaceVersion->IsValid())
						{
							if (MetasoundGraphEditor.IsValid())
							{
								const FText Notification = FText::Format(LOCTEXT("CannotDeleteInterfaceMemberNotificationFormat", "Cannot delete individual member of interface '{0}'."), FText::FromName(InterfaceVersion->Name));
								FNotificationInfo Info(Notification);
								Info.bFireAndForget = true;
								Info.bUseSuccessFailIcons = false;
								Info.ExpireDuration = 5.0f;

								MetasoundGraphEditor->AddNotification(Info, false /* bSuccess */);
							}
						}
						else
						{
							ActionToDelete = MetasoundAction;
							if (ActionToDelete.IsValid())
							{
								DeleteInterfaceItem(ActionToDelete);
							}
						}
					}
				}
			}
		}

		void FEditor::CutSelectedNodes()
		{
			CopySelectedNodes();

			// Cache off the old selection
			const FGraphPanelSelectionSet OldSelectedNodes = MetasoundGraphEditor->GetSelectedNodes();

			// Clear the selection and only select the nodes that can be duplicated
			FGraphPanelSelectionSet RemainingNodes;
			MetasoundGraphEditor->ClearSelectionSet();

			for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
			{
				UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
				if (Node && Node->CanUserDeleteNode())
				{
					MetasoundGraphEditor->SetNodeSelection(Node, true);
				}
				else
				{
					RemainingNodes.Add(Node);
				}
			}

			// Delete the deletable nodes
			DeleteSelectedNodes();

			// Clear deleted, and reselect remaining nodes from original selection
			MetasoundGraphEditor->ClearSelectionSet();
			for (UObject* RemainingNode : RemainingNodes)
			{
				if (UEdGraphNode* Node = Cast<UEdGraphNode>(RemainingNode))
				{
					MetasoundGraphEditor->SetNodeSelection(Node, true);
				}
			}
		}

		void FEditor::ExportNodesToText(FString& OutText) const
		{
			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (UObject* Object : SelectedNodes)
			{
				if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Object))
				{
					Node->CacheBreadcrumb();
				}
			}

			FEdGraphUtilities::ExportNodesToText(SelectedNodes, OutText);
		}

		void FEditor::CopySelectedNodes() const
		{
			FString NodeString;
			ExportNodesToText(NodeString);
			FPlatformApplicationMisc::ClipboardCopy(*NodeString);
		}

		bool FEditor::CanDuplicateNodes() const
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			// If any of the nodes can be duplicated then allow copying
			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
			{
				UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
				if (!Node)
				{
					return false;
				}
			}

			FString NodeString;
			FEdGraphUtilities::ExportNodesToText(SelectedNodes, NodeString);

			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(GetMetasoundObject());
			check(MetaSoundAsset);

			UEdGraph* Graph = MetaSoundAsset->GetGraph();
			if (!Graph)
			{
				return false;
			}

			return FEdGraphUtilities::CanImportNodesFromText(Graph, NodeString);
		}

		bool FEditor::CanDeleteNodes() const
		{
			if (MetasoundGraphEditor->GetSelectedNodes().IsEmpty())
			{
				return false;
			}

			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
			{
				// Allow deletion of comment nodes even on uneditable graphs 
				// because they were unintentionally addable at one point
				UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
				if (Node && Node->CanUserDeleteNode())
				{
					if (IsGraphEditable())
					{
						return true;
					}
				}
			}
			return false;
		}

		bool FEditor::CanDeleteInterfaceItems() const
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty())
			{
				return false;
			}

			TSharedPtr<FMetasoundGraphMemberSchemaAction> ActionToDelete;
			for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
			{
				TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (MetasoundAction.IsValid())
				{
					const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember();
					if (ensure(nullptr != GraphMember))
					{
						const FMetasoundFrontendVersion* InterfaceVersion = nullptr;
						if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember))
						{
							InterfaceVersion = &Vertex->GetInterfaceVersion();
						}

						// Interface members cannot be deleted
						const bool bIsInterfaceMember = InterfaceVersion && InterfaceVersion->IsValid();
						if (!bIsInterfaceMember)
						{
							return true;
						}
					}
					else
					{
						return true;
					}
				}
			}
			return false;
		}

		void FEditor::DuplicateNodes()
		{
			ExportNodesToText(NodeTextToPaste);
			PasteNodes(nullptr, LOCTEXT("MetaSoundEditorDuplicate", "Duplicate MetaSound Node(s)"));
		}

		void FEditor::PasteNodes(const FVector2D* InLocation)
		{
			PasteNodes(InLocation, LOCTEXT("MetaSoundEditorPaste", "Paste MetaSound Node(s)"));
		}

		void FEditor::PasteNodes(const FVector2D* InLocation, const FText& InTransactionText)
		{
			using namespace Frontend;

			FVector2D Location;
			if (InLocation)
			{
				Location = *InLocation;
			}
			else
			{
				check(MetasoundGraphEditor);
				Location = MetasoundGraphEditor->GetPasteLocation2f();
			}

			FDocumentPasteNotifications Notifications;
			TArray<UEdGraphNode*> PastedNodes = FDocumentClipboardUtils::PasteClipboardString(InTransactionText, NodeTextToPaste, Location, *GetMetasoundObject(), Notifications);

			// Paste notifications 
			if (Notifications.bPastedNodesCreateLoop)
			{
				NotifyNodePasteFailure_ReferenceLoop();
			}

			if (Notifications.bPastedNodesAddMultipleVariableSetters)
			{
				NotifyNodePasteFailure_MultipleVariableSetters();
			}

			if (Notifications.bPastedNodesAddMultipleOutputNodes)
			{
				NotifyNodePasteFailure_MultipleOutputs();
			}

			// Clear the selection set (newly pasted stuff will be selected)
			if (!PastedNodes.IsEmpty())
			{
				MetasoundGraphEditor->ClearSelectionSet();

				// Select the newly pasted stuff
				for (UEdGraphNode* GraphNode : PastedNodes)
				{
					MetasoundGraphEditor->SetNodeSelection(GraphNode, true);
				}

				MetasoundGraphEditor->NotifyGraphChanged();
			}

			NodeTextToPaste.Empty();
		}

		bool FEditor::CanRenameSelectedNodes() const
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
			{
				// Node is directly renameable (comment nodes)
				const UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
				if (Node && Node->GetCanRenameNode())
				{
					return true;
				}

				// Renameable member nodes
				if (const UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(*SelectedIter))
				{
					if (const UMetasoundEditorGraphMember* Member = MemberNode->GetMember()) 
					{
						return Member->CanRename();
					}
				}
			}
			return false;
		}

		bool FEditor::CanRenameSelectedInterfaceItems() const
		{
			if (GraphMembersMenu.IsValid())
			{
				TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
				GraphMembersMenu->GetSelectedActions(Actions);

				if (!Actions.IsEmpty())
				{
					for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
					{
						TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
						if (MetasoundAction.IsValid())
						{
							if (const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
							{
								if (GraphMember->CanRename())
								{
									return true;
								}
							}
						}
					}
				}
			}
			return false;
		}

		FGraphConnectionManager& FEditor::GetConnectionManager()
		{
			return *GraphConnectionManager.Get();
		}

		const FGraphConnectionManager& FEditor::GetConnectionManager() const
		{
			return *GraphConnectionManager.Get();
		}

		UAudioComponent* FEditor::GetAudioComponent() const
		{
			// TODO: Instance for each editor
			if (IsPlaying())
			{
				return GEditor->GetPreviewAudioComponent();
			}

			return nullptr;
		}

		const Frontend::FMetaSoundTemplate* FEditor::GetDocumentTemplate() const
		{
			if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
			{
				return DocBuilder->GetConstDocumentTemplate().GetPtr();
			}

			return nullptr;
		}

		FMetaSoundFrontendDocumentBuilder* FEditor::GetFrontendBuilder() const
		{
			if (Builder.IsValid())
			{
				FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
				if (DocBuilder.IsValid())
				{
					return &DocBuilder;
				}
			}

			return nullptr;
		}

		void FEditor::RenameSelectedNode()
		{
			if (!GraphMembersMenu.IsValid())
			{
				return;
			}

			const FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			const UMetasoundEditorGraphMember* ToRename = nullptr;
			for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
			{
				// Node is directly renameable (comment nodes)
				UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
				if (Node && Node->GetCanRenameNode())
				{
					if (TSharedPtr<SGraphEditor> GraphEditor = GetGraphEditor())
					{
						if (GraphEditor->IsNodeTitleVisible(Node, /*bRequestRename=*/ false))
						{
							GraphEditor->IsNodeTitleVisible(Node, /*bRequestRename=*/ true);
						}
						else
						{
							GraphEditor->JumpToNode(Node, /*bRequestRename=*/true);
						}
						return;
					}
				}

				// Break if renameable member node found (inputs/outputs/variables)
				if (UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(*SelectedIter))
				{
					if (const UMetasoundEditorGraphMember* Member = MemberNode->GetMember())
					{
						if (Member->CanRename())
						{
							ToRename = Member;
							break;
						}
					}
				}
			}

			if (ToRename)
			{
				GraphMembersMenu->SelectItemByName(ToRename->GetMemberName(), ESelectInfo::Direct, static_cast<int32>(ToRename->GetSectionID()));
				GraphMembersMenu->RefreshAllActions(/*bPreserveExpansion=*/ true, /*bHandleOnSelectionEvent=*/ true);
				GraphMembersMenu->OnRequestRenameOnActionNode();
			}
		}

		void FEditor::RenameSelectedInterfaceItem()
		{
			if (GraphMembersMenu.IsValid())
			{
				TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
				GraphMembersMenu->GetSelectedActions(Actions);

				if (!Actions.IsEmpty())
				{
					for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
					{
						TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
						if (MetasoundAction.IsValid())
						{
							if (const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
							{
								if (GraphMember->CanRename())
								{
									GraphMembersMenu->RefreshAllActions(/*bPreserveExpansion=*/ true, /*bHandleOnSelectionEvent=*/ true);
									GraphMembersMenu->OnRequestRenameOnActionNode();
								}
							}
						}
					}
				}
			}
		}

		bool FEditor::CanDuplicateSelectedMemberItems() const
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty())
			{
				return false;
			}

			for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
			{
				TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (MetasoundAction.IsValid())
				{
					if (const UMetasoundEditorGraphVertex* GraphVertex = Cast<UMetasoundEditorGraphVertex>(MetasoundAction->GetGraphMember()))
					{
						if (GraphVertex->IsInterfaceMember())
						{
							return false;
						}
					}
				}
			}

			return true;
		}

		void FEditor::DuplicateSelectedMemberItems()
		{
			using namespace Frontend;

			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return;
			}

			if (!GraphMembersMenu.IsValid())
			{
				return;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty())
			{
				return;
			}

			UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

			TArray<TObjectPtr<UObject>> SelectedObjects;
			FName NameToSelect;

			for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
			{
				TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (!MetasoundAction.IsValid())
				{
					continue;
				}

				if (const UMetasoundEditorGraphMember* SourceGraphMember = MetasoundAction->GetGraphMember())
				{
					const FScopedTransaction Transaction(TEXT(""), LOCTEXT("MetaSoundEditorDuplicateMember", "Duplicate MetaSound Member"), MetaSound);
					MetaSound->Modify();

					UMetasoundEditorGraphMember* NewGraphMember = nullptr;
					FMetaSoundFrontendDocumentBuilder& DocBuilder = Graph.GetBuilderChecked().GetBuilder();

					// Duplicate the Sources Frontend Node and add a new member from it
					if (const UMetasoundEditorGraphVariable* SourceGraphVariable = Cast<UMetasoundEditorGraphVariable>(SourceGraphMember))
					{
						const FName NewName = DocBuilder.GenerateUniqueVariableName(SourceGraphVariable->GetMemberName().ToString());
						const FMetasoundFrontendVariable* NewFrontendVariable = DocBuilder.DuplicateGraphVariable(SourceGraphVariable->GetMemberName(), NewName);
						if (ensure(NewFrontendVariable))
						{
							NewGraphMember = Graph.FindOrAddVariable(NewFrontendVariable->Name);
						}
					}
					else if (const UMetasoundEditorGraphVertex* SourceGraphVertex = Cast<UMetasoundEditorGraphVertex>(SourceGraphMember))
					{
						const FName SourceMemberName = SourceGraphVertex->GetMemberName();
						const EMetasoundFrontendClassType ClassType = SourceGraphVertex->GetClassType();
						
						if (ClassType == EMetasoundFrontendClassType::Input)
						{
							const FName Name = DocBuilder.GenerateUniqueInputName(SourceMemberName.ToString());
							if (const FMetasoundFrontendClassInput* NewInput = DocBuilder.DuplicateGraphInput(SourceMemberName, Name))
							{
								FGraphBuilder::SynchronizeGraphMembers(DocBuilder, Graph);
								NewGraphMember = Graph.FindInput(NewInput->Name);
							}
						}
						else if (ClassType == EMetasoundFrontendClassType::Output)
						{
							const FName Name = DocBuilder.GenerateUniqueOutputName(SourceMemberName.ToString());
							if (const FMetasoundFrontendClassOutput* NewOutput = DocBuilder.DuplicateGraphOutput(SourceMemberName, Name))
							{
								FGraphBuilder::SynchronizeGraphMembers(DocBuilder, Graph);
								NewGraphMember = Graph.FindOutput(NewOutput->Name);
							}
						}
					}

					// Duplicate the literal from the SourceGraphMember to the NewGraphMember added
					if (NewGraphMember)
					{
						if (UMetaSoundEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>())
						{
							FMetaSoundFrontendDocumentBuilder& DocumentBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
							TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> SubClass = SourceGraphMember->GetLiteral()->GetClass();
							EditorSubsystem->BindMemberMetadata(DocumentBuilder, *NewGraphMember, SubClass, SourceGraphMember->GetLiteral());

							NameToSelect = NewGraphMember->GetMemberName();
							SelectedObjects.Add(NewGraphMember);
						}
					}
				}
			}

			{
				Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
				RegOptions.bForceViewSynchronization = true;
				FGraphBuilder::RegisterGraphWithFrontend(*MetaSound, MoveTemp(RegOptions));
			}

			if (GraphMembersMenu.IsValid())
			{
				GraphMembersMenu->RefreshAllActions(true);
				if (!NameToSelect.IsNone())
				{
					GraphMembersMenu->SelectItemByName(NameToSelect);
					SetSelection(SelectedObjects);
					SetDelayedRename();
				}
			}
		}

		bool FEditor::CanCopySelectedMemberItems() const
		{
			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}

			if (!IsGraphEditable())
			{
				return false;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty())
			{
				return false;
			}

			for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
			{
				TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (MetasoundAction.IsValid())
				{
					if (const UMetasoundEditorGraphVertex* GraphVertex = Cast<UMetasoundEditorGraphVertex>(MetasoundAction->GetGraphMember()))
					{
						if (GraphVertex->IsInterfaceMember())
						{
							return false;
						}
					}
				}
			}

			return true;
		}

		void FEditor::CopySelectedMemberItems()
		{
			if (!GraphMembersMenu.IsValid())
			{
				return;
			}

			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty())
			{
				return;
			}
			
			TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Actions.Last());
			MetasoundAction->GetGraphMember()->CacheBreadcrumb();
			FDocumentClipboardUtils::CopyMemberToClipboard(MetasoundAction->GetGraphMember());
		}

		bool FEditor::CanCutSelectedMemberItems() const
		{
			return CanCopySelectedMemberItems() && CanDeleteInterfaceItems();
		}

		void FEditor::CutSelectedMemberItems()
		{
			CopySelectedMemberItems();
			DeleteSelectedInterfaceItems();
		}

		void FEditor::CopySelectionAsConfiguratorCodeToClipboard() const
		{
			if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
			{
				const FGraphPanelSelectionSet Selection = MetasoundGraphEditor->GetSelectedNodes();
				FMetasoundFrontendGraphLayer Layer = FDocumentClipboardUtils::CopySelectionToLayer(*DocBuilder, Selection);

				FString CodeString;
				Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), CodeString);
				FPlatformApplicationMisc::ClipboardCopy(*CodeString);
			}
		}

		bool FEditor::CanPasteSelectedMemberItems() const
		{
			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}
			
			if (!IsGraphEditable())
			{
				return false;
			}

			FString ClipboardText;
			FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

			if (ClipboardText.IsEmpty())
			{
				return false;
			}

			if (FDocumentClipboardUtils::CanImportMemberFromText(ClipboardText))
			{
				return true;
			}

			return false;
		}

		void FEditor::PasteSelectedMemberItems()
		{
			if (const UMetasoundEditorGraphMember* Member = FDocumentClipboardUtils::GetMemberFromClipboard())
			{
				UObject* MetaSound = GetMetasoundObject();
				if (!MetaSound)
				{
					return;
				}

				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
				const FText TransactionLabel = FText::Format(LOCTEXT("PasteSelectedMember_Vertex", "Paste {0} '{1}'"), Member->GetGraphMemberLabel(), FText::FromName(Member->GetMemberName()));
				const FScopedTransaction Transaction(TransactionLabel);
				MetaSound->Modify();
				Graph.Modify();

				FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Frontend::IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&Graph.GetMetasoundChecked());

				if (const UMetasoundEditorGraphVertex* SourceGraphVertex = Cast<UMetasoundEditorGraphVertex>(Member))
				{
					const FMetasoundEditorGraphVertexBreadcrumb Breadcrumb = SourceGraphVertex->GetBreadcrumb();

					FCreateNodeVertexParams VertexParams;
					VertexParams.DataType = SourceGraphVertex->GetDataType();
					VertexParams.AccessType = Breadcrumb.AccessType;

					const EMetasoundFrontendClassType ClassType = SourceGraphVertex->GetClassType();

					if (ClassType == EMetasoundFrontendClassType::Input)
					{
						TArray<FMetasoundFrontendClassInputDefault> InputDefaults;
						Algo::Transform(Breadcrumb.DefaultLiterals, InputDefaults, [](const TPair<FGuid, FMetasoundFrontendLiteral>& Pair)
						{
							return FMetasoundFrontendClassInputDefault(Pair.Key, Pair.Value);
						});

						FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(*MetaSound, VertexParams, InputDefaults, &Breadcrumb.MemberName);
						ClassInput.Metadata.SetDescription(Breadcrumb.Description);
						ClassInput.Metadata.SetIsAdvancedDisplay(Breadcrumb.bIsAdvancedDisplay);
						ClassInput.Metadata.SortOrderIndex = Breadcrumb.SortOrderIndex;
						
						DocumentBuilder.AddGraphInput(MoveTemp(ClassInput));
					}
					else if (ClassType == EMetasoundFrontendClassType::Output)
					{
						FMetasoundFrontendClassOutput ClassOutput = FGraphBuilder::CreateUniqueClassOutput(*MetaSound, VertexParams, &Breadcrumb.MemberName);
						ClassOutput.Metadata.SetDescription(Breadcrumb.Description);
						ClassOutput.Metadata.SetIsAdvancedDisplay(Breadcrumb.bIsAdvancedDisplay);
						ClassOutput.Metadata.SortOrderIndex = Breadcrumb.SortOrderIndex;

						DocumentBuilder.AddGraphOutput(MoveTemp(ClassOutput));
					}
					else
					{
						checkNoEntry();
					}
				}
				else if (const UMetasoundEditorGraphVariable* SourceGraphVariable = Cast<UMetasoundEditorGraphVariable>(Member))
				{
					const FMetasoundEditorGraphVariableBreadcrumb Breadcrumb = SourceGraphVariable->GetBreadcrumb();
					const FName VariableName = DocumentBuilder.GenerateUniqueVariableName(Breadcrumb.MemberName.ToString());
					const FMetasoundFrontendVariable* FrontendVariable = DocumentBuilder.AddGraphVariable(
						VariableName,
						SourceGraphVariable->GetDataType(),
						&Breadcrumb.DefaultLiteral,
						nullptr, // TODO: Add DisplayName to breadcrumb, and only paste if it doesn't conflict with existing variable
						&Breadcrumb.Description
					);
					
					if (ensure(FrontendVariable))
					{
						Graph.FindOrAddVariable(FrontendVariable->Name);
					}
				}
				else
				{
					checkNoEntry();
				}

				{
					Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
					RegOptions.bForceViewSynchronization = true;
					FGraphBuilder::RegisterGraphWithFrontend(*MetaSound, MoveTemp(RegOptions));
				}
			}
		}

		void FEditor::RefreshDetails()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::RefreshDetails);

			using namespace Frontend;

			if (MetasoundDetails.IsValid())
			{
				MetasoundDetails->ForceRefresh();
			}
		}

		void FEditor::RefreshPagesView()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::RefreshPages);

			if (PagesDetails.IsValid())
			{
				PagesDetails->ForceRefresh();
			}
		}

		void FEditor::RefreshInterfaceView()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::RefreshInterfaces);

			if (InterfacesDetails.IsValid())
			{
				InterfacesDetails->ForceRefresh();
			}
		}

		UMetasoundEditorGraphMember* FEditor::RefreshGraphMemberMenu()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::RefreshGraphMemberMenu);

			if (GraphMembersMenu.IsValid())
			{
				TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
				GraphMembersMenu->GetSelectedActions(SelectedActions);

				GraphMembersMenu->RefreshAllActions(true /* bPreserveExpansion */, false /*bHandleOnSelectionEvent*/);

				for (const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedActions)
				{
					TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
					if (MetasoundAction.IsValid())
					{
						if (UMetasoundEditorGraphMember* Member = MetasoundAction->GetGraphMember())
						{
							const FName ActionName = Member->GetMemberName();
							GraphMembersMenu->SelectItemByName(ActionName, ESelectInfo::Direct, Action->GetSectionID());
							return Member;
						}
					}
				}
			}
			return nullptr;
		}

		void FEditor::UpdateSelectedNodeClasses()
		{
			using namespace Metasound::Frontend;

			const FScopedTransaction Transaction(LOCTEXT("NodeVersionUpdate", "Update MetaSound Node(s) Class(es)"));
			UObject* MetaSound = GetMetasoundObject();
			check(MetaSound);
			MetaSound->Modify();

			UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
			Graph.Modify();

			bool bReplacedNodes = false;
			const FGraphPanelSelectionSet Selection = MetasoundGraphEditor->GetSelectedNodes();
			for (UObject* Object : Selection)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Object))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Metasound::Frontend::FNodeHandle NodeHandle = ExternalNode->GetNodeHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					const FMetasoundFrontendClassMetadata& Metadata = NodeHandle->GetClassMetadata();

					// Check for new version
					FMetasoundFrontendVersionNumber HighestVersion = ExternalNode->FindHighestVersionInRegistry();
					const bool bHasNewVersion = HighestVersion.IsValid() && HighestVersion > Metadata.GetVersion();

					// Check for asset classes
					const bool bIsAssetClass = IMetaSoundAssetManager::GetChecked().IsAssetClass(Metadata);
					if (bHasNewVersion || bIsAssetClass)
					{
						// These are ignored here when updating as the user is actively
						// forcing an update.
						constexpr TArray<INodeController::FVertexNameAndType>* DisconnectedInputs = nullptr;
						constexpr TArray<INodeController::FVertexNameAndType>* DisconnectedOutputs = nullptr;

						FNodeHandle NewNode = NodeHandle->ReplaceWithVersion(HighestVersion, DisconnectedInputs, DisconnectedOutputs);
						bReplacedNodes = true;
					}
				}
			}

			if (bReplacedNodes)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FDocumentHandle DocumentHandle = Graph.GetDocumentHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				DocumentHandle->RemoveUnreferencedDependencies();
				DocumentHandle->SynchronizeDependencyMetadata();
				FMetasoundFrontendDocumentModifyContext& ModifyContext = FGraphBuilder::GetOutermostMetaSoundChecked(Graph).GetModifyContext();
				ModifyContext.SetDocumentModified();
			}
		}

		void FEditor::HideUnconnectedPins()
		{			
			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (UObject* Object : SelectedNodes)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Object))
				{
					ExternalNode->HideUnconnectedPins(true);
				}	
			}
		}

		void FEditor::ShowUnconnectedPins()
		{
			const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (UObject* Object : SelectedNodes)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Object))
				{
					ExternalNode->HideUnconnectedPins(false);
				}
			}
		}

		bool FEditor::CanPasteNodes()
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
			FPlatformApplicationMisc::ClipboardPaste(NodeTextToPaste);
			if (FEdGraphUtilities::CanImportNodesFromText(&Graph, NodeTextToPaste))
			{
				return true;
			}

			NodeTextToPaste.Empty();
			return false;
		}

		void FEditor::UndoGraphAction()
		{
			check(GEditor);
			GEditor->UndoTransaction();
		}

		void FEditor::RedoGraphAction()
		{
			// Clear selection, to avoid holding refs to nodes that go away
			MetasoundGraphEditor->ClearSelectionSet();

			check(GEditor);
			GEditor->RedoTransaction();
		}

		void FEditor::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
		{
			using namespace Frontend;

			// Uses the builder rather than the local edit object as it may not be set
			// initially when loading the editor prior to init call on the underlying AssetToolKit.
			if (!Builder.IsValid())
			{
				return;
			}

			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();

			auto GetMemberCategory = [](FName InFullCategoryName)
			{
				FName InterfaceName;
				FName MemberName;
				Audio::FParameterPath::SplitName(InFullCategoryName, InterfaceName, MemberName);

				if (InterfaceName.IsNone())
				{
					return FText::GetEmpty();
				}

				FString CategoryString = InterfaceName.ToString();
				CategoryString.ReplaceInline(*Audio::FParameterPath::NamespaceDelimiter, TEXT("|"));
				return FText::FromString(CategoryString);
			};

			struct FAddActionParams
			{
				const FName FullName;
				const FText Tooltip;
				const FText MenuDesc;
				const ENodeSection Section;
				const FGuid MemberID;
			};

			constexpr bool bDisplayNamespace = false;
			const FMetasoundAssetBase& AssetBase = DocBuilder.GetMetasoundAsset();
			UEdGraph& EdGraph = AssetBase.GetGraphChecked();
			auto AddMemberAction = [&](const FAddActionParams& Params)
			{
				const FText Category = GetMemberCategory(Params.FullName);
				TSharedPtr<FMetasoundGraphMemberSchemaAction> NewFuncAction = MakeShared<FMetasoundGraphMemberSchemaAction>(
					Category,
					Params.MenuDesc,
					Params.Tooltip,
					1, /* Grouping */
					Params.Section);
				NewFuncAction->Graph = &EdGraph;
				NewFuncAction->SetMemberID(Params.MemberID);
				NewFuncAction->SetBuilder(*Builder.Get());
				OutAllActions.AddAction(NewFuncAction);
			};

			for (const FMetasoundFrontendClassInput& Input : DocBuilder.GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs)
			{
				if (const FMetasoundFrontendNode* Node = DocBuilder.FindGraphInputNode(Input.Name))
				{
					FText DisplayName;
					if (const FMetasoundFrontendClassInput* ClassInput = DocBuilder.FindGraphInput(Node->Name))
					{
						DisplayName = ClassInput->Metadata.GetDisplayName();
					}

					AddMemberAction(FAddActionParams
					{
						Input.Name, // FullName
						Input.Metadata.GetDescription(), // Tooltip
						INodeTemplate::ResolveMemberDisplayName(Node->Name, DisplayName, bDisplayNamespace), // MenuDesc
						ENodeSection::Inputs, // Section
						Node->GetID() // MemberID
					});
				}
			}

			const FMetasoundFrontendGraphClass& RootGraph = DocBuilder.GetConstDocumentChecked().RootGraph;
			for (const FMetasoundFrontendClassOutput& Output : RootGraph.GetDefaultInterface().Outputs)
			{
				if (const FMetasoundFrontendNode* Node = DocBuilder.FindGraphOutputNode(Output.Name))
				{
					FText DisplayName;
					if (const FMetasoundFrontendClassOutput* ClassOutput = DocBuilder.FindGraphOutput(Node->Name))
					{
						DisplayName = ClassOutput->Metadata.GetDisplayName();
					}

					AddMemberAction(FAddActionParams
					{
						Output.Name, // FullName
						Output.Metadata.GetDescription(), // Tooltip
						INodeTemplate::ResolveMemberDisplayName(Node->Name, DisplayName, bDisplayNamespace), // MenuDesc
						ENodeSection::Outputs, // Section
						Node->GetID() // MemberID
					});
				}
			}

			const FMetasoundFrontendGraph& Graph = DocBuilder.FindConstBuildGraphChecked();
			for (const FMetasoundFrontendVariable& Variable : Graph.Variables)
			{
				AddMemberAction(FAddActionParams
				{
					Variable.Name, // FullName
					Variable.Description, // Tooltip
					INodeTemplate::ResolveMemberDisplayName(Variable.Name, Variable.DisplayName, bDisplayNamespace), // MenuDesc
					ENodeSection::Variables, // Section
					Variable.ID // MemberID
				});
			}
		}

		void FEditor::CollectStaticSections(TArray<int32>& StaticSectionIDs)
		{
			const bool bIsReadOnly = Builder.IsValid() ? !Builder->GetConstBuilder().GetIsGraphEditable() : false;

			for (int32 i = 0; i < static_cast<int32>(ENodeSection::COUNT); ++i)
			{
				if (static_cast<ENodeSection>(i) != ENodeSection::None)
				{
					// ReadOnly graphs do not show variables
					if (bIsReadOnly && static_cast<ENodeSection>(i) == ENodeSection::Variables)
					{
						continue;
					}
					StaticSectionIDs.Add(i);
				}
			}
		}

		bool FEditor::HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
		{
			if (FMetasoundGraphMemberSchemaAction* Action = static_cast<FMetasoundGraphMemberSchemaAction*>(InAction))
			{
				return InName == Action->GetMemberName();
			}

			return false;
		}

		FReply FEditor::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
		{
			if (!MetasoundGraphEditor.IsValid() || InActions.IsEmpty())
			{
				return FReply::Unhandled();
			}

			TSharedPtr<FEdGraphSchemaAction> DragAction = InActions.Last();
			if (FMetasoundGraphMemberSchemaAction* MemberAction = static_cast<FMetasoundGraphMemberSchemaAction*>(DragAction.Get()))
			{
				if (UEdGraph* ActionGraph = MemberAction->Graph)
				{
					if (&GetMetaSoundGraphChecked() == ActionGraph)
					{
						TSharedPtr<FEditor> ThisEditor = StaticCastSharedRef<FEditor>(AsShared());
						return FReply::Handled().BeginDragDrop(MakeShared<FMetaSoundDragDropMemberAction>(ThisEditor, MemberAction->GetGraphMember()));
					}
				}
			}

			return FReply::Unhandled();
		}

		void FEditor::OnMemberActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions)
		{
			if (!MetasoundGraphEditor.IsValid() || InActions.IsEmpty())
			{
				return;
			}

			TSharedPtr<FMetasoundGraphMemberSchemaAction> MemberAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(InActions.Last());
			if (UMetasoundEditorGraphMember* Member = MemberAction->GetGraphMember())
			{
				JumpToNodes(Member->GetNodes());
			}
		}

		bool FEditor::CanJumpToNodesForSelectedInterfaceItem() const
		{
			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}
			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (!Actions.IsEmpty())
			{
				for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
				{
					TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
					if (MetasoundAction.IsValid())
					{
						if (const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
						{
							TArray<UMetasoundEditorGraphMemberNode*> Nodes = GraphMember->GetNodes();
							if (!Nodes.IsEmpty())
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		}

		void FEditor::JumpToNodesForSelectedInterfaceItem()
		{
			if (GraphMembersMenu.IsValid())
			{
				TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
				GraphMembersMenu->GetSelectedActions(Actions);

				if (!Actions.IsEmpty())
				{
					for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
					{
						TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
						if (MetasoundAction.IsValid())
						{
							if (const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
							{
								JumpToNodes(GraphMember->GetNodes());
								return;
							}
						}
					}
				}
			}
		}

		void FEditor::DeleteAllUnusedInSection()
		{
			TArray<TSharedPtr<FMetasoundGraphMemberSchemaAction>> ActionsToDelete;
			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedCategorySubActions(Actions);

			for (TSharedPtr<FEdGraphSchemaAction> Action : Actions)
			{
				const TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
				if (MetasoundAction.IsValid())
				{
					if (const UMetasoundEditorGraphMember* GraphMember = MetasoundAction->GetGraphMember())
					{
						const TArray<UMetasoundEditorGraphMemberNode*> Nodes = GraphMember->GetNodes();
						if (Nodes.IsEmpty())
						{
							const FMetasoundFrontendVersion* InterfaceVersion = nullptr;
							if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember))
							{
								InterfaceVersion = &Vertex->GetInterfaceVersion();
							}

							// Interface members cannot be deleted
							const bool bIsInterfaceMember = InterfaceVersion && InterfaceVersion->IsValid();
							if (!bIsInterfaceMember)
							{
								ActionsToDelete.Add(MetasoundAction);
							}
						}
					}
				}
			}

			for (TSharedPtr<FMetasoundGraphMemberSchemaAction> Action : ActionsToDelete)
			{
				DeleteInterfaceItem(Action);
			}
		}

		bool FEditor::CanDeleteUnusedMembers() const
		{
			if (!IsGraphEditable())
			{
				return false;
			}

			if (!GraphMembersMenu.IsValid())
			{
				return false;
			}

			//Check if there is any Actions to remove in the section
			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedCategorySubActions(Actions);
			if (Actions.IsEmpty())
			{
				return false;
			}

			//Check if selected is not a Member
			TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
			GraphMembersMenu->GetSelectedActions(SelectedActions);
			if (SelectedActions.IsEmpty())
			{
				return true;
			}
			
			return false;
		}

		FActionMenuContent FEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
		{
			TSharedRef<SMetasoundActionMenu> ActionMenu = SNew(SMetasoundActionMenu)
				.AutoExpandActionMenu(bAutoExpand)
				.Graph(&GetMetaSoundGraphChecked())
				.NewNodePosition(InNodePosition)
				.DraggedFromPins(InDraggedPins)
				.OnClosedCallback(InOnMenuClosed);
// 				.OnCloseReason(this, &FEditor::OnGraphActionMenuClosed);

			TSharedPtr<SWidget> FilterTextBox = StaticCastSharedRef<SWidget>(ActionMenu->GetFilterTextBox());
			return FActionMenuContent(StaticCastSharedRef<SWidget>(ActionMenu), FilterTextBox);
		}

		void FEditor::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, ESelectInfo::Type InSelectionType)
		{
			if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || InSelectionType == ESelectInfo::OnNavigation || InActions.IsEmpty())
			{
				TArray<UObject*> SelectedObjects;
				for (const TSharedPtr<FEdGraphSchemaAction>& Action : InActions)
				{
					TSharedPtr<FMetasoundGraphMemberSchemaAction> MetasoundMemberAction = StaticCastSharedPtr<FMetasoundGraphMemberSchemaAction>(Action);
					if (MetasoundMemberAction.IsValid())
					{
						SelectedObjects.Add(MetasoundMemberAction->GetGraphMember());
					}
				}

				if (InSelectionType != ESelectInfo::Direct && !InActions.IsEmpty())
				{
					if (MetasoundGraphEditor.IsValid())
					{
						bManuallyClearingGraphSelection = true;
						MetasoundGraphEditor->ClearSelectionSet();
						bManuallyClearingGraphSelection = false;
					}
					SetSelection(SelectedObjects);
				}
			}
		}

		// TODO: Add ability to filter inputs/outputs in "MetaSound" Tab
		FText FEditor::GetFilterText() const
		{
			return FText::GetEmpty();
		}

		TSharedRef<SWidget> FEditor::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
		{
			return SNew(SMetaSoundGraphPaletteItem, InCreateData);
		}

		TSharedPtr<SWidget> FEditor::OnContextMenuOpening()
		{
			if (!GraphMembersMenu.IsValid())
			{
				return nullptr;
			}

			if (!Builder.IsValid())
			{
				return nullptr;
			}

			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
			if (!DocBuilder.IsValid())
			{
				return nullptr;
			}

			FMenuBuilder MenuBuilder(true, ToolkitCommands);
			TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
			GraphMembersMenu->GetSelectedActions(Actions);

			if (Actions.IsEmpty()) //Section is selected
			{
				if (DocBuilder.GetIsGraphEditable())
				{
					MenuBuilder.BeginSection("GraphActionMenuSectionActions", LOCTEXT("SectionActionsMenuHeader", "Section Actions"));

					if (const UMetasoundEditorGraphMember* Member = FDocumentClipboardUtils::GetMemberFromClipboard())
					{
						FText PasteTooltip;

						if (Member->IsA<UMetasoundEditorGraphVertex>())
						{
							PasteTooltip = FText::Format(LOCTEXT("PasteVertexTooltip", "Pastes the {0} to this MetaSound"), Member->GetGraphMemberLabel());
						}
						else if (Member->IsA<UMetasoundEditorGraphVariable>())
						{
							PasteTooltip = LOCTEXT("PasteVariableTooltip", "Pastes the Variable to this Graph");
						}

						MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None, FText::Format(LOCTEXT("PasteMember","Paste {0}"), Member->GetGraphMemberLabel()), PasteTooltip);
					}

					MenuBuilder.AddMenuEntry(
						LOCTEXT("DeleteAllUnusedInSection", "Delete Unused Members"),
						LOCTEXT("DeleteAllUnusedInSectionTooltip", "Delete all Unused Members under this Section"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &FEditor::DeleteAllUnusedInSection),
							FCanExecuteAction::CreateSP(this, &FEditor::CanDeleteUnusedMembers)));
					MenuBuilder.EndSection();
				}
			}
			else //Member is selected
			{
				MenuBuilder.BeginSection("GraphActionMenuMemberActions", LOCTEXT("MemberActionsMenuHeader", "Member Actions"));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
				MenuBuilder.AddMenuEntry(
					LOCTEXT("JumpToNodesMenuEntry", "Jump to Node(s) in Graph"),
					LOCTEXT("JumpToNodesMenuEntryTooltip", "Jump to the corresponding node(s) in the MetaSound graph"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FEditor::JumpToNodesForSelectedInterfaceItem), 
						FCanExecuteAction::CreateSP(this, &FEditor::CanJumpToNodesForSelectedInterfaceItem)));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
				MenuBuilder.EndSection();
			}

			return MenuBuilder.MakeWidget();
		}

		void FEditor::RemoveInvalidSelection()
		{
			if (MetasoundDetails.IsValid())
			{
				const TArray<TWeakObjectPtr<UObject>>& Objects = MetasoundDetails->GetSelectedObjects();
				TArray<UObject*> NewSelection;

				TSet<const UMetasoundEditorGraphMember*> GraphMembers;
				GetMetaSoundGraphChecked().IterateMembers([&GraphMembers](UMetasoundEditorGraphMember& GraphMember) { GraphMembers.Add(&GraphMember); });

				for (const TWeakObjectPtr<UObject>& Object : Objects)
				{
					if (Object.IsValid())
					{
						if (const UMetasoundEditorGraphMember* Member = Cast<UMetasoundEditorGraphMember>(Object.Get()))
						{
							if (GraphMembers.Contains(Member))
							{
								NewSelection.Add(Object.Get());
							}
						}
						else
						{
							NewSelection.Add(Object.Get());
						}
					}
				}

				if (NewSelection.Num() != Objects.Num())
				{
					SetSelection(NewSelection);
				}
			}
		}

		void FEditor::Tick(float DeltaTime)
		{
			using namespace Metasound::Engine;

			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return;
			}

			RefreshEditorContext(*MetaSound);

			GraphConnectionManager->Update(DeltaTime);

			if (LoadingNotificationPtr.IsValid())
			{
				const int32 NumProcessing = FMetaSoundAssetManager::GetChecked().GetActiveAsyncLoadRequestCount();
				if (NumProcessing > 0)
				{
					LoadingNotificationPtr->SetText(FText::Format(AssetEditorPrivate::LoadingAssetsFormat, NumProcessing));
				}
				else
				{
					NotifyAssetLoadingComplete();
				}
			}
		}

		void FEditor::RefreshEditorContext(UObject& MetaSound)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::RefreshEditorContext);

			if (!Builder.IsValid())
			{
				return;
			}

			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
			if (!DocBuilder.IsValid())
			{
				return;
			}

			UMetasoundEditorGraph* Graph = nullptr;
			FGraphBuilder::BindEditorGraph(DocBuilder, &Graph);
			check(Graph);

			// Requires editable builder as transient registry context may need to be recached on the document.
			const bool bSynchronizedGraph = FGraphBuilder::SynchronizeGraph(Builder->GetBuilder(), *Graph, !bRefreshGraph);
			bRefreshGraph = false;

			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
			check(MetaSoundAsset);

			// Capture after synchronizing as the modification state may be modified therein
			const FMetasoundFrontendDocumentModifyContext& ModifyContext = MetaSoundAsset->GetConstModifyContext();
			const bool bForceRefreshViews = ModifyContext.GetForceRefreshViews();
			const TSet<FName>& InterfacesModified = ModifyContext.GetInterfacesModified();
			const TSet<FGuid>& MembersModified = ModifyContext.GetMemberIDsModified();
			const TSet<FGuid>& NodesModified = ModifyContext.GetNodeIDsModified();
			if (bSynchronizedGraph || bForceRefreshViews || !InterfacesModified.IsEmpty() || !NodesModified.IsEmpty() || !MembersModified.IsEmpty())
			{
				FGraphValidationResults Results = FGraphBuilder::ValidateGraph(MetaSound);

				for (const FGraphNodeValidationResult& Result : Results.GetResults())
				{
					UMetasoundEditorGraphNode& Node = Result.GetNodeChecked();
					const bool bClassChanged = Node.ContainsClassChange();
					const FText Title = Node.GetCachedTitle();
					Node.CacheTitle();
					const bool bTitleUpdated = !Title.IdenticalTo(Node.GetCachedTitle());
					const bool bRefreshNode = NodesModified.Contains(Node.GetNodeID());
					if (Result.GetHasDirtiedNode() || bTitleUpdated || bClassChanged || bForceRefreshViews || bRefreshNode)
					{
						Node.SyncChangeIDs();
						if (MetasoundGraphEditor.IsValid())
						{
							MetasoundGraphEditor->RefreshNode(Node);
						}
					}
				}

				TArray<UObject*> Selection;

				if (!MembersModified.IsEmpty() || bForceRefreshViews)
				{
					if (const Frontend::FMetaSoundTemplate* Template = GetDocumentTemplate();
						!Template || Template->GetEditorOptions().bMemberEditingEnabled)
					{
						UMetasoundEditorGraphMember* SelectedMember = RefreshGraphMemberMenu();

						// If no member was selected by an action (ex. undo/redo), select a modified member 
						if (!SelectedMember)
						{
							for (const FGuid& MemberGuid : MembersModified)
							{
								if (UObject* Member = Graph->FindMember(MemberGuid))
								{
									// Currently only one member can be selected at a time, so only first found is added
									Selection.Add(Member);
									break;
								}
							}
						}
					}
				}

				// Only refresh details panel if
				// 1. Forcing refresh with modify context option
				// 2. The currently selected object(s) is/are modified.
				// 3. If the selection is changed via the modify context, it will automatically dirty & refresh via 'SetSelection' below
				if (bForceRefreshViews)
				{
					RefreshDetails();
				}
				else if (!NodesModified.IsEmpty() || !MembersModified.IsEmpty())
				{
					if (MetasoundDetails.IsValid())
					{
						TArray<TWeakObjectPtr<UObject>> SelectedObjects = MetasoundDetails->GetSelectedObjects();
						const bool bShouldRefreshDetails = Algo::AnyOf(SelectedObjects, [&NodesModified, &MembersModified](const TWeakObjectPtr<UObject>& Obj)
							{
								if (const UMetasoundEditorGraphNode* Node = Cast<const UMetasoundEditorGraphNode>(Obj.Get()))
								{
									return NodesModified.Contains(Node->GetNodeID());
								}
								if (const UMetasoundEditorGraphMember* Member = Cast<const UMetasoundEditorGraphMember>(Obj.Get()))
								{
									return MembersModified.Contains(Member->GetMemberID());
								}
								return false;
							});
						if (bShouldRefreshDetails)
						{
							RefreshDetails();
						}
					}
				}

				if (!InterfacesModified.IsEmpty() || bForceRefreshViews)
				{
					RefreshInterfaceView();

					// Output Format may have changed, ensure analyzers are created with the correct channel count:
					if (UMetaSoundSource* MetaSoundSource = Cast<UMetaSoundSource>(&MetaSound))
					{
						CreateAnalyzers(*MetaSoundSource);
					}
				}

				constexpr bool bSetAuditionFocus = false;
				SyncAuditionState(bSetAuditionFocus);

				// Modify data has been observed both from synchronization & by
				// updating views by this point, so full reset is completed here.
				MetaSoundAsset->GetModifyContext().Reset();

				if (!Selection.IsEmpty())
				{
					// Don't invoke tab as this can be called in response
					// to another focused, referenced graph mutating (ex.
					// interface changing).
					constexpr bool bInvokeTabOnSelectionSet = false;
					SetSelection(Selection, bInvokeTabOnSelectionSet);
				}

				// Avoids details panel displaying
				// removed members in certain cases.
				RemoveInvalidSelection();
			}

			// Prompt to Rename if requested on Member Creation.
			if (bMemberRenameRequested)
			{
				GraphMembersMenu->RefreshAllActions(/*bPreserveExpansion=*/ true, /*bHandleOnSelectionEvent=*/ true);
				GraphMembersMenu->OnRequestRenameOnActionNode();
				bMemberRenameRequested = false;
			}
		}

		TStatId FEditor::GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FMetasoundEditor, STATGROUP_Tickables);
		}

		FText FEditor::GetSectionTitle(ENodeSection InSection) const
		{
			const int32 SectionIndex = static_cast<int32>(InSection);
			if (ensure(NodeSectionNames.IsValidIndex(SectionIndex)))
			{
				return NodeSectionNames[SectionIndex];
			}

			return FText::GetEmpty();
		}

		FText FEditor::OnGetSectionTitle(int32 InSectionID)
		{
			if (ensure(NodeSectionNames.IsValidIndex(InSectionID)))
			{
				return NodeSectionNames[InSectionID];
			}

			return FText::GetEmpty();
		}

		bool FEditor::IsAuditionable() const
		{
			if (const UObject* MetaSound = GetMetasoundObject())
			{
				return MetaSound->IsA<USoundBase>();
			}

			return false;
		}

		bool FEditor::IsGraphEditable() const
		{
			using namespace Metasound::Editor;

			if (Builder.IsValid())
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
				if (DocBuilder.IsValid())
				{
					const FMetasoundFrontendGraph& Graph = DocBuilder.FindConstBuildGraphChecked();
					return Graph.Style.bIsGraphEditable;
				}
			}

			return false;
		}

		void FEditor::ClearSelectionAndSelectNode(UEdGraphNode* Node)
		{
			if (MetasoundGraphEditor.IsValid())
			{
				MetasoundGraphEditor->ClearSelectionSet();
				MetasoundGraphEditor->SetNodeSelection(Node, /*bSelect=*/true);
			}
		}

		int32 FEditor::GetNumNodesSelected() const
		{
			if (MetasoundGraphEditor.IsValid())
			{
				return MetasoundGraphEditor->GetSelectedNodes().Num();
			}

			return 0;
		}

		TSharedRef<SWidget> FEditor::OnGetMenuSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID)
		{
			TWeakPtr<SWidget> WeakRowWidget = RowWidget;

			FText AddNewText;
			FName MetaDataTag;

			if (IsGraphEditable())
			{
				switch (static_cast<ENodeSection>(InSectionID))
				{
				case ENodeSection::Inputs:
				{
					AddNewText = LOCTEXT("AddNewInput", "Input");
					MetaDataTag = "AddNewInput";
					return CreateAddButton(InSectionID, AddNewText, MetaDataTag);
				}

				case ENodeSection::Outputs:
				{
					AddNewText = LOCTEXT("AddNewOutput", "Output");
					MetaDataTag = "AddNewOutput";
					return CreateAddButton(InSectionID, AddNewText, MetaDataTag);
				}
	
				case ENodeSection::Variables:
				{
					AddNewText = LOCTEXT("AddNewVariable", "Variable");
					MetaDataTag = "AddNewVariable";
					return CreateAddButton(InSectionID, AddNewText, MetaDataTag);
				}

				default:
					break;
				}
			}

			return SNullWidget::NullWidget;
		}

		bool FEditor::CanAddNewElementToSection(int32 InSectionID) const
		{
			return true;
		}

		FReply FEditor::OnAddButtonClickedOnSection(int32 InSectionID)
		{
			UObject* MetaSound = GetMetasoundObject();
			if (!MetaSound)
			{
				return FReply::Unhandled();
			}

			const FName DataTypeName = GetMetasoundDataTypeName<float>();

			UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

			TArray<TObjectPtr<UObject>> SelectedObjects;

			FName NameToSelect;
			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
			switch (static_cast<ENodeSection>(InSectionID))
			{
				case ENodeSection::Inputs:
				{
					const FScopedTransaction Transaction(LOCTEXT("AddInputNode", "Add MetaSound Input"));
					MetaSound->Modify();
					Graph.Modify();

					FCreateNodeVertexParams VertexParams;
					VertexParams.DataType = DataTypeName;

					FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(*MetaSound, VertexParams);
					if (const FMetasoundFrontendNode* NewNode = DocBuilder.AddGraphInput(MoveTemp(ClassInput)))
					{
						NameToSelect = NewNode->Name;

						TObjectPtr<UMetasoundEditorGraphInput> Input = Graph.FindOrAddInput(NewNode->GetID());
						if (ensure(Input))
						{
							SelectedObjects.Add(Input);
						}
					}
				}
				break;

				case ENodeSection::Outputs:
				{
					const FScopedTransaction Transaction(LOCTEXT("AddOutputNode", "Add MetaSound Output"));
					MetaSound->Modify();
					Graph.Modify();

					FCreateNodeVertexParams VertexParams;
					VertexParams.DataType = DataTypeName;

					FMetasoundFrontendClassOutput ClassOutput = FGraphBuilder::CreateUniqueClassOutput(*MetaSound, VertexParams);
					if (const FMetasoundFrontendNode* NewNode = DocBuilder.AddGraphOutput(MoveTemp(ClassOutput)))
					{
						NameToSelect = NewNode->Name;

						TObjectPtr<UMetasoundEditorGraphOutput> Output = Graph.FindOrAddOutput(NewNode->GetID());
						if (ensure(Output))
						{
							SelectedObjects.Add(Output);
						}
					}
				}
				break;

				case ENodeSection::Variables:
				{
					const FScopedTransaction Transaction(TEXT(""), LOCTEXT("AddVariableNode", "Add MetaSound Variable"), MetaSound);
					MetaSound->Modify();
					Graph.Modify();

					const FName VariableName = DocBuilder.GenerateUniqueVariableName(TEXT("Variable"));
					const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.AddGraphVariable(VariableName, DataTypeName);
					if (ensure(FrontendVariable))
					{
						TObjectPtr<UMetasoundEditorGraphVariable> EditorVariable = Graph.FindOrAddVariable(FrontendVariable->Name);
						if (ensure(EditorVariable))
						{
							SelectedObjects.Add(EditorVariable);
							NameToSelect = EditorVariable->GetMemberName();
						}
					}
				}
				break;

				default:
				return FReply::Unhandled();
			}

			{
				Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
				RegOptions.bForceViewSynchronization = true;
				FGraphBuilder::RegisterGraphWithFrontend(*MetaSound, MoveTemp(RegOptions));
			}

			if (GraphMembersMenu.IsValid())
			{
				GraphMembersMenu->RefreshAllActions(/* bPreserveExpansion */ true);
				if (!NameToSelect.IsNone())
				{
					GraphMembersMenu->SelectItemByName(NameToSelect);
					SetSelection(SelectedObjects);
					SetDelayedRename();
				}
			}
			return FReply::Handled();
		}

		TSharedRef<SWidget> FEditor::CreateAddButton(int32 InSectionID, FText AddNewText, FName MetaDataTag)
		{
			return
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &FEditor::OnAddButtonClickedOnSection, InSectionID)
				.IsEnabled(this, &FEditor::CanAddNewElementToSection, InSectionID)
				.ContentPadding(FMargin(1, 0))
				.AddMetaData<FTagMetaData>(FTagMetaData(MetaDataTag))
				.ToolTipText(AddNewText)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}

		void FEditor::ShowFindInMetaSound()
		{
			TabManager->TryInvokeTab(TabNamesPrivate::Find);
			if (FindWidget.IsValid())
			{
				FindWidget->FocusForUse();
			}
		}

		void FEditor::FindSelectedNodeInGraph()
		{
			TabManager->TryInvokeTab(TabNamesPrivate::Find);
			if (FindWidget.IsValid())
			{		
				const FGraphPanelSelectionSet& SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
				for (UObject* Object : SelectedNodes)
				{
					if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(Object))
					{
						FString SearchTerms = SelectedNode->GetFindReferenceSearchString(EGetFindReferenceSearchStringFlags::UseSearchSyntax);
						FindWidget->FocusForUse(SearchTerms);
					}
				}
			}
		}

		void FEditor::SetDelayedRename()
		{
			bMemberRenameRequested = true;
		}

		void FEditor::UpdateEnabledTabs()
		{
			using namespace Frontend;

			if (!TabManager)
			{
				return;
			}

			auto ResolveTabEnablement = [this](FName TabName, bool bEnabled)
			{
				if (TSharedPtr<FTabSpawnerEntry> Spawner = TabManager->FindTabSpawnerFor(TabName))
				{
					Spawner->SetMenuType(bEnabled ? ETabSpawnerMenuType::Enabled : ETabSpawnerMenuType::Hidden);
				}

				if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(TabName))
				{
					if (!bEnabled)
					{
						Tab->RequestCloseTab();
					}
				}
				else
				{
					if (bEnabled)
					{
						TabManager->TryInvokeTab(TabName, false /* bInvokeAsInactive */);
					}
				}
			};

			auto RefreshViews = [this]()
			{
				RefreshGraphMemberMenu();
				RefreshInterfaceView();
				RefreshDetails();
			};

			if (FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder())
			{
				const bool bHasProjectPageValues = DocBuilder->GetConstDocumentChecked().RootGraph.GetConstGraphPages().Num() > 1;

				const bool bEnablePageTab = IsPageGraphEditingEnabled(*DocBuilder, bHasProjectPageValues);

				bool bConfigVisibilitySet = false;
				if (const FMetaSoundTemplate* Template = GetDocumentTemplate())
				{
					if (!AssetEditorPrivate::bIgnoreDocTemplateVisibilitySettings)
					{
						const FMetaSoundTemplate::FEditorOptions& EdOptions = Template->GetEditorOptions();
						const bool bShowGraph = EdOptions.bGraphEditorVisible || AssetEditorPrivate::bShowGraphOverride;
						ResolveTabEnablement(TabNamesPrivate::Find, bShowGraph);
						ResolveTabEnablement(TabNamesPrivate::GraphCanvas, bShowGraph);
						ResolveTabEnablement(TabNamesPrivate::Interfaces, EdOptions.bInterfaceEditingEnabled);
						ResolveTabEnablement(TabNamesPrivate::Members, EdOptions.bMemberEditingEnabled);

						// 'IsPageGraphEditingEnabled' above internally checks if config supports page graph editing,
						// so need to resolve enablement for the pages tab here.

						// Don't show custom config tab if no config widget was constructed, even if enabled
						for (const FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
						{
							ResolveTabEnablement(Pair.Value.TabInfo.TabName, EdOptions.bCustomTabEnabled);
						}

						RefreshViews();
						bConfigVisibilitySet = true;
					}
				}

				if (!bConfigVisibilitySet)
				{
					{
						constexpr bool bEnableTab = true;
						ResolveTabEnablement(TabNamesPrivate::Find, bEnableTab);
						ResolveTabEnablement(TabNamesPrivate::GraphCanvas, bEnableTab);
						ResolveTabEnablement(TabNamesPrivate::Interfaces, bEnableTab);
						ResolveTabEnablement(TabNamesPrivate::Members, bEnableTab);
					}

					{
						// Doc config widgets are only visible when visibility is set unless ignore cvar is true.
						const bool bEnableTab = !AssetEditorPrivate::bIgnoreDocTemplateVisibilitySettings;
						for (const FBPIdWidgetDataPair& Pair : DocTemplateWidgetData)
						{
							ResolveTabEnablement(Pair.Value.TabInfo.TabName, bEnableTab);
						}
					}
				}

				// Pages need additional checks to determine edit eligibility
				ResolveTabEnablement(TabNamesPrivate::Pages, bEnablePageTab);

				{
					UObject* MetaSoundObject = GetMetasoundObject();
					const bool bEnableTab = MetaSoundObject && MetaSoundObject->IsA<USoundBase>();
					ResolveTabEnablement(TabNamesPrivate::Analyzers, bEnableTab);
				}

				RefreshViews();
			}
		}

		TUniquePtr<FGraphConnectionManager> FEditor::RebuildConnectionManager(UAudioComponent* PreviewComp) const
		{
			using namespace Engine;
			using namespace Frontend;

			if (!PreviewComp || !Builder.IsValid())
			{
				return MakeUnique<FGraphConnectionManager>();
			}

			UMetaSoundSource* Source = Cast<UMetaSoundSource>(GetMetasoundObject());
			if (!Source)
			{
				return MakeUnique<FGraphConnectionManager>();
			}

			if (const FMetasoundFrontendGraph* GraphPage = FindPreferredPage(Source->GetConstDocument().RootGraph.GetConstGraphPages(), UMetaSoundSettings::GetPageOrder()))
			{
				if (GraphPage->PageID !=  Builder->GetConstBuilder().GetBuildPageID())
				{
					return MakeUnique<FGraphConnectionManager>();
				}
			}
			else
			{
				return MakeUnique<FGraphConnectionManager>();
			}

			FAudioDevice* AudioDevice = PreviewComp->GetAudioDevice();
			if (!AudioDevice)
			{
				return MakeUnique<FGraphConnectionManager>();
			}

			const FSampleRate DeviceSampleRate = static_cast<FSampleRate>(AudioDevice->GetSampleRate());
			const uint32 PlayOrder = PreviewComp->GetLastPlayOrder();
			const uint64 TransmitterID = Audio::GetTransmitterID(PreviewComp->GetAudioComponentID(), 0, PlayOrder);

			return MakeUnique<FGraphConnectionManager>(*Source, *PreviewComp, TransmitterID, Source->GetOperatorSettings(DeviceSampleRate));
		}

		void FEditor::UpdatePageInfo(bool bIsPlaying)
		{
			using namespace Engine;
			using namespace Frontend;

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings)

			if (PageStatsWidget.IsValid())
			{
				const FSlateColor* Color = nullptr;
				const FMetaSoundPageSettings* GraphPageSettings = nullptr;
				const FMetaSoundPageSettings* AuditionPageSettings = nullptr;

				if (Builder.IsValid() && ShowPageGraphDetails())
				{
					if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
					{
						AuditionPageSettings = Settings->FindPageSettings(EditorSettings->GetAuditionPage());
					}

					const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
					const FGuid& PageID = DocBuilder.GetBuildPageID();
					if (bIsPlaying)
					{
						if (const FMetasoundFrontendGraph* Graph= FindPreferredPage(DocBuilder.GetConstDocumentChecked().RootGraph.GetConstGraphPages(), UMetaSoundSettings::GetPageOrder()))
						{
							if (Graph->PageID == PageID)
							{
								Color = &Style::GetPageExecutingColor();
							}
						}
					}
					GraphPageSettings = Settings->FindPageSettings(PageID);
				}

				PageStatsWidget->Update(AuditionPageSettings, GraphPageSettings, Color);

				// Update tab label with page name
				TSharedPtr<SDockTab> GraphTab = TabManager->FindExistingLiveTab(TabNamesPrivate::GraphCanvas);
				if (GraphTab.IsValid())
				{
					const FText GraphTabLabel = FText::Format(LOCTEXT("GraphCanvasPageTab", "Graph Page: {0}"), PageStatsWidget.IsValid() ?
						FText::FromName(PageStatsWidget->GetDisplayedPageName()) : LOCTEXT("MetasoundGraphCanvasTitle", "MetaSound Graph"));
					GraphTab->SetLabel(GraphTabLabel);
				}
			}
		}

		void FEditor::UpdateRenderInfo(bool bIsPlaying, float InDeltaTime)
		{
			if (!bIsPlaying)
			{
				SetPreviewID(INDEX_NONE);
			}

			if (RenderStatsWidget.IsValid())
			{
				RenderStatsWidget->Update(bIsPlaying, InDeltaTime, Cast<const UMetaSoundSource>(GetMetasoundObject()));
			}
		}

		void FEditor::FDocumentListener::OnBuilderReloaded(Frontend::FDocumentModifyDelegates& OutDelegates)
		{
			OutDelegates.PageDelegates.OnPageSet.AddSP(this, &FDocumentListener::OnPageSet);
			OutDelegates.InterfaceDelegates.OnInputDefaultChanged.AddSP(this, &FDocumentListener::OnInputDefaultChanged);
			OutDelegates.InterfaceDelegates.OnMemberMetadataSet.AddSP(this, &FDocumentListener::OnMemberMetadataSet);
			OutDelegates.OnDocumentTemplateChanged.AddSP(this, &FDocumentListener::OnDocumentTemplateChanged);
		}

		void FEditor::FDocumentListener::OnDocumentTemplateChanged(const Frontend::FDocumentTemplateChangedArgs& Args)
		{
			if (TSharedPtr<FEditor> ParentPtr = Parent.Pin())
			{
				if (Args.Type != Frontend::EDocumentTemplateChangeType::Property)
				{
					ParentPtr->ReloadAllDocTemplateTabs();
					ParentPtr->UpdateEnabledTabs();
				}
			}
		}

		void FEditor::FDocumentListener::OnPageSet(const Frontend::FDocumentMutatePageArgs& Args)
		{
			if (TSharedPtr<FEditor> ParentPtr = Parent.Pin())
			{
				ParentPtr->Stop();
				ParentPtr->UpdatePageInfo(false);
				ParentPtr->bRefreshGraph = true;
				ParentPtr->RefreshExecVisibility(Args.PageID);

				if (ParentPtr->GraphMembersMenu.IsValid())
				{
					ParentPtr->GraphMembersMenu->RefreshAllActions(true);
				}
			}
		}

		void FEditor::RebuildUserWidgets(
			UWorld& World,
			UAudioWidgetSubsystem& WidgetSubsystem,
			TSubclassOf<UInterface> InterfaceClass,
			TFunction<bool(UUserWidget*)>Filter,
			TFunction<FMetaSoundTemplateWidgetTabInfo(UUserWidget*)>OnTabConstruct,
			FGuid TargetBlueprintId)
		{
			TFunction<bool(UBlueprint*)> BlueprintFilter;
			if (TargetBlueprintId.IsValid())
			{
				BlueprintFilter = [&TargetBlueprintId](UBlueprint* OtherBlueprint)
				{
					return OtherBlueprint && OtherBlueprint->GetBlueprintGuid() == TargetBlueprintId;
				};
			}

			TArray<UUserWidget*> NewWidgets = WidgetSubsystem.CreateUserWidgets(World, InterfaceClass, MoveTemp(Filter), BlueprintFilter);
			for (UUserWidget* UserWidget : NewWidgets)
			{
				check(UserWidget);
				if (UBlueprint* OwningBlueprint = UBlueprint::GetBlueprintFromClass(UserWidget->GetClass()))
				{
					const FBlueprintId& OwningBPId = OwningBlueprint->GetBlueprintGuid();

					FDelegateHandle RecompileHandle;

					// If request to rebuild provides a target blueprint ID, reuse the
					// existing handle if found. Otherwise, assign a new delegate handler.
					if (TargetBlueprintId.IsValid())
					{
						if (FUserWidgetData* WidgetData = DocTemplateWidgetData.Find(TargetBlueprintId))
						{
							RecompileHandle = MoveTemp(WidgetData->RecompileHandle);
						}
					}
					else
					{
						OwningBlueprint->OnCompiled().RemoveAll(this);
					}

					DocTemplateWidgetData.Remove(OwningBPId);

					if (!RecompileHandle.IsValid())
					{
						RecompileHandle = OwningBlueprint->OnCompiled().AddSP(this, &FEditor::OnDocTemplateWidgetBlueprintCompiled);
					}

					TStrongObjectPtr<UUserWidget> UserWidgetPtr(UserWidget);
					FUserWidgetData& NewData = DocTemplateWidgetData.Add(OwningBPId, FUserWidgetData
					{
						.Class = InterfaceClass,
						.RecompileHandle = MoveTemp(RecompileHandle),
						.Widget = MoveTemp(UserWidgetPtr),
						.TabInfo = OnTabConstruct(UserWidget)
					});

					FMetaSoundFrontendDocumentBuilder* DocBuilder = GetFrontendBuilder();
					checkf(DocBuilder, TEXT("Builder must exist by this point in user widget rebuild"));

					const UScriptStruct* ConfigStruct = DocBuilder->GetConstDocumentTemplate().GetScriptStruct();
					checkf(ConfigStruct, TEXT("Builder must have a config set by this point in user widget rebuild"));

					AssetEditorPrivate::SanitizeTabInfo(*ConfigStruct, NewData.TabInfo);
				}
			}
		}

		void FEditor::RefreshExecVisibility(const FGuid& InPageID) const
		{
			if (PageStatsWidget.IsValid())
			{
				TAttribute<EVisibility> ExecVisibility = TAttribute<EVisibility>::CreateSPLambda(AsShared(), [this, InPageID]()
				{
					using namespace Engine;
					if (Builder.IsValid() && ShowPageGraphDetails())
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
						const bool bIsPreviewing = IsPreviewingPageGraph(DocBuilder, InPageID);
						return bIsPreviewing ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
					}

					return EVisibility::Collapsed;
				});
				PageStatsWidget->SetExecVisibility(MoveTemp(ExecVisibility));
			}
		}

		void FEditor::FDocumentListener::OnInputDefaultChanged(const Frontend::FDocumentArrayPagedInputArgs& Args)
		{
			if (TSharedPtr<FEditor> ParentPtr = Parent.Pin())
			{
				UMetasoundEditorGraph* MetaSoundGraph = ParentPtr->GetMetaSoundGraph();
				if (!MetaSoundGraph)
				{
					return;
				}

				const FMetaSoundFrontendDocumentBuilder& ConstBuilder = ParentPtr->Builder->GetConstBuilder();
				const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
				const FMetasoundFrontendGraph& Graph = GraphClass.GetConstDefaultGraph();

				const TArray<FMetasoundFrontendClassInput>& GraphInputs = GraphClass.GetDefaultInterface().Inputs;
				if (GraphInputs.IsValidIndex(Args.Index))
				{
					const FMetasoundFrontendClassInput& GraphInput = GraphClass.GetDefaultInterface().Inputs[Args.Index];
					const FName InputName = GraphInput.Name;
					UMetasoundEditorGraphInput* EditorGraphInput = MetaSoundGraph->FindInput(InputName);

					// Literal does not need to be updated if the data type doesn't match,
					// as it should be getting overwritten by the new literal type after this delegate call
					if (EditorGraphInput && GraphInput.TypeName == EditorGraphInput->GetDataType())
					{
						UMetasoundEditorGraphMemberDefaultLiteral* Literal = EditorGraphInput->GetLiteral();
						if (Literal)
						{
							// Synchronize editor literal and update preview instance
							Literal->Synchronize();
							if (GEditor && MetaSoundGraph->IsPreviewing())
							{
								UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent();
								check(PreviewComponent);

								if (TScriptInterface<IAudioParameterControllerInterface> ParamInterface = PreviewComponent)
								{
									Literal->UpdatePreviewInstance(InputName, ParamInterface);
								}
							}
						}
					}
				}
			}
		}

		void FEditor::FDocumentListener::OnMemberMetadataSet(FGuid MemberID)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FEditor::OnMemberMetadataSet);

			if (TSharedPtr<FEditor> ParentPtr = Parent.Pin())
			{
				UMetasoundEditorGraph* MetaSoundGraph = ParentPtr->GetMetaSoundGraph();
				if (!MetaSoundGraph)
				{
					return;
				}

				UMetasoundEditorGraphMember* Member = MetaSoundGraph->FindMember(MemberID);
				if (Member)
				{
					Member->InitializeLiteral(/*bForceRebind=*/ true);
				}
			}
		}

		bool FEditor::CanPromoteToInput()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
				if (const UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu(); ensure(TargetPin))
				{
					return TargetPin->Direction == EGPD_Input;
				}
			}
			return false;
		}

		void FEditor::PromoteToInput()
		{
			using namespace Metasound::Frontend;

			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

				UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu();
				check(TargetPin);

				UEdGraphNode* OwningNode = TargetPin->GetOwningNode();
				FVector2D Location = FVector2D(OwningNode->NodePosX, OwningNode->NodePosY);
				SchemaUtils::PromoteToInput(&Graph, TargetPin, Location - DisplayStyle::NodeLayout::DefaultOffsetX, /*bSelectNewNode=*/ true);
			}
		}

		bool FEditor::CanPromoteToOutput()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
				if (const UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu(); ensure(TargetPin))
				{
					return TargetPin->Direction == EGPD_Output;
				}
			}
			return false;
		}

		void FEditor::PromoteToOutput()
		{
			using namespace Frontend;

			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

				UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu();
				check(TargetPin);

				UEdGraphNode* OwningNode = TargetPin->GetOwningNode();
				FVector2D Location = FVector2D(OwningNode->NodePosX, OwningNode->NodePosY);
				SchemaUtils::PromoteToOutput(&Graph, TargetPin, Location + DisplayStyle::NodeLayout::DefaultOffsetX * 2.f, /*bSelectNewNode=*/ true);
			}
		}

		bool FEditor::CanPromoteToVariable()
		{
			return true;
		}

		void FEditor::PromoteToVariable()
		{
			using namespace Metasound::Frontend;

			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

				UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu();
				check(TargetPin);

				UEdGraphNode* OwningNode = TargetPin->GetOwningNode();
				FVector2D Location = FVector2D(OwningNode->NodePosX, OwningNode->NodePosY);
				if (TargetPin->Direction == EGPD_Input)
				{
					SchemaUtils::PromoteToVariable(&Graph, TargetPin, Location - DisplayStyle::NodeLayout::DefaultOffsetX, /*bSelectNewNode=*/ true);
				}
				else
				{
					SchemaUtils::PromoteToMutatorVariable(&Graph, TargetPin, Location + DisplayStyle::NodeLayout::DefaultOffsetX * 2.f, /*bSelectNewNode=*/ true);
				}
			}
		}

		bool FEditor::CanPromoteToDeferredVariable()
		{
			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

				UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu();
				check(TargetPin);

				if (TargetPin->Direction == EGPD_Input)
				{
					return true;
				}
			}
			return false;
		}

		void FEditor::PromoteToDeferredVariable()
		{
			using namespace Frontend;

			if (MetasoundGraphEditor.IsValid())
			{
				UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();

				UEdGraphPin* TargetPin = MetasoundGraphEditor->GetGraphPinForMenu();
				check(TargetPin);

				UEdGraphNode* OwningNode = TargetPin->GetOwningNode();
				FVector2D Location = FVector2D(OwningNode->NodePosX, OwningNode->NodePosY);
				SchemaUtils::PromoteToDeferredVariable(&Graph, TargetPin, Location - DisplayStyle::NodeLayout::DefaultOffsetX, /*bSelectNewNode=*/ true);
			}
		}

		int32 FEditor::PromotableSelectedNodes()
		{
			int32 Counter = 0;

			FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt);
				for (const UEdGraphPin* Pin : Node->Pins)
				{
					if (Pin->Direction == EGPD_Input && !Pin->HasAnyConnections())
					{
						Counter += 1;
						break;
					}
				}
			}
			return Counter;
		}

		bool FEditor::CanPromoteAllToInputs()
		{
			return PromotableSelectedNodes() > 0;
		}

		void FEditor::PromoteAllToInputs()
		{
			using namespace Frontend;

			UObject& ParentMetasound = *GetMetasoundObject();
			UMetasoundEditorGraph& MetasoundGraph = GetMetaSoundGraphChecked();

			const FScopedTransaction Transaction(LOCTEXT("PromoteNodeInputsToGraphInputs", "Promote MetaSound Node Inputs to Graph Inputs"));
			ParentMetasound.Modify();
			MetasoundGraph.Modify();

			FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				UMetasoundEditorGraphNode* EdGraphNode = Cast<UMetasoundEditorGraphNode>(*NodeIt);
				FVector2D NodeOffset = FVector2D(0.f);

				for (UEdGraphPin* Pin : EdGraphNode->Pins)
				{
					if (Pin->Direction != EGPD_Input || Pin->HasAnyConnections())
					{
						continue;
					}

					FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();

					FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, Pin);
					check(InputVertexHandle.IsSet());
					const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
					check(InputVertex);

					FName Name = DocBuilder.GenerateUniqueInputName(Pin->GetName());

					EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
					FMetasoundFrontendLiteral Literal;
					FGraphBuilder::GetPinLiteral(*Pin, Literal);
					bool bIsConstructorInput = DocBuilder.GetNodeInputAccessType(InputVertexHandle.NodeID, InputVertexHandle.VertexID) == EMetasoundFrontendVertexAccessType::Value;
					FMetaSoundBuilderNodeOutputHandle OutputHandle = Builder->AddGraphInputNode(Name, InputVertex->TypeName, Literal, Result, bIsConstructorInput);
					check(Result == EMetaSoundBuilderResult::Succeeded);

					FVector2D Location = FVector2D(EdGraphNode->NodePosX, EdGraphNode->NodePosY);
					Location -= DisplayStyle::NodeLayout::DefaultOffsetX;
					Location += NodeOffset;
					NodeOffset += DisplayStyle::NodeLayout::DefaultOffsetY * 0.5f;

					Builder->SetNodeLocation(OutputHandle.NodeID, Location, Result);
					check(Result == EMetaSoundBuilderResult::Succeeded);

					if (const FMetasoundFrontendNode* NewTemplateNode = FInputNodeTemplate::CreateNode(DocBuilder, Name))
					{
						if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetasound, NewTemplateNode->GetID()))
						{
							FMetaSoundNodeHandle NewNodeHandle(NewGraphNode->GetFrontendNode()->GetID());
							FName OutputName = NewGraphNode->GetFrontendNode()->Interface.Outputs[0].Name;
							OutputHandle = Builder->FindNodeOutputByName(NewNodeHandle, OutputName, Result);
							check(Result == EMetaSoundBuilderResult::Succeeded);

							FMetaSoundNodeHandle SourceNodeHandle(EdGraphNode->GetFrontendNode()->GetID());
							FMetaSoundBuilderNodeInputHandle InputHandle = Builder->FindNodeInputByName(SourceNodeHandle, InputVertex->Name, Result);
							check(Result == EMetaSoundBuilderResult::Succeeded);

							Builder->ConnectNodes(OutputHandle, InputHandle, Result);
							check(Result == EMetaSoundBuilderResult::Succeeded);
						}
					}
				}
			}

			{
				Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
				RegOptions.bForceViewSynchronization = true;
				FGraphBuilder::RegisterGraphWithFrontend(ParentMetasound, MoveTemp(RegOptions));
			}
		}

		bool FEditor::CanPromoteAllToCommonInputs()
		{
			return PromotableSelectedNodes() > 1;
		}

		void FEditor::PromoteAllToCommonInputs()
		{
			using namespace Frontend;

			UObject& ParentMetasound = *GetMetasoundObject();
			UMetasoundEditorGraph& MetasoundGraph = GetMetaSoundGraphChecked();

			const FScopedTransaction Transaction(LOCTEXT("PromoteNodeInputsToCommonGraphInputs", "Promote MetaSound Node Inputs to Shared Graph Inputs"));
			ParentMetasound.Modify();
			MetasoundGraph.Modify();

			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();

			// PinsMap.Key { pin name, pin data type }
			TMap<TPair<FName, FName>, TArray<UEdGraphPin*>> PinsMap;
			TMap<FGuid, FVector2D> NodeOffsets;

			// Find common pins and save them for processing at later stage
			FGraphPanelSelectionSet SelectedNodes = MetasoundGraphEditor->GetSelectedNodes();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(*NodeIt);
				for (UEdGraphPin* Pin : EdGraphNode->Pins)
				{
					if (Pin->Direction == EGPD_Input && !Pin->HasAnyConnections())
					{
						// Get type name from pin
						FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, Pin);
						check(InputVertexHandle.IsSet());
						const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
						check(InputVertex);

						TPair<FName, FName> Pair = { Pin->GetFName(), InputVertex->TypeName };
						
						if (TArray<UEdGraphPin*>* PtrArr = PinsMap.Find(Pair))
						{
							PtrArr->Add(Pin);
						}
						else
						{
							TArray<UEdGraphPin*> Arr;
							Arr.Add(Pin);
							PinsMap.Add(Pair, Arr);
						}
					}

					NodeOffsets.Add(EdGraphNode->NodeGuid, FVector2D(0.f));
				}
			}

			for (const TPair<TPair<FName, FName>, TArray<UEdGraphPin*>>& Pair : PinsMap)
			{
				check(Pair.Value.Num() != 0);
				
				FName PinName = Pair.Key.Key;
				FName TypeName = Pair.Key.Value;
				UEdGraphPin* SourcePin = Pair.Value[0];
				FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, SourcePin);
				FName InputName = DocBuilder.GenerateUniqueInputName(PinName.ToString());
				
				EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
				FMetasoundFrontendLiteral Literal;
				FGraphBuilder::GetPinLiteral(*SourcePin, Literal);
				bool bIsConstructorInput = DocBuilder.GetNodeInputAccessType(InputVertexHandle.NodeID, InputVertexHandle.VertexID) == EMetasoundFrontendVertexAccessType::Value;
				FMetaSoundBuilderNodeOutputHandle OutputHandle = Builder->AddGraphInputNode(InputName, TypeName, Literal, Result, bIsConstructorInput);
				check(Result == EMetaSoundBuilderResult::Succeeded);

				FVector2D* NodeOffset = NodeOffsets.Find(SourcePin->GetOwningNode()->NodeGuid);
				check(NodeOffset);

				FVector2D Location = FVector2D(SourcePin->GetOwningNode()->NodePosX, SourcePin->GetOwningNode()->NodePosY);
				Location -= DisplayStyle::NodeLayout::DefaultOffsetX;
				Location += *NodeOffset;
				*NodeOffset += DisplayStyle::NodeLayout::DefaultOffsetY * 0.5f;
				
				Builder->SetNodeLocation(OutputHandle.NodeID, Location, Result);
				check(Result == EMetaSoundBuilderResult::Succeeded);

				if (const FMetasoundFrontendNode* NewTemplateNode = FInputNodeTemplate::CreateNode(DocBuilder, InputName))
				{
					if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetasound, NewTemplateNode->GetID()))
					{
						FMetaSoundNodeHandle NewNodeHandle(NewGraphNode->GetFrontendNode()->GetID());
						FName OutputName = NewGraphNode->GetFrontendNode()->Interface.Outputs[0].Name;
						OutputHandle = Builder->FindNodeOutputByName(NewNodeHandle, OutputName, Result);
						check(Result == EMetaSoundBuilderResult::Succeeded);

						for (const UEdGraphPin* Pin : Pair.Value)
						{
							UMetasoundEditorGraphNode* EdGraphNode = Cast<UMetasoundEditorGraphNode>(Pin->GetOwningNode());

							FMetaSoundNodeHandle SourceNodeHandle(EdGraphNode->GetFrontendNode()->GetID());
							FMetaSoundBuilderNodeInputHandle InputHandle = Builder->FindNodeInputByName(SourceNodeHandle, PinName, Result);
							check(Result == EMetaSoundBuilderResult::Succeeded);
							
							Builder->ConnectNodes(OutputHandle, InputHandle, Result);
							check(Result == EMetaSoundBuilderResult::Succeeded);
						}
					}
				}
			}

			{
				Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
				RegOptions.bForceViewSynchronization = true;
				FGraphBuilder::RegisterGraphWithFrontend(ParentMetasound, MoveTemp(RegOptions));
			}
		}

		void FEditor::SetDroppedObjectsInfo(FDroppedObjectsInfo InInfo)
		{
			DroppedObjectsInfo = TOptional<FDroppedObjectsInfo>(MoveTemp(InInfo));
		}

		const TOptional<FDroppedObjectsInfo>& FEditor::GetDroppedObjectsInfo() const
		{
			return DroppedObjectsInfo;
		}

		void FEditor::ClearDroppedObjectsInfo()
		{
			DroppedObjectsInfo.Reset();
		}
	}
}

#undef LOCTEXT_NAMESPACE
