// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraph/EdGraphNode.h"
#include "EditorUndoClient.h"
#include "Framework/Commands/UICommandList.h"
#include "GraphEditor.h"
#include "IDetailsView.h"
#include "IMetasoundEditor.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathUtility.h"
#include "MetasoundEditorGraphConnectionManager.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundTemplateWidgetInterface.h"
#include "Misc/NotifyHook.h"
#include "SGraphActionMenu.h"
#include "SGraphPanel.h"
#include "SMetasoundPalette.h"
#include "SMetasoundStats.h"
#include "Sound/AudioBus.h"
#include "Textures/SlateIcon.h"
#include "TickableEditorObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SPanel.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#include "MetasoundEditor.generated.h"


// Forward Declarations
class UAudioWidgetSubsystem;
class FSlateRect;
class FTabManager;
class IDetailsView;
class IToolkitHost;
class SDockableTab;
class SGraphEditor;
class SMetasoundPalette;
class SVerticalBox;
class UAudioComponent;
class UEdGraphNode;
class UMetasoundEditorGraph;
class UMetasoundEditorGraphNode;
class UMetaSoundPatch;
class UMetaSoundSource;
class UUserWidget;
class UWorld;

struct FGraphActionNode;
struct FMetaSoundFrontendDocumentBuilder;
struct FPropertyChangedEvent;

// Forward Declarations
namespace AudioWidgets
{
	class FAudioAnalyzerRack;
} // AudioWidgets

namespace Metasound::Editor
{
	class SFindInMetasound;
	class SRenderStats;
} // namespace Metasound::Editor


// Base implementation for editor wrappers that provide edit customizations
UCLASS(MinimalAPI)
class UMetasoundEditorViewBase : public UObject
{
	GENERATED_BODY()
public:

	void SetMetasound(UObject* InMetasound) { Metasound = InMetasound; }
	const UObject* GetMetasound() const { return Metasound.Get(); }
	UObject* GetMetasound() { return Metasound.Get(); }

private:
	TWeakObjectPtr<UObject> Metasound;
};


// Simple class for the interfaces details tab to keep track of its corresponding Metasound 
UCLASS(MinimalAPI)
class UMetasoundInterfacesView : public UMetasoundEditorViewBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMetasoundPagesView : public UMetasoundEditorViewBase
{
	GENERATED_BODY()
};

namespace Metasound::Editor
{
	bool IsPreviewingPageInputDefault(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassInput& InClassInput, const FGuid& InPageID);
	bool IsPreviewingPageGraph(const FMetaSoundFrontendDocumentBuilder& Builder, const FGuid& InPageID);

	bool IsPageAuditioningEnabled(const FMetaSoundFrontendDocumentBuilder& Builder, bool bHasProjectPageValues);
	bool IsPageGraphEditingEnabled(const FMetaSoundFrontendDocumentBuilder& Builder, bool bHasProjectPageValues);
	bool IsInputPageEditingEnabled(const FMetaSoundFrontendDocumentBuilder& Builder, bool bHasProjectPageValues);

	// Forward Declarations
	class FMetasoundGraphMemberSchemaAction;

	/* Enums to use when grouping the members in the list panel. Enum order dictates visible order. */
	enum class ENodeSection : uint8
	{
		None,
		Inputs,
		Outputs,
		Variables,

		COUNT
	};

	class FEditor : public IMetasoundEditor, public FNotifyHook, public FEditorUndoClient, public FTickableEditorObject
	{
	public:
		static const FLazyName EditorName;

		FEditor();
		virtual ~FEditor();

		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

		TSharedPtr<SGraphEditor> GetGraphEditor() const;

		/** Edits the specified Metasound object */
		void InitMetasoundEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit);

		UAudioComponent* GetAudioComponent() const;

		const Frontend::FMetaSoundTemplate* GetDocumentTemplate() const;
		FMetaSoundFrontendDocumentBuilder* GetFrontendBuilder() const;

		/** IMetasoundEditor interface */
		virtual UObject* GetMetasoundObject() const override;
		virtual void SetSelection(const TArray<UObject*>& SelectedObjects, bool bInvokeTabOnSelectionSet = true) override;
		virtual bool GetBoundsForSelectedNodes(FSlateRect& Rect, float Padding) override;

		/** IToolkit interface */
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual const FSlateBrush* GetDefaultTabIcon() const override;
		virtual FLinearColor GetDefaultTabColor() const override;

		/** IAssetEditorInstance interface */
		virtual FName GetEditorName() const override;

		virtual FString GetDocumentationLink() const override
		{
			return FString(TEXT("working-with-audio/sound-sources/meta-sounds"));
		}

		/** FEditorUndoClient Interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

		/** FTickableEditorObject Interface */
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

		virtual void Play() override;
		virtual void Stop() override;

		/** Returns whether or not template visibility settings should be ignored (for debugging/development) */
		static bool IgnoreDocTemplateVisibilitySettings();

		/** Returns whether or not live audition is enabled for previewing */
		static bool IsLiveAuditionEnabled();

		virtual bool IsPlaying() const override;

		/** Whether pasting the current data on the clipboard to the focused graph is permissible */
		bool CanPasteNodes();

		/** Duplicates the selected node(s) in the graph */
		void DuplicateNodes();

		/** Forces all UX pertaining to the root graph's details panel to be refreshed. */
		void RefreshDetails();
			
		/** Pastes node(s) from the clipboard to the graph */
		void PasteNodes(const FVector2D* InLocation = nullptr);
		void PasteNodes(const FVector2D* InLocation, const FText& InTransactionText);

		/** Returns Graph Connection Manager associated with this editor */
		FGraphConnectionManager& GetConnectionManager();
		const FGraphConnectionManager& GetConnectionManager() const;

		/** Forces all UX pertaining to the root graph's interface to be refreshed, returning the first selected member. */
		UMetasoundEditorGraphMember* RefreshGraphMemberMenu();

		/** Updates selected node classes to highest class found in the MetaSound Class Registry. */
		void UpdateSelectedNodeClasses();

		/** Whether or not MetaSound can be auditioned */
		bool IsAuditionable() const;

		/* Whether the displayed graph is marked as editable */
		bool IsGraphEditable() const;

		void ClearSelectionAndSelectNode(UEdGraphNode* Node);

		int32 GetNumNodesSelected() const;

		/** Creates analyzers */
		void CreateAnalyzers(UMetaSoundSource& MetaSoundSource);

		/** Destroys analyzers */
		void DestroyAnalyzers();

		/** Jumps to the given nodes on the graph (templated to support arrays of various MetaSound graph node types) */
		template<typename TNodeType = UMetasoundEditorGraphNode>
		void JumpToNodes(const TArray<TNodeType*>& InNodes)
		{
			if (!MetasoundGraphEditor.IsValid())
			{
				return;
			}

			MetasoundGraphEditor->ClearSelectionSet();
			const UMetasoundEditorGraph& Graph = GetMetaSoundGraphChecked();
			if (!InNodes.IsEmpty())
			{
				if (SGraphPanel* GraphPanel = MetasoundGraphEditor->GetGraphPanel())
				{
					FVector2D BottomLeft = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
					FVector2D TopRight = { TNumericLimits<float>::Min(), TNumericLimits<float>::Min() };
					for (TNodeType* Node : InNodes)
					{
						if (!Node || Node->GetGraph() != &Graph)
						{
							continue;
						}

						constexpr bool bSelected = true;
						MetasoundGraphEditor->SetNodeSelection(Node, bSelected);
						BottomLeft.X = FMath::Min(BottomLeft.X, Node->NodePosX);
						BottomLeft.Y = FMath::Min(BottomLeft.Y, Node->NodePosY);
						TopRight.X = FMath::Max(TopRight.X, Node->NodePosX + Node->EstimateNodeWidth());
						TopRight.Y = FMath::Max(TopRight.Y, Node->NodePosY);
					}

					GraphPanel->JumpToRect(BottomLeft, TopRight);
				}
			}
		}

		void SetDelayedRename();

		void SetDroppedObjectsInfo(FDroppedObjectsInfo InInfo);
		const TOptional<FDroppedObjectsInfo>& GetDroppedObjectsInfo() const;

		void ClearDroppedObjectsInfo();

		/** Reloads all document template tabs */
		void ReloadAllDocTemplateTabs();

	protected:
		// Callbacks for action tree
		bool CanRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const;
		bool CanAddNewElementToSection(int32 InSectionID) const;
		void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
		void CollectStaticSections(TArray<int32>& StaticSectionIDs);
		TSharedRef<SWidget> CreateAddButton(int32 InSectionID, FText AddNewText, FName MetaDataTag);
		FText GetFilterText() const;
		bool HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;
		FReply OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent);
		void OnMemberActionDoubleClicked(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions);
		FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);
		void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, ESelectInfo::Type InSelectionType);
		FReply OnAddButtonClickedOnSection(int32 InSectionID);
		TSharedRef<SWidget> OnGetMenuSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID);
		FText GetSectionTitle(ENodeSection InSection) const;
		FText OnGetSectionTitle(int32 InSectionID);
		TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
		TSharedPtr<SWidget> OnContextMenuOpening();

		/** Called when the selection changes in the GraphEditor */
		void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);

		FGraphAppearanceInfo GetGraphAppearance() const;

		UMetasoundEditorGraph& GetMetaSoundGraphChecked();
		UMetasoundEditorGraph* GetMetaSoundGraph();

		FText GetGraphStatusDescription() const;
		const FSlateIcon& GetPlayIcon() const;
		const FSlateIcon& GetStopIcon() const;

		/**
			* Called when a node's title is committed for a rename
			*
			* @param	NewText				New title text
			* @param	CommitInfo			How text was committed
			* @param	NodeBeingChanged	The node being changed
			*/
		void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

		/** Deletes from the Metasound Menu (i.e. input or output) if in focus, or the currently selected nodes if the graph editor is in focus. */
		void DeleteSelected();

		void DeleteInterfaceItem(TSharedPtr<FMetasoundGraphMemberSchemaAction> ActionToDelete);

		/** Delete the currently selected nodes */
		void DeleteSelectedNodes();

		/** Delete the currently selected interface items */
		void DeleteSelectedInterfaceItems();

		/** Cut the currently selected nodes */
		void CutSelectedNodes();

		/** Copy the currently selected nodes */
		void CopySelectedNodes() const;

		/** Whether or not the currently selected node(s) can be duplicated */
		bool CanDuplicateNodes() const;

		/** Whether the currently selected node(s) can be deleted */
		bool CanDeleteNodes() const;

		/** Whether the currently selected interface item(s) can be deleted */
		bool CanDeleteInterfaceItems() const;

		/** Whether at least one of the currently selected node(s) can be renamed. */
		bool CanRenameSelectedNodes() const;

		/** Rename selected node (currently applies to comments and member nodes). */
		void RenameSelectedNode();

		/** Whether at least one of the currently selected interface item(s) can be renamed. */
		bool CanRenameSelectedInterfaceItems() const;

		/** Rename selected interface item. */
		void RenameSelectedInterfaceItem();

		/** Whether the currently selected Member item(s) can be duplicated. */
		bool CanDuplicateSelectedMemberItems() const;

		/** Duplicate selected Member items. */
		void DuplicateSelectedMemberItems();

		/** Whether the currently selected Member item(s) can be copied. */
		bool CanCopySelectedMemberItems() const;

		/** Copy selected Member items. */
		void CopySelectedMemberItems();
		
		/** Whether the currently selected Member item(s) can be cut. */
		bool CanCutSelectedMemberItems() const;

		/** Cut selected Member items. */
		void CutSelectedMemberItems();

		/** Whether the currently selected Member item(s) can be pasted. */
		bool CanPasteSelectedMemberItems() const;

		/** Copies the selected nodes, shared connections, and required interfaces to the clipboard as configurator c++ */
		void CopySelectionAsConfiguratorCodeToClipboard() const;

		/** Paste selected Member items. */
		void PasteSelectedMemberItems();

		/** Whether there are nodes to jump to for the currently selected interface item. */
		bool CanJumpToNodesForSelectedInterfaceItem() const;

		/** Jumps to the nodes corresponding to the first valid currently selected interface item. */
		void JumpToNodesForSelectedInterfaceItem();
			
		/** Delete all unused members from the selected section*/
		void DeleteAllUnusedInSection();

		/** Whether the selection is not to a valid member*/
		bool CanDeleteUnusedMembers() const;

		/** Called to undo the last action */
		void UndoGraphAction();

		/** Called to redo the last undone action */
		void RedoGraphAction();

		void RefreshEditorContext(UObject& MetaSound);

		/** Show and focus the Find in MetaSound tab. */
		void ShowFindInMetaSound();

		/** Find selected node from Graph*/
		void FindSelectedNodeInGraph();

		/**Hide pins without connection*/
		void HideUnconnectedPins();
			
		/**Show pins without connection*/
		void ShowUnconnectedPins();

		/** Checks if pin can be promoted */
		bool CanPromoteToInput();
			
		/** Promotes pin to graph input */
		void PromoteToInput();

		/** Checks if pin can be promoted */
		bool CanPromoteToOutput();

		/** Promotes pin to graph output */
		void PromoteToOutput();

		/** Checks if pin can be promoted */
		bool CanPromoteToVariable();

		/** Promotes pin to graph variable */
		void PromoteToVariable();

		/** Checks if pin can be promoted */
		bool CanPromoteToDeferredVariable();

		/** Promotes pin to graph deferred variable */
		void PromoteToDeferredVariable();

		/** Checks if node's inputs can be promoted */
		bool CanPromoteAllToInputs();

		/** Promotes node's inputs to unique graph inputs */
		void PromoteAllToInputs();

		/** Checks if node's inputs can be promoted */
		bool CanPromoteAllToCommonInputs();

		/** Promotes node's inputs to shared graph inputs */
		void PromoteAllToCommonInputs();

	private:
		int32 PromotableSelectedNodes();

		// If target BP Id set, only rebuilds the widget associated with that BP.
		void RebuildUserWidgets(
			UWorld& World,
			UAudioWidgetSubsystem& WidgetSubsystem,
			TSubclassOf<UInterface> InterfaceClass,
			TFunction<bool(UUserWidget*)>Filter,
			TFunction<FMetaSoundTemplateWidgetTabInfo(UUserWidget*)>OnTabConstruct,
			FGuid TargetBlueprintId = FGuid());

		void RefreshExecVisibility(const FGuid& InPageID) const;

		/** Forces refresh of pages view. */
		void RefreshPagesView();

		/** Forces refresh of interfaces view. */
		void RefreshInterfaceView();

		void RemoveInvalidSelection();

		void UpdateEnabledTabs();

		void SetPreviewID(uint32 InPreviewID);

		void ExportNodesToText(FString& OutText) const;

		void SyncAuditionState(bool bSetAuditionFocus = true);

		/** FNotifyHook interface */
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

		/** Creates all internal widgets for the tabs to point at */
		void CreateInternalWidgets(UObject& MetaSound);

		void RebuildDocTemplateUserWidgets(const FGuid& TargetBlueprintId = FGuid());

		/** For teardown and regeneration of user document template widget on world change
			Based on UEditorUtilityWidgetBlueprint::ChangeTabWorld */
		void ChangeDocTemplateWidgetWorld(UWorld* World, EMapChangeType MapChangeType);

		/** Reload document template widget tab on associated widget blueprint recompile */
		void OnDocTemplateWidgetBlueprintCompiled(UBlueprint* Blueprint);

		/** Builds the toolbar widget for the Metasound editor */
		void ExtendToolbarInternal();

		/** Binds new graph commands to delegates */
		void BindGraphCommands();

		FSlateIcon GetImportStatusImage() const;

		FSlateIcon GetExportStatusImage() const;

		// TODO: Move import/export out of editor and into import/export asset actions
		void Import();
		void Export();

		/** Toolbar command methods */
		void ExecuteNode();

		/** Whether we can play the current selection of nodes */
		bool CanExecuteNode() const;

		/** Either play the Metasound or stop currently playing sound */
		void TogglePlayback();

		/** Whether or not to show the toggle for showing/hiding a graph (i.e. is or is not a MetaSound with a template set) */
		bool CanToggleShowHideGraph() const;

		/** Toggles showing the graph for the given configured MetaSound */
		void ToggleShowHideGraph();

		/** Executes specified node (If supported) */
		void ExecuteNode(UEdGraphNode* Node);

		/** Sync the content browser to the current selection of nodes */
		void SyncInBrowser();

		/** Shows dialog confirming converting from configured document to make parent MetaSound openly modifiable. */
		void ShowConvertFromConfigDialog();

		/* Whether or not page details should be visible/enabled. */
		bool ShowPageGraphDetails() const;

		/** Creates audition menu options */
		TSharedRef<SWidget> CreateAuditionMenuOptions();

		/** Creates template menu options */
		TSharedRef<SWidget> CreateTemplateMenuOptions();

		/** Creates page menu options */
		void CreateAuditionPageSubMenuOptions(FMenuBuilder& MenuBuilder);

		/** Creates template menu options */
		void CreateTemplateSubMenuOptions(FMenuBuilder& MenuBuilder);

		/** Show the Metasound object's Source settings in the Details panel */
		void EditSourceSettings();

		/** Show the Metasound object's settings in the Details panel */
		void EditMetasoundSettings();

		TSharedPtr<FTabManager::FStack> GraphCanvasTabStack;

		/** Add an input to the currently selected node */
		void AddInput();

		/** Whether we can add an input to the currently selected node */
		bool CanAddInput() const;

		/* Create comment node on graph */
		void OnCreateComment();

		/** Create new graph editor widget */
		void CreateGraphEditorWidget(UObject& MetaSound);

		void EditObjectSettings();

		void NotifyAssetLoadingComplete();
		void NotifyAssetLoadingInProgress(int32 NumProcessing);
		void NotifyDocumentVersioned();
		void NotifyNodePasteFailure_MultipleVariableSetters();
		void NotifyNodePasteFailure_ReferenceLoop();
		void NotifyNodePasteFailure_MultipleOutputs();

		TUniquePtr<FGraphConnectionManager> RebuildConnectionManager(UAudioComponent* PreviewComponent = nullptr) const;

		/** Updates the page info widget. */
		void UpdatePageInfo(bool bIsPlaying);

		/** Updates the render info widget. */
		void UpdateRenderInfo(bool bIsPlaying, float InDeltaTime = 0.0f);

		/** List of open tool panels; used to ensure only one exists at any one time */
		TMap<FName, TWeakPtr<SDockableTab>> SpawnedToolPanels;

		/** New Graph Editor */
		TSharedPtr<SGraphEditor> MetasoundGraphEditor;

		/** Details tab */
		TSharedPtr<IDetailsView> MetasoundDetails;

		/** Pages tab */
		TSharedPtr<IDetailsView> PagesDetails;
		TStrongObjectPtr<UMetasoundPagesView> PagesView;

		/** Interfaces tab */
		TSharedPtr<IDetailsView> InterfacesDetails;
		TStrongObjectPtr<UMetasoundInterfacesView> InterfacesView;

		/** Metasound graph members menu */
		TSharedPtr<SGraphActionMenu> GraphMembersMenu;

		/** Displayed in the analyzer tab for visualizing preview output. */
		TSharedPtr<AudioWidgets::FAudioAnalyzerRack> AnalyzerRack;

		/** Find in MetaSound widget. */
		TSharedPtr<SFindInMetasound> FindWidget;

		/** Palette of Node types */
		TSharedPtr<SMetasoundPalette> Palette;

		/** Widget showing page info regarding page info that overlays the graph tab content */
		TSharedPtr<SPageStats> PageStatsWidget;

		/** Widget showing render performance information that overlays the graph */
		TSharedPtr<SRenderStats> RenderStatsWidget;

		struct FUserWidgetData
		{
			TSubclassOf<UInterface> Class;
			FDelegateHandle RecompileHandle;
			TStrongObjectPtr<UUserWidget> Widget;
			FMetaSoundTemplateWidgetTabInfo TabInfo;

			void SetAuditionState(UAudioComponent* AudioComponent, bool bIsAuditioning) const;
		};
		using FBlueprintId = FGuid;
		using FBPIdWidgetDataPair = TPair<FBlueprintId, FUserWidgetData>;

		/** User defined widget data displayed when editing a document template with user widgets enabled.*/
		TMap<FBlueprintId, FUserWidgetData> DocTemplateWidgetData;

		FWorkflowAllowedTabSet DocTemplateTabSet;

		struct FTemplateWidgetSummoner : public FWorkflowTabFactory
		{
		public:
			FTemplateWidgetSummoner(FName FactoryName, FBlueprintId InBlueprintId, TSharedPtr<class FAssetEditorToolkit> InHostingApp);

			virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
			virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;
			virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

		private:
			TSharedPtr<FEditor> GetParentEditor() const;
			const FUserWidgetData* GetWidgetData() const;
			FBlueprintId BlueprintId;
		};

		TUniquePtr<FGraphConnectionManager> GraphConnectionManager;

		/** Command list for this editor */
		TSharedPtr<FUICommandList> GraphEditorCommands;

		/** Pointer to builder being actively used to mutate MetaSound asset */
		TStrongObjectPtr<UMetaSoundBuilderBase> Builder;

		/** Whether or not metasound being edited is valid */
		bool bPassedValidation = true;

		/** Text content used when either duplicating or pasting from clipboard (avoids deserializing twice) */
		FString NodeTextToPaste;

		/** Boolean state for when selection change handle should not respond due to selection state
			* being manually applied in code */
		bool bManuallyClearingGraphSelection = false;

		/** Highest message severity set on last validation pass of graph. */
		int32 HighestMessageSeverity = EMessageSeverity::Info;

		/** If set, used to inform user of validation results on hover of play icon. */
		FText GraphStatusDescriptionOverride;

		TSharedPtr<SNotificationItem> LoadingNotificationPtr;

		bool bMemberRenameRequested = false;

		bool bRefreshGraph = false;

		class FDocumentListener : public Frontend::IDocumentBuilderTransactionListener
		{
			TWeakPtr<FEditor> Parent;

		public:
			FDocumentListener() = default;
			FDocumentListener(TSharedRef<FEditor> InParent)
				: Parent(InParent)
			{
			}

			virtual ~FDocumentListener() = default;

		private:
			virtual void OnBuilderReloaded(Frontend::FDocumentModifyDelegates& OutDelegates) override;
			void OnDocumentTemplateChanged(const Frontend::FDocumentTemplateChangedArgs& Args);
			void OnPageSet(const Frontend::FDocumentMutatePageArgs& Args);
			void OnInputDefaultChanged(const Frontend::FDocumentArrayPagedInputArgs& Args);
			void OnMemberMetadataSet(FGuid MemberID);
		};

		TSharedPtr<FDocumentListener> DocListener;

		// Objects that are currently being dropped onto the editor
		TOptional<FDroppedObjectsInfo> DroppedObjectsInfo;
	};
} // namespace Metasound::Editor
