// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorBuilderListener.h"

#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorBuilderListener)

void UMetaSoundEditorBuilderListener::Init(const TWeakObjectPtr<UMetaSoundBuilderBase> InBuilder)
{
	if (InBuilder.IsValid())
	{
		Builder = InBuilder;
		BuilderListener = MakeShared<FEditorBuilderListener>(this);
		Builder->AddTransactionListener(BuilderListener->AsShared());
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentDisplayNameChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FText& DisplayName = GraphClass.Metadata.GetDisplayName();

		OnDocumentDisplayNameChangedDelegate.Broadcast(DisplayName);
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentDescriptionChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FText& Description = GraphClass.Metadata.GetDescription();

		OnDocumentDescriptionChangedDelegate.Broadcast(Description);
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentAuthorChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FString& Author = GraphClass.Metadata.GetAuthor();

		OnDocumentAuthorChangedDelegate.Broadcast(Author);
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentKeywordsChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FText>& Keywords = GraphClass.Metadata.GetKeywords();

		OnDocumentKeywordsChangedDelegate.Broadcast(Keywords);
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentCategoryHierarchyChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FText>& CategoryHierarchy = GraphClass.Metadata.GetCategoryHierarchy();

		OnDocumentCategoryHierarchyChangedDelegate.Broadcast(CategoryHierarchy);
	}
}

void UMetaSoundEditorBuilderListener::OnDocumentIsDeprecatedChanged()
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const bool bIsDeprecated = EnumHasAnyFlags(GraphClass.Metadata.GetAccessFlags(), EMetasoundFrontendClassAccessFlags::Deprecated);

		OnDocumentIsDeprecatedChangedDelegate.Broadcast(bIsDeprecated);
	}
}

void UMetaSoundEditorBuilderListener::OnPageAdded(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs)
{
	if (Builder.IsValid())
	{
		const FName PageName = UMetaSoundSettings::GetPageName(PageArgs.PageID);
		OnPageAddedDelegate.Broadcast(PageName);
	}
}

void UMetaSoundEditorBuilderListener::OnRemovingPage(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs)
{
	if (Builder.IsValid())
	{
		const FName PageName = UMetaSoundSettings::GetPageName(PageArgs.PageID);
		OnRemovingPageDelegate.Broadcast(PageName);
	}
}

void UMetaSoundEditorBuilderListener::OnBuildPageChanged(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs)
{
	if (Builder.IsValid())
	{
		const FName PageName = UMetaSoundSettings::GetPageName(PageArgs.PageID);
		OnBuildPageChangedDelegate.Broadcast(PageName);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputAdded(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputAddedDelegate.Broadcast(GraphInput.Name, GraphInput.TypeName);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputDefaultChanged(int32 Index)
{
}

void UMetaSoundEditorBuilderListener::OnGraphInputDefaultChanged(const Metasound::Frontend::FDocumentArrayPagedInputArgs& Args)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Args.Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Args.Index];
			FMetasoundFrontendLiteral DefaultLiteral;
			// If default literal was removed, then it will be null
			if (const FMetasoundFrontendLiteral* CurrentLiteral = GraphInput.FindConstDefault(Args.PageID))
			{
				DefaultLiteral = *CurrentLiteral;
			}
			const FName PageName = UMetaSoundSettings::GetPageName(Args.PageID);
			OnGraphInputDefaultChangedWithArgsDelegate.Broadcast(FGraphInputDefaultChangedArgs(
					GraphInput.Name, PageName, DefaultLiteral, Args.ChangeType));
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputInheritsDefaultChanged(int32 Index)
{
	using namespace Metasound::Frontend;
	
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			bool bInheritsDefault = false;
			if (const FPresetVertexMetadata* VertexMetadata = ConstBuilder.FindConstVertexMetadata<FPresetVertexMetadata>(GraphInput.NodeID))
			{
				bInheritsDefault = !VertexMetadata->bOverrideInheritedDefault;
			}
			OnGraphInputInheritsDefaultChangedDelegate.Broadcast(GraphInput.Name, bInheritsDefault);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnRemovingGraphInput(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnRemovingGraphInputDelegate.Broadcast(GraphInput.Name, GraphInput.TypeName);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputAdded(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputAddedDelegate.Broadcast(GraphOutput.Name, GraphOutput.TypeName);
	}
}

void UMetaSoundEditorBuilderListener::OnRemovingGraphOutput(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnRemovingGraphOutputDelegate.Broadcast(GraphOutput.Name, GraphOutput.TypeName);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputDataTypeChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputDataTypeChangedDelegate.Broadcast(GraphInput.Name, GraphInput.TypeName);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputDisplayNameChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputDisplayNameChangedDelegate.Broadcast(GraphInput.Name, GraphInput.Metadata.GetDisplayName());
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputDescriptionChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputDescriptionChangedDelegate.Broadcast(GraphInput.Name, GraphInput.Metadata.GetDescription());
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputSortOrderIndexChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputSortOrderIndexChangedDelegate.Broadcast(GraphInput.Name, GraphInput.Metadata.SortOrderIndex);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputIsConstructorPinChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputIsConstructorPinChangedDelegate.Broadcast(GraphInput.Name, GraphInput.AccessType == EMetasoundFrontendVertexAccessType::Value);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputIsAdvancedDisplayChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		if (ensure(Inputs.IsValidIndex(Index)))
		{
			const FMetasoundFrontendClassInput& GraphInput = Inputs[Index];
			OnGraphInputIsAdvancedDisplayChangedDelegate.Broadcast(GraphInput.Name, GraphInput.Metadata.bIsAdvancedDisplay);
		}
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputDataTypeChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputDataTypeChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.TypeName);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputDisplayNameChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputDisplayNameChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.Metadata.GetDisplayName());
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputDescriptionChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputDescriptionChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.Metadata.GetDescription());
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputSortOrderIndexChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputSortOrderIndexChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.Metadata.SortOrderIndex);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputIsConstructorPinChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputIsConstructorPinChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.AccessType == EMetasoundFrontendVertexAccessType::Value);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputIsAdvancedDisplayChanged(int32 Index)
{
	if (Builder.IsValid())
	{
		const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendGraphClass& GraphClass = ConstBuilder.GetConstDocumentChecked().RootGraph;
		const FMetasoundFrontendClassOutput& GraphOutput = GraphClass.GetDefaultInterface().Outputs[Index];

		OnGraphOutputIsAdvancedDisplayChangedDelegate.Broadcast(GraphOutput.Name, GraphOutput.Metadata.bIsAdvancedDisplay);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphInputNameChanged(FName OldName, FName NewName)
{
	if (Builder.IsValid())
	{
		OnGraphInputNameChangedDelegate.Broadcast(OldName, NewName);
	}
}

void UMetaSoundEditorBuilderListener::OnGraphOutputNameChanged(FName OldName, FName NewName)
{
	if (Builder.IsValid())
	{
		OnGraphOutputNameChangedDelegate.Broadcast(OldName, NewName);
	}
}

void UMetaSoundEditorBuilderListener::RemoveAllDelegates()
{
	// Remove multicast BP delegates
	OnGraphInputAddedDelegate.RemoveAll(this);
	OnRemovingGraphInputDelegate.RemoveAll(this);
	OnGraphInputNameChangedDelegate.RemoveAll(this);
	OnGraphInputDataTypeChangedDelegate.RemoveAll(this);
	OnGraphInputDescriptionChangedDelegate.RemoveAll(this);
	OnGraphInputSortOrderIndexChangedDelegate.RemoveAll(this);
	OnGraphInputIsAdvancedDisplayChangedDelegate.RemoveAll(this);
	OnGraphInputIsConstructorPinChangedDelegate.RemoveAll(this);
	OnGraphInputDefaultChangedWithArgsDelegate.RemoveAll(this);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnGraphInputDefaultChangedDelegate.RemoveAll(this);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OnPageAddedDelegate.RemoveAll(this);
	OnRemovingPageDelegate.RemoveAll(this);
	OnBuildPageChangedDelegate.RemoveAll(this);

	OnGraphOutputAddedDelegate.RemoveAll(this);
	OnRemovingGraphOutputDelegate.RemoveAll(this);
	OnGraphOutputNameChangedDelegate.RemoveAll(this);
	OnGraphOutputDataTypeChangedDelegate.RemoveAll(this);
	OnGraphOutputDescriptionChangedDelegate.RemoveAll(this);
	OnGraphOutputSortOrderIndexChangedDelegate.RemoveAll(this);
	OnGraphOutputIsConstructorPinChangedDelegate.RemoveAll(this);
	OnGraphOutputIsAdvancedDisplayChangedDelegate.RemoveAll(this);

	// Remove document delegates
	if (Builder.IsValid())
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		Metasound::Frontend::FDocumentModifyDelegates& BuilderDelegates = DocumentBuilder.GetDocumentDelegates();
		Metasound::Frontend::FInterfaceModifyDelegates& InterfaceDelegates = BuilderDelegates.InterfaceDelegates;

		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentDisplayNameChangedHandle);
		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentDescriptionChangedHandle);
		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentAuthorChangedHandle);
		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentKeywordsChangedHandle);
		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentCategoryHierarchyChangedHandle);
		BuilderDelegates.OnDocumentMetadataChanged.Remove(OnDocumentIsDeprecatedChangedHandle);
		
		BuilderDelegates.PageDelegates.OnPageAdded.Remove(OnPageAddedHandle);
		BuilderDelegates.PageDelegates.OnRemovingPage.Remove(OnRemovingPageHandle);
		BuilderDelegates.PageDelegates.OnPageSet.Remove(OnBuildPageChangedHandle);

		InterfaceDelegates.OnInputAdded.Remove(OnInputAddedHandle);
		InterfaceDelegates.OnRemovingInput.Remove(OnRemovingInputHandle);
		InterfaceDelegates.OnInputNameChanged.Remove(OnInputNameChangedHandle);
		InterfaceDelegates.OnInputDisplayNameChanged.Remove(OnInputDisplayNameChangedHandle);
		InterfaceDelegates.OnInputDataTypeChanged.Remove(OnInputDataTypeChangedHandle);
		InterfaceDelegates.OnInputDescriptionChanged.Remove(OnInputDescriptionChangedHandle);
		InterfaceDelegates.OnInputSortOrderIndexChanged.Remove(OnInputSortOrderIndexChangedHandle);
		InterfaceDelegates.OnInputIsConstructorPinChanged.Remove(OnInputIsConstructorPinChangedHandle);
		InterfaceDelegates.OnInputIsAdvancedDisplayChanged.Remove(OnInputIsAdvancedDisplayChangedHandle);
		InterfaceDelegates.OnInputDefaultChanged.Remove(OnInputDefaultChangedHandle);
		InterfaceDelegates.OnInputInheritsDefaultChanged.Remove(OnInputInheritsDefaultChangedHandle);

		InterfaceDelegates.OnOutputAdded.Remove(OnOutputAddedHandle);
		InterfaceDelegates.OnRemovingOutput.Remove(OnRemovingOutputHandle);
		InterfaceDelegates.OnOutputNameChanged.Remove(OnOutputNameChangedHandle);
		InterfaceDelegates.OnOutputDisplayNameChanged.Remove(OnOutputDisplayNameChangedHandle);
		InterfaceDelegates.OnOutputDataTypeChanged.Remove(OnOutputDataTypeChangedHandle);
		InterfaceDelegates.OnOutputDescriptionChanged.Remove(OnOutputDescriptionChangedHandle);
		InterfaceDelegates.OnOutputSortOrderIndexChanged.Remove(OnOutputSortOrderIndexChangedHandle);
		InterfaceDelegates.OnOutputIsConstructorPinChanged.Remove(OnOutputIsConstructorPinChangedHandle);
		InterfaceDelegates.OnOutputIsAdvancedDisplayChanged.Remove(OnOutputIsAdvancedDisplayChangedHandle);
	}
}

void UMetaSoundEditorBuilderListener::FEditorBuilderListener::OnBuilderReloaded(Metasound::Frontend::FDocumentModifyDelegates& OutDelegates)
{
	if (Parent)
	{
		Metasound::Frontend::FInterfaceModifyDelegates& InterfaceDelegates = OutDelegates.InterfaceDelegates;

		Parent->OnDocumentDisplayNameChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentDisplayNameChanged);
		Parent->OnDocumentDescriptionChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentDescriptionChanged);
		Parent->OnDocumentAuthorChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentAuthorChanged);
		Parent->OnDocumentKeywordsChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentKeywordsChanged);
		Parent->OnDocumentCategoryHierarchyChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentCategoryHierarchyChanged);
		Parent->OnDocumentIsDeprecatedChangedHandle = OutDelegates.OnDocumentMetadataChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnDocumentIsDeprecatedChanged);

		Parent->OnPageAddedHandle = OutDelegates.PageDelegates.OnPageAdded.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnPageAdded);
		Parent->OnRemovingPageHandle = OutDelegates.PageDelegates.OnRemovingPage.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnRemovingPage);
		Parent->OnBuildPageChangedHandle = OutDelegates.PageDelegates.OnPageSet.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnBuildPageChanged);

		Parent->OnInputAddedHandle = InterfaceDelegates.OnInputAdded.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputAdded);
		Parent->OnRemovingInputHandle = InterfaceDelegates.OnRemovingInput.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnRemovingGraphInput);
		Parent->OnInputNameChangedHandle = InterfaceDelegates.OnInputNameChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputNameChanged);
		Parent->OnInputDisplayNameChangedHandle = InterfaceDelegates.OnInputDisplayNameChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputDisplayNameChanged);
		Parent->OnInputDataTypeChangedHandle = InterfaceDelegates.OnInputDataTypeChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputDataTypeChanged);
		Parent->OnInputDescriptionChangedHandle = InterfaceDelegates.OnInputDescriptionChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputDescriptionChanged);
		Parent->OnInputSortOrderIndexChangedHandle = InterfaceDelegates.OnInputSortOrderIndexChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputSortOrderIndexChanged);
		Parent->OnInputIsConstructorPinChangedHandle = InterfaceDelegates.OnInputIsConstructorPinChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputIsConstructorPinChanged);
		Parent->OnInputIsAdvancedDisplayChangedHandle = InterfaceDelegates.OnInputIsAdvancedDisplayChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputIsAdvancedDisplayChanged);
		Parent->OnInputDefaultChangedHandle = InterfaceDelegates.OnInputDefaultChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputDefaultChanged);
		Parent->OnInputInheritsDefaultChangedHandle = InterfaceDelegates.OnInputInheritsDefaultChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphInputInheritsDefaultChanged);

		Parent->OnOutputAddedHandle = InterfaceDelegates.OnOutputAdded.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputAdded);
		Parent->OnRemovingOutputHandle = InterfaceDelegates.OnRemovingOutput.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnRemovingGraphOutput);
		Parent->OnOutputNameChangedHandle = InterfaceDelegates.OnOutputNameChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputNameChanged);
		Parent->OnOutputDisplayNameChangedHandle = InterfaceDelegates.OnOutputDisplayNameChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputDisplayNameChanged);
		Parent->OnOutputDataTypeChangedHandle = InterfaceDelegates.OnOutputDataTypeChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputDataTypeChanged);
		Parent->OnOutputDescriptionChangedHandle = InterfaceDelegates.OnOutputDescriptionChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputDescriptionChanged);
		Parent->OnOutputSortOrderIndexChangedHandle = InterfaceDelegates.OnOutputSortOrderIndexChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputSortOrderIndexChanged);
		Parent->OnOutputIsConstructorPinChangedHandle = InterfaceDelegates.OnOutputIsConstructorPinChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputIsConstructorPinChanged);
		Parent->OnOutputIsAdvancedDisplayChangedHandle = InterfaceDelegates.OnOutputIsAdvancedDisplayChanged.AddUObject(Parent, &UMetaSoundEditorBuilderListener::OnGraphOutputIsAdvancedDisplayChanged);
	}
}
