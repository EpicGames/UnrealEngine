// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorCommands.h"

#define LOCTEXT_NAMESPACE "MetasoundEditorCommands"

namespace Metasound
{
	namespace Editor
	{
		void FEditorCommands::RegisterCommands()
		{
			UI_COMMAND(Play, "Play", "Plays (or restarts) the MetaSound", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(Stop, "Stop", "Stops MetaSound (If currently playing)", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(TogglePlayback, "Toggle Playback", "Plays or stops the currently playing MetaSound", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));

			UI_COMMAND(ToggleShowGraph, "Show Graph", "Toggle showing/hiding the graph for the given configured MetaSound", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control | EModifierKey::Shift));

			UI_COMMAND(Import, "Import", "Imports MetaSound from Json", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(Export, "Export", "Exports MetaSound to Json", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(BrowserSync, "Browse", "Selects the MetaSound in the content browser. If referencing MetaSound nodes are selected, selects referenced assets instead.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(AddInput, "Add Input", "Adds an input to the node", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(EditMetasoundSettings, "MetaSound", "Edit MetaSound settings", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(EditSourceSettings, "Source", "Edit source specific settings", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(UpdateNodeClass, "Update Node Class", "Update selected node(s) class(es) that have an available update.", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(Delete, "Delete Selected", "Delete selected items.", EUserInterfaceActionType::None, FInputChord(EKeys::Delete));
			UI_COMMAND(FindInMetaSound, "Find in MetaSound", "Find a node or pin within this MetaSound.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
		
			UI_COMMAND(PromoteToInput, "Promote To Graph Input", "Promotes pin to graph input.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(PromoteToOutput, "Promote To Graph Output", "Promotes pin to graph output.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(PromoteToVariable, "Promote To Graph Variable", "Promotes pin to graph variable.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(PromoteToDeferredVariable, "Promote To Deferred Graph Variable", "Promotes pin to deferred graph variable.", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(PromoteAllToInput, "Promote All To Graph Input", "Promotes unconnected node inputs to graph inputs.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(PromoteAllToCommonInputs, "Promote All To Common Graph Inputs", "Promotes unconnected input pins from selected nodes to graph inputs, sharing inputs if possible.", EUserInterfaceActionType::Button, FInputChord());

			UI_COMMAND(CopySelectionAsConfiguratorCodeToClipboard,
				"Copy Selected As Configurator Code to Clipboard",
				"Reflect configurator C++ code to the clipboard for the given graph selection",
				EUserInterfaceActionType::Button,
				FInputChord(EKeys::C, EModifierKey::Control | EModifierKey::Shift));
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
