// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorDocumentClipboardUtils.h"

#include "Algo/Transform.h"
#include "EdGraphUtilities.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Logging/TokenizedMessage.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontendGraphLayer.h"
#include "MetasoundFrontendSearchEngine.h"
#include "Misc/StringOutputDevice.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "ScopedTransaction.h"
#include "UnrealExporter.h"

namespace Metasound::Editor
{
	namespace DocumentClipboardUtilsPrivate
	{
		class FMemberClipboardObjectTextFactory : public FCustomizableTextObjectFactory
		{
		public:
			FMemberClipboardObjectTextFactory()
				: FCustomizableTextObjectFactory(GWarn)
			{
			}

			UMetasoundEditorGraphMember* Member = nullptr;

		protected:
			virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
			{
				return InObjectClass->IsChildOf(UMetasoundEditorGraphMember::StaticClass());
			}

			virtual void ProcessConstructedObject(UObject* CreatedObject) override
			{
				Member = Cast<UMetasoundEditorGraphMember>(CreatedObject);
			}
		};
	} // DocumentClipboardUtilsPrivate

	void FDocumentClipboardUtils::ProcessPastedInputNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes)
	{
		using namespace Engine;
		using namespace Frontend;

		TMap<FName, TObjectPtr<UMetasoundEditorGraphInput>> MappedGeneratedInputNames;
		UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&OutAsset.GetGraphChecked());
		for (int32 Index = OutPastedNodes.Num() - 1; Index >= 0; --Index)
		{
			UMetasoundEditorGraphInputNode* InputNode = Cast<UMetasoundEditorGraphInputNode>(OutPastedNodes[Index]);
			if (!InputNode)
			{
				continue;
			}

			InputNode->CreateNewGuid();
			TObjectPtr<UMetasoundEditorGraphInput>& Input = InputNode->Input;
			if (!Input || !Graph.ContainsInput(*Input))
			{
				Input = nullptr;

				bool bNameMatchFound = false;
				const FMetasoundEditorGraphVertexNodeBreadcrumb& Breadcrumb = InputNode->GetBreadcrumb();
				Graph.IterateInputs([&Input, &InputNode, &Graph, &Breadcrumb, &bNameMatchFound](UMetasoundEditorGraphInput& TestInput)
				{
					FConstNodeHandle InputHandle = TestInput.GetConstNodeHandle();
					FConstOutputHandle TestOutput = InputHandle->GetConstOutputs().Last();
					const bool bTypeMatches = TestOutput->GetDataType() == Breadcrumb.DataType;
					const bool bAccessMatches = TestOutput->GetVertexAccessType() == Breadcrumb.AccessType;
					const bool bNameMatches = InputHandle->GetNodeName() == Breadcrumb.MemberName;
					bNameMatchFound |= bNameMatches;
					if (bTypeMatches && bAccessMatches && bNameMatches)
					{
						Input = &TestInput;
					}
				});

				if (!Input)
				{
					FDataTypeRegistryInfo Info;
					if (!Input && IDataTypeRegistry::Get().GetDataTypeInfo(Breadcrumb.DataType, Info))
					{
						const FName InputName = Breadcrumb.MemberName;
						if (TObjectPtr<UMetasoundEditorGraphInput>* InputNodeHandle = MappedGeneratedInputNames.Find(InputName))
						{
							Input = *InputNodeHandle;
						}
						else
						{
							FCreateNodeVertexParams VertexParams;
							VertexParams.DataType = Breadcrumb.DataType;
							VertexParams.AccessType = Breadcrumb.AccessType;

							TArray<FMetasoundFrontendClassInputDefault> InputDefaults;
							Algo::Transform(Breadcrumb.DefaultLiterals, InputDefaults, [](const TPair<FGuid, FMetasoundFrontendLiteral>& Pair)
							{
								return FMetasoundFrontendClassInputDefault(Pair.Key, Pair.Value);
							});

							const FMetasoundFrontendNode* NewNode = nullptr;
							{
								FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(*OutAsset.GetOwningAsset(), VertexParams, InputDefaults, &Breadcrumb.MemberName);
								ClassInput.Metadata = Breadcrumb.VertexMetadata;
								NewNode = Builder.AddGraphInput(MoveTemp(ClassInput));
							}

							if (NewNode)
							{
								Input = Graph.FindOrAddInput(NewNode->GetID());
								if (Breadcrumb.MemberMetadataPath.IsSet())
								{
									UObject* MemberMetadata = Breadcrumb.MemberMetadataPath->TryLoad();
									if (UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Cast<UMetasoundEditorGraphMemberDefaultLiteral>(MemberMetadata))
									{
										Builder.ClearMemberMetadata(NewNode->GetID());
										UMetaSoundEditorSubsystem& MetaSoundEditorSubsystem = UMetaSoundEditorSubsystem::GetChecked();
										TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass = MetaSoundEditorSubsystem.GetLiteralClassForType(Breadcrumb.DataType);
										MetaSoundEditorSubsystem.BindMemberMetadata(Builder, *Input, LiteralClass, DefaultLiteral);
									}
								}
								MappedGeneratedInputNames.Add(Breadcrumb.MemberName, Input);
							}
						}
					}
				}
			}

			if (Input)
			{
				const FMetasoundFrontendNode* InputTemplateNode = FInputNodeTemplate::CreateNode(Builder, Input->GetMemberName());
				if (ensure(InputTemplateNode))
				{
					FGuid TemplateNodeID = InputTemplateNode->GetID();
					InputNode->NodeID = TemplateNodeID;

					// Remove default node location from input node. 
					// Correct node location from the ed graph node will be set subsequently in ProcessPastedNodePositions
					TArray<FGuid> NodeLocationGuids;
					InputTemplateNode->Style.Display.Locations.GetKeys(NodeLocationGuids);
					if (!NodeLocationGuids.IsEmpty())
					{
						Builder.RemoveNodeLocation(TemplateNodeID);
					}
				}
			}
			else
			{
				constexpr bool bAllowShrinking = false;
				Graph.RemoveNode(InputNode);
				OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
			}
		}
	}

	void FDocumentClipboardUtils::ProcessPastedOutputNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications)
	{
		using namespace Frontend;
		using namespace Engine;
		UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&OutAsset.GetGraphChecked());
		for (int32 Index = OutPastedNodes.Num() - 1; Index >= 0; --Index)
		{
			UMetasoundEditorGraphOutputNode* OutputNode = Cast<UMetasoundEditorGraphOutputNode>(OutPastedNodes[Index]);
			if (!OutputNode)
			{
				continue;
			}

			OutputNode->CreateNewGuid();

			if (OutputNode->Output && Graph.ContainsOutput(*OutputNode->Output))
			{
				auto IsOtherMatchingNode = [&OutputNode](const TObjectPtr<UEdGraphNode>& EdNode)
				{
					if (OutputNode != EdNode.Get())
					{
						if (UMetasoundEditorGraphOutputNode* OtherOutputNode = Cast<UMetasoundEditorGraphOutputNode>(EdNode))
						{
							return OutputNode->GetNodeID() == OtherOutputNode->GetNodeID();
						}
					}
					return false;
				};

				// Can only have one output reference node
				if (Graph.Nodes.ContainsByPredicate(IsOtherMatchingNode))
				{
					OutNotifications.bPastedNodesAddMultipleOutputNodes = true;
					Graph.RemoveNode(OutputNode);
					OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
				}
			}
			else
			{
				// Add output if doesn't exist 
				const FMetasoundEditorGraphVertexNodeBreadcrumb& Breadcrumb = OutputNode->GetBreadcrumb();

				FDataTypeRegistryInfo Info;
				if (IDataTypeRegistry::Get().GetDataTypeInfo(Breadcrumb.DataType, Info))
				{
					FCreateNodeVertexParams VertexParams;
					VertexParams.DataType = Breadcrumb.DataType;
					VertexParams.AccessType = Breadcrumb.AccessType;

					FMetasoundFrontendClassOutput ClassOutput = FGraphBuilder::CreateUniqueClassOutput(*OutAsset.GetOwningAsset(), VertexParams, &Breadcrumb.MemberName);
					ClassOutput.Metadata = Breadcrumb.VertexMetadata;

					if (const FMetasoundFrontendNode* NewNode = Builder.AddGraphOutput(ClassOutput))
					{
						UMetasoundEditorGraphOutput* Output = Graph.FindOrAddOutput(NewNode->GetID());
						if (Output)
						{
							if (Breadcrumb.MemberMetadataPath.IsSet())
							{
								UObject* MemberMetadata = Breadcrumb.MemberMetadataPath->TryLoad();
								if (UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Cast<UMetasoundEditorGraphMemberDefaultLiteral>(MemberMetadata))
								{
									Builder.ClearMemberMetadata(ClassOutput.NodeID);
									UMetaSoundEditorSubsystem& MetaSoundEditorSubsystem = UMetaSoundEditorSubsystem::GetChecked();
									TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass = MetaSoundEditorSubsystem.GetLiteralClassForType(Breadcrumb.DataType);
									MetaSoundEditorSubsystem.BindMemberMetadata(Builder, *Output, LiteralClass, DefaultLiteral);
								}
							}

							// Remove default node location from output node. 
							// Correct node location from the ed graph node will be set subsequently in ProcessPastedNodePositions
							TArray<FGuid> NodeLocationGuids;
							NewNode->Style.Display.Locations.GetKeys(NodeLocationGuids);
							if (!NodeLocationGuids.IsEmpty())
							{
								Builder.RemoveNodeLocation(NewNode->GetID());
							}

							OutputNode->Output = Output;
						}
						else
						{
							Graph.RemoveNode(OutputNode);
							OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
						}
					}
				}
			}
		}
	}

	void FDocumentClipboardUtils::ProcessPastedVariableNodes(FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications)
	{
		using namespace Frontend;

		OutNotifications.bPastedNodesAddMultipleVariableSetters = false;

		UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&OutAsset.GetGraphChecked());
		FMetaSoundFrontendDocumentBuilder& DocBuilder = Graph.GetBuilderChecked().GetBuilder();
		TMap<FName, TObjectPtr<UMetasoundEditorGraphVariable>> MappedGeneratedVariableNames;
		Graph.IterateVariables([&MappedGeneratedVariableNames](UMetasoundEditorGraphVariable& Variable)
		{
			MappedGeneratedVariableNames.Add(Variable.GetMemberName(), &Variable);
		});

		for (int32 Index = OutPastedNodes.Num() - 1; Index >= 0; --Index)
		{
			UMetasoundEditorGraphVariableNode* VariableNode = Cast<UMetasoundEditorGraphVariableNode>(OutPastedNodes[Index]);
			if (!VariableNode)
			{
				continue;
			}

			VariableNode->CreateNewGuid();

			TObjectPtr<UMetasoundEditorGraphVariable>& Variable = VariableNode->Variable;
			const FMetasoundEditorGraphMemberNodeBreadcrumb PastedBreadcrumb = VariableNode->Breadcrumb;
			if (!Variable || !Graph.ContainsVariable(*Variable) || Variable->GetFrontendVariable() == nullptr)
			{
				const FName BaseName = PastedBreadcrumb.MemberName;
				TObjectPtr<UMetasoundEditorGraphVariable> CachedVariable = MappedGeneratedVariableNames.FindRef(BaseName);
				const FMetasoundFrontendVariable* FrontendVariable = nullptr;
				if (CachedVariable)
				{
					FrontendVariable = CachedVariable->GetFrontendVariable();
				}

				if (FrontendVariable && FrontendVariable->TypeName == PastedBreadcrumb.DataType)
				{
					Variable = CachedVariable;
				}
				else
				{
					const FMetasoundFrontendLiteral* Literal = PastedBreadcrumb.DefaultLiterals.Find(Frontend::DefaultPageID);
					const FName VariableName = DocBuilder.GenerateUniqueVariableName(BaseName.ToString());
					FrontendVariable = DocBuilder.AddGraphVariable(
						VariableName,
						PastedBreadcrumb.DataType,
						Literal,
						&PastedBreadcrumb.VertexMetadata.GetDisplayName(),
						&PastedBreadcrumb.VertexMetadata.GetDescription()
					);

					Variable = Graph.FindOrAddVariable(FrontendVariable->Name);
					check(Variable);

					VariableNode->CacheBreadcrumb(); // Name of referenced variable/node state has changed so make sure up-to-date in case breadcrumb is used later
					MappedGeneratedVariableNames.Add(BaseName, Variable);
				}
			}

			const FMetasoundFrontendVariable* FrontendVariable = Variable->GetFrontendVariable();
			if (ensure(FrontendVariable))
			{
				// Can only have one mutator/setter node. Check pasted breadcrumb as nodes may have
				// changed due to it being added above (i.e. node was pasted from another graph).
				const FNodeClassName NodeClassName = PastedBreadcrumb.ClassName.ToNodeClassName();
				const bool bMatchesMutatorNodeID = VariableNode->GetNodeID() == FrontendVariable->MutatorNodeID;
				bool bIsDuplicateMutatorNode = false;
				if (const FMetasoundFrontendNode* MutatorNode = DocBuilder.FindNode(FrontendVariable->MutatorNodeID))
				{
					const FMetasoundFrontendClass* MutatorClass = DocBuilder.FindDependency(MutatorNode->ClassID);
					check(MutatorClass);
					bIsDuplicateMutatorNode = MutatorClass->Metadata.GetClassName() == NodeClassName;
				}
				if (bMatchesMutatorNodeID || bIsDuplicateMutatorNode)
				{
					OutNotifications.bPastedNodesAddMultipleVariableSetters = true;
					OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
					Graph.RemoveNode(VariableNode);
				}
				else
				{
					// Add new variable node
					FMetasoundFrontendClass FrontendClass;
					bool bDidFindClassWithName = ISearchEngine::Get().FindClassWithHighestVersion(NodeClassName, FrontendClass);
					if (ensure(bDidFindClassWithName))
					{
						if (const FMetasoundFrontendNode* NewNode = DocBuilder.AddGraphVariableNode(FrontendVariable->Name, FrontendClass.Metadata.GetType()))
						{
							VariableNode->SetNodeID(NewNode->GetID());
							VariableNode->CacheBreadcrumb(); // Cached again here to ensure node class is up-to-date on the copied node.
						}
						else
						{
							OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
							Graph.RemoveNode(VariableNode);
						}
					}
				}
			}
			else
			{
				OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
				Graph.RemoveNode(VariableNode);
			}
		}
	}

	void FDocumentClipboardUtils::ProcessPastedExternalNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications)
	{
		using namespace Engine;
		using namespace Frontend;

		OutNotifications.bPastedNodesCreateLoop = false;

		UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&OutAsset.GetGraphChecked());
		for (int32 Index = OutPastedNodes.Num() - 1; Index >= 0; --Index)
		{
			UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(OutPastedNodes[Index]);
			if (!ExternalNode)
			{
				continue;
			}

			ExternalNode->CreateNewGuid();
			FMetasoundFrontendClassMetadata LookupMetadata;
			const FMetasoundEditorGraphNodeBreadcrumb& Breadcrumb = ExternalNode->GetBreadcrumb();
			LookupMetadata.SetClassName(Breadcrumb.ClassName);
			LookupMetadata.SetType(EMetasoundFrontendClassType::External);
			const FNodeRegistryKey PastedRegistryKey(LookupMetadata);
			UObject& MetaSound = *OutAsset.GetOwningAsset();

			if (const FMetasoundAssetBase* Asset = IMetaSoundAssetManager::GetChecked().FindAsset(PastedRegistryKey))
			{
				if (OutAsset.AddingReferenceCausesLoop(*Asset))
				{
					FMetasoundFrontendClass MetaSoundClass;
					FMetasoundFrontendRegistryContainer::Get()->FindFrontendClassFromRegistered(PastedRegistryKey, MetaSoundClass);
					FString FriendlyClassName = MetaSoundClass.Metadata.GetDisplayName().ToString();
					if (FriendlyClassName.IsEmpty())
					{
						FriendlyClassName = MetaSoundClass.Metadata.GetClassName().ToString();
					}
					UE_LOGF(LogMetaSound, Warning, "Failed to paste node with class '%ls'.  Class would introduce cyclic asset dependency.", *FriendlyClassName);
					OutNotifications.bPastedNodesCreateLoop = true;
					OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
					Graph.RemoveNode(ExternalNode);
				}
				else
				{
					FMetaSoundClassInfo ClassInfo;
					if (ISearchEngine::Get().FindRegisteredClass(Breadcrumb.ClassName, Breadcrumb.Version.Major, ClassInfo))
					{
						if (EnumHasAnyFlags(ClassInfo.AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
						{
							const FMetasoundFrontendNode* NewNode = Builder.AddNodeByClassName(Breadcrumb.ClassName, Breadcrumb.Version.Major);
							if (NewNode)
							{
								const FGuid& NewNodeID = NewNode->GetID();
								ExternalNode->NodeID = NewNodeID;
								Builder.SetNodeConfiguration(NewNodeID, Breadcrumb.NodeConfiguration);
							}
							else
							{
								OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
								Graph.RemoveNode(ExternalNode);
							}
						}
						else
						{
							OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
							Graph.RemoveNode(ExternalNode);
							UE_LOGF(LogMetaSound, Warning, "Failed to add new node by name of class '%ls': Class access flag '%ls' not set.",
								*Breadcrumb.ClassName.ToString(),
								*LexToString(EMetasoundFrontendClassAccessFlags::Referenceable));
						}
					}
					else
					{
						OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
						Graph.RemoveNode(ExternalNode);
						UE_LOGF(LogMetaSound, Error, "Failed to add new node by class name '%ls' and major version '%d': Class not found", *Breadcrumb.ClassName.ToString(), Breadcrumb.Version.Major);
					}
				}
			}
			else
			{
				if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Breadcrumb.ClassName))
				{
					const FNodeTemplateGenerateInterfaceParams TemplateParams = Breadcrumb.TemplateParams.IsSet() ? *Breadcrumb.TemplateParams : FNodeTemplateGenerateInterfaceParams();
					const FMetasoundFrontendNode* TemplateNode = Builder.AddNodeByTemplate(*Template, TemplateParams);
					if (TemplateNode)
					{
						ExternalNode->NodeID = TemplateNode->GetID();
					}
					else
					{
						OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
						Graph.RemoveNode(ExternalNode);
					}
				}
				else
				{
					FMetaSoundClassInfo ClassInfo;
					if (ISearchEngine::Get().FindRegisteredClass(Breadcrumb.ClassName, Breadcrumb.Version.Major, ClassInfo))
					{
						if (EnumHasAnyFlags(ClassInfo.AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
						{
							const FMetasoundFrontendNode* NewNode = Builder.AddNodeByClassName(Breadcrumb.ClassName, Breadcrumb.Version.Major);
							if (NewNode)
							{
								const FGuid& NewNodeID = NewNode->GetID();
								ExternalNode->NodeID = NewNodeID;
								Builder.SetNodeConfiguration(NewNodeID, Breadcrumb.NodeConfiguration);
							}
							else
							{
								OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
								Graph.RemoveNode(ExternalNode);
							}
						}
						else
						{
							OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
							Graph.RemoveNode(ExternalNode);
							UE_LOGF(LogMetaSound, Warning, "Failed to add new node by name of class '%ls': Class access flag '%ls' not set.",
								*Breadcrumb.ClassName.ToString(),
								*LexToString(EMetasoundFrontendClassAccessFlags::Referenceable));
						}
					}
					else
					{
						OutPastedNodes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
						Graph.RemoveNode(ExternalNode);
						UE_LOGF(LogMetaSound, Warning, "Cannot add pasted node with class '%ls': Node class not found", *Breadcrumb.ClassName.ToString());
					}
				}
			}
		}
	}

	void FDocumentClipboardUtils::ProcessPastedCommentNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, const TArrayView<UMetasoundEditorGraphCommentNode*> CommentNodes)
	{
		using namespace Engine;
		using namespace Frontend;

		UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&OutAsset.GetGraphChecked());

		for (UMetasoundEditorGraphCommentNode* CommentNode : CommentNodes)
		{
			// Regenerate id
			CommentNode->CreateNewGuid();
			CommentNode->SetCommentID(CommentNode->NodeGuid);

			// Update frontend node
			FMetaSoundFrontendGraphComment& NewComment = Builder.FindOrAddGraphComment(CommentNode->GetCommentID());
			UMetasoundEditorGraphCommentNode::ConvertToFrontendComment(*CommentNode, NewComment);
		}
	}

	void FDocumentClipboardUtils::ProcessPastedNodePositions(FMetaSoundFrontendDocumentBuilder& OutBuilder, const FVector2D& InLocation, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, const TArrayView<UMetasoundEditorGraphCommentNode*> CommentNodes)
	{
		using namespace Frontend;

		// Find average midpoint of nodes and offset subgraph accordingly
		FVector2D AvgNodePosition = FVector2D::ZeroVector;
		for (UEdGraphNode* Node : OutPastedNodes)
		{
			AvgNodePosition.X += Node->NodePosX;
			AvgNodePosition.Y += Node->NodePosY;
		}
		for (UEdGraphNode* Node : CommentNodes)
		{
			AvgNodePosition.X += Node->NodePosX;
			AvgNodePosition.Y += Node->NodePosY;
		}

		if (!OutPastedNodes.IsEmpty())
		{
			float InvNumNodes = 1.0f / (OutPastedNodes.Num() + CommentNodes.Num());
			AvgNodePosition.X *= InvNumNodes;
			AvgNodePosition.Y *= InvNumNodes;
		}

		// Set new node positions
		for (UEdGraphNode* GraphNode : OutPastedNodes)
		{
			GraphNode->NodePosX = (GraphNode->NodePosX - AvgNodePosition.X) + InLocation.X;
			GraphNode->NodePosY = (GraphNode->NodePosY - AvgNodePosition.Y) + InLocation.Y;

			GraphNode->SnapToGrid(SNodePanel::GetSnapGridSize());
			if (UMetasoundEditorGraphNode* MetasoundGraphNode = Cast<UMetasoundEditorGraphNode>(GraphNode))
			{
				const FVector2D NewNodeLocation = FVector2D(GraphNode->NodePosX, GraphNode->NodePosY);
				OutBuilder.SetNodeLocation(MetasoundGraphNode->GetNodeID(), NewNodeLocation, &MetasoundGraphNode->NodeGuid);
			}
		}

		// Set new comment node positions 
		for (UMetasoundEditorGraphCommentNode* CommentNode : CommentNodes)
		{
			CommentNode->NodePosX = (CommentNode->NodePosX - AvgNodePosition.X) + InLocation.X;
			CommentNode->NodePosY = (CommentNode->NodePosY - AvgNodePosition.Y) + InLocation.Y;
			CommentNode->UpdateFrontendNodeLocation();
		}
	}

	void FDocumentClipboardUtils::ProcessPastedNodeConnections(FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes)
	{
		using namespace Frontend;

		for (UEdGraphNode* GraphNode : OutPastedNodes)
		{
			for (UEdGraphPin* Pin : GraphNode->Pins)
			{
				if (Pin->Direction != EGPD_Input)
				{
					continue;
				}

				FInputHandle InputHandle = FGraphBuilder::GetInputHandleFromPin(Pin);
				if (InputHandle->IsValid() && InputHandle->GetDataType() != GetMetasoundDataTypeName<FTrigger>())
				{
					FMetasoundFrontendLiteral LiteralValue;
					if (FGraphBuilder::GetPinLiteral(*Pin, LiteralValue))
					{
						if (const FMetasoundFrontendLiteral* ClassDefault = InputHandle->GetClassDefaultLiteral())
						{
							// Check equivalence with class default and don't set if they are equal. Copied node
							// pin has no information to indicate whether or not the literal was already set.
							if (!LiteralValue.IsEqual(*ClassDefault))
							{
								InputHandle->SetLiteral(LiteralValue);
							}
						}
						else
						{
							InputHandle->SetLiteral(LiteralValue);
						}
					}
				}

				for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
				{
					if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(LinkedPin->GetOwningNode()))
					{
						FGraphBuilder::ConnectNodes(*Pin, *LinkedPin, false /* bConnectEdPins */);
					}
				}
			}
		}
	}

	TArray<UEdGraphNode*> FDocumentClipboardUtils::PasteClipboardString(const FText& InTransactionText, const FString& InClipboardString, const FVector2D& InLocation, UObject& OutMetaSound, FDocumentPasteNotifications& OutNotifications)
	{
		using namespace Metasound::Engine;

		FMetasoundAssetBase* Asset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&OutMetaSound);
		check(Asset);

		const FScopedTransaction Transaction(InTransactionText);

		OutMetaSound.Modify();
		Asset->GetGraphChecked().Modify();

		TArray<UMetasoundEditorGraphCommentNode*> PastedCommentNodes;
		TArray<UMetasoundEditorGraphNode*> PastedGraphNodes;
		{
			TSet<UEdGraphNode*> PastedNodeSet;
			FEdGraphUtilities::ImportNodesFromText(Asset->GetGraph(), InClipboardString, PastedNodeSet);

			auto CastToMetaSoundNode = [](UEdGraphNode* Node) { return Cast<UMetasoundEditorGraphNode>(Node); };
			Algo::TransformIf(PastedNodeSet, PastedGraphNodes, CastToMetaSoundNode, CastToMetaSoundNode);

			auto CastToCommentNode = [](UEdGraphNode* Node) { return Cast<UMetasoundEditorGraphCommentNode>(Node); };
			Algo::TransformIf(PastedNodeSet, PastedCommentNodes, CastToCommentNode, CastToCommentNode);
		}

		TArray<UEdGraphNode*> PastedNodes;
		if (PastedGraphNodes.IsEmpty() && PastedCommentNodes.IsEmpty())
		{
			return PastedNodes;
		}

		FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Asset->GetOwningAsset());

		ProcessPastedCommentNodes(Builder, *Asset, PastedCommentNodes);
		ProcessPastedInputNodes(Builder, *Asset, PastedGraphNodes);
		ProcessPastedOutputNodes(Builder, *Asset, PastedGraphNodes, OutNotifications);
		ProcessPastedVariableNodes(*Asset, PastedGraphNodes, OutNotifications);
		ProcessPastedExternalNodes(Builder, *Asset, PastedGraphNodes, OutNotifications);
		ProcessPastedNodePositions(Builder, InLocation, PastedGraphNodes, PastedCommentNodes);
		ProcessPastedNodeConnections(*Asset, PastedGraphNodes);

		PastedNodes.Append(MoveTemp(PastedGraphNodes));
		PastedNodes.Append(MoveTemp(PastedCommentNodes));

		return PastedNodes;
	}

	void FDocumentClipboardUtils::CopyMemberToClipboard(UMetasoundEditorGraphMember* Content)
	{
		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		// Export the clipboard to text.
		FStringOutputDevice Archive;
		const FExportObjectInnerContext Context;
		UExporter::ExportToOutputDevice(&Context, Content, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Content->GetOuter());
		FPlatformApplicationMisc::ClipboardCopy(*Archive);
	}

	FMetasoundFrontendGraphLayer FDocumentClipboardUtils::CopySelectionToLayer(const FMetaSoundFrontendDocumentBuilder& Builder, const FGraphPanelSelectionSet& Selection)
{
		using namespace Engine;
		using namespace Frontend;

		FMetasoundFrontendGraphLayer Layer;
		Layer.Graph.PageID = Builder.GetBuildPageID();

		TSet<FGuid> NodeIDs;
		TSet<FGuid> AddedVariableIDs;
		TMap<FGuid, FMetasoundFrontendClass> DependenciesToAdd;
		for (UObject* Object : Selection)
		{
			if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Object))
			{
				const FMetasoundFrontendNode& FrontendNode = ExternalNode->GetFrontendNodeChecked();
				if (const FMetasoundFrontendClass* Dependency = Builder.FindDependency(FrontendNode.ClassID))
				{
					Layer.Graph.Nodes.Add(FrontendNode);
					NodeIDs.Add(FrontendNode.GetID());
					DependenciesToAdd.Add(Dependency->ID, *Dependency);
				}
			}
			else if (const UMetasoundEditorGraphInputNode* EdInputNode = Cast<UMetasoundEditorGraphInputNode>(Object))
			{
				if (const UMetasoundEditorGraphInput* EdInput = EdInputNode->Input)
				{
					const FMetasoundFrontendNode& TemplateNode = EdInputNode->GetFrontendNodeChecked();
					{
						if (const FMetasoundFrontendNode* InputNode = Builder.FindNode(EdInput->NodeID))
						{
							if (const FMetasoundFrontendClassInput* FrontendInput = Builder.FindGraphInput(EdInput->GetMemberName()))
							{
								const FMetasoundFrontendClass* TemplateDependency = Builder.FindDependency(TemplateNode.ClassID);
								const FMetasoundFrontendClass* InputDependency = Builder.FindDependency(InputNode->ClassID);

								if (TemplateDependency && InputDependency)
								{
									Layer.Graph.Nodes.Add(TemplateNode);
									NodeIDs.Add(TemplateNode.GetID());

									DependenciesToAdd.Add(TemplateDependency->ID, *TemplateDependency);
									DependenciesToAdd.Add(InputDependency->ID, *InputDependency);

									if (!NodeIDs.Contains(InputNode->GetID()))
									{
										Layer.Graph.Nodes.Add(*InputNode);
										NodeIDs.Add(InputNode->GetID());
										Layer.Inputs.Add(*FrontendInput);
									}
								}
							}
						}
					}
				}
			}
			else if (const UMetasoundEditorGraphOutputNode* OutputNode = Cast<UMetasoundEditorGraphOutputNode>(Object))
			{
				const FMetasoundFrontendNode& FrontendNode = OutputNode->GetFrontendNodeChecked();
				if (const FMetasoundFrontendClass* Dependency = Builder.FindDependency(FrontendNode.ClassID))
				{
					Layer.Graph.Nodes.Add(FrontendNode);
					NodeIDs.Add(FrontendNode.GetID());
					DependenciesToAdd.Add(Dependency->ID, *Dependency);

					if (UMetasoundEditorGraphOutput* Output = OutputNode->Output)
					{
						if (const FMetasoundFrontendClassOutput* FrontendOutput = Builder.FindGraphOutput(Output->GetMemberName()))
						{
							Layer.Outputs.Add(*FrontendOutput);
						}
					}
				}
			}
			else if (const UMetasoundEditorGraphVariableNode* VariableNode = Cast<UMetasoundEditorGraphVariableNode>(Object))
			{
				const FGuid VariableNodeID = VariableNode->GetNodeID();
				if (const FMetasoundFrontendNode* FrontendNode = Builder.FindNode(VariableNodeID))
				{
					if (const FMetasoundFrontendClass* Dependency = Builder.FindDependency(FrontendNode->ClassID))
					{
						Layer.Graph.Nodes.Add(*FrontendNode);
						NodeIDs.Add(FrontendNode->GetID());
						DependenciesToAdd.Add(Dependency->ID, *Dependency);

						// Add the associated variable definition (once per variable)
						if (const FMetasoundFrontendVariable* Variable = Builder.FindGraphVariableByNodeID(VariableNodeID))
						{
							if (!AddedVariableIDs.Contains(Variable->ID))
							{
								AddedVariableIDs.Add(Variable->ID);
								Layer.Graph.Variables.Add(*Variable);

								// Also add the hidden variable node and its dependency if not already present
								if (Variable->VariableNodeID.IsValid() && !NodeIDs.Contains(Variable->VariableNodeID))
								{
									if (const FMetasoundFrontendNode* VarNode = Builder.FindNode(Variable->VariableNodeID))
									{
										Layer.Graph.Nodes.Add(*VarNode);
										NodeIDs.Add(VarNode->GetID());
										if (const FMetasoundFrontendClass* VarDep = Builder.FindDependency(VarNode->ClassID))
										{
											DependenciesToAdd.Add(VarDep->ID, *VarDep);
										}
									}
								}
							}
						}
					}
				}
			}
		}

		for (const UObject* Object : Selection)
		{
			if (const UMetasoundEditorGraphNode* EditorNode = Cast<const UMetasoundEditorGraphNode>(Object))
			{
				const FMetasoundFrontendNode& FrontendNode = EditorNode->GetFrontendNodeChecked();
				for (const FMetasoundFrontendVertex& Vertex : FrontendNode.Interface.Inputs)
				{
					const FMetasoundFrontendNode* OutputNode = nullptr;
					if (const FMetasoundFrontendVertex* Output = Builder.FindNodeOutputConnectedToNodeInput(EditorNode->GetNodeID(), Vertex.VertexID, &OutputNode))
					{
						check(OutputNode);
						if (NodeIDs.Contains(OutputNode->GetID()))
						{
							Layer.Graph.Edges.Add(FMetasoundFrontendEdge { OutputNode->GetID(), Output->VertexID, FrontendNode.GetID(), Vertex.VertexID });
						}
					}
				}
			}
		}

		DependenciesToAdd.GenerateValueArray(Layer.Dependencies);
		return Layer;
	}

	const UMetasoundEditorGraphMember* FDocumentClipboardUtils::GetMemberFromClipboard()
	{
		using namespace DocumentClipboardUtilsPrivate;

		// Get the text from the clipboard.
		FString ClipboardText;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

		FMemberClipboardObjectTextFactory Factory;
		if (Factory.CanCreateObjectsFromText(ClipboardText))
		{
			Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardText);
			return Factory.Member;
		}
		
		return nullptr;
	}

	const bool FDocumentClipboardUtils::CanImportMemberFromText(const FString& TextToImport)
	{
		using namespace DocumentClipboardUtilsPrivate;
		return FMemberClipboardObjectTextFactory().CanCreateObjectsFromText(TextToImport);
	}
} // namespace Metasound::Editor
