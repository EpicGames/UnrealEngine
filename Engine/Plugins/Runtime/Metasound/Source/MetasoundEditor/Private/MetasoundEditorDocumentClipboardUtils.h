// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"

// Forward Declarations

struct FMetasoundFrontendGraphLayer;


namespace Metasound::Editor
{
	struct FDocumentPasteNotifications
	{
		bool bPastedNodesAddMultipleVariableSetters = false;
		bool bPastedNodesCreateLoop = false;
		bool bPastedNodesAddMultipleOutputNodes = false;
	};

	struct FDocumentClipboardUtils
	{
		static TArray<UEdGraphNode*> PasteClipboardString(const FText& InTransactionText, const FString& InClipboardString, const FVector2D& InLocation, UObject& OutMetaSound, FDocumentPasteNotifications& OutNotifications);

		/** Copy MetasoundMember to Clipboard*/
		static void CopyMemberToClipboard(class UMetasoundEditorGraphMember* Content);

		static FMetasoundFrontendGraphLayer CopySelectionToLayer(const FMetaSoundFrontendDocumentBuilder& Builder, const FGraphPanelSelectionSet& Selection);

		/** Return MetasoundMember that is in Clipboard, if any*/
		static const UMetasoundEditorGraphMember* GetMemberFromClipboard();

		/** Whether the string can be imported to a MetasoundMember */
		static const bool CanImportMemberFromText(const FString& TextToImport);

	private:
		static void ProcessPastedCommentNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, const TArrayView<UMetasoundEditorGraphCommentNode*> CommentNodes);

		static void ProcessPastedExternalNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications);

		static void ProcessPastedInputNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes);

		static void ProcessPastedNodePositions(FMetaSoundFrontendDocumentBuilder& OutAsset, const FVector2D& InLocation, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, const TArrayView<UMetasoundEditorGraphCommentNode*> CommentNodes);

		static void ProcessPastedNodeConnections(FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes);

		static void ProcessPastedOutputNodes(FMetaSoundFrontendDocumentBuilder& Builder, FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications);

		static void ProcessPastedVariableNodes(FMetasoundAssetBase& OutAsset, TArray<UMetasoundEditorGraphNode*>& OutPastedNodes, FDocumentPasteNotifications& OutNotifications);
	};
} // namespace Metasound::Editor
