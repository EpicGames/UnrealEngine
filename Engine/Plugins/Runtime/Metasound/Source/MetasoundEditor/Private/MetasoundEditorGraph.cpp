// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraph.h"

#include "Algo/Transform.h"
#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "AudioParameterControllerInterface.h"
#include "Components/AudioComponent.h"
#include "EdGraph/EdGraphNode.h"
#include "Interfaces/ITargetPlatform.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLog.h"
#include "MetasoundSettings.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundVariableNodes.h"
#include "MetasoundVertex.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraph)

#define LOCTEXT_NAMESPACE "MetaSoundEditor"

// Parameter names do not support analyzer path separator, but do support
// spaces (to be as consistent as possible with other systems such as Blueprint)
#define INVALID_PARAMETER_NAME_CHARACTERS TEXT("\"',\n\r\t") METASOUND_ANALYZER_PATH_SEPARATOR


namespace Metasound::Editor
{
	namespace GraphPrivate
	{
		static const FText SetMemberAccessTypeTransactionLabelFormat = LOCTEXT("RenameGraphMemberAccessTypeFormat", "Set MetaSound {0} '{1}' AccessType");
		static const FText SetMemberDefaultTransactionLabelFormat = LOCTEXT("SetGraphMemberDefaultFormat", "Set MetaSound {0} '{1}' Default(s)");
		static const FText SetMemberDescriptionTransactionLabelFormat = LOCTEXT("SetGraphMemberTooltipFormat", "Set MetaSound {0} '{1}' ToolTip");
		static const FText SetMemberDisplayNameTransactionLabelFormat = LOCTEXT("RenameGraphMemberDisplayNameFormat", "Set MetaSound {0} '{1}' DisplayName to '{2}'");
		static const FText SetMemberNameTransactionLabelFormat = LOCTEXT("RenameGraphVertexMemberNameFormat", "Set MetaSound {0} Namespace and Name from '{1}' to '{2}'");

		FName GetUniqueTransientMemberName()
		{
			// Use unique instance ID to avoid copy/paste logic resolving invalid relationship between graphs.
			// Equality is properly resolved based on associated Frontend node's Name, TypeName, & AccessType.
			// This can bloat the name table within a editor given session, but ed graph is not serialized so
			// purely for editing.
			return *FString::Printf(TEXT("Member_%s"), *FGuid::NewGuid().ToString());
		}

		void OnLiteralChanged(UMetasoundEditorGraphMember& InMember, const FGuid* InPageID, EPropertyChangeType::Type InChangeType)
		{
			constexpr bool bPostTransaction = false;
			EMetaSoundLiteralChangeType LiteralChangeType = EMetaSoundLiteralChangeType::Set;
			if (InChangeType & EPropertyChangeType::ResetToDefault)
			{
				LiteralChangeType = EMetaSoundLiteralChangeType::Reset;
			}
			InMember.UpdateFrontendDefaultLiteral(bPostTransaction, InPageID, LiteralChangeType);

			const bool bCommitChange = InChangeType != EPropertyChangeType::Interactive;
			if (bCommitChange)
			{
				if (UObject* MetaSound = InMember.GetOutermostObject())
				{
					{
						Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
						RegOptions.bIgnoreIfLiveAuditioning = true;
						FGraphBuilder::RegisterGraphWithFrontend(*MetaSound, MoveTemp(RegOptions));
					}

					if (FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSound))
					{
						MetaSoundAsset->GetModifyContext().AddMemberIDsModified({ InMember.GetMemberID() });
					}
				}
			}
		}

		// Avoids member literal setting the node literal if its not required (which in turn
		// avoids 'Reset To Default' action from being enabled when the default is equal)
		void SetOrClearIfLiteralMatchesNodeVertexDefault(FMetaSoundFrontendDocumentBuilder& InBuilder, const FMetasoundFrontendVertexHandle& VertexHandle, const FMetasoundFrontendLiteral& InDefaultLiteral)
		{
			using namespace Frontend; 

			const FMetasoundFrontendVertex* Vertex = InBuilder.FindNodeInput(VertexHandle.NodeID, VertexHandle.VertexID);
			check(Vertex);

			bool bClearLiteral = false;
			const TArray<FMetasoundFrontendClassInputDefault>* ClassDefaults = InBuilder.FindNodeClassInputDefaults(VertexHandle.NodeID, Vertex->Name);
			if (ClassDefaults)
			{
				if (const FMetasoundFrontendClassInputDefault* ClassDefault = FindPreferredPage(*ClassDefaults, UMetaSoundSettings::GetPageOrder()))
				{
					bClearLiteral = ClassDefault->Literal.IsEqual(InDefaultLiteral);
				}
			}

			if (!bClearLiteral)
			{
				FMetasoundFrontendLiteral DefaultTypeLiteral;
				DefaultTypeLiteral.SetFromLiteral(Frontend::IDataTypeRegistry::Get().CreateDefaultLiteral(Vertex->TypeName));
				bClearLiteral = InDefaultLiteral.IsEqual(DefaultTypeLiteral);
			}

			if (bClearLiteral)
			{
				InBuilder.RemoveNodeInputDefault(VertexHandle.NodeID, VertexHandle.VertexID);
			}
			else
			{
				InBuilder.SetNodeInputDefault(VertexHandle.NodeID, VertexHandle.VertexID, InDefaultLiteral);
			}
		}

		void UpdatePreviewParameter(UMetasoundEditorGraph* MetaSoundGraph, FName MemberName, UMetasoundEditorGraphMemberDefaultLiteral& Literal)
		{
			if (GEditor)
			{
				if (MetaSoundGraph && MetaSoundGraph->IsPreviewing())
				{
					UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent();
					check(PreviewComponent);

					if (TScriptInterface<IAudioParameterControllerInterface> ParamInterface = PreviewComponent)
					{
						Literal.UpdatePreviewInstance(MemberName, ParamInterface);
					}
				}
			}
		}
	} // namespace GraphPrivate
} // namespace Metasound::Editor

FMetaSoundFrontendDocumentBuilder& UMetasoundEditorGraphMember::GetFrontendBuilderChecked() const
{
	using namespace Metasound::Frontend;

	const UMetasoundEditorGraph* Graph = GetOwningGraph();
	check(Graph);
	UObject& MetaSound = Graph->GetMetasoundChecked();
	return IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&MetaSound);
}

UMetasoundEditorGraph* UMetasoundEditorGraphMember::GetOwningGraph()
{
	// Due to a prior document migration that enables ed graphs to be built from the frontend document exclusively,
	// MetaSound objects may contain more than one editor graph, so must check outer rather than accessing the
	// transient graph from the FMetaSoundAssetBase layer.
	return Cast<UMetasoundEditorGraph>(GetOuter());
}

const UMetasoundEditorGraph* UMetasoundEditorGraphMember::GetOwningGraph() const
{
	// Due to a prior document migration that enables ed graphs to be built from the frontend document exclusively,
	// MetaSound objects may contain more than one editor graph, so must check outer rather than accessing the
	// transient graph from the FMetaSoundAssetBase layer.
	return Cast<const UMetasoundEditorGraph>(GetOuter());
}

void UMetasoundEditorGraphMember::InitializeLiteral(bool bForceRebind)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FDataTypeRegistryInfo DataTypeInfo;
	IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	IDataTypeRegistry::Get().GetDataTypeInfo(TypeName, DataTypeInfo);
	const EMetasoundFrontendLiteralType LiteralType = static_cast<EMetasoundFrontendLiteralType>(DataTypeInfo.PreferredLiteralType);

	TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass = EditorModule.FindDefaultLiteralClass(LiteralType);
	if (!LiteralClass)
	{
		LiteralClass = UMetasoundEditorGraphMemberDefaultLiteral::StaticClass();
	}

	if (bForceRebind || !Literal || Literal->GetClass() != LiteralClass)
	{
		FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
		const bool bIsNew = UMetaSoundEditorSubsystem::GetChecked().BindMemberMetadata(Builder, *this, LiteralClass);
		checkf(Literal, TEXT("Bind is required to initialize literal field on this member"));

		if (bIsNew)
		{
			Literal->Initialize();
		}
	}
}

FName UMetasoundEditorGraphMember::GetDataType() const
{
	return TypeName;
}

#if WITH_EDITOR
void UMetasoundEditorGraphMember::PostEditUndo()
{
	using namespace Metasound;

	Super::PostEditUndo();

	if (IsValidChecked(this))
	{
		constexpr bool bPostTransaction = false;
		SetDataType(TypeName, bPostTransaction);
		UpdateFrontendDefaultLiteral(bPostTransaction);
	}
}
#endif // WITH_EDITOR

bool UMetasoundEditorGraphMember::Synchronize()
{
	bool bModified = false;
	if (!Literal)
	{
		bModified = true;
		InitializeLiteral();
	}

	return bModified;
}

void UMetasoundEditorGraphVertex::InitMember(FName InDataType, const FMetasoundFrontendLiteral& InDefaultLiteral, FGuid InNodeID, FMetasoundFrontendClassName&& InClassName)
{
	TypeName = InDataType;
	NodeID = InNodeID;
	ClassName = MoveTemp(InClassName);

	InitializeLiteral();

	if (ensure(Literal))
	{
		Literal->SetFromLiteral(InDefaultLiteral);
	}
}

const FMetasoundFrontendNode* UMetasoundEditorGraphVertex::GetFrontendNode() const
{
	using namespace Metasound::Engine;

	if (const UMetasoundEditorGraph* Graph = GetOwningGraph())
	{
		const FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&Graph->GetMetasoundChecked());
		return Builder.FindNode(NodeID);
	}

	return nullptr;
}

TArray<UMetasoundEditorGraphMemberNode*> UMetasoundEditorGraphVertex::GetNodes() const
{
	TArray<UMetasoundEditorGraphMemberNode*> Nodes;

	const UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (ensure(Graph))
	{
		Graph->GetNodesOfClassEx<UMetasoundEditorGraphMemberNode>(Nodes);
		for (int32 i = Nodes.Num() -1; i >= 0; --i)
		{
			UMetasoundEditorGraphNode* Node = Nodes[i];
			if (Node && Node->GetNodeID() != NodeID)
			{
				Nodes.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

	return Nodes;
}

void UMetasoundEditorGraphVertex::SetDescription(const FText& InDescription, bool bPostTransaction)
{
	Breadcrumb.Description = InDescription;
}

FGuid UMetasoundEditorGraphVertex::GetMemberID() const 
{ 
	return NodeID;
}

FName UMetasoundEditorGraphVertex::GetMemberName() const
{
	using namespace Metasound::Frontend;

	if (!Breadcrumb.MemberName.IsNone())
	{
		return Breadcrumb.MemberName;
	}

	if (const FMetasoundFrontendNode* FrontendNode = GetFrontendNode())
	{
		return FrontendNode->Name;
	}

	return FName();
}

void UMetasoundEditorGraphVertex::SetMemberName(const FName& InNewName, bool bPostTransaction)
{
	constexpr bool bPropagateToPinNames = true;
	SetMemberNameInternal(InNewName, bPropagateToPinNames, bPostTransaction);
}

void UMetasoundEditorGraphVertex::SetMemberNameInternal(const FName& InNewName, bool bPropagateToPinNames, bool bPostTransaction)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
	FName OldName;
	if (const FMetasoundFrontendNode* Node = DocBuilder.FindNode(NodeID); ensure(Node))
	{
		if (Node->Name == InNewName)
		{
			return;
		}
		OldName = Node->Name;
	}

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberNameTransactionLabelFormat, GetGraphMemberLabel(), FText::FromName(OldName), FText::FromName(InNewName));
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();

	RenameFrontendMemberInternal(DocBuilder, OldName, InNewName);
	Breadcrumb.MemberName = InNewName;

	if (bPropagateToPinNames)
	{
		const TArray<UMetasoundEditorGraphMemberNode*> Nodes = GetNodes();
		for (UMetasoundEditorGraphMemberNode* Node : Nodes)
		{
			const TArray<UEdGraphPin*>& Pins = Node->GetAllPins();
			ensure(Pins.Num() == 1);

			for (UEdGraphPin* Pin : Pins)
			{
				Pin->Modify();
				Pin->PinName = InNewName;
			}
		}
	}

	Graph->RegisterGraphWithFrontend();
}

FText UMetasoundEditorGraphVertex::GetDisplayName() const
{
	constexpr bool bIncludeNamespace = true;
	return Metasound::Editor::FGraphBuilder::GetDisplayName(*GetConstNodeHandle(), bIncludeNamespace);
}

#if WITH_EDITORONLY_DATA
bool UMetasoundEditorGraphVertex::SetIsAdvancedDisplay(const bool IsAdvancedDisplay)
{
	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return false;
	}

	const FText TransactionLabel = FText::Format(LOCTEXT("SetGraphVertexIsAdvancedDisplayState", "Set Metasound {0} IsAdvancedDisplay"), GetGraphMemberLabel());
	const FScopedTransaction Transaction(TransactionLabel, true);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();
	Modify();

	bool bSucceeded = false;
	
	if (GetClassType() == EMetasoundFrontendClassType::Input)
	{
		bSucceeded = GetFrontendBuilderChecked().SetGraphInputAdvancedDisplay(GetMemberName(), IsAdvancedDisplay);
		
	}
	else if(GetClassType() == EMetasoundFrontendClassType::Output)
	{
		bSucceeded = GetFrontendBuilderChecked().SetGraphOutputAdvancedDisplay(GetMemberName(), IsAdvancedDisplay);
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();	
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID))
	{
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			ClassName = Class->Metadata.GetClassName();
		}
	}

	Graph->RegisterGraphWithFrontend();
	return bSucceeded;
}
#endif // WITH_EDITORONLY_DATA

void UMetasoundEditorGraphVertex::CacheBreadcrumb()
{
	Breadcrumb = { };

	if (const FMetasoundFrontendNode* FrontendNode = GetFrontendNode())
	{
		Breadcrumb.MemberName = FrontendNode->Name;
	}

	if (const FMetasoundFrontendClassVertex* FrontendVertex = GetFrontendClassVertex())
	{
		Breadcrumb.bIsAdvancedDisplay = FrontendVertex->GetIsAdvancedDisplay();
	}

	Breadcrumb.AccessType = GetVertexAccessType();
	Breadcrumb.Description = GetDescription();
	Breadcrumb.SortOrderIndex = GetSortOrderIndex();
}

Metasound::Frontend::FNodeHandle UMetasoundEditorGraphVertex::GetNodeHandle()
{
	using namespace Metasound;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	check(Graph);

	UObject* Object = Graph->GetMetasound();
	if (!ensure(Object))
	{
		return Frontend::INodeController::GetInvalidHandle();
	}

	FMetasoundAssetBase* MetasoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(Object);
	check(MetasoundAsset);

	return MetasoundAsset->GetRootGraphHandle()->GetNodeWithID(NodeID);
}

Metasound::Frontend::FConstNodeHandle UMetasoundEditorGraphVertex::GetConstNodeHandle() const
{
	using namespace Metasound;

	const FMetasoundAssetBase& MetaSound = Editor::FGraphBuilder::GetOutermostConstMetaSoundChecked(*this);
	return MetaSound.GetRootGraphHandle()->GetNodeWithID(NodeID);
}

const FMetasoundFrontendVersion& UMetasoundEditorGraphVertex::GetInterfaceVersion() const
{
	return GetConstNodeHandle()->GetInterfaceVersion();
}

bool UMetasoundEditorGraphVertex::IsInterfaceMember(FMetasoundFrontendInterface* OutInterface) const
{
	return false;
}

bool UMetasoundEditorGraphVertex::NameContainsInterfaceNamespace(FMetasoundFrontendInterface* OutInterface) const
{
	using namespace Metasound::Frontend;

	const FName MemberName = GetMemberName();
	FName InterfaceNamespace;
	FName ParamName;
	Audio::FParameterPath::SplitName(MemberName, InterfaceNamespace, ParamName);

	if (!InterfaceNamespace.IsNone())
	{
		FMetasoundFrontendVersion InterfaceVersion;
		const bool bFoundVersion = ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceNamespace, InterfaceVersion);
		if (bFoundVersion)
		{
			if (OutInterface)
			{
				const bool bIsValid = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, *OutInterface);
				checkf(bIsValid, TEXT("Version found in search engine must be registered with interface registry"));
				return bIsValid;
			}
			return true;
		}
	}

	if (OutInterface)
	{
		*OutInterface = { };
	}
	return false;
}

bool UMetasoundEditorGraphVertex::CanRename() const
{
	FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	return Builder.GetIsGraphEditable() && !IsInterfaceMember();
}

bool UMetasoundEditorGraphVertex::CanRename(const FText& InNewText, FText& OutError) const
{
	using namespace Metasound::Frontend;

	if (InNewText.IsEmptyOrWhitespace())
	{
		OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_NameEmpty", "{0} cannot be empty string."), InNewText);
		return false;
	}

	const FString NewNameString = InNewText.ToString();
	if (!FName::IsValidXName(NewNameString, INVALID_PARAMETER_NAME_CHARACTERS, &OutError))
	{
		return false;
	}

	if (NewNameString.Len() >= NAME_SIZE)
	{
		OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_NameTooLong", "Name cannot be longer than {0} characters."), NAME_SIZE);
		return false;
	}

	if (IsInterfaceMember())
	{
		const FText CurrentMemberName = FText::FromName(GetMemberName());
		OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_GraphVertexRequired", "{0} is interface member and cannot be renamed."), CurrentMemberName);
		return false;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (!Builder.GetIsGraphEditable())
	{
		OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_Preset", "{0} is a vertex in a read-only graph and cannot be renamed."), InNewText);
		return false;
	}

	const FName NewName(*NewNameString);
	FName Namespace;
	FName ParameterName;
	Audio::FParameterPath::SplitName(NewName, Namespace, ParameterName);

	bool bIsNameValid = true;
	Builder.IterateNodesByClassType([&](const FMetasoundFrontendClass&, const FMetasoundFrontendNode& NodeToCompare)
	{
		if (NodeID != NodeToCompare.GetID())
		{
			FName OtherName = NodeToCompare.Name;
			if (NewName == OtherName)
			{
				bIsNameValid = false;
				OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_NameTaken", "{0} is already in use"), InNewText);
			}
			else if (Namespace == OtherName)
			{
				bIsNameValid = false;
				OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_NamespaceTaken", "Namespace of '{0}' cannot be the same as an existing member's name"), InNewText);
			}
			else
			{
				FName OtherNamespace;
				Audio::FParameterPath::SplitName(OtherName, OtherNamespace, OtherName);
				if (OtherNamespace == NewName)
				{
					bIsNameValid = false;
					OutError = FText::Format(LOCTEXT("GraphVertexRenameInvalid_NamespaceTaken2", "Name of '{0}' cannot be the same as an existing member's namespace"), InNewText);
				}
			}
		}
	}, GetClassType());

	return bIsNameValid;
}

bool UMetasoundEditorGraphVertex::Synchronize()
{
	bool bModified = Super::Synchronize();

	if (const FMetasoundFrontendClassVertex* Vertex = GetFrontendClassVertex(); ensure(Vertex))
	{
		if (TypeName != Vertex->TypeName)
		{
			bModified = true;
			TypeName = Vertex->TypeName;

			InitializeLiteral();
		}

		FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
		if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID); ensure(Node))
		{
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID); ensure(Class))
			{
				const FMetasoundFrontendClassName& FrontendClassName = Class->Metadata.GetClassName();
				if (ClassName != FrontendClassName)
				{
					bModified = true;
					ClassName = FrontendClassName;
				}
			}
		}
	}

	return bModified;
}

UMetasoundEditorGraphMember* UMetasoundEditorGraphMemberDefaultLiteral::FindMember() const
{
	using namespace Metasound;

	const FMetasoundAssetBase& MetaSound = Editor::FGraphBuilder::GetOutermostConstMetaSoundChecked(*this);
	if (UMetasoundEditorGraph* Graph = Cast<UMetasoundEditorGraph>(MetaSound.GetGraph()))
	{
		return Graph->FindMember(MemberID);
	}

	return nullptr;
}

void UMetasoundEditorGraphMemberDefaultLiteral::ForceRefresh()
{
}

FName UMetasoundEditorGraphMemberDefaultLiteral::GetDataType() const
{
	return { };
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultLiteral::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::None;
}

void UMetasoundEditorGraphMemberDefaultLiteral::InitDefault(const FGuid& InPageID)
{
}

void UMetasoundEditorGraphMemberDefaultLiteral::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	FMetasoundFrontendLiteral Literal;
	if (TryFindDefault(Literal))
	{
		Iter(Metasound::Frontend::DefaultPageID, MoveTemp(Literal));
	}
}

bool UMetasoundEditorGraphMemberDefaultLiteral::RemoveDefault(const FGuid& InPageID)
{
	return false;
}

void UMetasoundEditorGraphMemberDefaultLiteral::ResetDefaults()
{
}

void UMetasoundEditorGraphMemberDefaultLiteral::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
}

bool UMetasoundEditorGraphMemberDefaultLiteral::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	OutLiteral = { };
	return true;
}

#if WITH_EDITOR
void UMetasoundEditorGraphMemberDefaultLiteral::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (UMetasoundEditorGraphMember* Member = FindMember())
	{
		Metasound::Editor::GraphPrivate::OnLiteralChanged(*Member, nullptr, InPropertyChangedEvent.ChangeType);
	}
}

void UMetasoundEditorGraphMemberDefaultLiteral::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	if (const FEditPropertyChain::TDoubleLinkedListNode* MemberNode = InPropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		if (const FProperty* ChildProperty = MemberNode->GetValue())
		{
			const FName ChildPropertyName = ChildProperty->GetFName();

			if (ChildPropertyName.IsEqual(GetDefaultsPropertyName()))
			{
				ResolvePageDefaults();
				SortPageDefaults();
			}
		}
	}

	if (UMetasoundEditorGraphMember* Member = FindMember())
	{
		Metasound::Editor::GraphPrivate::OnLiteralChanged(*Member, nullptr, InPropertyChangedEvent.ChangeType);
	}
}

void UMetasoundEditorGraphMemberDefaultLiteral::PostEditUndo()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!IsValidChecked(this))
	{
		Super::PostEditUndo();
		return;
	}

	// Builder must be reloaded to refresh caches, which may be stale after data reapplication.
	// Must happen before Super PostEditUndo which can call PostEditChangeProperty 
	// which can call RegisterGraphWithFrontend which depends on this data.
	UMetasoundEditorGraphMember* Member = FindMember();
	if (Member)
	{
		const FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
		const FMetasoundFrontendClassName& ClassName = Builder.GetConstDocumentChecked().RootGraph.Metadata.GetClassName();
		IDocumentBuilderRegistry::GetChecked().ReloadBuilder(ClassName);
	}

	Super::PostEditUndo();

	constexpr bool bPostTransaction = false;
	if (Member)
	{
		Member->UpdateFrontendDefaultLiteral(bPostTransaction);
	}
}
#endif // WITH_EDITOR

bool UMetasoundEditorGraphMemberDefaultLiteral::TryGetPreviewPageID(FGuid& OutPreviewPageID) const
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	const UMetasoundEditorGraphMember* Member = FindMember();
	if (!ensure(Member))
	{
		OutPreviewPageID = Metasound::Frontend::DefaultPageID;
		return false;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassInput* Input = Builder.FindGraphInput(Member->GetMemberName()))
	{
		const UMetasoundEditorSettings* EdSettings = ::GetDefault<UMetasoundEditorSettings>();
		const UMetaSoundSettings* Settings = ::GetDefault<UMetaSoundSettings>();
		if (Settings && EdSettings)
		{
			const FMetasoundFrontendClassInputDefault* DefaultLiteral = FindPreferredPage(Input->GetDefaults(), Settings->GetPageOrder());
			const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(EdSettings->GetAuditionPage());

			if (DefaultLiteral && PageSettings)
			{
				if (DefaultLiteral->PageID == PageSettings->UniqueId)
				{
					OutPreviewPageID = PageSettings->UniqueId;
					return true;
				}
			}
		}
	}

	OutPreviewPageID = Metasound::Frontend::DefaultPageID;
	return false;
}

Metasound::Editor::ENodeSection UMetasoundEditorGraphInput::GetSectionID() const 
{
	return Metasound::Editor::ENodeSection::Inputs;
}

const FText& UMetasoundEditorGraphInput::GetGraphMemberLabel() const
{
	static const FText Label = LOCTEXT("GraphMemberLabel_Input", "Input");
	return Label;
}

const FMetasoundFrontendClassVertex* UMetasoundEditorGraphInput::GetFrontendClassVertex() const
{
	return GetFrontendBuilderChecked().FindGraphInput(GetMemberName());
}

FText UMetasoundEditorGraphInput::GetDescription() const
{
	if (!Breadcrumb.Description.IsEmpty())
	{
		return Breadcrumb.Description;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassInput* Input = Builder.FindGraphInput(GetMemberName()))
	{
		return Input->Metadata.GetDescription();
	}

	return { };
}

int32 UMetasoundEditorGraphInput::GetSortOrderIndex() const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	const UMetasoundEditorGraph* MetaSoundGraph = GetOwningGraph();
	FConstGraphHandle GraphHandle = MetaSoundGraph->GetGraphHandle();
	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const Metasound::FVertexName& NodeName = NodeHandle->GetNodeName();
	return GraphHandle->GetSortOrderIndexForInput(NodeName);
}

TArray<UMetasoundEditorGraphMemberNode*> UMetasoundEditorGraphInput::GetNodes() const
{
	TArray<UMetasoundEditorGraphMemberNode*> Nodes;

	const UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (ensure(Graph))
	{
		TArray<UMetasoundEditorGraphInputNode*> InputNodes;
		Graph->GetNodesOfClassEx<UMetasoundEditorGraphInputNode>(InputNodes);
		Algo::TransformIf(InputNodes, Nodes,
			[this](const UMetasoundEditorGraphMemberNode* Node)
			{
				return Node->GetMember() == this;
			},
			[](UMetasoundEditorGraphMemberNode* Node)
			{
				return Node;
			});
	}

	return Nodes;
}

bool UMetasoundEditorGraphInput::IsDefaultPaged() const
{
	// Triggers are special and do not show their default value, but are visible
	// to allow for interact button when auditioning.  Therefore, default paging
	// is unnecessary.
	return TypeName != Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>();
}

bool UMetasoundEditorGraphInput::IsInterfaceMember(FMetasoundFrontendInterface* OutInterface) const
{
	FMetasoundFrontendInterface Interface;
	if (NameContainsInterfaceNamespace(&Interface))
	{
		// Is interface declared on this MetaSound 
		const UObject& MetaSoundObject = GetOwningGraph()->GetMetasoundChecked();
		const FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSoundObject);
		if (MetaSoundAsset && MetaSoundAsset->IsInterfaceDeclared(Interface.Metadata.Version))
		{
			// Check if Input is a member of the found interface
			if (const FMetasoundFrontendNode* InputNode = GetFrontendNode())
			{
				const FMetasoundFrontendVertex& Input = InputNode->Interface.Inputs.Last();
				auto IsInput = [&Input](const FMetasoundFrontendClassInput& InterfaceInput)
				{
					return FMetasoundFrontendVertex::IsFunctionalEquivalent(Input, InterfaceInput);
				};

				if (Interface.Inputs.ContainsByPredicate(IsInput))
				{
					if (OutInterface)
					{
						*OutInterface = MoveTemp(Interface);
					}
					return true;
				}
			}
		}
	}

	if (OutInterface)
	{
		*OutInterface = { };
	}
	return false;
}

void UMetasoundEditorGraphInput::SetSortOrderIndex(int32 InSortOrderIndex)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* MetaSoundGraph = GetOwningGraph();
	check(MetaSoundGraph);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGraphHandle GraphHandle = MetaSoundGraph->GetGraphHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const Metasound::FVertexName& NodeName = NodeHandle->GetNodeName();

	GraphHandle->SetSortOrderIndexForInput(NodeName, InSortOrderIndex);
	FGraphBuilder::GetOutermostMetaSoundChecked(*MetaSoundGraph).GetModifyContext().AddMemberIDsModified({ GetMemberID() });
}

void UMetasoundEditorGraphInput::ResetToClassDefault()
{
	using namespace Metasound;
	using namespace Metasound::Editor;

	if (ensure(Literal))
	{
		FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();

		Builder.CastDocumentObjectChecked<UObject>().Modify();
		Literal->Modify();

		const FName MemberName = GetMemberName();
		Builder.ResetGraphInputDefault(MemberName);

		constexpr bool bPostTransaction = false;
		UpdateFrontendDefaultLiteral(bPostTransaction);

		GraphPrivate::UpdatePreviewParameter(GetOwningGraph(), MemberName, *Literal);
	}
}

void UMetasoundEditorGraphInput::SetDataType(FName InNewType, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InNewType != GetDataType())
	{
		if (UMetasoundEditorGraph* Graph = GetOwningGraph())
		{
			UObject& MetaSound = Graph->GetMetasoundChecked();
			const FScopedTransaction Transaction(LOCTEXT("SetGraphInputData", "Set MetaSound Graph Input DataType"), bPostTransaction);
			MetaSound.Modify();
			Graph->Modify();
			Modify();

			FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
			const bool bSuccess = Builder.SetGraphInputDataType(GetMemberName(), InNewType);
			ensure(bSuccess);

			// Cached TypeName here must be set prior to re-initializing literal below
			TypeName = InNewType;

			if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID); ensure(Node))
			{
				if (const FMetasoundFrontendClass* Dependency = Builder.FindDependency(Node->ClassID); ensure(Dependency))
				{
					ClassName = Dependency->Metadata.GetClassName();
				}
			}

			InitializeLiteral();

			FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
		}
	}
}

void UMetasoundEditorGraphInput::SetDescription(const FText& InDescription, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDescriptionTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName());
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();
	GetFrontendBuilderChecked().SetGraphInputDescription(GetMemberName(), InDescription);
	Breadcrumb.Description = InDescription;
	Graph->RegisterGraphWithFrontend();

	Super::SetDescription(InDescription, bPostTransaction);
}

void UMetasoundEditorGraphInput::SetDisplayName(const FText& InNewName, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
	const FMetasoundFrontendClassInput* Input = DocBuilder.FindGraphInput(GetMemberName());
	if (!Input || Input->Metadata.GetDisplayName().EqualTo(InNewName))
	{
		return;
	}

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDisplayNameTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName(), InNewName);
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();

	DocBuilder.SetGraphInputDisplayName(Input->Name, InNewName);

	const TArray<UMetasoundEditorGraphMemberNode*> Nodes = GetNodes();
	for (UMetasoundEditorGraphMemberNode* Node : Nodes)
	{
		const TArray<UEdGraphPin*>& Pins = Node->GetAllPins();
		ensure(Pins.Num() == 1);

		for (UEdGraphPin* Pin : Pins)
		{
			Pin->PinFriendlyName = InNewName;
		}
	}

	Graph->RegisterGraphWithFrontend();
}

void UMetasoundEditorGraphInput::SetVertexAccessType(EMetasoundFrontendVertexAccessType InNewAccessType, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InNewAccessType != GetVertexAccessType())
	{
		if (UMetasoundEditorGraph* Graph = GetOwningGraph(); ensure(Graph))
		{
			const FScopedTransaction Transaction(FText::Format(GraphPrivate::SetMemberAccessTypeTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName()), bPostTransaction);
			Graph->GetMetasoundChecked().Modify();
			Graph->Modify();
			Modify();

			FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
			DocBuilder.SetGraphInputAccessType(GetMemberName(), InNewAccessType);

			if (const FMetasoundFrontendNode* Node = GetFrontendNode(); ensure(Node))
			{
				if (const FMetasoundFrontendClass* Class = DocBuilder.FindDependency(Node->ClassID); ensure(Class))
				{
					ClassName = Class->Metadata.GetClassName();
				}
			}
			Graph->RegisterGraphWithFrontend();
		}
	}
}

void UMetasoundEditorGraphInput::SetMemberName(const FName& InNewName, bool bPostTransaction)
{
	// Renaming vertex members must stop the preview component to avoid confusion afterward
	// with newly named input not passing updated values to active previewed instance.
	if (UMetasoundEditorGraph* Graph = GetOwningGraph())
	{
		if (GEditor && Graph->IsPreviewing())
		{
			GEditor->ResetPreviewAudioComponent();
		}
	}

	// Input nodes are templates, which don't have specialized pin names and get their display names via custom function.
	constexpr bool bPropagateToPinNames = false;
	SetMemberNameInternal(InNewName, bPropagateToPinNames, bPostTransaction);
}

bool UMetasoundEditorGraphInput::RenameFrontendMemberInternal(FMetaSoundFrontendDocumentBuilder& Builder, FName OldName, FName InNewName) const
{
	return Builder.SetGraphInputName(OldName, InNewName);
}

bool UMetasoundEditorGraphInput::Synchronize()
{
	bool bModified = Super::Synchronize();

	if (ensure(Literal))
	{
		bModified |= Literal->Synchronize();
	}

	return bModified;
}

void UMetasoundEditorGraphInput::CacheBreadcrumb()
{
	Super::CacheBreadcrumb();

	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassInput* Input = Builder.FindGraphInput(GetMemberName()))
	{
		Input->IterateDefaults([this](const FGuid& InPageID, const FMetasoundFrontendLiteral& InLiteral)
		{
			Breadcrumb.DefaultLiterals.Add(InPageID, InLiteral);
		});
	}
}

void UMetasoundEditorGraphInput::UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID, EMetaSoundLiteralChangeType InChangeType)
{
	using namespace Metasound::Editor;

	if (Literal)
	{
		FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();

		const FScopedTransaction Transaction(FText::Format(GraphPrivate::SetMemberDefaultTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName()), bPostTransaction);
		Builder.CastDocumentObjectChecked<UObject>().Modify();
		Literal->Modify();

		const FName MemberName = GetMemberName();
		if (InPageID)
		{
			FMetasoundFrontendLiteral Default;
			if (Literal->TryFindDefault(Default, InPageID))
			{
				const FMetasoundFrontendLiteral* ExistingDefault = Builder.GetGraphInputDefault(MemberName, InPageID);
				if (!ExistingDefault || *ExistingDefault != Default)
				{
					Builder.SetGraphInputDefault(MemberName, Default, InPageID, InChangeType);
				}
			}
		}
		else
		{
			bool bDefaultsModified = false; 
			TArray<FMetasoundFrontendClassInputDefault> NewDefaults;
			Literal->IterateDefaults([&NewDefaults, &MemberName, &Builder, &bDefaultsModified](const FGuid& PageID, FMetasoundFrontendLiteral Default)
			{
				const FMetasoundFrontendLiteral* ExistingDefault = Builder.GetGraphInputDefault(MemberName, &PageID);
				if (!ExistingDefault || *ExistingDefault != Default)
				{
					bDefaultsModified |= true;
				}
				NewDefaults.Add(FMetasoundFrontendClassInputDefault(PageID, MoveTemp(Default)));
			});
			
			const FMetasoundFrontendClassInput* GraphInput = Builder.FindGraphInput(MemberName);
			if (!GraphInput || bDefaultsModified || NewDefaults.Num() != GraphInput->GetDefaults().Num())
			{
				Builder.SetGraphInputDefaults(MemberName, MoveTemp(NewDefaults), InChangeType);
			}
		}
	}
}

EMetasoundFrontendVertexAccessType UMetasoundEditorGraphInput::GetVertexAccessType() const
{
	if (Breadcrumb.AccessType != EMetasoundFrontendVertexAccessType::Unset)
	{
		return Breadcrumb.AccessType;
	}

	const FName MemberName = GetMemberName();
	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassInput* Input = Builder.FindGraphInput(MemberName))
	{
		return Input->AccessType;
	}

	return EMetasoundFrontendVertexAccessType::Reference;
}

FText UMetasoundEditorGraphOutput::GetDescription() const
{
	if (!Breadcrumb.Description.IsEmpty())
	{
		return Breadcrumb.Description;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassOutput* Output = Builder.FindGraphOutput(GetMemberName()))
	{
		return Output->Metadata.GetDescription();
	}

	return { };
}

const FMetasoundFrontendClassVertex* UMetasoundEditorGraphOutput::GetFrontendClassVertex() const
{
	return GetFrontendBuilderChecked().FindGraphOutput(GetMemberName());
}

int32 UMetasoundEditorGraphOutput::GetSortOrderIndex() const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	const UMetasoundEditorGraph* MetaSoundGraph = GetOwningGraph();
	FConstGraphHandle GraphHandle = MetaSoundGraph->GetGraphHandle();
	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const Metasound::FVertexName& NodeName = NodeHandle->GetNodeName();
	return GraphHandle->GetSortOrderIndexForOutput(NodeName);
}

bool UMetasoundEditorGraphOutput::IsInterfaceMember(FMetasoundFrontendInterface* OutInterface) const
{
	using namespace Metasound::Frontend;
	
	FMetasoundFrontendInterface Interface;
	if (NameContainsInterfaceNamespace(&Interface))
	{
		// Is interface declared on this MetaSound 
		const UObject& MetaSoundObject = GetOwningGraph()->GetMetasoundChecked();
		const FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSoundObject);
		if (MetaSoundAsset && MetaSoundAsset->IsInterfaceDeclared(Interface.Metadata.Version))
		{
			// Check if Output is a member of the found interface
			if (const FMetasoundFrontendNode* OutputNode = GetFrontendNode())
			{
				const FMetasoundFrontendVertex& Output = OutputNode->Interface.Outputs.Last();
				auto IsOutput = [&Output](const FMetasoundFrontendClassOutput& InterfaceOutput)
				{
					return FMetasoundFrontendVertex::IsFunctionalEquivalent(Output, InterfaceOutput);
				};

				if (Interface.Outputs.ContainsByPredicate(IsOutput))
				{
					if (OutInterface)
					{
						*OutInterface = MoveTemp(Interface);
					}
					return true;
				}
			}
		}
	}
	
	if (OutInterface)
	{
		*OutInterface = { };
	}
	return false;
}

void UMetasoundEditorGraphOutput::SetSortOrderIndex(int32 InSortOrderIndex)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* MetaSoundGraph = GetOwningGraph();
	check(MetaSoundGraph);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGraphHandle GraphHandle = MetaSoundGraph->GetGraphHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const Metasound::FVertexName& NodeName = NodeHandle->GetNodeName();

	GraphHandle->SetSortOrderIndexForOutput(NodeName, InSortOrderIndex);
	FGraphBuilder::GetOutermostMetaSoundChecked(*MetaSoundGraph).GetModifyContext().AddMemberIDsModified({ GetMemberID() });
}

const FText& UMetasoundEditorGraphOutput::GetGraphMemberLabel() const
{
	static const FText Label = LOCTEXT("GraphMemberLabel_Output", "Output");
	return Label;
}

void UMetasoundEditorGraphOutput::ResetToClassDefault()
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	TArray<UMetasoundEditorGraphMemberNode*> Nodes = GetNodes();
	for (UMetasoundEditorGraphMemberNode* Node : Nodes)
	{
		TArray<const FMetasoundFrontendVertex*> InputVertices = Builder.FindNodeInputs(Node->GetNodeID());
		if (ensure(InputVertices.Num() == 1))
		{
			Builder.RemoveNodeInputDefault(Node->GetNodeID(), InputVertices.Last()->VertexID);
		}
	}
}

void UMetasoundEditorGraphOutput::SetDataType(FName InNewType, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InNewType != GetDataType())
	{
		if (UMetasoundEditorGraph* Graph = GetOwningGraph())
		{
			const FScopedTransaction Transaction(LOCTEXT("SetGraphOutputData", "Set MetaSound Graph Output DataType"), bPostTransaction);
			Graph->GetMetasoundChecked().Modify();
			Graph->Modify();
			Modify();

			EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
			Graph->GetBuilderChecked().SetGraphOutputDataType(GetMemberName(), InNewType, Result);
			ensure(Result == EMetaSoundBuilderResult::Succeeded);

			// Cached TypeName here must be set prior to re-initializing literal below
			TypeName = InNewType;
			ClassName = GetConstNodeHandle()->GetClassMetadata().GetClassName();

			InitializeLiteral();

			Graph->RegisterGraphWithFrontend();
		}
	}
}

void UMetasoundEditorGraphOutput::SetDescription(const FText& InDescription, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	} 

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDescriptionTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName());
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();
	GetFrontendBuilderChecked().SetGraphOutputDescription(GetMemberName(), InDescription);
	Graph->RegisterGraphWithFrontend();

	Super::SetDescription(InDescription, bPostTransaction);
}

void UMetasoundEditorGraphOutput::SetDisplayName(const FText& InNewName, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
	const FMetasoundFrontendClassOutput* Output = DocBuilder.FindGraphOutput(GetMemberName());
	if (!Output || Output->Metadata.GetDisplayName().EqualTo(InNewName))
	{
		return;
	}

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDisplayNameTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName(), InNewName);
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	Graph->GetMetasoundChecked().Modify();

	DocBuilder.SetGraphOutputDisplayName(Output->Name, InNewName);

	const TArray<UMetasoundEditorGraphMemberNode*> Nodes = GetNodes();
	for (UMetasoundEditorGraphMemberNode* Node : Nodes)
	{
		const TArray<UEdGraphPin*>& Pins = Node->GetAllPins();
		ensure(Pins.Num() == 1);

		for (UEdGraphPin* Pin : Pins)
		{
			Pin->PinFriendlyName = InNewName;
		}
	}

	Graph->RegisterGraphWithFrontend();
}

void UMetasoundEditorGraphOutput::SetVertexAccessType(EMetasoundFrontendVertexAccessType InNewAccessType, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InNewAccessType != GetVertexAccessType())
	{
		if (UMetasoundEditorGraph* Graph = GetOwningGraph(); ensure(Graph))
		{
			const FScopedTransaction Transaction(FText::Format(GraphPrivate::SetMemberAccessTypeTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName()), bPostTransaction);
			Graph->GetMetasoundChecked().Modify();
			Graph->Modify();
			Modify();

			FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
			DocBuilder.SetGraphOutputAccessType(GetMemberName(), InNewAccessType);

			if (const FMetasoundFrontendNode* Node = GetFrontendNode(); ensure(Node))
			{
				if (const FMetasoundFrontendClass* Class = DocBuilder.FindDependency(Node->ClassID); ensure(Class))
				{
					ClassName = Class->Metadata.GetClassName();
				}
			}
			Graph->RegisterGraphWithFrontend();
		}
	}
}

void UMetasoundEditorGraphOutput::UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID, EMetaSoundLiteralChangeType InChangeType)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UObject* Metasound = nullptr;
	UMetasoundEditorGraph* MetaSoundGraph = GetOwningGraph();
	if (ensure(MetaSoundGraph))
	{
		Metasound = MetaSoundGraph->GetMetasound();
	}

	if (!ensure(Metasound))
	{
		return;
	}

	if (!ensure(Literal))
	{
		return;
	}

	// Use the default page ID here as output defaults do *not* support paged defaults (they exist per paged graph on the singleton output node)
	FMetasoundFrontendLiteral DefaultLiteral;
	if (ensure(Literal->TryFindDefault(DefaultLiteral)))
	{
		const FScopedTransaction Transaction(FText::Format(GraphPrivate::SetMemberDefaultTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName()), bPostTransaction);
		Metasound->Modify();

		if (const FMetasoundFrontendNode* FrontendNode = GetFrontendNode())
		{
			FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
			const FGuid& VertexID = FrontendNode->Interface.Inputs.Last().VertexID;
			const FMetasoundFrontendVertexHandle VertexHandle { NodeID, VertexID };
			GraphPrivate::SetOrClearIfLiteralMatchesNodeVertexDefault(Builder, VertexHandle, DefaultLiteral);
		}
	}
}

EMetasoundFrontendVertexAccessType UMetasoundEditorGraphOutput::GetVertexAccessType() const
{
	if (Breadcrumb.AccessType != EMetasoundFrontendVertexAccessType::Unset)
	{
		return Breadcrumb.AccessType;
	}

	const FName MemberName = GetMemberName();
	const FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassOutput* Output = Builder.FindGraphOutput(MemberName))
	{
		return Output->AccessType;
	}

	return EMetasoundFrontendVertexAccessType::Reference;
}

Metasound::Editor::ENodeSection UMetasoundEditorGraphOutput::GetSectionID() const 
{
	return Metasound::Editor::ENodeSection::Outputs;
}

bool UMetasoundEditorGraphOutput::Synchronize()
{
	using namespace Metasound;

	bool bModified = Super::Synchronize();

	FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	const FName MemberName = GetMemberName();
	if (const FMetasoundFrontendClassOutput* ClassOutput = Builder.FindGraphOutput(MemberName))
	{
		if (ensure(Literal))
		{
			TOptional<FMetasoundFrontendLiteral> NewDefault;
			if (const FMetasoundFrontendNode* OutputNode = Builder.FindGraphOutputNode(GetMemberName()); ensure(OutputNode))
			{
				FMetasoundFrontendLiteral DefaultLiteral;
				Literal->TryFindDefault(DefaultLiteral);
				if (!OutputNode->InputLiterals.IsEmpty())
				{
					const FMetasoundFrontendVertexLiteral& VertexLiteral = OutputNode->InputLiterals.Last();
					if (!VertexLiteral.Value.IsEqual(DefaultLiteral))
					{
						NewDefault = VertexLiteral.Value;
					}
				}
				else
				{
					FMetasoundFrontendLiteral TypeDefault;
					TypeDefault.SetFromLiteral(Frontend::IDataTypeRegistry::Get().CreateDefaultLiteral(TypeName));
					if (!TypeDefault.IsEqual(DefaultLiteral))
					{
						NewDefault = TypeDefault;
					}
				}
			}

			if (NewDefault.IsSet())
			{
				bModified = true;
				Literal->ResetDefaults();
				Literal->SetFromLiteral(*NewDefault);
			}
		}
	}

	return bModified;
}

bool UMetasoundEditorGraphOutput::RenameFrontendMemberInternal(FMetaSoundFrontendDocumentBuilder& Builder, FName OldName, FName InNewName) const
{
	return Builder.SetGraphOutputName(OldName, InNewName);
}

void UMetasoundEditorGraphVariable::InitMember(FName InDataType, const FMetasoundFrontendLiteral& InDefaultLiteral, FGuid InVariableID)
{
	TypeName = InDataType;
	VariableID = InVariableID;

	InitializeLiteral();

	if (ensure(Literal))
	{
		Literal->SetFromLiteral(InDefaultLiteral);
	}
}

const FText& UMetasoundEditorGraphVariable::GetGraphMemberLabel() const
{
	static const FText Label = LOCTEXT("GraphMemberLabel_Variable", "Variable");
	return Label;
}

void UMetasoundEditorGraphVariable::CacheBreadcrumb()
{
	Breadcrumb = { };

	if (const FMetasoundFrontendVariable* Variable = GetFrontendVariable())
	{
		Breadcrumb.MemberName = Variable->Name;
		Breadcrumb.DefaultLiteral = Variable->Literal;
		Breadcrumb.Description = Variable->Description;
	}
}

Metasound::Frontend::FVariableHandle UMetasoundEditorGraphVariable::GetVariableHandle()
{
	using namespace Metasound;

	FMetasoundAssetBase& MetasoundAsset = Editor::FGraphBuilder::GetOutermostMetaSoundChecked(*this);
	return MetasoundAsset.GetRootGraphHandle()->FindVariable(VariableID);
}

Metasound::Frontend::FConstVariableHandle UMetasoundEditorGraphVariable::GetConstVariableHandle() const
{
	using namespace Metasound;

	const FMetasoundAssetBase& MetaSound = Editor::FGraphBuilder::GetOutermostConstMetaSoundChecked(*this);
	return MetaSound.GetRootGraphHandle()->FindVariable(VariableID);
}

void UMetasoundEditorGraphVariable::SetMemberName(const FName& InNewName, bool bPostTransaction)
{
	using namespace Metasound::Editor;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	const FName OldName = GetMemberName();
	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberNameTransactionLabelFormat, GetGraphMemberLabel(), FText::FromName(OldName), FText::FromName(InNewName));
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	Graph->Modify();
	UObject& MetaSound = Graph->GetMetasoundChecked();
	MetaSound.Modify();

	Graph->GetBuilderChecked().GetBuilder().SetGraphVariableName(OldName, InNewName);
}

FGuid UMetasoundEditorGraphVariable::GetMemberID() const 
{ 
	return VariableID;
}

FName UMetasoundEditorGraphVariable::GetMemberName() const
{
	if (!Breadcrumb.MemberName.IsNone())
	{
		return Breadcrumb.MemberName;
	}

	if (const FMetasoundFrontendVariable* Variable = GetFrontendVariable())
	{
		return Variable->Name;
	}

	return { };
}

Metasound::Editor::ENodeSection UMetasoundEditorGraphVariable::GetSectionID() const 
{ 
	return Metasound::Editor::ENodeSection::Variables;
}

FText UMetasoundEditorGraphVariable::GetDescription() const
{
	if (!Breadcrumb.Description.IsEmpty())
	{
		return Breadcrumb.Description;
	}

	if (const FMetasoundFrontendVariable* Variable = GetFrontendVariable())
	{
		return Variable->Description;
	}

	return { };
}

void UMetasoundEditorGraphVariable::SetDescription(const FText& InDescription, bool bPostTransaction)
{
	using namespace Metasound::Editor;

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDescriptionTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName());
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);

	if (UMetasoundEditorGraph* Graph = GetOwningGraph())
	{
		Graph->Modify();
		UObject& MetaSound = Graph->GetMetasoundChecked();
		MetaSound.Modify();

		Graph->GetBuilderChecked().GetBuilder().SetGraphVariableDescription(GetMemberName(), InDescription);
	}
}

bool UMetasoundEditorGraphVariable::CanRename() const
{
	return true;
}

bool UMetasoundEditorGraphVariable::CanRename(const FText& InNewText, FText& OutError) const
{
	using namespace Metasound::Frontend;

	if (InNewText.IsEmptyOrWhitespace())
	{
		OutError = FText::Format(LOCTEXT("GraphVariableRenameInvalid_NameEmpty", "{0} cannot be empty string."), InNewText);
		return false;
	}
	
	const FString NewNameString = InNewText.ToString();
	if (!FName::IsValidXName(NewNameString, INVALID_PARAMETER_NAME_CHARACTERS, &OutError))
	{
		return false;
	}

	if (NewNameString.Len() >= NAME_SIZE)
	{
		OutError = FText::Format(LOCTEXT("GraphVariableRenameInvalid_NameTooLong", "Name cannot be longer than {0} characters."), NAME_SIZE);
		return false;
	}

	const FName NewName(*NewNameString);
	FName Namespace;
	FName ParameterName;
	Audio::FParameterPath::SplitName(NewName, Namespace, ParameterName);

	FConstVariableHandle VariableHandle = GetConstVariableHandle();
	TArray<FConstVariableHandle> Variables = VariableHandle->GetOwningGraph()->GetVariables();
	for (const FConstVariableHandle& OtherVariable : Variables)
	{
		if (VariableID != OtherVariable->GetID())
		{
			FName OtherName = OtherVariable->GetName();
			if (NewName == OtherName)
			{
				OutError = FText::Format(LOCTEXT("GraphVariableRenameInvalid_NameTaken", "{0} is already in use"), InNewText);
				return false;
			}

			if (Namespace == OtherName)
			{
				OutError = FText::Format(LOCTEXT("GraphVariableRenameInvalid_NamespaceTaken", "Namespace of '{0}' cannot be the same as an existing variable's name"), InNewText);
				return false;
			}

			FName OtherNamespace;
			Audio::FParameterPath::SplitName(OtherName, OtherNamespace, OtherName);
			if (OtherNamespace == NewName)
			{
				OutError = FText::Format(LOCTEXT("GraphVariableRenameInvalid_NamespaceTaken2", "Name of '{0}' cannot be the same as an existing variable's namespace"), InNewText);
				return false;
			}
		}
	}

	return true;
}

TArray<UMetasoundEditorGraphMemberNode*> UMetasoundEditorGraphVariable::GetNodes() const
{
	TArray<UMetasoundEditorGraphMemberNode*> Nodes;

	FVariableEditorNodes EditorNodes = GetVariableNodes();
	if (nullptr != EditorNodes.MutatorNode)
	{
		Nodes.Add(EditorNodes.MutatorNode);
	}
	Nodes.Append(EditorNodes.AccessorNodes);
	Nodes.Append(EditorNodes.DeferredAccessorNodes);

	return Nodes;
}

FText UMetasoundEditorGraphVariable::GetDisplayName() const
{
	constexpr bool bIncludeNamespace = true;
	return Metasound::Editor::FGraphBuilder::GetDisplayName(*GetConstVariableHandle(), bIncludeNamespace);
}

void UMetasoundEditorGraphVariable::SetDisplayName(const FText& InNewName, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	const FText TransactionLabel = FText::Format(GraphPrivate::SetMemberDisplayNameTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName(), InNewName);
	const FScopedTransaction Transaction(TransactionLabel, bPostTransaction);
	{
		Graph->Modify();
		Graph->GetMetasoundChecked().Modify();
	}

	GetFrontendBuilderChecked().SetGraphVariableDisplayName(GetMemberName(), InNewName);
}

void UMetasoundEditorGraphVariable::SetDataType(FName InNewType, bool bPostTransaction)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InNewType == GetDataType())
	{
		return;
	}

	UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (!ensure(Graph))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetGraphVariableType", "Set MetaSound GraphVariable Type"), bPostTransaction);
	{
		Graph->GetMetasoundChecked().Modify();
		Graph->Modify();
		Modify();

		// Changing the data type requires that the variable and the associated nodes
		// be removed and readded. Before removing, cache required info to be set after
		// readding. It is assumed that connections are discarded because connections
		// require data types to be equal between to TO and FROM pin. 
		struct FCachedData
		{
			FName MemberName;
			FText DisplayName;
			FText Description;
			FVariableNodeLocations Locations;
		} CachedData;

		const FMetasoundFrontendVariable* OrigVariable = GetFrontendVariable();
		check(OrigVariable);

		// Cache variable metadata
		CachedData.MemberName = OrigVariable->Name;
		CachedData.DisplayName = OrigVariable->DisplayName;
		CachedData.Description = OrigVariable->Description;
		CachedData.Locations = GetVariableNodeLocations();

		// Remove the current variable
		{
			const TArray<UMetasoundEditorGraphMemberNode*> Nodes = GetNodes();
			for (UMetasoundEditorGraphMemberNode* Node : Nodes)
			{
				Graph->RemoveNode(Node);
			}
		}

		FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
		DocBuilder.RemoveGraphVariable(CachedData.MemberName);
		VariableID = FGuid();

		// Add variable with new type to frontend
		const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.AddGraphVariable(
			CachedData.MemberName,
			InNewType,
			nullptr,
			&CachedData.DisplayName,
			&CachedData.Description);

		if (!ensure(FrontendVariable))
		{
			// Failed to add a new variable with the given data type. 
			return;
		}

		// Setup this object with new variable data
		VariableID = FrontendVariable->ID;
		TypeName = InNewType;
		InitializeLiteral();

		{
			FMetasoundFrontendLiteral DefaultLiteral;
			DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(GetDataType()));
			check(Literal); // Should exist from prior InitializeLiteral() call
			Literal->SetFromLiteral(DefaultLiteral);
			Literal->MemberID = VariableID;
		}

		// Add the nodes with the same identifier data but new datatype.
		AddVariableNodes(Graph->GetMetasoundChecked(), CachedData.Locations);
	}

}

UMetasoundEditorGraphVariable::FVariableEditorNodes UMetasoundEditorGraphVariable::GetVariableNodes() const
{
	using namespace Metasound::Frontend;

	FVariableEditorNodes VariableNodes;
	TArray<UMetasoundEditorGraphMemberNode*> AllMetasoundNodes;

	const UMetasoundEditorGraph* Graph = GetOwningGraph();
	if (ensure(Graph))
	{
		Graph->GetNodesOfClassEx<UMetasoundEditorGraphMemberNode>(AllMetasoundNodes);
		FConstVariableHandle FrontendVariable = GetConstVariableHandle();

		// Find the mutator node if it exists.
		{
			FConstNodeHandle FrontendMutatorNode = FrontendVariable->FindMutatorNode();
			if (FrontendMutatorNode->IsValid())
			{
				const FGuid& MutatorNodeID = FrontendMutatorNode->GetID();
				auto IsNodeWithID = [&MutatorNodeID](const UMetasoundEditorGraphMemberNode* InNode)
				{
					return (nullptr != InNode) && (MutatorNodeID == InNode->GetNodeID());
				};

				if (UMetasoundEditorGraphMemberNode** FoundMutatorNode = AllMetasoundNodes.FindByPredicate(IsNodeWithID))
				{
					VariableNodes.MutatorNode = *FoundMutatorNode;
				}
			}
		}

		// Find all accessor nodes
		{
			TSet<FGuid> AccessorNodeIDs;
			for (const FConstNodeHandle& FrontendAccessorNode : FrontendVariable->FindAccessorNodes())
			{
				AccessorNodeIDs.Add(FrontendAccessorNode->GetID());
			}
			auto IsNodeInAccessorSet = [&AccessorNodeIDs](const UMetasoundEditorGraphMemberNode* InNode)
			{
				return (nullptr != InNode) && AccessorNodeIDs.Contains(InNode->GetNodeID());
			};
			VariableNodes.AccessorNodes = AllMetasoundNodes.FilterByPredicate(IsNodeInAccessorSet);
		}

		// Find all deferred accessor nodes
		{
			TSet<FGuid> DeferredAccessorNodeIDs;
			for (const FConstNodeHandle& FrontendAccessorNode : FrontendVariable->FindDeferredAccessorNodes())
			{
				DeferredAccessorNodeIDs.Add(FrontendAccessorNode->GetID());
			}
			auto IsNodeInDeferredAccessorSet = [&DeferredAccessorNodeIDs](const UMetasoundEditorGraphMemberNode* InNode)
			{
				return (nullptr != InNode) && DeferredAccessorNodeIDs.Contains(InNode->GetNodeID());
			};
			VariableNodes.DeferredAccessorNodes = AllMetasoundNodes.FilterByPredicate(IsNodeInDeferredAccessorSet);
		}
	}

	return VariableNodes;

}

UMetasoundEditorGraphVariable::FVariableNodeLocations UMetasoundEditorGraphVariable::GetVariableNodeLocations() const
{
	FVariableNodeLocations Locations;
	// Cache current node positions 
	FVariableEditorNodes EditorNodes = GetVariableNodes();
	auto GetNodeLocation = [](const UMetasoundEditorGraphMemberNode* InNode) { return FVector2D(InNode->NodePosX, InNode->NodePosY); };

	if (nullptr != EditorNodes.MutatorNode)
	{
		Locations.MutatorLocation = GetNodeLocation(EditorNodes.MutatorNode);
	}
	Algo::Transform(EditorNodes.AccessorNodes, Locations.AccessorLocations, GetNodeLocation);
	Algo::Transform(EditorNodes.DeferredAccessorNodes, Locations.DeferredAccessorLocations, GetNodeLocation);

	return Locations;
}

void UMetasoundEditorGraphVariable::AddVariableNodes(UObject& InMetaSound, const FVariableNodeLocations& InNodeLocs)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	auto AddAndSyncEdGraphNode = [&](const FMetasoundFrontendNode* InNode, const FVector2D& Location)
	{
		check(InNode);

		UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddVariableNode(InMetaSound, InNode->GetID(), false /* bInSelectNewNode */);
		check(NewGraphNode);

		NewGraphNode->UpdateFrontendNodeLocation(Location);
		NewGraphNode->SyncLocationFromFrontendNode();
	};

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
	const FMetasoundFrontendVariable* FrontendVariable = GetFrontendVariable();
	check(FrontendVariable);
	if (InNodeLocs.MutatorLocation)
	{
		if (ensure(!FrontendVariable->MutatorNodeID.IsValid()))
		{
			const FMetasoundFrontendNode* NewFrontendNode = DocBuilder.AddGraphVariableMutatorNode(FrontendVariable->Name);
			AddAndSyncEdGraphNode(NewFrontendNode, *InNodeLocs.MutatorLocation);
		}
	}

	for (const FVector2D& Location : InNodeLocs.AccessorLocations)
	{
		const FMetasoundFrontendNode* NewFrontendNode = DocBuilder.AddGraphVariableAccessorNode(FrontendVariable->Name);
		AddAndSyncEdGraphNode(NewFrontendNode, Location);
	}

	for (const FVector2D& Location : InNodeLocs.DeferredAccessorLocations)
	{
		const FMetasoundFrontendNode* NewFrontendNode = DocBuilder.AddGraphVariableDeferredAccessorNode(FrontendVariable->Name);
		AddAndSyncEdGraphNode(NewFrontendNode, Location);
	}
}

const FGuid& UMetasoundEditorGraphVariable::GetVariableID() const
{
	return VariableID;
}

const FMetasoundFrontendVariable* UMetasoundEditorGraphVariable::GetFrontendVariable() const
{
	return GetFrontendBuilderChecked().FindGraphVariable(VariableID);
}

void UMetasoundEditorGraphVariable::ResetToClassDefault()
{
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	FMetasoundFrontendLiteral DefaultLiteral;
	DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(GetDataType()));

	Literal->Modify();
	Literal->SetFromLiteral(DefaultLiteral);

	UObject* MetaSound = Literal->GetOutermostObject();
	check(MetaSound);
	MetaSound->Modify();

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetFrontendBuilderChecked();
	DocBuilder.SetGraphVariableDefault(GetMemberName(), MoveTemp(DefaultLiteral));

	if (const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariable(GetMemberName()); ensure(FrontendVariable))
	{
		if (const FMetasoundFrontendVertex* FrontendVertex = DocBuilder.FindNodeInput(FrontendVariable->MutatorNodeID, METASOUND_GET_PARAM_NAME(InputData)))
		{
			DocBuilder.RemoveNodeInputDefault(FrontendVariable->MutatorNodeID, FrontendVertex->VertexID);
		}
	}
}

bool UMetasoundEditorGraphVariable::Synchronize()
{
	using namespace Metasound;

	bool bModified = Super::Synchronize();
	FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	const FMetasoundFrontendGraph& Graph = Builder.FindConstBuildGraphChecked();
	const FMetasoundFrontendVariable* Variable = GetFrontendVariable();
	if (!Variable)
	{
		return false;
	}

	if (TypeName != Variable->TypeName)
	{
		bModified = true;
		TypeName = Variable->TypeName;

		InitializeLiteral();
	}

	if (ensure(Literal))
	{
		TOptional<FMetasoundFrontendLiteral> NewDefault;
		FMetasoundFrontendLiteral DefaultLiteral;
		Literal->TryFindDefault(DefaultLiteral);
		if (const FMetasoundFrontendNode* MutatorNode = Builder.FindNode(Variable->MutatorNodeID))
		{
			if (!MutatorNode->InputLiterals.IsEmpty())
			{
				const FMetasoundFrontendVertexLiteral& VertexLiteral = MutatorNode->InputLiterals.Last();
				if (!VertexLiteral.Value.IsEqual(DefaultLiteral))
				{
					NewDefault = VertexLiteral.Value;
				}
			}
			else
			{
				FMetasoundFrontendLiteral TypeDefault;
				TypeDefault.SetFromLiteral(Frontend::IDataTypeRegistry::Get().CreateDefaultLiteral(TypeName));
				if (!TypeDefault.IsEqual(DefaultLiteral))
				{
					NewDefault = TypeDefault;
				}
			}
		}
		else if (!Variable->Literal.IsEqual(DefaultLiteral))
		{
			NewDefault = Variable->Literal;
		}

		if (NewDefault.IsSet())
		{
			bModified = true;
			Literal->ResetDefaults();
			Literal->SetFromLiteral(*NewDefault);
		}
	}

	return bModified;
}

void UMetasoundEditorGraphVariable::UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID, EMetaSoundLiteralChangeType InChangeType)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	if (!Literal)
	{
		return;
	}

	const FMetasoundFrontendVariable* Variable = GetFrontendVariable();
	if (!Variable)
	{
		return;
	}

	const FScopedTransaction Transaction(FText::Format(GraphPrivate::SetMemberDefaultTransactionLabelFormat, GetGraphMemberLabel(), GetDisplayName()), bPostTransaction);
	FMetaSoundFrontendDocumentBuilder& Builder = GetFrontendBuilderChecked();
	Builder.CastDocumentObjectChecked<UObject>().Modify();

	// Use the default page ID here as variables do *not* support paged defaults
	// (they, as well as their mutator node which has a matching default, exist in each paged graph).
	FMetasoundFrontendLiteral DefaultLiteral;
	Literal->TryFindDefault(DefaultLiteral);

	// Page ID is passed along to the builder from here because the builder needs the current BuildPageID to access the appropriate in-graph variable
	// (variables can have the same IDs/names in different paged graphs).
	Builder.SetGraphVariableDefault(Variable->Name, DefaultLiteral, InPageID);

	if (const FMetasoundFrontendNode* MutatorNode = Builder.FindNode(Variable->MutatorNodeID, InPageID))
	{
		const FMetasoundFrontendVertex* Input = MutatorNode->Interface.Inputs.FindByPredicate([](const FMetasoundFrontendVertex& Vertex) { return Vertex.Name == METASOUND_GET_PARAM_NAME(InputData); });
		if (ensure(Input))
		{
			const FMetasoundFrontendVertexHandle VertexHandle { MutatorNode->GetID(), Input->VertexID };
			GraphPrivate::SetOrClearIfLiteralMatchesNodeVertexDefault(Builder, VertexHandle, DefaultLiteral);
		}
	}
}

UMetasoundEditorGraphInputNode* UMetasoundEditorGraph::CreateInputNode(Metasound::Frontend::FNodeHandle InNodeHandle, bool bInSelectNewNode)
{
	checkNoEntry();
	return nullptr;
}

Metasound::Frontend::FDocumentHandle UMetasoundEditorGraph::GetDocumentHandle()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetGraphHandle()->GetOwningDocument();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

Metasound::Frontend::FConstDocumentHandle UMetasoundEditorGraph::GetDocumentHandle() const
{
	return GetGraphHandle()->GetOwningDocument();
}

Metasound::Frontend::FGraphHandle UMetasoundEditorGraph::GetGraphHandle() 
{
	FMetasoundAssetBase* MetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&GetMetasoundChecked());
	check(MetasoundAsset);

	return MetasoundAsset->GetRootGraphHandle();
}

Metasound::Frontend::FConstGraphHandle UMetasoundEditorGraph::GetGraphHandle() const
{
	const FMetasoundAssetBase* MetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&GetMetasoundChecked());
	check(MetasoundAsset);

	return MetasoundAsset->GetRootGraphHandle();
}

void UMetasoundEditorGraph::PreSave(FObjectPreSaveContext InSaveContext)
{
	using namespace Metasound::Frontend;

#if WITH_EDITOR
	if (!InSaveContext.IsCooking())
	{
		TArray<UMetasoundEditorGraphNode*> MetaSoundNodes;
		GetNodesOfClass<UMetasoundEditorGraphNode>(MetaSoundNodes);
		for (UMetasoundEditorGraphNode* Node : MetaSoundNodes)
		{
			if (const FMetasoundFrontendNode* FrontendNode = Node->GetFrontendNode())
			{
				FrontendNode->Style.bMessageNodeUpdated = false;
			}
		}
	}
#endif // WITH_EDITOR

	Super::PreSave(InSaveContext);
}

UMetaSoundBuilderBase& UMetasoundEditorGraph::GetBuilderChecked() const
{
	using namespace Metasound::Engine;
	return FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(GetMetasoundChecked());
}

UObject* UMetasoundEditorGraph::GetMetasound() const
{
	return GetOutermostObject();
}

UObject& UMetasoundEditorGraph::GetMetasoundChecked() const
{
	UObject* ParentMetasound = GetMetasound();
	check(ParentMetasound);
	return *ParentMetasound;
}

void UMetasoundEditorGraph::RegisterGraphWithFrontend(Metasound::Frontend::FMetaSoundAssetRegistrationOptions* RegOptions)
{
	using namespace Metasound::Editor;

	if (UObject* ParentMetasound = GetOutermostObject())
	{
		FGraphBuilder::RegisterGraphWithFrontend(*ParentMetasound, RegOptions
			? *RegOptions
			: Metasound::Editor::FGraphBuilder::GetDefaultRegistrationOptions());
	}
}

UMetasoundEditorGraphInput* UMetasoundEditorGraph::FindInput(FGuid InNodeID) const
{
	const TObjectPtr<UMetasoundEditorGraphInput>* Input = Inputs.FindByPredicate([InNodeID](const TObjectPtr<UMetasoundEditorGraphInput>& InInput)
	{
		if (InInput)
		{
			return InInput->NodeID == InNodeID;
		}

		return false;
	});
	return Input ? Input->Get() : nullptr;
}

UMetasoundEditorGraphInput* UMetasoundEditorGraph::FindInput(FName InName) const
{
	const TObjectPtr<UMetasoundEditorGraphInput>* Input = Inputs.FindByPredicate([InName](const TObjectPtr<UMetasoundEditorGraphInput>& InInput)
	{
		if (InInput)
		{
			const FName NodeName = InInput->GetMemberName();
			return NodeName == InName;
		}

		return false;
	});
	return Input ? Input->Get() : nullptr;
}

UMetasoundEditorGraphInput* UMetasoundEditorGraph::FindOrAddInput(const FGuid& InNodeID)
{
	using namespace Metasound;

	if (TObjectPtr<UMetasoundEditorGraphInput> Input = FindInput(InNodeID))
	{
		return Input;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderChecked().GetConstBuilder();
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(InNodeID))
	{
		if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(Node->Name))
		{
			const FMetasoundFrontendLiteral& DefaultLiteral = ClassInput->FindConstDefaultChecked(Frontend::DefaultPageID);
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
			{
				FMetasoundFrontendClassName ClassName = Class->Metadata.GetClassName();

				UMetasoundEditorGraphInput* NewInput = NewObject<UMetasoundEditorGraphInput>(this, Editor::GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
				if (ensure(NewInput))
				{
					NewInput->InitMember(ClassInput->TypeName, DefaultLiteral, InNodeID, MoveTemp(ClassName));
					Inputs.Add(NewInput);
				}

				return NewInput;
			}
		}
	}

	return nullptr;
}

UMetasoundEditorGraphInput* UMetasoundEditorGraph::FindOrAddInput(Metasound::Frontend::FConstNodeHandle InNodeHandle)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FConstGraphHandle Graph = InNodeHandle->GetOwningGraph();

	FName TypeName;
	FGuid VertexID;

	ensure(InNodeHandle->GetNumInputs() == 1);
	InNodeHandle->IterateConstInputs([InGraph = &Graph, InTypeName = &TypeName, InVertexID = &VertexID](FConstInputHandle InputHandle)
	{
		*InTypeName = InputHandle->GetDataType();
		*InVertexID = (*InGraph)->GetVertexIDForInputVertex(InputHandle->GetName());
	});

	const FGuid NodeID = InNodeHandle->GetID();
	if (TObjectPtr<UMetasoundEditorGraphInput> Input = FindInput(NodeID))
	{
		ensure(Input->TypeName == TypeName);
		return Input;
	}

	UMetasoundEditorGraphInput* NewInput = NewObject<UMetasoundEditorGraphInput>(this, GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
	if (ensure(NewInput))
	{
		FMetasoundFrontendLiteral DefaultLiteral = Graph->GetDefaultInput(VertexID);
		FMetasoundFrontendClassName ClassName = InNodeHandle->GetClassMetadata().GetClassName();
		NewInput->InitMember(TypeName, DefaultLiteral, NodeID, MoveTemp(ClassName));
		Inputs.Add(NewInput);

		return NewInput;
	}

	return nullptr;
}

UMetasoundEditorGraphOutput* UMetasoundEditorGraph::FindOutput(FGuid InNodeID) const
{
	const TObjectPtr<UMetasoundEditorGraphOutput>* Output = Outputs.FindByPredicate([InNodeID](const TObjectPtr<UMetasoundEditorGraphOutput>& InOutput)
	{
		if (InOutput)
		{
			return InOutput->NodeID == InNodeID;
		}
		return false;
	});
	return Output ? Output->Get() : nullptr;
}

UMetasoundEditorGraphOutput* UMetasoundEditorGraph::FindOutput(FName InName) const
{
	const TObjectPtr<UMetasoundEditorGraphOutput>* Output = Outputs.FindByPredicate([&InName](const TObjectPtr<UMetasoundEditorGraphOutput>& InOutput)
	{
		if (InOutput)
		{
			return InName == InOutput->GetMemberName();
		}
		return false;
	});
	return Output ? Output->Get() : nullptr;
}

UMetasoundEditorGraphOutput* UMetasoundEditorGraph::FindOrAddOutput(const FGuid& InNodeID)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (TObjectPtr<UMetasoundEditorGraphOutput> Output = FindOutput(InNodeID))
	{
		return Output;
	}

	const FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderChecked().GetConstBuilder();
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(InNodeID))
	{
		if (const FMetasoundFrontendClassOutput* ClassOutput = Builder.FindGraphOutput(Node->Name))
		{
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
			{
				FMetasoundFrontendClassName ClassName = Class->Metadata.GetClassName();

				UMetasoundEditorGraphOutput* NewOutput = NewObject<UMetasoundEditorGraphOutput>(this, GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
				if (ensure(NewOutput))
				{
					FMetasoundFrontendLiteral DefaultLiteral;
					DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(ClassOutput->TypeName));
					NewOutput->InitMember(ClassOutput->TypeName, DefaultLiteral, InNodeID, MoveTemp(ClassName));
					Outputs.Add(NewOutput);
				}

				return NewOutput;
			}
		}
	}

	return nullptr;
}

UMetasoundEditorGraphOutput* UMetasoundEditorGraph::FindOrAddOutput(Metasound::Frontend::FConstNodeHandle InNodeHandle)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FConstGraphHandle Graph = InNodeHandle->GetOwningGraph();

	FName TypeName;
	FGuid VertexID;

	ensure(InNodeHandle->GetNumOutputs() == 1);
	InNodeHandle->IterateConstOutputs([InGraph = &Graph, InTypeName = &TypeName, InVertexID = &VertexID](FConstOutputHandle OutputHandle)
	{
		*InTypeName = OutputHandle->GetDataType();
		*InVertexID = (*InGraph)->GetVertexIDForInputVertex(OutputHandle->GetName());
	});

	const FGuid NodeID = InNodeHandle->GetID();
	if (TObjectPtr<UMetasoundEditorGraphOutput> Output = FindOutput(NodeID))
	{
		ensure(Output->TypeName == TypeName);
		return Output;
	}

	UMetasoundEditorGraphOutput* NewOutput = NewObject<UMetasoundEditorGraphOutput>(this, GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
	if (ensure(NewOutput))
	{
		FMetasoundFrontendLiteral DefaultLiteral;
		DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(TypeName));

		FMetasoundFrontendClassName ClassName = InNodeHandle->GetClassMetadata().GetClassName();
		NewOutput->InitMember(TypeName, DefaultLiteral, NodeID, MoveTemp(ClassName));
		Outputs.Add(NewOutput);

		return NewOutput;
	}

	return nullptr;
}

UMetasoundEditorGraphVariable* UMetasoundEditorGraph::FindVariable(const FGuid& InVariableID) const
{
	const TObjectPtr<UMetasoundEditorGraphVariable>* Variable = Variables.FindByPredicate([&InVariableID](const TObjectPtr<UMetasoundEditorGraphVariable>& InVariable)
	{
		if (InVariable)
		{
			return InVariable->GetVariableID() == InVariableID;
		}

		return false;
	});
	return Variable ? Variable->Get() : nullptr;
}

UMetasoundEditorGraphVariable* UMetasoundEditorGraph::FindOrAddVariable(FName VariableName)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendVariable* Variable = GetBuilderChecked().GetBuilder().FindGraphVariable(VariableName))
	{
		if (TObjectPtr<UMetasoundEditorGraphVariable> EditorVariable = FindVariable(Variable->ID))
		{
			ensure(EditorVariable->TypeName == Variable->TypeName);
			return EditorVariable;
		}

		UMetasoundEditorGraphVariable* NewVariable = NewObject<UMetasoundEditorGraphVariable>(this, GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
		if (ensure(NewVariable))
		{
			NewVariable->InitMember(Variable->TypeName, Variable->Literal, Variable->ID);
			Variables.Add(NewVariable);
			return NewVariable;
		}
	}

	return nullptr;
}

UMetasoundEditorGraphVariable* UMetasoundEditorGraph::FindOrAddVariable(const Metasound::Frontend::FConstVariableHandle& InVariableHandle)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FName TypeName = InVariableHandle->GetDataType();
	const FGuid VariableID = InVariableHandle->GetID();

	if (TObjectPtr<UMetasoundEditorGraphVariable> EditorVariable = FindVariable(VariableID))
	{
		ensure(EditorVariable->TypeName == TypeName);
		return EditorVariable;
	}

	UMetasoundEditorGraphVariable* NewVariable = NewObject<UMetasoundEditorGraphVariable>(this, GraphPrivate::GetUniqueTransientMemberName(), RF_Transactional);
	if (ensure(NewVariable))
	{
		const FMetasoundFrontendLiteral DefaultLiteral = InVariableHandle->GetLiteral();
		NewVariable->InitMember(InVariableHandle->GetDataType(), DefaultLiteral, VariableID);
		Variables.Add(NewVariable);
		return NewVariable;
	}

	return nullptr;
}

UMetasoundEditorGraphMember* UMetasoundEditorGraph::FindMember(FGuid InMemberID) const
{
	if (UMetasoundEditorGraphOutput* Output = FindOutput(InMemberID))
	{
		return Output;
	}

	if (UMetasoundEditorGraphInput* Input = FindInput(InMemberID))
	{
		return Input;
	}

	return FindVariable(InMemberID);
}

UMetasoundEditorGraphMember* UMetasoundEditorGraph::FindAdjacentMember(const UMetasoundEditorGraphMember& InMember)
{
	auto CheckPredicate = [&InMember](const TObjectPtr<UMetasoundEditorGraphMember>& InGraphMember)
	{
		return &InMember == ToRawPtr(InGraphMember);
	};

	// Input
	int32 IndexInArray = Inputs.IndexOfByPredicate(CheckPredicate);

	if (INDEX_NONE != IndexInArray)
	{
		if (TObjectPtr<UMetasoundEditorGraphInput> Input = FindAdjacentMemberFromSorted<UMetasoundEditorGraphInput>(Inputs, CheckPredicate))
		{
			return Input;
		}
		else if (Outputs.Num() > 0)
		{
			return Outputs[0];
		}
		else if (Variables.Num() > 0)
		{
			return Variables[0];
		}

		return nullptr;
	}

	// Output
	IndexInArray = Outputs.IndexOfByPredicate(CheckPredicate);

	if (INDEX_NONE != IndexInArray)
	{
		if (TObjectPtr<UMetasoundEditorGraphOutput> Output = FindAdjacentMemberFromSorted<UMetasoundEditorGraphOutput>(Outputs, CheckPredicate))
		{
			return Output;
		}
		else if (Inputs.Num() > 0)
		{
			return Inputs.Last();
		}
		else if (Variables.Num() > 0)
		{
			return Variables[0];
		}

		return nullptr;
	}

	// Variable
	IndexInArray = Variables.IndexOfByPredicate(CheckPredicate);

	if (INDEX_NONE != IndexInArray)
	{
		if (TObjectPtr<UMetasoundEditorGraphVariable> Variable = FindAdjacentMemberFromSorted<UMetasoundEditorGraphVariable>(Variables, CheckPredicate))
		{
			return Variable;
		}
		else if (Outputs.Num() > 0)
		{
			return Outputs.Last();
		}
		else if (Inputs.Num() > 0)
		{
			return Inputs.Last();
		}

		return nullptr;
	}

	return nullptr;
}

bool UMetasoundEditorGraph::ContainsInput(const UMetasoundEditorGraphInput& InInput) const
{
	return Inputs.Contains(&InInput);
}

bool UMetasoundEditorGraph::ContainsOutput(const UMetasoundEditorGraphOutput& InOutput) const
{
	return Outputs.Contains(&InOutput);
}

bool UMetasoundEditorGraph::ContainsVariable(const UMetasoundEditorGraphVariable& InVariable) const
{
	return Variables.Contains(&InVariable);
}

void UMetasoundEditorGraph::MigrateEditorDocumentData(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	// 1. Add node comments to style for non-comment nodes (comment nodes processed separately below)
	TArray<UMetasoundEditorGraphNode*> AllMetaSoundNodes;
	GetNodesOfClass(AllMetaSoundNodes);

	constexpr bool bIsVisible = true;
	for (UMetasoundEditorGraphNode* Node : AllMetaSoundNodes)
	{
		// Comment nodes are migrated in the next loop
		if (!Node->IsA<UEdGraphNode_Comment>() && !Node->NodeComment.IsEmpty())
		{
			const FGuid& NodeID = Node->GetNodeID();
			if (NodeID.IsValid())
			{
				OutBuilder.SetNodeComment(NodeID, MoveTemp(Node->NodeComment));
				OutBuilder.SetNodeCommentVisible(NodeID, Node->bCommentBubblePinned);
			}
		}
	}

	const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FInputNodeTemplate::GetRegistryKey());
	checkf(Template, TEXT("Failed to find InputNodeTemplate, which is required for migrating editor document data"));

	// 2. Move inputs to input template nodes, using connection data within the ed graph as a way to inform
	// which template node should effectively represent which input template node and own which connections.
	// Cache literals in the literal metadata map to ensure data is serialized appropriately.
	IterateInputs([this, &OutBuilder, Template](UMetasoundEditorGraphInput& Input)
	{
#if WITH_EDITOR
		if (GEditor) // Have to check if valid as its unavailable in standalone editor builds`
		{
			if (UMetaSoundEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>())
			{
				if (UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Input.GetLiteral())
				{
					TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> SubClass = DefaultLiteral->GetClass();
					EditorSubsystem->BindMemberMetadata(OutBuilder, Input, SubClass, DefaultLiteral);
				}
			}
		}
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Presets get rebuilt anyway and may have invalid connections (newly referenced vertices)
		// that need to be rebuilt later in asset load.
		if (OutBuilder.GetConstDocumentChecked().RootGraph.PresetOptions.bIsPreset)
		{
			return;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Cache data to be used in edge swapping below, being careful to not reference the input node pointer
		// or vertex reference within the lower inner loop as the underlying node array may be reallocated by
		// template nodes being added.
		const FMetasoundFrontendNode* InputNode = OutBuilder.FindNode(Input.NodeID);

		// Potentially not used input, which is perfectly valid so early out
		if (!InputNode)
		{
			return;
		}

		const FName InputName = InputNode->Name;
		if (InputNode->Interface.Outputs.IsEmpty())
		{
			UE_LOGF(LogMetaSound, Error, "Failed to migrate input node '%ls': Input has no outputs assigned on interface.", *InputName.ToString());
			return;
		}

		const FMetasoundFrontendVertex InputNodeOutputVertex = InputNode->Interface.Outputs.Last();

		const FGuid InputNodeID = InputNode->GetID();
		FMetasoundFrontendEdge EdgeToRemove { InputNodeID, InputNodeOutputVertex.VertexID };

		TArray<UMetasoundEditorGraphMemberNode*> Nodes = Input.GetNodes();
		for (UMetasoundEditorGraphMemberNode* EdNode : Nodes)
		{
			FNodeTemplateGenerateInterfaceParams Params { { InputNodeOutputVertex.TypeName }, { } };
			const FMetasoundFrontendNode* TemplateNode = OutBuilder.AddNodeByTemplate(*Template, MoveTemp(Params));
			check(TemplateNode);

			const FGuid& TemplateNodeID = TemplateNode->GetID();
			const FGuid& TemplateInputID = TemplateNode->Interface.Inputs.Last().VertexID;
			const FGuid& TemplateOutputID = TemplateNode->Interface.Outputs.Last().VertexID;
			OutBuilder.SetNodeLocation(TemplateNodeID, FVector2D(EdNode->NodePosX, EdNode->NodePosY));

			// Transform comment to template from input node
			OutBuilder.SetNodeComment(TemplateNodeID, MoveTemp(EdNode->NodeComment));
			OutBuilder.SetNodeCommentVisible(TemplateNodeID, EdNode->bCommentBubblePinned);

			// Add edge between input node and new template node
			OutBuilder.AddEdge(FMetasoundFrontendEdge
			{
				InputNodeID,
				InputNodeOutputVertex.VertexID,
				TemplateNodeID,
				TemplateInputID
			});

			for (const UEdGraphPin* Pin : EdNode->Pins)
			{
				if (Pin && Pin->Direction == EGPD_Output)
				{
					for (const UEdGraphPin* Linked : Pin->LinkedTo)
					{
						if (Linked)
						{
							const UMetasoundEditorGraphNode* ConnectedNode = CastChecked<const UMetasoundEditorGraphNode>(Linked->GetOwningNode());
							const FGuid ConnectedNodeID = ConnectedNode->GetNodeID();
							if (const FMetasoundFrontendVertex* ConnectedInput = OutBuilder.FindNodeInput(ConnectedNodeID, Linked->GetFName()))
							{
								// Swap connection from input node to connected node to now be from template node to connected node
								EdgeToRemove.ToNodeID = ConnectedNodeID,
								EdgeToRemove.ToVertexID = ConnectedInput->VertexID;
								bool bEdgeRemoved = OutBuilder.RemoveEdge(EdgeToRemove);
								if (!bEdgeRemoved)
								{
									UE_LOGF(LogMetaSound, Display, "Editor graph '%ls' migration failed to remove exact node '%ls' connection to class output '%ls': Removing any existing connections.",
										*ConnectedNode->GetDisplayName().ToString(),
										*Linked->GetName(),
										*InputName.ToString());
									bEdgeRemoved = OutBuilder.RemoveEdgeToNodeInput(ConnectedNodeID, ConnectedInput->VertexID);
								}

								if (bEdgeRemoved)
								{
									OutBuilder.AddEdge(FMetasoundFrontendEdge
									{
										TemplateNodeID,
										TemplateOutputID,
										ConnectedNodeID,
										ConnectedInput->VertexID
									});
								}
								else
								{
									UE_LOGF(LogMetaSound, Display, "Editor graph '%ls' migration failed to remove connected for node '%ls' class output '%ls': Ignoring connection upgrade from input '%ls' ",
										*OutBuilder.GetDebugName(),
										*ConnectedNode->GetDisplayName().ToString(),
										*Linked->GetName(),
										*InputName.ToString());
								}
							}
							else
							{
								UE_LOGF(LogMetaSound, Display, "Editor graph '%ls' migration failed to find node '%ls' class output '%ls': Ignoring connection upgrade from input '%ls'",
									*OutBuilder.GetDebugName(),
									*ConnectedNode->GetDisplayName().ToString(),
									*Linked->GetName(),
									*InputName.ToString());
							}
						}
					}
				}
			}
		}
	});

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// 4. Add comment nodes as builder graph comments to frontend document
	// (No need to propagate comments for presets)
	if (!OutBuilder.GetConstDocumentChecked().RootGraph.PresetOptions.bIsPreset)
	{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		TArray<UEdGraphNode_Comment*> CommentNodes;
		GetNodesOfClass(CommentNodes);
		for (UEdGraphNode_Comment* Node : CommentNodes)
		{
			FMetaSoundFrontendGraphComment& NewComment = OutBuilder.FindOrAddGraphComment(FGuid::NewGuid());
			UMetasoundEditorGraphCommentNode::ConvertToFrontendComment(*Node, NewComment);
		}
	}

	// 5. Remove input locations and ensure that all other nodes only have at most one
	// location represented in the style/editor graph (0 is acceptable as some member
	// node types (eg. variables) may not contain locations and that's ok).
	const TArray<FMetasoundFrontendNode>& GraphNodes = OutBuilder.FindConstBuildGraphChecked().Nodes;
	TMap<FGuid, const UMetasoundEditorGraphNode*> EdNodeMap;
	Algo::Transform(AllMetaSoundNodes, EdNodeMap, [](const UEdGraphNode* Node)
	{
		return TTuple<FGuid, const UMetasoundEditorGraphNode*>(Node->NodeGuid, Cast<const UMetasoundEditorGraphNode>(Node));
	});
	for (const FMetasoundFrontendNode& Node : GraphNodes)
	{
		if (const FMetasoundFrontendClass* Class = OutBuilder.FindDependency(Node.ClassID))
		{
			// Inputs no longer have locational data as input template nodes provide that
			if (Class->Metadata.GetType() == EMetasoundFrontendClassType::Input)
			{
				OutBuilder.RemoveNodeLocation(Node.GetID());
			}
			else
			{
				const TMap<FGuid, FVector2D>& Locations = Node.Style.Display.Locations;
				if (Locations.Num() > 1)
				{
					TPair<FGuid, FVector2D> DefaultLocation;
					for (const TPair<FGuid, FVector2D>& Pair : Locations)
					{
						DefaultLocation = Pair;
						if (const UMetasoundEditorGraphNode* MetaSoundNode = EdNodeMap.FindRef(Pair.Key))
						{
							if (MetaSoundNode->GetNodeID() == Node.GetID())
							{
								break;
							}
						}
					}

					// Remove first in case there are multiple locations and the editor guid may be different
					OutBuilder.RemoveNodeLocation(Node.GetID());
					OutBuilder.SetNodeLocation(Node.GetID(), DefaultLocation.Value, &DefaultLocation.Key);
				}
			}
		}
	}
}

void UMetasoundEditorGraph::SetPreviewID(uint32 InPreviewID)
{
	PreviewID = InPreviewID;
}

bool UMetasoundEditorGraph::IsPreviewing() const
{
	if (GEditor)
	{
		UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent();
		if (!PreviewComponent)
		{
			return false;
		}

		if (!PreviewComponent->IsPlaying())
		{
			return false;
		}

		return PreviewComponent->GetUniqueID() == PreviewID;
	}

	return false;
}

bool UMetasoundEditorGraph::IsEditable() const
{
	return GetGraphHandle()->GetGraphStyle().bIsGraphEditable;
}

void UMetasoundEditorGraph::IterateInputs(TFunctionRef<void(UMetasoundEditorGraphInput&)> InFunction) const
{
	for (UMetasoundEditorGraphInput* Input : Inputs)
	{
		if (Input)
		{
			InFunction(*Input);
		}
	}
}

void UMetasoundEditorGraph::IterateOutputs(TFunctionRef<void(UMetasoundEditorGraphOutput&)> InFunction) const
{
	for (UMetasoundEditorGraphOutput* Output : Outputs)
	{
		if (ensure(Output))
		{
			InFunction(*Output);
		}
	}
}

void UMetasoundEditorGraph::IterateVariables(TFunctionRef<void(UMetasoundEditorGraphVariable&)> InFunction) const
{
	for (UMetasoundEditorGraphVariable* Variable : Variables)
	{
		if (ensure(Variable))
		{
			InFunction(*Variable);
		}
	}
}

void UMetasoundEditorGraph::IterateMembers(TFunctionRef<void(UMetasoundEditorGraphMember&)> InFunction) const
{
	for (UMetasoundEditorGraphInput* Input : Inputs)
	{
		if (Input)
		{
			InFunction(*Input);
		}
	}

	for (UMetasoundEditorGraphOutput* Output : Outputs)
	{
		if (ensure(Output))
		{
			InFunction(*Output);
		}
	}

	for (UMetasoundEditorGraphVariable* Variable : Variables)
	{
		if (ensure(Variable))
		{
			InFunction(*Variable);
		}
	}
}

void UMetasoundEditorGraph::ValidateInternal(Metasound::Editor::FGraphValidationResults& OutResults)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	OutResults = FGraphValidationResults();
	TSet<FGuid> NodeGuids; 
	TArray<UMetasoundEditorGraphNode*> NodesToValidate;
	GetNodesOfClass<UMetasoundEditorGraphNode>(NodesToValidate);
	bool bNodeIdFound = false;
	const FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&GetMetasoundChecked());
	for (UMetasoundEditorGraphNode* Node : NodesToValidate)
	{
		Node->CacheBreadcrumb();
		FGraphNodeValidationResult NodeResult(*Node);

		// Validate there is only 1 editor node per guid 
		// Input nodes are currently not 1:1 with their frontend representation
		// but when they are, they can be validated here as well 
		if (!Node->IsA<UMetasoundEditorGraphInputNode>() && !Node->IsA<UMetasoundEditorGraphVariableNode>())
		{
			NodeGuids.Add(Node->GetNodeID(), &bNodeIdFound);
			if (bNodeIdFound)
			{
				NodeResult.SetMessage(EMessageSeverity::Warning, TEXT("The internal node this represents is referenced multiple times and may have unintended behavior. Please delete and readd this node."));
			}
		}

		Node->Validate(Builder, NodeResult);

		OutResults.NodeResults.Add(MoveTemp(NodeResult));
	}
}
#undef LOCTEXT_NAMESPACE
