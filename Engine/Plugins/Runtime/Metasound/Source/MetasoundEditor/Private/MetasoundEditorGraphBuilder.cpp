// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphBuilder.h"

#include "Algo/AnyOf.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GraphEditor.h"
#include "GraphEditorSettings.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundEditorModule.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFactory.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLiteral.h"
#include "MetasoundSettings.h"
#include "MetasoundTime.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundVariableNodes.h"
#include "MetasoundVertex.h"
#include "MetasoundWaveTable.h"
#include "Modules/ModuleManager.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "Templates/Tuple.h"
#include "Toolkits/ToolkitManager.h"
#include "WaveTable.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound
{
	namespace Editor
	{
		namespace GraphBuilderPrivate
		{
			template <
				typename TEdGraphNode,
				typename FInitMetaSoundNodeFunc = TFunctionRef<void(UMetasoundEditorGraph&, TEdGraphNode&)>>
			TEdGraphNode* AddNode(UObject& InMetaSound, FInitMetaSoundNodeFunc InitNodeFunc, bool bInSelectNewNode)
			{
				FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
				check(MetaSoundAsset);
				UEdGraph& Graph = MetaSoundAsset->GetGraphChecked();
				FGraphNodeCreator<TEdGraphNode> NodeCreator(Graph);
				if (TEdGraphNode* NewGraphNode = NodeCreator.CreateNode(bInSelectNewNode))
				{
					// Required to happen prior to caching title and syncing location in case underlying type requires
					// additional logic to initialize state in order to cache the title/sync the location
					UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(&Graph);
					check(MetasoundGraph);
					InitNodeFunc(*MetasoundGraph, *NewGraphNode);

					NodeCreator.Finalize();

					NewGraphNode->CacheTitle();

					// Override finalized EdGraphNode ID if a location is found.
					constexpr bool bUpdateEditorNodeID = true;
					NewGraphNode->SyncLocationFromFrontendNode(bUpdateEditorNodeID);
					NewGraphNode->SyncCommentFromFrontendNode();

					return NewGraphNode;
				}

				return nullptr;
			}

			void RecurseClearDocumentModified(FMetasoundAssetBase& InAssetBase)
			{
				using namespace Metasound::Frontend;

				InAssetBase.GetModifyContext().ClearDocumentModified();

				TArray<FMetasoundAssetBase*> References;
				ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(InAssetBase, References));
				for (FMetasoundAssetBase* Reference : References)
				{
					check(Reference);
					Reference->GetModifyContext().ClearDocumentModified();
					RecurseClearDocumentModified(*Reference);
				}
			};

			void SynchronizeGraphRecursively(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph, bool bSkipIfModifyContextUnchanged)
			{
				using namespace Engine;
				using namespace Frontend;

				UObject& MetaSound = InBuilder.CastDocumentObjectChecked<UObject>();
				FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
				check(MetaSoundAsset);

				// Synchronize referenced graphs first to ensure all editor data
				// is up-to-date prior to synchronizing this referencing graph.
				TArray<FMetasoundAssetBase*> References;
				ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(*MetaSoundAsset, References));
				for (FMetasoundAssetBase* Reference : References)
				{
					check(Reference);
					UObject* RefObject = Reference->GetOwningAsset();
					check(RefObject);
					FMetaSoundFrontendDocumentBuilder& RefBuilder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(RefObject);

					UMetasoundEditorGraph* Graph = nullptr;
					FGraphBuilder::BindEditorGraph(RefBuilder, &Graph);
					SynchronizeGraphRecursively(RefBuilder, *Graph, bSkipIfModifyContextUnchanged);
				}

				if (bSkipIfModifyContextUnchanged && !MetaSoundAsset->GetConstModifyContext().GetDocumentModified())
				{
					return;
				}

				FGraphBuilder::SynchronizeComments(InBuilder, OutGraph);
				FGraphBuilder::SynchronizeGraphMembers(InBuilder, OutGraph);
				FGraphBuilder::SynchronizeOutputNodes(InBuilder, OutGraph);
				FGraphBuilder::SynchronizeNodes(InBuilder, OutGraph);
				FGraphBuilder::SynchronizeConnections(InBuilder, OutGraph);
			}
		} // namespace GraphBuilderPrivate

		bool FGraphBuilder::CanInspectPin(const UEdGraphPin* InPin)
		{
			// Can't inspect the value on an invalid pin object.
			if (!InPin || InPin->IsPendingKill())
			{
				return false;
			}

			// Can't inspect the value on an orphaned pin object.
			if (InPin->bOrphanedPin)
			{
				return false;
			}

			// Currently only inspection of connected pins is supported.
			if (InPin->LinkedTo.IsEmpty())
			{
				return false;
			}

			// Can't inspect the value on an unknown pin object or if the owning node is disabled.
			const UEdGraphNode* OwningNode = InPin->GetOwningNodeUnchecked();
			if (!OwningNode || !OwningNode->IsNodeEnabled())
			{
				return false;
			}

			TSharedPtr<FEditor> Editor = GetEditorForPin(*InPin);
			if (!Editor.IsValid())
			{
				return false;
			}

			if (!Editor->IsPlaying())
			{
				return false;
			}

			FName DataType;
			if (InPin->Direction == EGPD_Input)
			{
				Frontend::FConstInputHandle InputHandle = GetConstInputHandleFromPin(InPin);
				DataType = InputHandle->GetDataType();
			}
			else
			{
				Frontend::FConstOutputHandle OutputHandle = GetConstOutputHandleFromPin(InPin);
				DataType = OutputHandle->GetDataType();
			}

			const bool bIsSupportedType = DataType == GetMetasoundDataTypeName<float>()
				|| DataType == GetMetasoundDataTypeName<int32>()
				|| DataType == GetMetasoundDataTypeName<FString>()
				|| DataType == GetMetasoundDataTypeName<bool>()
				|| DataType == GetMetasoundDataTypeName<FAudioBuffer>();

			if (!bIsSupportedType)
			{
				return false;
			}

			const UEdGraphPin* ReroutedPin = FindReroutedOutputPin(InPin);
			if (ReroutedPin != InPin)
			{
				return false;
			}

			return true;
		}

		UMetasoundEditorGraphCommentNode* FGraphBuilder::CreateCommentNode(UObject& InMetaSound, bool bInSelectNewNode, FGuid InCommentID)
		{
			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
			check(MetaSoundAsset);
			UEdGraph& Graph = MetaSoundAsset->GetGraphChecked();
			FGraphNodeCreator<UMetasoundEditorGraphCommentNode> NodeCreator(Graph);
			if (UMetasoundEditorGraphCommentNode* NewNode = NodeCreator.CreateNode(bInSelectNewNode))
			{
				NewNode->CommentID = InCommentID;
				NodeCreator.Finalize();
				return NewNode;
			}

			return nullptr;
		}

		FText FGraphBuilder::GetDisplayName(const FMetasoundFrontendClassMetadata& InClassMetadata, FName InNodeName, bool bInIncludeNamespace)
		{
			using namespace Frontend;

			FText DisplayName;
			auto GetAssetDisplayNameFromMetadata = [&DisplayName](const FMetasoundFrontendClassMetadata& Metadata)
			{
				DisplayName = Metadata.GetDisplayName();
				if (DisplayName.IsEmptyOrWhitespace())
				{
					const bool bIsAssetClass = IMetaSoundAssetManager::GetChecked().IsAssetClass(Metadata);
					if (bIsAssetClass)
					{
						const FTopLevelAssetPath Path = IMetaSoundAssetManager::GetChecked().FindAssetPath(FMetaSoundAssetKey(Metadata));
						if (Path.IsValid())
						{
							DisplayName = FText::FromName(Path.GetAssetName());
						}
					}
				}
			};

			// 1. Try to get display name from metadata or asset if one can be found from the asset manager
			GetAssetDisplayNameFromMetadata(InClassMetadata);

			// 2. If version is missing from the registry or from asset system, then this node
			// will not provide a useful DisplayName.  In that case, attempt to find the next highest
			// class & associated DisplayName.
			if (DisplayName.IsEmptyOrWhitespace())
			{
				FMetasoundFrontendClass ClassWithHighestVersion;
				if (ISearchEngine::Get().FindClassWithHighestVersion(InClassMetadata.GetClassName(), ClassWithHighestVersion))
				{
					GetAssetDisplayNameFromMetadata(ClassWithHighestVersion.Metadata);
				}
			}

			if (DisplayName.IsEmptyOrWhitespace() || bInIncludeNamespace)
			{
				FName Namespace;
				FName ParameterName;
				Audio::FParameterPath::SplitName(InNodeName, Namespace, ParameterName);

				// 3. If that cannot be found, build a title from the cached node registry FName.
				if (DisplayName.IsEmptyOrWhitespace())
				{
					DisplayName = FText::FromString(ParameterName.ToString());
				}

				// 4. Tack on the namespace if requested
				if (bInIncludeNamespace)
				{
					if (!Namespace.IsNone())
					{
						return FText::Format(LOCTEXT("ClassMetadataDisplayNameWithNamespaceFormat", "{0} ({1})"), DisplayName, FText::FromName(Namespace));
					}
				}
			}

			return DisplayName;
		}

		FText FGraphBuilder::GetDisplayName(const Frontend::INodeController& InFrontendNode, bool bInIncludeNamespace)
		{
			using namespace Frontend;

			FText DisplayName = InFrontendNode.GetDisplayName();
			if (!DisplayName.IsEmptyOrWhitespace())
			{
				return DisplayName;
			}

			return GetDisplayName(InFrontendNode.GetClassMetadata(), InFrontendNode.GetNodeName(), bInIncludeNamespace);
		}

		FText FGraphBuilder::GetDisplayName(const Frontend::IInputController& InFrontendInput)
		{
			FText DisplayName = InFrontendInput.GetDisplayName();
			if (DisplayName.IsEmptyOrWhitespace())
			{
				DisplayName = FText::FromName(InFrontendInput.GetName());
			}
			return DisplayName;
		}

		FText FGraphBuilder::GetDisplayName(const Frontend::IOutputController& InFrontendOutput)
		{
			FText DisplayName = InFrontendOutput.GetDisplayName();
			if (DisplayName.IsEmptyOrWhitespace())
			{
				DisplayName = FText::FromName(InFrontendOutput.GetName());
			}
			return DisplayName;
		}

		FText FGraphBuilder::GetDisplayName(const Frontend::IVariableController& InFrontendVariable, bool bInIncludeNamespace)
		{
			FText DisplayName = InFrontendVariable.GetDisplayName();
			if (DisplayName.IsEmptyOrWhitespace())
			{
				FName Namespace;
				FName ParameterName;
				Audio::FParameterPath::SplitName(InFrontendVariable.GetName(), Namespace, ParameterName);

				DisplayName = FText::FromName(ParameterName);
				if (bInIncludeNamespace && !Namespace.IsNone())
				{
					return FText::Format(LOCTEXT("ClassMetadataDisplayNameWithNamespaceFormat", "{0} ({1})"), DisplayName, FText::FromName(Namespace));
				}
			}

			return DisplayName;
		}

		UMetasoundEditorGraphExternalNode* FGraphBuilder::AddTemplateNode(UObject& InMetaSound, const FGuid& InNodeID, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode)
		{
			const EMetasoundFrontendClassType ClassType = InMetadata.GetType();
			if (ensureMsgf(ClassType == EMetasoundFrontendClassType::Template, TEXT("Cannot call 'AddTemplateNode' with node of class type '%s'."), LexToString(ClassType)))
			{
				auto InitNodeFunc = [&InMetadata, &InNodeID](UMetasoundEditorGraph&, UMetasoundEditorGraphExternalNode& NewGraphNode)
				{
					NewGraphNode.NodeID = InNodeID;
					NewGraphNode.ClassName = InMetadata.GetClassName();
				};
				return GraphBuilderPrivate::AddNode<UMetasoundEditorGraphExternalNode>(InMetaSound, InitNodeFunc, bInSelectNewNode);
			}

			return nullptr;
		}

		UMetasoundEditorGraphExternalNode* FGraphBuilder::AddExternalNode(UObject& InMetaSound, const FGuid& InNodeID, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode)
		{
			using namespace Frontend;

			const EMetasoundFrontendClassType ClassType = InMetadata.GetType();
			if (ensureMsgf(ClassType == EMetasoundFrontendClassType::External, TEXT("Cannot call 'AddExternalNode' with node of class type '%s'."), LexToString(ClassType)))
			{
				auto InitNodeFunc = [&InMetadata, &InNodeID](UMetasoundEditorGraph& MetasoundGraph, UMetasoundEditorGraphExternalNode& NewGraphNode)
				{
					const bool bIsAssetClass = IMetaSoundAssetManager::GetChecked().IsAssetClass(InMetadata);
					NewGraphNode.bIsClassNative = !bIsAssetClass;
					NewGraphNode.NodeID = InNodeID;
					NewGraphNode.ClassName = InMetadata.GetClassName();
				};

				return GraphBuilderPrivate::AddNode<UMetasoundEditorGraphExternalNode>(InMetaSound, InitNodeFunc, bInSelectNewNode);
			}

			return nullptr;
		}

		UMetasoundEditorGraphExternalNode* FGraphBuilder::AddExternalNode(UObject& InMetaSound, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode)
		{
			using namespace Frontend;
			using namespace Engine;

			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(InMetaSound);
			EMetaSoundBuilderResult Result;
			const FMetaSoundNodeHandle NewNodeHandle = Builder.AddNodeByClassName(InMetadata.GetClassName(), Result, InMetadata.GetVersion().Major);
			if (Result == EMetaSoundBuilderResult::Succeeded && NewNodeHandle.IsSet())
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetConstBuilder();
				const FMetasoundFrontendNode* NewNode = DocBuilder.FindNode(NewNodeHandle.NodeID);
				check(NewNode);
				const FMetasoundFrontendClass* Dependency = DocBuilder.FindDependency(NewNode->ClassID);
				if (ensure(Dependency))
				{
					FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
					check(MetaSoundAsset);
					MetaSoundAsset->GetModifyContext().AddNodeIDModified(NewNode->GetID());
					return AddExternalNode(InMetaSound, NewNode->GetID(), Dependency->Metadata, bInSelectNewNode);
				}
			}

			return nullptr;
		}

		Frontend::FNodeHandle FGraphBuilder::AddExternalNodeHandle(UObject& InMetaSound, const FMetasoundFrontendClassName& InClassName)
		{
			using namespace Frontend;

			FMetasoundFrontendClass FrontendClass;
			bool bDidFindClassWithName = ISearchEngine::Get().FindClassWithHighestVersion(InClassName, FrontendClass);
			if (ensure(bDidFindClassWithName))
			{
				FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
				check(MetaSoundAsset);
				return MetaSoundAsset->GetRootGraphHandle()->AddNode(FrontendClass.Metadata);
			}

			return INodeController::GetInvalidHandle();
		}

		UMetasoundEditorGraphVariableNode* FGraphBuilder::AddVariableNode(UObject& InMetaSound, const FGuid& InNodeID, bool bInSelectNewNode)
		{
			using namespace Frontend;

			FMetaSoundFrontendDocumentBuilder& Builder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&InMetaSound);

			const FMetasoundFrontendNode* FrontendNode = Builder.FindNode(InNodeID);
			check(FrontendNode);

			const FMetasoundFrontendClass* Class = Builder.FindDependency(FrontendNode->ClassID);
			check(Class);

			const FMetasoundFrontendClassMetadata& Metadata = Class->Metadata;
			EMetasoundFrontendClassType ClassType = Metadata.GetType();
			const bool bIsSupportedClassType = (ClassType == EMetasoundFrontendClassType::VariableAccessor)
				|| (ClassType == EMetasoundFrontendClassType::VariableDeferredAccessor)
				|| (ClassType == EMetasoundFrontendClassType::VariableMutator);

			if (ensureMsgf(bIsSupportedClassType, TEXT("Cannot call 'AddVariableNode' with node of class type '%s'"), LexToString(ClassType)))
			{
				const FMetasoundFrontendVariable* FrontendVariable = Builder.FindGraphVariableByNodeID(InNodeID);
				if (ensure(FrontendVariable))
				{
					auto InitNodeFunc = [&FrontendVariable, &InNodeID, &Metadata](UMetasoundEditorGraph& MetasoundGraph, UMetasoundEditorGraphVariableNode& NewGraphNode)
					{
						UMetasoundEditorGraphVariable* Variable = MetasoundGraph.FindOrAddVariable(FrontendVariable->Name);
						if (ensure(Variable))
						{
							NewGraphNode.Variable = Variable;
							NewGraphNode.NodeID = InNodeID;
							NewGraphNode.ClassName = Metadata.GetClassName();
							NewGraphNode.ClassType = Metadata.GetType();
						}
					};

					return GraphBuilderPrivate::AddNode<UMetasoundEditorGraphVariableNode>(InMetaSound, InitNodeFunc, bInSelectNewNode);
				}
			}

			return nullptr;
		}

		UMetasoundEditorGraphInputNode* FGraphBuilder::AddInputNode(UObject& InMetaSound, const FGuid& InTemplateNodeID, bool bInSelectNewNode)
		{
			using namespace Frontend;

			using namespace Engine;
			using namespace Frontend;

			FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&InMetaSound);
			if (const FMetasoundFrontendNode* TemplateNode = Builder.FindNode(InTemplateNodeID))
			{
				if (const FMetasoundFrontendClass* Class = Builder.FindDependency(TemplateNode->ClassID))
				{
					const EMetasoundFrontendClassType& ClassType = Class->Metadata.GetType();
					if (ensureMsgf(ClassType == EMetasoundFrontendClassType::Template, TEXT("Cannot call 'AddInputNode' with node of class type '%s': Must be input template."), LexToString(ClassType)))
					{
						if (ensureMsgf(Class->Metadata.GetClassName() == FInputNodeTemplate::ClassName, TEXT("Cannot call 'AddInputNode with node handle that is not of class '%s'"), *FInputNodeTemplate::ClassName.ToString()))
						{
							const FGuid& TemplateNodeInputVertexID = TemplateNode->Interface.Inputs.Last().VertexID;
							auto InitNodeFunc = [&Builder, &TemplateNodeInputVertexID, &InTemplateNodeID](UMetasoundEditorGraph& MetasoundGraph, UMetasoundEditorGraphInputNode& NewGraphNode)
							{
								const FMetasoundFrontendNode* ConnectedInputNode = nullptr;
								Builder.FindNodeOutputConnectedToNodeInput(InTemplateNodeID, TemplateNodeInputVertexID, &ConnectedInputNode);
								if (ensureMsgf(ConnectedInputNode, TEXT("Failed to find required input connected to template node")))
								{
									UMetasoundEditorGraphInput* Input = MetasoundGraph.FindOrAddInput(ConnectedInputNode->GetID());
									if (ensure(Input))
									{
										NewGraphNode.Input = Input;
										NewGraphNode.NodeID = InTemplateNodeID;
									}
								}
							};

							return GraphBuilderPrivate::AddNode<UMetasoundEditorGraphInputNode>(InMetaSound, InitNodeFunc, bInSelectNewNode);
						}
					}
				}
			}

			return nullptr;
		}

		UMetasoundEditorGraphOutputNode* FGraphBuilder::AddOutputNode(UObject& InMetaSound, const FGuid& InNodeID, bool bInSelectNewNode)
		{
			using namespace Engine;
			using namespace Frontend;

			FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&InMetaSound);
			if (const FMetasoundFrontendNode* Node = Builder.FindNode(InNodeID))
			{
				if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
				{
					const EMetasoundFrontendClassType& ClassType = Class->Metadata.GetType();
					if (ensureMsgf(ClassType == EMetasoundFrontendClassType::Output, TEXT("Cannot call 'AddOutputNode' with node of class type '%s'"), LexToString(ClassType)))
					{
						auto InitNodeFunc = [&InNodeID](UMetasoundEditorGraph& MetasoundGraph, UMetasoundEditorGraphOutputNode& NewGraphNode)
						{
							UMetasoundEditorGraphOutput* Output = MetasoundGraph.FindOrAddOutput(InNodeID);
							if (ensure(Output))
							{
								NewGraphNode.Output = Output;
							}
						};

						Builder.GetConstDocumentChecked().Metadata.ModifyContext.AddNodeIDModified(InNodeID);
						return GraphBuilderPrivate::AddNode<UMetasoundEditorGraphOutputNode>(InMetaSound, InitNodeFunc, bInSelectNewNode);
					}
				}
			}

			return nullptr;
		}

		FGraphValidationResults FGraphBuilder::ValidateGraph(UObject& InMetaSound)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphBuilder::ValidateGraph);

			using namespace Frontend;

			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
			check(MetaSoundAsset);

			// Validate referenced graphs first to ensure all editor data
			// is up-to-date prior to validating this referencing graph to
			// allow errors to bubble up.
			TArray<FMetasoundAssetBase*> References;
			ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(*MetaSoundAsset, References));
			for (FMetasoundAssetBase* Reference : References)
			{
				check(Reference);
				ValidateGraph(*Reference->GetOwningAsset());
			}

			FGraphValidationResults Results;
			UMetasoundEditorGraph& Graph = *CastChecked<UMetasoundEditorGraph>(&MetaSoundAsset->GetGraphChecked());
			Graph.ValidateInternal(Results);
			return Results;
		}

		UMetaSoundBuilderBase& FGraphBuilder::GetBuilderFromPinChecked(const UEdGraphPin& InPin)
		{
			const UMetasoundEditorGraphNode* Node = CastChecked<UMetasoundEditorGraphNode>(InPin.GetOwningNode());
			check(Node);
			return Node->GetBuilderChecked();
		}

		TArray<FString> FGraphBuilder::GetDataTypeNameCategories(const FName& InDataTypeName)
		{
			FString CategoryString = InDataTypeName.ToString();

			TArray<FString> Categories;
			CategoryString.ParseIntoArray(Categories, TEXT(":"));

			if (Categories.Num() > 0)
			{
				// Remove name
				Categories.RemoveAt(Categories.Num() - 1);
			}

			return Categories;
		}

		TSharedPtr<FEditor> FGraphBuilder::GetEditorForMetasound(const UObject& Metasound)
		{
			// TODO: FToolkitManager is deprecated. Replace with UAssetEditorSubsystem.
			if (TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(&Metasound))
			{
				if (FEditor::EditorName == FoundAssetEditor->GetToolkitFName())
				{
					return StaticCastSharedPtr<FEditor, IToolkit>(FoundAssetEditor);
				}
			}

			return { };
		}

		TSharedPtr<FEditor> FGraphBuilder::GetEditorForGraph(const UEdGraph& EdGraph)
		{
			if (const UMetasoundEditorGraph* MetasoundGraph = Cast<const UMetasoundEditorGraph>(&EdGraph))
			{
				return GetEditorForMetasound(MetasoundGraph->GetMetasoundChecked());
			}

			return { };
		}

		TSharedPtr<FEditor> FGraphBuilder::GetEditorForNode(const UEdGraphNode& InEdNode)
		{
			if (const UMetasoundEditorGraph* Graph = Cast<UMetasoundEditorGraph>(InEdNode.GetGraph()))
			{
				return FGraphBuilder::GetEditorForGraph(*Graph);
			}

			return { };
		}

		TSharedPtr<FEditor> FGraphBuilder::GetEditorForPin(const UEdGraphPin& InEdPin)
		{
			if (const UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(InEdPin.GetOwningNode()))
			{
				return GetEditorForNode(*Node);
			}

			return { };
		}

		Frontend::FInputHandle FGraphBuilder::GetInputHandleFromPin(const UEdGraphPin* InPin)
		{
			using namespace Frontend;
			using namespace VariableNames;

			if (InPin && ensure(InPin->Direction == EGPD_Input))
			{
				if (UMetasoundEditorGraphVariableNode* EdVariableNode = Cast<UMetasoundEditorGraphVariableNode>(InPin->GetOwningNode()))
				{
					// UEdGraphPins on variable nodes use the variable's name for display
					// purposes instead of the underlying vertex's name. The frontend vertices
					// of a variable node have consistent names no matter what the 
					// variable is named.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return EdVariableNode->GetNodeHandle()->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputData));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else if (UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return EdNode->GetNodeHandle()->GetInputWithVertexName(InPin->GetFName());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}

			return IInputController::GetInvalidHandle();
		}

		Frontend::FConstInputHandle FGraphBuilder::GetConstInputHandleFromPin(const UEdGraphPin* InPin)
		{
			using namespace Frontend;
			using namespace VariableNames;

			if (InPin && ensure(InPin->Direction == EGPD_Input))
			{
				if (const UMetasoundEditorGraphVariableNode* EdVariableNode = Cast<UMetasoundEditorGraphVariableNode>(InPin->GetOwningNode()))
				{
					// UEdGraphPins on variable nodes use the variable's name for display
					// purposes instead of the underlying vertex's name. The frontend vertices
					// of a variable node have consistent names no matter what the 
					// variable is named.
					return EdVariableNode->GetConstNodeHandle()->GetConstInputWithVertexName(METASOUND_GET_PARAM_NAME(InputData));
				}
				else if (const UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
				{
					return EdNode->GetConstNodeHandle()->GetConstInputWithVertexName(InPin->GetFName());
				}
			}

			return IInputController::GetInvalidHandle();
		}

		FName FGraphBuilder::GetPinDataType(const UEdGraphPin* InPin)
		{
			using namespace Frontend;

			if (InPin)
			{
				if (InPin->Direction == EGPD_Input)
				{
					FConstInputHandle InputHandle = GetConstInputHandleFromPin(InPin);
					return InputHandle->GetDataType();
				}
				else // EGPD_Output
				{
					FConstOutputHandle OutputHandle = GetConstOutputHandleFromPin(InPin);
					return OutputHandle->GetDataType();
				}
			}

			return { };
		}

		FMetasoundFrontendVertexHandle FGraphBuilder::GetPinVertexHandle(const FMetaSoundFrontendDocumentBuilder& InBuilder, const UEdGraphPin* InPin)
		{
			using namespace VariableNames;

			if (!InPin)
			{
				return { };
			}

			const UMetasoundEditorGraphNode* OwningNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode());

			const FGuid NodeID = OwningNode->GetNodeID();
			const FMetasoundFrontendNode* Node = InBuilder.FindNode(NodeID);
			if (!Node)
			{
				return { };
			}

			const FMetasoundFrontendClass* Class = InBuilder.FindDependency(Node->ClassID);
			if (!Class)
			{
				return { };
			}

			const FMetasoundFrontendVertex* Vertex = nullptr;
			switch (Class->Metadata.GetType())
			{
				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					// All variables nodes use the same pin name for user-modifiable node
					// inputs and outputs and the editor does not display the pin's name. The
					// editor instead displays the variable's name in place of the pin name to
					// maintain a consistent look and behavior to input and output nodes.
					Vertex = InPin->Direction == EGPD_Input
						? InBuilder.FindNodeInput(NodeID, METASOUND_GET_PARAM_NAME(InputData))
						: InBuilder.FindNodeOutput(NodeID, METASOUND_GET_PARAM_NAME(OutputData));
					break;
				}

				case EMetasoundFrontendClassType::Input:
				{
					Vertex = &Node->Interface.Outputs.Last();
					break;
				}

				case EMetasoundFrontendClassType::Output:
				{
					Vertex = &Node->Interface.Inputs.Last();
					break;
				}

				default:
				{
					Vertex = InPin->Direction == EGPD_Input
						? InBuilder.FindNodeInput(NodeID, InPin->GetFName())
						: InBuilder.FindNodeOutput(NodeID, InPin->GetFName());
				}
			}

			FMetasoundFrontendVertexHandle VertexHandle { NodeID };
			if (Vertex)
			{
				VertexHandle.VertexID = Vertex->VertexID;
			}
			return VertexHandle;
		}

		const FMetasoundFrontendVertex* FGraphBuilder::GetPinVertex(const FMetaSoundFrontendDocumentBuilder& InBuilder, const UEdGraphPin* InPin, const FMetasoundFrontendNode** Node)
		{
			using namespace VariableNames;

			if (Node)
			{
				*Node = nullptr;
			}

			if (!InPin)
			{
				return nullptr;
			}

			const UMetasoundEditorGraphNode* OwningNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode());

			const FGuid NodeID = OwningNode->GetNodeID();
			const FMetasoundFrontendNode* FoundNode = InBuilder.FindNode(NodeID);
			if (!FoundNode)
			{
				return nullptr;
			}

			if (Node)
			{
				*Node = FoundNode;
			}

			const FMetasoundFrontendClass* Class = InBuilder.FindDependency(FoundNode->ClassID);
			if (!Class)
			{
				return nullptr;
			}

			switch (Class->Metadata.GetType())
			{
				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					// All variables nodes use the same pin name for user-modifiable node
					// inputs and outputs and the editor does not display the pin's name. The
					// editor instead displays the variable's name in place of the pin name to
					// maintain a consistent look and behavior to input and output nodes.
					return InPin->Direction == EGPD_Input
						? InBuilder.FindNodeInput(NodeID, METASOUND_GET_PARAM_NAME(InputData))
						: InBuilder.FindNodeOutput(NodeID, METASOUND_GET_PARAM_NAME(OutputData));
				}
				case EMetasoundFrontendClassType::Input:
				{
					ensureMsgf(InPin->Direction == EGPD_Output, TEXT("Querying for hidden input node output vertex, which should never be represented on an editor graph."));
					return &FoundNode->Interface.Outputs.Last();
				}
				case EMetasoundFrontendClassType::Output:
				{
					ensureMsgf(InPin->Direction == EGPD_Input, TEXT("Querying for hidden output node input vertex, which should never be represented on an editor graph."));
					return &FoundNode->Interface.Inputs.Last();
				}

				default:
				{
					return InPin->Direction == EGPD_Input
						? InBuilder.FindNodeInput(NodeID, InPin->GetFName())
						: InBuilder.FindNodeOutput(NodeID, InPin->GetFName());
				}
			}
		}

		Frontend::FOutputHandle FGraphBuilder::GetOutputHandleFromPin(const UEdGraphPin* InPin)
		{
			using namespace Frontend;
			using namespace VariableNames; 

			if (InPin && ensure(InPin->Direction == EGPD_Output))
			{
				if (UMetasoundEditorGraphVariableNode* EdVariableNode = Cast<UMetasoundEditorGraphVariableNode>(InPin->GetOwningNode()))
				{
					// UEdGraphPins on variable nodes use the variable's name for display
					// purposes instead of the underlying vertex's name. The frontend vertices
					// of a variable node have consistent names no matter what the 
					// variable is named.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return EdVariableNode->GetNodeHandle()->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputData));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else if (UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return EdNode->GetNodeHandle()->GetOutputWithVertexName(InPin->GetFName());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}

			return IOutputController::GetInvalidHandle();
		}

		Frontend::FConstOutputHandle FGraphBuilder::GetConstOutputHandleFromPin(const UEdGraphPin* InPin)
		{
			using namespace Frontend;
			using namespace VariableNames;

			if (InPin && ensure(InPin->Direction == EGPD_Output))
			{
				if (const UMetasoundEditorGraphVariableNode* EdVariableNode = Cast<UMetasoundEditorGraphVariableNode>(InPin->GetOwningNode()))
				{
					// UEdGraphPins on variable nodes use the variable's name for display
					// purposes instead of the underlying vertex's name. The frontend vertices
					// of a variable node have consistent names no matter what the 
					// variable is named.
					return EdVariableNode->GetConstNodeHandle()->GetConstOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputData));
				}
				else if (const UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
				{
					return EdNode->GetConstNodeHandle()->GetConstOutputWithVertexName(InPin->GetFName());
				}
			}

			return IOutputController::GetInvalidHandle();
		}

		bool FGraphBuilder::IsPreviewingMetaSound(const UObject& InMetaSound)
		{
			using namespace Engine;

			if (GEditor)
			{
				if (const UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent())
				{
					if (PreviewComponent->IsPlaying())
					{
						if (const USoundBase* Sound = PreviewComponent->Sound)
						{
							return Sound->GetUniqueID() == InMetaSound.GetUniqueID();
						}
					}
				}
			}

			return false;
		}

		UEdGraphPin* FGraphBuilder::FindReroutedOutputPin(UEdGraphPin* OutputPin)
		{
			using namespace Frontend;

			if (OutputPin)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(OutputPin->GetOwningNode()))
				{
					if (ExternalNode->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
					{
						auto IsInput = [](UEdGraphPin* Pin) { check(Pin); return Pin->Direction == EGPD_Input; };
						if (UEdGraphPin** RerouteInput = ExternalNode->Pins.FindByPredicate(IsInput); RerouteInput && *RerouteInput)
						{
							TArray<UEdGraphPin*>& LinkedTo = (*RerouteInput)->LinkedTo;
							if (!LinkedTo.IsEmpty())
							{
								UEdGraphPin* ReroutedOutput = LinkedTo.Last();
								return FindReroutedOutputPin(ReroutedOutput);
							}
						}
					}
				}
			}

			return OutputPin;
		}

		const UEdGraphPin* FGraphBuilder::FindReroutedOutputPin(const UEdGraphPin* OutputPin)
		{
			using namespace Frontend;

			if (OutputPin)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(OutputPin->GetOwningNode()))
				{
					if (ExternalNode->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
					{
						auto IsInput = [](const UEdGraphPin* Pin) { check(Pin); return Pin->Direction == EGPD_Input; };
						if (const UEdGraphPin* const* RerouteInput = ExternalNode->Pins.FindByPredicate(IsInput))
						{
							const TArray<UEdGraphPin*>& LinkedTo = (*RerouteInput)->LinkedTo;
							if (!LinkedTo.IsEmpty())
							{
								const UEdGraphPin* ReroutedOutput = LinkedTo.Last();
								return FindReroutedOutputPin(ReroutedOutput);
							}
						}
					}
				}
			}

			return OutputPin;
		}

		Frontend::FConstOutputHandle FGraphBuilder::FindReroutedConstOutputHandleFromPin(const UEdGraphPin* OutputPin)
		{
			using namespace Frontend;

			if (OutputPin)
			{
				if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(OutputPin->GetOwningNode()))
				{
					if (ExternalNode->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
					{
						auto IsInput = [](UEdGraphPin* Pin) { check(Pin); return Pin->Direction == EGPD_Input; };
						if (const UEdGraphPin* const* RerouteInput = ExternalNode->Pins.FindByPredicate(IsInput))
						{
							const TArray<UEdGraphPin*>& LinkedTo = (*RerouteInput)->LinkedTo;
							if (!LinkedTo.IsEmpty())
							{
								const UEdGraphPin* ReroutedOutput = LinkedTo.Last();
								return FindReroutedConstOutputHandleFromPin(ReroutedOutput);
							}
						}
					}
				}

				return GetConstOutputHandleFromPin(OutputPin);
			}

			return IOutputController::GetInvalidHandle();
		}

		void FGraphBuilder::FindReroutedInputPins(UEdGraphPin* InPinToCheck, TArray<UEdGraphPin*>& InOutInputPins)
		{
			using namespace Frontend;

			if (InPinToCheck && InPinToCheck->Direction == EGPD_Input)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(InPinToCheck->GetOwningNode()))
				{
					if (ExternalNode->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
					{
						for (UEdGraphPin* Pin : ExternalNode->Pins)
						{
							if (Pin->Direction == EGPD_Output)
							{
								for (UEdGraphPin* LinkedInput : Pin->LinkedTo)
								{
									FindReroutedInputPins(LinkedInput, InOutInputPins);
								}
							}
						}

						return;
					}
				}

				InOutInputPins.Add(InPinToCheck);
			}
		}

		bool FGraphBuilder::GetPinLiteral(const UEdGraphPin& InInputPin, FMetasoundFrontendLiteral& OutDefaultLiteral)
		{
			using namespace Frontend;

			IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);

			const FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderFromPinChecked(InInputPin).GetConstBuilder();
			FMetasoundFrontendVertexHandle InputHandle = GetPinVertexHandle(Builder, &InInputPin);
			if (!ensure(InputHandle.IsSet()))
			{
				return false;
			}

			const FMetasoundFrontendVertex* Vertex = GetPinVertex(Builder, &InInputPin);
			if (!ensure(Vertex))
			{
				return false;
			}

			const FString& InStringValue = InInputPin.DefaultValue;
			const FName TypeName = Vertex->TypeName;

			FDataTypeRegistryInfo DataTypeInfo;
			IDataTypeRegistry::Get().GetDataTypeInfo(TypeName, DataTypeInfo);
			switch (DataTypeInfo.PreferredLiteralType)
			{
				case ELiteralType::Boolean:
				{
					// Currently don't support triggers being initialized to boolean in-graph
					if (GetMetasoundDataTypeName<FTrigger>() != TypeName)
					{
						OutDefaultLiteral.Set(FCString::ToBool(*InStringValue));
					}
				}
				break;

				case ELiteralType::Float:
				{
					OutDefaultLiteral.Set(FCString::Atof(*InStringValue));
				}
				break;

				case ELiteralType::Integer:
				{
					OutDefaultLiteral.Set(FCString::Atoi(*InStringValue));
				}
				break;

				case ELiteralType::String:
				{
					OutDefaultLiteral.Set(InStringValue);
				}
				break;

				case ELiteralType::UObjectProxy:
				{
					bool bObjectFound = false;
					if (!InInputPin.DefaultValue.IsEmpty())
					{
						if (UClass* Class = IDataTypeRegistry::Get().GetUClassForDataType(TypeName))
						{
							FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

							// Remove class prefix if included in default value path
							FString ObjectPath = InInputPin.DefaultValue;
							ObjectPath.RemoveFromStart(Class->GetName() + TEXT(" "));

							FARFilter Filter;
							Filter.bRecursiveClasses = false;
							Filter.SoftObjectPaths.Add(FSoftObjectPath(ObjectPath));

							TArray<FAssetData> AssetData;
							AssetRegistryModule.Get().GetAssets(Filter, AssetData);
							if (!AssetData.IsEmpty())
							{
								if (UObject* AssetObject = AssetData.GetData()->GetAsset())
								{
									const UClass* AssetClass = AssetObject->GetClass();
									if (ensureAlways(AssetClass))
									{
										if (AssetClass->IsChildOf(Class))
										{
											Filter.ClassPaths.Add(Class->GetClassPathName());
											OutDefaultLiteral.Set(AssetObject);
											bObjectFound = true;
										}
									}
								}
							}
						}
					}

					if (!bObjectFound)
					{
						// If the class default literal is the default (type is None), then the literal should be set to that.
						// However, if the class default literal is set to an object, the literal should be set to a valid, null object.
						// This is used for reset to default behavior, where an valid object literal with a null value is a separate case
						// from an inherited cleared default literal. 
						const TArray<FMetasoundFrontendClassInputDefault>* ClassInputDefaults = Builder.FindNodeClassInputDefaults(InputHandle.NodeID, Vertex->Name);
						if (ClassInputDefaults)
						{
							if (const FMetasoundFrontendClassInputDefault* ClassDefault = FindPreferredPage(*ClassInputDefaults, UMetaSoundSettings::GetPageOrder()))
							{
								if (ClassDefault->Literal.GetType() == EMetasoundFrontendLiteralType::None)
								{
									OutDefaultLiteral.Clear();
								}
								else
								{
									OutDefaultLiteral = ClassDefault->Literal;
								}
							}
							return true;
						}

						OutDefaultLiteral.Set(static_cast<UObject*>(nullptr));
					}
				}
				break;

				case ELiteralType::BooleanArray:
				{
					OutDefaultLiteral.Set(TArray<bool>());
				}
				break;

				case ELiteralType::FloatArray:
				{
					OutDefaultLiteral.Set(TArray<float>());
				}
				break;

				case ELiteralType::IntegerArray:
				{
					OutDefaultLiteral.Set(TArray<int32>());
				}
				break;

				case ELiteralType::NoneArray:
				{
					OutDefaultLiteral.Set(FMetasoundFrontendLiteral::FDefaultArray());
				}
				break;

				case ELiteralType::StringArray:
				{
					OutDefaultLiteral.Set(TArray<FString>());
				}
				break;

				case ELiteralType::UObjectProxyArray:
				{
					OutDefaultLiteral.Set(TArray<UObject*>());
				}
				break;

				case ELiteralType::None:
				{
					OutDefaultLiteral.Set(FMetasoundFrontendLiteral::FDefault());
				}
				break;

				case ELiteralType::Invalid:
				default:
				{
					static_assert(static_cast<int32>(ELiteralType::COUNT) == 13, "Possible missing ELiteralType case coverage.");
					ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
					return false;
				}
			}

			return true;
		}

		FMetasoundFrontendClassInput FGraphBuilder::CreateUniqueClassInput(UObject& InMetaSound, const FCreateNodeVertexParams& InParams, const TArray<FMetasoundFrontendClassInputDefault>& InDefaultLiterals, const FName* InNameBase)
		{
			using namespace Engine;
			using namespace Frontend;

			FMetasoundFrontendClassInput ClassInput;
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(InMetaSound);
			const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetConstBuilder();
			const FMetasoundFrontendDocument& Document = DocBuilder.GetConstDocumentChecked();
			ClassInput.Name = DocBuilder.GenerateUniqueInputName(InNameBase ? InNameBase->ToString() : TEXT("Input"));
			ClassInput.TypeName = InParams.DataType;
			ClassInput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(Document);
			ClassInput.NodeID = FDocumentIDGenerator::Get().CreateNodeID(Document);

			// Can be unset if attempting to mirror parameters from a reroute, so default to reference
			ClassInput.AccessType = InParams.AccessType == EMetasoundFrontendVertexAccessType::Unset ? EMetasoundFrontendVertexAccessType::Reference : InParams.AccessType;

			// Should always have at least one value
			if (InDefaultLiterals.IsEmpty())
			{
				ClassInput.InitDefault();
			}
			else
			{
				if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
				{
					TSet<FGuid> ValidPageIDs;
					Settings->IteratePageSettings([&ValidPageIDs](const FMetaSoundPageSettings& PageSetting)
					{
						ValidPageIDs.Add(PageSetting.UniqueId);
					});
					for (const FMetasoundFrontendClassInputDefault& InputDefault : InDefaultLiterals)
					{
						if (ValidPageIDs.Contains(InputDefault.PageID))
						{
							ClassInput.AddDefault(InputDefault.PageID) = InputDefault.Literal;
						}
					}
				}
			}

			return ClassInput;
		}

		FMetasoundFrontendClassOutput FGraphBuilder::CreateUniqueClassOutput(UObject& InMetaSound, const FCreateNodeVertexParams& InParams, const FName* InNameBase)
		{
			using namespace Metasound::Engine;

			FMetasoundFrontendClassOutput ClassOutput;
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(InMetaSound);
			ClassOutput.Name = Builder.GetConstBuilder().GenerateUniqueOutputName(InNameBase ? InNameBase->ToString() : TEXT("Output"));
			ClassOutput.TypeName = InParams.DataType;
			ClassOutput.VertexID = FGuid::NewGuid();
			ClassOutput.NodeID = FGuid::NewGuid();

			// Can be unset if attempting to mirror parameters from a reroute, so default to reference
			ClassOutput.AccessType = InParams.AccessType == EMetasoundFrontendVertexAccessType::Unset ? EMetasoundFrontendVertexAccessType::Reference : InParams.AccessType;

			return ClassOutput;
		}

		FMetasoundAssetBase& FGraphBuilder::GetOutermostMetaSoundChecked(UObject& InSubObject)
		{
			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(InSubObject.GetOutermostObject());
			check(MetaSoundAsset);
			return *MetaSoundAsset;
		}

		const FMetasoundAssetBase& FGraphBuilder::GetOutermostConstMetaSoundChecked(const UObject& InSubObject)
		{
			const FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(InSubObject.GetOutermostObject());
			check(MetaSoundAsset);
			return *MetaSoundAsset;
		}

		bool FGraphBuilder::ConnectNodes(UEdGraphPin& InInputPin, UEdGraphPin& InOutputPin, bool bInConnectEdPins)
		{
			using namespace Frontend;

			// When true, will recursively call back into this function
			// from the schema if the editor pins are successfully connected
			if (bInConnectEdPins)
			{
				const UEdGraphSchema* Schema = InInputPin.GetSchema();
				if (ensure(Schema))
				{
					if (!Schema->TryCreateConnection(&InInputPin, &InOutputPin))
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderFromPinChecked(InInputPin).GetBuilder();
			const FMetasoundFrontendVertexHandle InputVertexHandle = GetPinVertexHandle(Builder, &InInputPin);
			const FMetasoundFrontendVertexHandle OutputVertexHandle = GetPinVertexHandle(Builder, &InOutputPin);
			if (!InputVertexHandle.IsSet() || !OutputVertexHandle.IsSet())
			{
				InInputPin.BreakLinkTo(&InOutputPin);
				return false;
			}

			Builder.RemoveEdgeToNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);

			FMetasoundFrontendEdge NewEdge
			{
				OutputVertexHandle.NodeID,
				OutputVertexHandle.VertexID,
				InputVertexHandle.NodeID,
				InputVertexHandle.VertexID,
			};
			Builder.AddEdge(MoveTemp(NewEdge));
			return true;
		}

		void FGraphBuilder::DisconnectPinVertex(UEdGraphPin& InPin)
		{
			FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderFromPinChecked(InPin).GetBuilder();
			const FMetasoundFrontendVertexHandle VertexHandle = GetPinVertexHandle(Builder, &InPin);
			if (VertexHandle.IsSet())
			{
				if (InPin.Direction == EGPD_Input)
				{
					Builder.RemoveEdgeToNodeInput(VertexHandle.NodeID, VertexHandle.VertexID);
				}
				else
				{
					Builder.RemoveEdgesFromNodeOutput(VertexHandle.NodeID, VertexHandle.VertexID);
				}
			}
		}

		bool FGraphBuilder::DeleteNode(UEdGraphNode& InNode, bool bRemoveUnusedDependencies)
		{
			using namespace Engine;
			using namespace Frontend;

			UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(InNode.GetGraph());
			UObject& MetaSound = Graph->GetMetasoundChecked();
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);

			if (UMetasoundEditorGraphCommentNode* Node = Cast<UMetasoundEditorGraphCommentNode>(&InNode))
			{
				const bool bFrontendCommentRemoved = Builder.RemoveGraphComment(Node->CommentID);
				if (ensure(bFrontendCommentRemoved))
				{
					return ensure(Graph->RemoveNode(&InNode));
				}
			}

			UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(&InNode);
			if (ensure(Node))
			{
//				Need to split out delete ed vs. frontend impl which will happen in subsequent change.
//				For now, just ignore removal failure as a hack as this gets called from ed graph sync.
				EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
				Builder.RemoveNode({ Node->GetNodeID() }, Result, bRemoveUnusedDependencies);
// 				ensure(Result == EMetaSoundBuilderResult::Succeeded);
			}

			return ensure(Graph->RemoveNode(&InNode));
		}

		bool FGraphBuilder::BindEditorGraph(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph** OutGraph)
		{
			using namespace Frontend;

			bool bNewGraphBound = false;
			FMetasoundAssetBase& MetaSound = InBuilder.GetMetasoundAsset();
			UMetasoundEditorGraph* Graph = Cast<UMetasoundEditorGraph>(MetaSound.GetGraph());
			if (!Graph)
			{
				Graph = NewObject<UMetasoundEditorGraph>(MetaSound.GetOwningAsset(), FName(), RF_Transactional | RF_Transient);
				Graph->Schema = UMetasoundEditorGraphSchema::StaticClass();
				MetaSound.SetGraph(Graph);
				bNewGraphBound = true;
			}

			if (OutGraph)
			{
				*OutGraph = Graph;
			}
			return bNewGraphBound;
		}

		void FGraphBuilder::RefreshPinMetadata(UEdGraphPin& InPin, bool bAdvancedView)
		{
			UMetasoundEditorGraphNode* MetasoundGraphNode = Cast<UMetasoundEditorGraphNode>(InPin.GetOwningNode());
			check(MetasoundGraphNode);

			// Pin ToolTips are no longer cached on pins, and are instead dynamically generated via UMetasoundEditorGraphNode::GetPinHoverText
			InPin.PinToolTip = { };
			InPin.bAdvancedView = bAdvancedView;

			const FMetasoundFrontendNode& FrontendNode = MetasoundGraphNode->GetFrontendNodeChecked();
			if (InPin.bAdvancedView || FrontendNode.Style.bUnconnectedPinsHidden)
			{
				UEdGraphNode* OwningNode = InPin.GetOwningNode();
				check(OwningNode);
				if (OwningNode->AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
				{
					OwningNode->AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
				}
			}
		}

		void FGraphBuilder::RegisterGraphWithFrontend(UObject& InMetaSound, Frontend::FMetaSoundAssetRegistrationOptions RegOptions)
		{
			using namespace Frontend;

			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
			check(MetaSoundAsset);

			TArray<FMetasoundAssetBase*> EditedReferencingMetaSounds;
			if (GEditor)
			{
				if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					TArray<UObject*> EditedAssets = AssetSubsystem->GetAllEditedAssets();
					for (UObject* Asset : EditedAssets)
					{
						if (Asset != &InMetaSound)
						{
							if (FMetasoundAssetBase* EditedMetaSound = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(Asset))
							{
								EditedMetaSound->RebuildReferencedAssetClasses();
								if (EditedMetaSound->IsReferencedAsset(*MetaSoundAsset))
								{
									EditedReferencingMetaSounds.Add(EditedMetaSound);
								}
							}
						}
					}
				}
			}

			auto CheckShouldIgnore = [&](const FMetasoundAssetBase& InAsset)
			{
				if (FEditor::IsLiveAuditionEnabled())
				{
					if (RegOptions.bIgnoreIfLiveAuditioning)
					{
						const UObject* AssetToTest = InAsset.GetOwningAsset();
						check(AssetToTest);
						const bool bIsPreviewing = FGraphBuilder::IsPreviewingMetaSound(*AssetToTest);
						if (bIsPreviewing)
						{
							return true;
						}

						return false;
					}
				}

				return false;
			};

			// if EditedReferencingMetaSounds is empty, then no MetaSounds are open
			// that reference this MetaSound, so just register this asset. Otherwise,
			// this graph will recursively get updated when the open referencing graphs
			// are registered recursively via bRegisterDependencies flag.
			if (EditedReferencingMetaSounds.IsEmpty())
			{
				if (!CheckShouldIgnore(*MetaSoundAsset))
				{
					MetaSoundAsset->UpdateAndRegisterForExecution(RegOptions);
				}
			}
			else
			{
				for (FMetasoundAssetBase* RefMetaSound : EditedReferencingMetaSounds)
				{
					if (!CheckShouldIgnore(*RefMetaSound))
					{
						RefMetaSound->UpdateAndRegisterForExecution(RegOptions);
					}
				}
			}
		}

		bool FGraphBuilder::IsMatchingInputAndPin(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FMetasoundFrontendVertex& InVertex, const UEdGraphPin& InEditorPin)
		{
			if (InEditorPin.Direction == EGPD_Input)
			{
				const FMetasoundFrontendVertex* PinVertex = GetPinVertex(InBuilder, &InEditorPin);
				return PinVertex && PinVertex->VertexID == InVertex.VertexID && InVertex.Name == InEditorPin.PinName;
			}

			return false;
		}

		bool FGraphBuilder::IsMatchingOutputAndPin(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FMetasoundFrontendVertex& InVertex, const UEdGraphPin& InEditorPin)
		{
			if (InEditorPin.Direction == EGPD_Output)
			{
				const FMetasoundFrontendVertex* PinVertex = GetPinVertex(InBuilder, &InEditorPin);
				return PinVertex && PinVertex->VertexID == InVertex.VertexID && InVertex.Name == InEditorPin.PinName;
			}

			return false;
		}

		void FGraphBuilder::DepthFirstTraversal(UEdGraphNode* InInitialNode, FDepthFirstVisitFunction InVisitFunction)
		{
			// Non recursive depth first traversal.
			TArray<UEdGraphNode*> Stack({InInitialNode});
			TSet<UEdGraphNode*> Visited;

			while (Stack.Num() > 0)
			{
				UEdGraphNode* CurrentNode = Stack.Pop();
				if (Visited.Contains(CurrentNode))
				{
					// Do not revisit a node that has already been visited. 
					continue;
				}

				TArray<UEdGraphNode*> Children = InVisitFunction(CurrentNode).Array();
				Stack.Append(Children);

				Visited.Add(CurrentNode);
			}
		}

		UEdGraphPin* FGraphBuilder::AddPinToNode(
			const IMetasoundEditorModule& EditorModule,
			const FMetaSoundFrontendDocumentBuilder& InBuilder,
			UMetasoundEditorGraphNode& InEditorNode,
			const FMetasoundFrontendClass& InClass,
			const FMetasoundFrontendNode& InNode,
			const FMetasoundFrontendVertex& InVertex,
			EEdGraphPinDirection Direction)
		{
			using namespace Frontend;

			FEdGraphPinType PinType;
			if (const FEdGraphPinType* RegisteredPinType = FMetaSoundGraphPanelPinFactory::GetChecked()->FindPinType(InVertex.TypeName))
			{
				PinType = *RegisteredPinType;
			}

			UEdGraphPin* NewPin = InEditorNode.CreatePin(Direction, PinType, InVertex.Name);
			if (ensure(NewPin))
			{
				auto IsVertexMetadata = [&InVertex](const FMetasoundFrontendClassVertex& ClassVertex) { return ClassVertex.Name == InVertex.Name; };
				const FMetasoundFrontendClassVertex* ClassVertex = nullptr;
				if (Direction == EGPD_Input)
				{
					ClassVertex = InClass.GetInterfaceForNode(InNode).Inputs.FindByPredicate(IsVertexMetadata);
					SynchronizePinLiteral(InBuilder, *NewPin);
				}
				else
				{
					ClassVertex = InClass.GetInterfaceForNode(InNode).Outputs.FindByPredicate(IsVertexMetadata);
				}

				RefreshPinMetadata(*NewPin, ClassVertex ? ClassVertex->Metadata.bIsAdvancedDisplay : false);

			}

			return NewPin;
		}

		bool FGraphBuilder::RecurseGetDocumentModified(FMetasoundAssetBase& InAssetBase)
		{
			using namespace Metasound::Frontend;

			if (InAssetBase.GetConstModifyContext().GetDocumentModified())
			{
				return true;
			}

			TArray<FMetasoundAssetBase*> References;
			ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(InAssetBase, References));
			for (FMetasoundAssetBase* Reference : References)
			{
				check(Reference);
				const bool bReferenceDocumentModified = RecurseGetDocumentModified(*Reference);
				if (bReferenceDocumentModified)
				{
					return true;
				}
			}

			return false;
		}

		bool FGraphBuilder::SynchronizePinType(const IMetasoundEditorModule& InEditorModule, UEdGraphPin& InPin, const FName InDataType)
		{
			FEdGraphPinType PinType;
			if (const FEdGraphPinType* RegisteredPinType = FMetaSoundGraphPanelPinFactory::GetChecked()->FindPinType(InDataType))
			{
				PinType = *RegisteredPinType;
			}

			if (InPin.PinType != PinType)
			{
				InPin.PinType = PinType;
				return true;
			}

			return false;
		}

		bool FGraphBuilder::SynchronizeComments(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph)
		{
			bool bModified = false;

			TMap<FGuid, UMetasoundEditorGraphCommentNode*> CommentIDToEdNode;
			Algo::TransformIf(OutGraph.Nodes, CommentIDToEdNode,
				[](const UEdGraphNode* Node)
				{
					return Node->IsA<UMetasoundEditorGraphCommentNode>();
				},
				[](UEdGraphNode* Node)
				{
					UMetasoundEditorGraphCommentNode* CommentNode = CastChecked<UMetasoundEditorGraphCommentNode>(Node);
					return TPair<FGuid, UMetasoundEditorGraphCommentNode*>(CommentNode->GetCommentID(), CommentNode);
				});

			const FMetasoundFrontendDocument& Document = InBuilder.GetConstDocumentChecked();
			const TMap<FGuid, FMetaSoundFrontendGraphComment>& Comments = InBuilder.FindConstBuildGraphChecked().Style.Comments;
			for (const TPair<FGuid, FMetaSoundFrontendGraphComment>& Pair : Comments)
			{
				UMetasoundEditorGraphCommentNode* CommentNode = nullptr;
				if (!CommentIDToEdNode.RemoveAndCopyValue(Pair.Key, CommentNode))
				{
					bModified = true;
					constexpr bool bSelectNode = false;

					// Can't use spawn node action because it modifies the transaction stack, so just generate from CDO and backport frontend data accordingly.
					UClass* CommentClass = UMetasoundEditorGraphCommentNode::StaticClass();
					check(CommentClass);
					UMetasoundEditorGraphCommentNode* CommentCDO = CommentClass->GetDefaultObject<UMetasoundEditorGraphCommentNode>();
					check(CommentCDO);

					const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();
					check(GraphEditorSettings);

					UMetasoundEditorGraphCommentNode* NewNode = CastChecked<UMetasoundEditorGraphCommentNode>(DuplicateObject<UEdGraphNode>(CommentCDO, &OutGraph));
					NewNode->SetFlags(RF_Transactional);
					OutGraph.AddNode(NewNode, false, false);

					NewNode->CreateNewGuid();
					NewNode->bCommentBubbleVisible_InDetailsPanel = GraphEditorSettings->bShowCommentBubbleWhenZoomedOut;

					// Pull position, color, etc. from the existing frontend data.
					NewNode->CommentID = Pair.Key;
					UMetasoundEditorGraphCommentNode::ConvertFromFrontendComment(Pair.Value, *NewNode);
				}
				else
				{
					const FMetaSoundFrontendGraphComment* Comment = InBuilder.FindGraphComment(Pair.Key);
					UMetasoundEditorGraphCommentNode::ConvertFromFrontendComment(*Comment, *CommentNode);
				}
			}

			// Remaining items are stale, so they are removed
			bModified |= !CommentIDToEdNode.IsEmpty();
			for (const TPair<FGuid, UMetasoundEditorGraphCommentNode*>& Comment : CommentIDToEdNode)
			{
				constexpr bool bMarkDirty = false;
				constexpr bool bBreakAllLinks = true;
				OutGraph.RemoveNode(Comment.Value, bBreakAllLinks, bMarkDirty);
			}

			return bModified;
		}

		bool FGraphBuilder::SynchronizeConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph)
		{
			using namespace Frontend;

			bool bIsGraphDirty = false;

			TArray<UMetasoundEditorGraphNode*> EditorNodes;
			{
				OutGraph.GetNodesOfClass(EditorNodes);
			}

			TMap<FGuid, UMetasoundEditorGraphNode*> EditorNodesByFrontendID;
			for (UMetasoundEditorGraphNode* EditorNode : EditorNodes)
			{
				const FGuid& NodeID = EditorNode->GetNodeID();
				if (EditorNodesByFrontendID.Contains(NodeID))
				{
					UE_LOGF(LogMetasoundEditor, Error, "Multiple editor nodes associated with FrontendDocument node with ID '%ls'", *NodeID.ToString());
				}
				else
				{
					EditorNodesByFrontendID.Add(NodeID, EditorNode);
				}
			}

			// Iterate through all nodes in metasound editor graph and synchronize connections.
			for (UMetasoundEditorGraphNode* EditorNode : EditorNodes)
			{
				bool bIsNodeDirty = false;

				TArray<UEdGraphPin*> Pins = EditorNode->GetAllPins();
				TArray<const FMetasoundFrontendVertex*> NodeInputs = InBuilder.FindUserModifiableNodeInputs(EditorNode->GetNodeID());
				for (const FMetasoundFrontendVertex* NodeInput : NodeInputs)
				{
					auto IsMatchingInputPin = [&](const UEdGraphPin* Pin) { return IsMatchingInputAndPin(InBuilder, *NodeInput, *Pin); };

					UEdGraphPin* MatchingPin = nullptr;
					const int32 PinIndex = Pins.IndexOfByPredicate(IsMatchingInputPin);
					if (!ensure(PinIndex != INDEX_NONE))
					{
						continue;
					}

					MatchingPin = Pins[PinIndex];
					Pins.RemoveAtSwap(PinIndex, EAllowShrinking::No);

					const FMetasoundFrontendNode* ConnectedOutputNode = nullptr;
					if (const FMetasoundFrontendVertex* Output = InBuilder.FindNodeOutputConnectedToNodeInput(EditorNode->GetNodeID(), NodeInput->VertexID, &ConnectedOutputNode))
					{
						bool bAddLink = false;

						if (MatchingPin->LinkedTo.IsEmpty())
						{
							// No link currently exists. Add the appropriate link.
							bAddLink = true;
						}
						else if (!IsMatchingOutputAndPin(InBuilder, *Output, *MatchingPin->LinkedTo[0]))
						{
							// The wrong link exists.
							constexpr bool bNotifyNodes = false;
							constexpr bool bMarkDirty = false;
							MatchingPin->BreakAllPinLinks(bNotifyNodes, bMarkDirty);
							bAddLink = true;
						}

						if (bAddLink)
						{
							const FText& OwningNodeName = EditorNode->GetDisplayName();
							UMetasoundEditorGraphNode* OutputEditorNode = EditorNodesByFrontendID.FindRef(ConnectedOutputNode->GetID());
							if (OutputEditorNode)
							{
								UEdGraphPin* OutputPin = OutputEditorNode->FindPinChecked(Output->Name, EEdGraphPinDirection::EGPD_Output);

								UE_LOGF(LogMetasoundEditor, Verbose, "Synchronizing Node '%ls' Connection: Linking Pin '%ls' to '%ls'", *OwningNodeName.ToString(), *MatchingPin->GetName(), *OutputPin->GetName());

								constexpr bool bAlwaysMarkDirty = false;
								MatchingPin->MakeLinkTo(OutputPin, bAlwaysMarkDirty);
								bIsNodeDirty = true;
							}
							else
							{
								UE_LOGF(LogMetasoundEditor, Warning, "Failed to synchronize Frontend Node '%ls' Connection: Pin '%ls'", *OwningNodeName.ToString(), *MatchingPin->GetName());
							}
						}
					}
					else
					{
						// No link should exist.
						if (!MatchingPin->LinkedTo.IsEmpty())
						{
							constexpr bool bNotifyNodes = false;
							constexpr bool bMarkDirty = false;
							MatchingPin->BreakAllPinLinks(bNotifyNodes, bMarkDirty);
							bIsNodeDirty = true;
						}
					}

					SynchronizePinLiteral(InBuilder, *MatchingPin);
				}

				// Handle node outputs to break connections for the case 
				// where the connected node input is on a node that has been deleted,
				// so it wasn't handled by the node inputs direction above
				TArray<const FMetasoundFrontendVertex*> NodeOutputs = InBuilder.FindUserModifiableNodeOutputs(EditorNode->GetNodeID());
				for (const FMetasoundFrontendVertex* NodeOutput : NodeOutputs)
				{
					// Find pin
					auto IsMatchingOutputPin = [&](const UEdGraphPin* Pin) { return IsMatchingOutputAndPin(InBuilder, *NodeOutput, *Pin); };

					UEdGraphPin* MatchingPin = nullptr;
					if (UEdGraphPin** DoublePointer = Pins.FindByPredicate(IsMatchingOutputPin))
					{
						MatchingPin = *DoublePointer;
					}

					if (!ensure(MatchingPin))
					{
						continue;
					}

					// Remove connected pins from removal list
					TArray<UEdGraphPin*> PinsToBreak = MatchingPin->LinkedTo;
					TArray<const FMetasoundFrontendNode*> ConnectedInputNodes;
					TArray<const FMetasoundFrontendVertex*> ConnectedInputs = InBuilder.FindNodeInputsConnectedToNodeOutput(EditorNode->GetNodeID(), NodeOutput->VertexID, &ConnectedInputNodes);
					for (const FMetasoundFrontendVertex* ConnectedInput : ConnectedInputs)
					{
						UEdGraphPin** InputPin = PinsToBreak.FindByPredicate([&](const UEdGraphPin* Pin) { return IsMatchingInputAndPin(InBuilder, *ConnectedInput, *Pin); });
						if (InputPin)
						{
							UEdGraphPin* FoundPin = *InputPin;
							if (FoundPin)
							{
								PinsToBreak.Remove(FoundPin);
							}
						}
					}

					// Break remaining invalid connections
					for (UEdGraphPin* PinToBreak : PinsToBreak)
					{
						constexpr bool bMarkDirty = false;
						MatchingPin->BreakLinkTo(PinToBreak, bMarkDirty);
						bIsNodeDirty = true;
					}
				}

				bIsGraphDirty |= bIsNodeDirty;
			}

			return bIsGraphDirty;
		}

		bool FGraphBuilder::SynchronizeGraph(FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph, bool bSkipIfModifyContextUnchanged)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphBuilder::SynchronizeGraph);

			using namespace Frontend;

			UObject& MetaSound = InBuilder.CastDocumentObjectChecked<UObject>();
			FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
			check(MetaSoundAsset);

			if (!bSkipIfModifyContextUnchanged || RecurseGetDocumentModified(*MetaSoundAsset))
			{
				TSet<FMetasoundAssetBase*> EditedReferencingMetaSounds;
				if (GEditor)
				{
					if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
					{
						TArray<UObject*> EditedAssets = AssetSubsystem->GetAllEditedAssets();
						for (UObject* Asset : EditedAssets)
						{
							if (Asset != &MetaSound)
							{
								if (FMetasoundAssetBase* EditedMetaSound = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(Asset))
								{
									if (EditedMetaSound->IsReferencedAsset(*MetaSoundAsset))
									{
										// Ensure the graph has been initialized. Its possible the "edited asset" is a MetaSound
										// but its transient EdGraph has yet to be initialized due to the active editor not necessarily
										// being the MetaSound editor (ex. property matrix editor).
										if (const UEdGraph* EdGraph = EditedMetaSound->GetGraph())
										{
											EditedReferencingMetaSounds.Add(EditedMetaSound);
										}
									}
								}
							}
						}
					}
				}

				if (EditedReferencingMetaSounds.IsEmpty())
				{
					InBuilder.CacheRegistryMetadata();
					GraphBuilderPrivate::SynchronizeGraphRecursively(InBuilder, OutGraph, bSkipIfModifyContextUnchanged);
					GraphBuilderPrivate::RecurseClearDocumentModified(*MetaSoundAsset);
				}
				else
				{
					for (FMetasoundAssetBase* EditedMetaSound : EditedReferencingMetaSounds)
					{
						check(EditedMetaSound);
						UObject* OwningMetaSound = EditedMetaSound->GetOwningAsset();
						check(OwningMetaSound);
						FMetaSoundFrontendDocumentBuilder& EditedBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(OwningMetaSound);
						SynchronizeGraph(EditedBuilder, *CastChecked<UMetasoundEditorGraph>(&EditedMetaSound->GetGraphChecked()), bSkipIfModifyContextUnchanged);
					}
				}

				return true;
			}

			return false;
		}

		bool FGraphBuilder::SynchronizeOutputNodes(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphBuilder::SynchronizeOutputNodes);

			using namespace Frontend;

			bool bEditorGraphModified = false;

			const UObject& MetaSoundObject = InBuilder.CastDocumentObjectChecked<const UObject>();
			const FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSoundObject);
			check(MetaSoundAsset);
			FConstGraphHandle GraphHandle = MetaSoundAsset->GetRootGraphHandle();

			TArray<UMetasoundEditorGraphOutputNode*> OutputNodes;
			OutGraph.GetNodesOfClassEx<UMetasoundEditorGraphOutputNode>(OutputNodes);
			for (UMetasoundEditorGraphOutputNode* Node : OutputNodes)
			{
				FConstNodeHandle NodeHandle = Node->GetConstNodeHandle();
				if (!NodeHandle->IsValid())
				{
					for (UEdGraphPin* Pin : Node->Pins)
					{
						check(Pin);
						FConstClassOutputAccessPtr ClassOutputPtr = GraphHandle->FindClassOutputWithName(Pin->PinName);
						if (const FMetasoundFrontendClassOutput* Output = ClassOutputPtr.Get())
						{
							const FGuid& InitialID = Node->GetNodeID();
							if (NodeHandle->GetID() != Output->NodeID)
							{
								Node->SetNodeID(Output->NodeID);
								UE_LOGF(LogMetasoundEditor, Verbose, "Editor Output Node '%ls' interface versioned", *Node->GetDisplayName().ToString());

								bEditorGraphModified = true;
							}
						}
					}
				}
			}

			return bEditorGraphModified;
		}

		void FGraphBuilder::SynchronizeNodes(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphBuilder::SynchronizeNodes);

			using namespace Frontend;
			using FNodeClassPair = TPair<const FMetasoundFrontendNode*, const FMetasoundFrontendClass*>;

			// Get all external nodes from Frontend graph.  Input and output references will only be added/synchronized
			// if required when synchronizing connections (as they are not required to inhabit editor graph).
			UObject& MetaSoundObject = InBuilder.CastDocumentObjectChecked<UObject>();
			const FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSoundObject);
			check(MetaSoundAsset);
			TArray<FNodeClassPair> FrontendNodeAndClass;
			TArray<UMetasoundEditorGraphNode*> EditorNodes;

			{
				InBuilder.IterateNodes([&FrontendNodeAndClass](const FMetasoundFrontendClass& Class, const FMetasoundFrontendNode& Node)
				{
					// Input nodes use template input nodes to visually distinguish location, so ignore them.
					if (Class.Metadata.GetType() != EMetasoundFrontendClassType::Input)
					{
						FrontendNodeAndClass.Add(FNodeClassPair(&Node, &Class));
					}
				});
				OutGraph.GetNodesOfClass(EditorNodes);
			}

			// Find existing array of editor nodes associated with Frontend node
			struct FAssociatedNodes
			{
				TArray<UMetasoundEditorGraphNode*> EditorNodes;
				const FMetasoundFrontendNode* Node = nullptr;
				const FMetasoundFrontendClass* Class = nullptr;
			};
			TMap<FGuid, FAssociatedNodes> AssociatedNodes;

			{
				// Reverse iterate so paired nodes can safely be removed from the array.
				for (int32 i = FrontendNodeAndClass.Num() - 1; i >= 0; i--)
				{
					bool bFoundEditorNode = false;
					const FMetasoundFrontendNode& Node = *FrontendNodeAndClass[i].Key;
					const FMetasoundFrontendClass& Class = *FrontendNodeAndClass[i].Value;
					for (int32 j = EditorNodes.Num() - 1; j >= 0; --j)
					{
						UMetasoundEditorGraphNode* EditorNode = EditorNodes[j];
						EditorNode->CacheBreadcrumb();
						if (EditorNode->GetNodeID() == Node.GetID())
						{
							// Editor node may have the same Frontend NodeID as another page,
							// but may have been assigned a different Editor NodeID, so synchronize
							// from frontend data here only if node location was able to sync.
							constexpr bool bUpdateEditorNodeID = true;
							const bool bLocationFound = EditorNode->SyncLocationFromFrontendNode(bUpdateEditorNodeID);
							if (bLocationFound)
							{
								FAssociatedNodes& AssociatedNodeData = AssociatedNodes.FindOrAdd(Node.GetID());
								
								// If another editor node already exists for this frontend node, skip it 
								// (leaving it in EditorNodes will cause it to be removed later)
								// This is an editor only "fix" for frontend nodes with multiple locations, which should not be allowed past 
								// transient ed graph migration (doc version 1.12)
								if (AssociatedNodeData.EditorNodes.IsEmpty())
								{
									bFoundEditorNode = true;

									if (AssociatedNodeData.Node)
									{
										ensure(AssociatedNodeData.Node == &Node);
									}
									else
									{
										AssociatedNodeData.Node = &Node;
										AssociatedNodeData.Class = &Class;
									}

									EditorNode->SyncCommentFromFrontendNode();

									AssociatedNodeData.EditorNodes.Add(EditorNode);
									EditorNodes.RemoveAtSwap(j, EAllowShrinking::No);
								}
							}
						}
					}

					if (bFoundEditorNode)
					{
						FrontendNodeAndClass.RemoveAtSwap(i, EAllowShrinking::No);
					}
				}
			}

			// FrontendNodes now contains nodes which need to be added to the editor graph.
			// EditorNodes now contains nodes that need to be removed from the editor graph.
			// AssociatedNodes contains pairs which we have to check have synchronized pins

			// Add and remove nodes first in order to make sure correct editor nodes
			// exist before attempting to synchronize connections.
			for (UMetasoundEditorGraphNode* EditorNode : EditorNodes)
			{
				constexpr bool bMarkDirty = false;
				constexpr bool bBreakAllLinks = true;
				OutGraph.RemoveNode(EditorNode, bBreakAllLinks, bMarkDirty);
			}

			// Add missing editor nodes marked as visible.
			for (const FNodeClassPair& Pair : FrontendNodeAndClass)
			{
				const FMetasoundFrontendNodeStyle& CurrentStyle = Pair.Key->Style;
				const FGuid& NodeID = Pair.Key->GetID();
				const FMetasoundFrontendClassMetadata& ClassMetadata = Pair.Value->Metadata;
				for (const TPair<FGuid, FVector2D>& Location : CurrentStyle.Display.Locations)
				{
					UMetasoundEditorGraphNode* NewGraphNode = nullptr;
					switch (ClassMetadata.GetType())
					{
						case EMetasoundFrontendClassType::External:
						{
							NewGraphNode = Cast<UMetasoundEditorGraphNode>(AddExternalNode(MetaSoundObject, NodeID, ClassMetadata, false));
						}
						break;

						case EMetasoundFrontendClassType::Template:
						{
							if (Pair.Value->Metadata.GetClassName() == FInputNodeTemplate::ClassName)
							{
								NewGraphNode = Cast<UMetasoundEditorGraphNode>(AddInputNode(MetaSoundObject, NodeID, false));
							}
							else
							{
								NewGraphNode = Cast<UMetasoundEditorGraphNode>(AddTemplateNode(MetaSoundObject, NodeID, ClassMetadata, false));
							}
						}
						break;

						case EMetasoundFrontendClassType::Output:
						{
							NewGraphNode = Cast<UMetasoundEditorGraphNode>(AddOutputNode(MetaSoundObject, NodeID, false));
						}
						break;

						case EMetasoundFrontendClassType::VariableMutator:
						case EMetasoundFrontendClassType::VariableAccessor:
						case EMetasoundFrontendClassType::VariableDeferredAccessor:
						case EMetasoundFrontendClassType::Variable:
						{
							NewGraphNode = Cast<UMetasoundEditorGraphNode>(AddVariableNode(MetaSoundObject, NodeID, false));
						}
						break;

						case EMetasoundFrontendClassType::Invalid:

						// Class type needs to be deprecated 
						case EMetasoundFrontendClassType::Graph:

						// Not supported in editor
						case EMetasoundFrontendClassType::Literal:

						// Since MetaSound Document v1.12 update, the editor uses template input nodes, so no direct node representation of inputs no longer exists
						case EMetasoundFrontendClassType::Input: 
						default:
						{
							checkNoEntry();
							static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing FMetasoundFrontendClassType case coverage");
						}
						break;
					}

					if (ensure(NewGraphNode))
					{
						// Establish the editor node's identity GUID directly from the frontend Display.Locations
						// key being iterated. SyncLocationFromFrontendNode (called via GraphBuilderPrivate::AddNode)
						// picks the first map entry, which is order-sensitive across duplicated page graphs and can
						// diverge from the outer iterator's current Location.Key.
						if (Location.Key.IsValid())
						{
							NewGraphNode->NodeGuid = Location.Key;
						}
						else
						{
							// Legacy assets occasionally carry an invalid GUID key from older interface
							// migrations. Leave the freshly-finalized editor NodeGuid in place; the node
							// will reposition the next time the user moves it (which routes through
							// SetNodeLocation and rewrites the entry with a valid GUID).
							UE_LOGF(LogMetasoundEditor, Display, "Frontend node %ls has an invalid Display.Locations key; editor node will use a fresh GUID until next user move.", *NodeID.ToString());
						}

						FAssociatedNodes& AssociatedNodeData = AssociatedNodes.FindOrAdd(NodeID);
						if (AssociatedNodeData.EditorNodes.IsEmpty())
						{
							if (AssociatedNodeData.Node)
							{
								ensure(AssociatedNodeData.Node == Pair.Key);
							}
							else
							{
								AssociatedNodeData.Node = Pair.Key;
								AssociatedNodeData.Class = Pair.Value;
							}

							AssociatedNodeData.EditorNodes.Add(NewGraphNode);
						}
						// If another editor node already exists for this frontend node, remove it 
						// This is an editor only "fix" for frontend nodes with multiple locations, which should not be allowed past 
						// transient ed graph migration (doc version 1.12)
						else
						{
							constexpr bool bMarkDirty = false;
							constexpr bool bBreakAllLinks = true;
							OutGraph.RemoveNode(NewGraphNode, bBreakAllLinks, bMarkDirty);
						}
					}
				}
			}

			// Synchronize pins on node associations.
			for (const TPair<FGuid, FAssociatedNodes>& IdNodePair : AssociatedNodes)
			{
				for (UMetasoundEditorGraphNode* EditorNode : IdNodePair.Value.EditorNodes)
				{
					SynchronizeNodePins(InBuilder, *EditorNode, *IdNodePair.Value.Node, *IdNodePair.Value.Class);
				}
			}
		}

		bool FGraphBuilder::SynchronizeNodePins(
			const FMetaSoundFrontendDocumentBuilder& InBuilder,
			UMetasoundEditorGraphNode& InEditorNode,
			const FMetasoundFrontendNode& InNode,
			const FMetasoundFrontendClass& InClass,
			bool bRemoveUnusedPins)
		{
			using namespace Frontend;

			bool bIsNodeDirty = false;

			IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);

			const FGuid& NodeID = InNode.GetID();
			TArray<const FMetasoundFrontendVertex*> NodeInputs = InBuilder.FindUserModifiableNodeInputs(NodeID);
			TArray<const FMetasoundFrontendVertex*> NodeOutputs = InBuilder.FindUserModifiableNodeOutputs(NodeID);

			// Filter out pins which are not paired.
			TArray<UEdGraphPin*> EditorPins = InEditorNode.Pins;
			for (int32 i = EditorPins.Num() - 1; i >= 0; i--)
			{
				UEdGraphPin* Pin = EditorPins[i];
				switch (Pin->Direction)
				{
					case EEdGraphPinDirection::EGPD_Input:
					{
						auto IsMatchingInput = [&](const FMetasoundFrontendVertex* Input) { return IsMatchingInputAndPin(InBuilder, *Input, *Pin); };
						const int32 MatchingInputIndex = NodeInputs.FindLastByPredicate(IsMatchingInput);
						if (INDEX_NONE != MatchingInputIndex)
						{
							bIsNodeDirty |= SynchronizePinType(EditorModule, *EditorPins[i], NodeInputs[MatchingInputIndex]->TypeName);
							NodeInputs.RemoveAtSwap(MatchingInputIndex);
							EditorPins.RemoveAtSwap(i);
						}
					}
					break;

					case EEdGraphPinDirection::EGPD_Output:
					{
						auto IsMatchingOutput = [&](const FMetasoundFrontendVertex* Output) { return IsMatchingOutputAndPin(InBuilder, *Output, *Pin); };
						const int32 MatchingOutputIndex = NodeOutputs.FindLastByPredicate(IsMatchingOutput);
						if (INDEX_NONE != MatchingOutputIndex)
						{
							bIsNodeDirty |= SynchronizePinType(EditorModule, *EditorPins[i], NodeOutputs[MatchingOutputIndex]->TypeName);
							NodeOutputs.RemoveAtSwap(MatchingOutputIndex);
							EditorPins.RemoveAtSwap(i);
						}
					}
					break;
				}
			}

			// Remove any unused editor pins.
			if (bRemoveUnusedPins)
			{
				bIsNodeDirty |= !EditorPins.IsEmpty();
				for (UEdGraphPin* Pin : EditorPins)
				{
					InEditorNode.RemovePin(Pin);
				}
			}


			if (!NodeInputs.IsEmpty())
			{
				bIsNodeDirty = true;
				for (const FMetasoundFrontendVertex* NodeInput : NodeInputs)
				{
					AddPinToNode(EditorModule, InBuilder, InEditorNode, InClass, InNode, *NodeInput, EGPD_Input);
				}
			}

			if (!NodeOutputs.IsEmpty())
			{
				bIsNodeDirty = true;
				for (const FMetasoundFrontendVertex* NodeOutput : NodeOutputs)
				{
					AddPinToNode(EditorModule, InBuilder, InEditorNode, InClass, InNode, *NodeOutput, EGPD_Output);
				}
			}

			// Order pins
			NodeInputs = InBuilder.FindUserModifiableNodeInputs(NodeID);
			NodeOutputs = InBuilder.FindUserModifiableNodeOutputs(NodeID);

			auto GetInputDisplayName = [&InBuilder, &NodeID](const FMetasoundFrontendVertex& Vertex)
			{
				return InBuilder.GetNodeInputDisplayName(NodeID, Vertex.Name);
			};
			InClass.GetInterfaceForNode(InNode).GetInputStyle().SortVertices(NodeInputs, GetInputDisplayName);

			auto GetOutputDisplayName = [&InBuilder, &NodeID](const FMetasoundFrontendVertex& Vertex)
			{
				return InBuilder.GetNodeOutputDisplayName(NodeID, Vertex.Name);
			};
			InClass.GetInterfaceForNode(InNode).GetOutputStyle().SortVertices(NodeOutputs, GetOutputDisplayName);

			auto SwapAndDirty = [&](int32 IndexA, int32 IndexB)
			{
				const bool bRequiresSwap = IndexA != IndexB;
				if (bRequiresSwap)
				{
					InEditorNode.Pins.Swap(IndexA, IndexB);
					bIsNodeDirty |= bRequiresSwap;
				}
			};

			for (int32 i = InEditorNode.Pins.Num() - 1; i >= 0; --i)
			{
				UEdGraphPin* Pin = InEditorNode.Pins[i];
				if (Pin->Direction == EGPD_Input)
				{
					if (!NodeInputs.IsEmpty())
					{
						const FMetasoundFrontendVertex* Input = NodeInputs.Pop(EAllowShrinking::No);
						for (int32 j = i; j >= 0; --j)
						{
							if (IsMatchingInputAndPin(InBuilder, *Input, *InEditorNode.Pins[j]))
							{
								SwapAndDirty(i, j);
								break;
							}
						}
					}
				}
				else /* Pin->Direction == EGPD_Output */
				{
					if (!NodeOutputs.IsEmpty())
					{
						const FMetasoundFrontendVertex* Output = NodeOutputs.Pop(EAllowShrinking::No);
						for (int32 j = i; j >= 0; --j)
						{
							if (IsMatchingOutputAndPin(InBuilder, *Output, *InEditorNode.Pins[j]))
							{
								SwapAndDirty(i, j);
								break;
							}
						}
					}
				}
			}

			if (bIsNodeDirty)
			{
				InBuilder.GetConstDocumentChecked().Metadata.ModifyContext.AddNodeIDModified(NodeID);
			}
			return bIsNodeDirty;
		}

		bool FGraphBuilder::SynchronizePinLiteral(const FMetaSoundFrontendDocumentBuilder& InBuilder, UEdGraphPin& InPin)
		{
			using namespace Engine;
			using namespace Frontend;

			if (!ensure(InPin.Direction == EGPD_Input))
			{
				return false;
			}

			const FString OldValue = InPin.DefaultValue;

			const FMetasoundFrontendVertex* InputVertex = GetPinVertex(InBuilder, &InPin);
			if (!ensure(InputVertex))
			{
				return false;
			}

			FMetasoundFrontendVertexHandle InputHandle = GetPinVertexHandle(InBuilder, &InPin);
			check(InputHandle.IsSet());

			if (const FMetasoundFrontendVertexLiteral* VertexLiteral = InBuilder.FindNodeInputDefault(InputHandle.NodeID, InputVertex->Name))
			{
				InPin.DefaultValue = VertexLiteral->Value.ToString();
				return OldValue != InPin.DefaultValue;
			}

			const TArray<FMetasoundFrontendClassInputDefault>* ClassDefaults = InBuilder.FindNodeClassInputDefaults(InputHandle.NodeID, InputVertex->Name);
			if (ClassDefaults)
			{
				const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
				check(EditorSettings);

				TArray<FGuid> PageIDs;
				Algo::Transform(*ClassDefaults, PageIDs, [](const FMetasoundFrontendClassInputDefault& InputDefault) { return InputDefault.PageID; });
				const FGuid PageID = EditorSettings->ResolveAuditionPage(PageIDs, InBuilder.GetBuildPageID());
				auto MatchesPageID = [&PageID](const FMetasoundFrontendClassInputDefault& InputDefault) { return InputDefault.PageID == PageID; };
				if (const FMetasoundFrontendClassInputDefault* ClassDefault = ClassDefaults->FindByPredicate(MatchesPageID))
				{
					InPin.DefaultValue = ClassDefault->Literal.ToString();
					return OldValue != InPin.DefaultValue;
				}
			}

			FMetasoundFrontendLiteral DefaultLiteral;
			DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(InputVertex->TypeName));

			InPin.DefaultValue = DefaultLiteral.ToString();
			return OldValue != InPin.DefaultValue;
		}

		bool FGraphBuilder::SynchronizeGraphMembers(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphBuilder::SynchronizeGraphMembers);

			using namespace Frontend;

			bool bEditorGraphModified = false;
			auto GetMemberID = [](const TObjectPtr<UMetasoundEditorGraphMember>& Member) { return Member->GetMemberID(); };
			auto IsValidMember = [](const TObjectPtr<UMetasoundEditorGraphMember>& Member) { return Member != nullptr; };

			auto DeleteMemberNodes = [&OutGraph](const TObjectPtr<UMetasoundEditorGraphMember>& Member)
			{
				if (Member)
				{
					const TArray<UMetasoundEditorGraphMemberNode*> Nodes = Member->GetNodes();
					for (UMetasoundEditorGraphMemberNode* Node : Nodes)
					{
						OutGraph.RemoveNode(Node);
					}
				}
			};

			const FMetasoundFrontendGraphClass& RootGraph = InBuilder.GetConstDocumentChecked().RootGraph;

			// Input Sync
			{
				TSet<FGuid> EdMemberIDs;
				Algo::TransformIf(OutGraph.Inputs, EdMemberIDs, IsValidMember, GetMemberID);
				for (const FMetasoundFrontendClassInput& FrontendInput : RootGraph.GetDefaultInterface().Inputs)
				{
					if (EdMemberIDs.Remove(FrontendInput.NodeID) == 0)
					{
						bEditorGraphModified = true;
						OutGraph.FindOrAddInput(FrontendInput.NodeID);
					}
				}

				for (int32 Index = OutGraph.Inputs.Num() - 1; Index >= 0; --Index)
				{
					TObjectPtr<UMetasoundEditorGraphMember> Member = OutGraph.Inputs[Index];
					if (!Member || EdMemberIDs.Contains(Member->GetMemberID()))
					{
						DeleteMemberNodes(Member);
						OutGraph.Inputs.RemoveAtSwap(Index, EAllowShrinking::No);
						bEditorGraphModified = true;
					}
					else
					{
						Member->CacheBreadcrumb();
						bEditorGraphModified |= Member->Synchronize();
					}
				}
				OutGraph.Inputs.Shrink();
			}

			// Output Sync
			{
				TSet<FGuid> EdMemberIDs;
				Algo::TransformIf(OutGraph.Outputs, EdMemberIDs, IsValidMember, GetMemberID);
				for (const FMetasoundFrontendClassOutput& FrontendOutput : RootGraph.GetDefaultInterface().Outputs)
				{
					if (EdMemberIDs.Remove(FrontendOutput.NodeID) == 0)
					{
						bEditorGraphModified = true;
						OutGraph.FindOrAddOutput(FrontendOutput.NodeID);
					}
				}

				for (int32 Index = OutGraph.Outputs.Num() - 1; Index >= 0; --Index)
				{
					TObjectPtr<UMetasoundEditorGraphMember> Member = OutGraph.Outputs[Index];
					if (!Member || EdMemberIDs.Contains(Member->GetMemberID()))
					{
						DeleteMemberNodes(Member);
						OutGraph.Outputs.RemoveAtSwap(Index, EAllowShrinking::No);
					}
					else
					{
						Member->CacheBreadcrumb();
						bEditorGraphModified |= Member->Synchronize();
					}
				}
				OutGraph.Outputs.Shrink();
			}

			// Variable Sync
			{
				TSet<FGuid> EdMemberIDs;
				Algo::TransformIf(OutGraph.Variables, EdMemberIDs, IsValidMember, GetMemberID);
				for (const FMetasoundFrontendVariable& FrontendVariable : InBuilder.FindConstBuildGraphChecked().Variables)
				{
					if (EdMemberIDs.Remove(FrontendVariable.ID) == 0)
					{
						bEditorGraphModified = true;
						OutGraph.FindOrAddVariable(FrontendVariable.Name);
					}
				}

				for (int32 Index = OutGraph.Variables.Num() - 1; Index >= 0; --Index)
				{
					TObjectPtr<UMetasoundEditorGraphMember> Member = OutGraph.Variables[Index];
					if (!Member || EdMemberIDs.Contains(Member->GetMemberID()))
					{
						DeleteMemberNodes(Member);
						OutGraph.Variables.RemoveAtSwap(Index, EAllowShrinking::No);
					}
					else
					{
						Member->CacheBreadcrumb();
						bEditorGraphModified |= Member->Synchronize();
					}
				}
				OutGraph.Variables.Shrink();
			}

			return bEditorGraphModified;
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
