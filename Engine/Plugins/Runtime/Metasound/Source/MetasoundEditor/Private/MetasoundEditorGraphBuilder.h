// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "Logging/TokenizedMessage.h"
#include "MetasoundAssetManager.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundSettings.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"


// Forward Declarations
class SGraphEditor;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UMetaSoundPatch;
class UMetasoundEditorGraphCommentNode;
class UMetasoundEditorGraphExternalNode;
class UMetasoundEditorGraphNode;
class UMetasoundEditorGraphInputNode;
class UMetasoundEditorGraphOutputNode;
class UMetasoundEditorGraphVariableNode;

struct FEdGraphPinType;
struct FMetasoundFrontendNodeStyle;
struct FMetasoundFrontendLiteral;


namespace Metasound
{
	// Forward Declarations
	struct FLiteral;

	namespace Editor
	{
		// Forward Declarations
		class FEditor;
		class IMetasoundEditorModule;
		struct FCreateNodeVertexParams;

		class FGraphBuilder
		{
		public:
			static const FText FunctionMenuName;
			static const FText GraphMenuName;

			// Convenience functions for retrieving the editor for the given UObject
			static TSharedPtr<FEditor> GetEditorForMetasound(const UObject& InMetaSound);
			static TSharedPtr<FEditor> GetEditorForGraph(const UEdGraph& InEdGraph);
			static TSharedPtr<FEditor> GetEditorForNode(const UEdGraphNode& InEdNode);
			static TSharedPtr<FEditor> GetEditorForPin(const UEdGraphPin& InEdPin);

			// Validates MetaSound graph, returning the highest EMessageSeverity integer value
			static FGraphValidationResults ValidateGraph(UObject& InMetaSound);

			// Recursively checks whether the provided Asset's Document is marked as modified since last
			// EdGraph synchronization, or if any of its referenced asset graphs have been marked as modified.
			static bool RecurseGetDocumentModified(FMetasoundAssetBase& InAssetBase);

			// Returns default registration options when registering from this graph builder API.
			static Frontend::FMetaSoundAssetRegistrationOptions GetDefaultRegistrationOptions()
			{
				Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
				RegOptions.bForceViewSynchronization = false;
				RegOptions.PageOrder = UMetaSoundSettings::GetPageOrder();
				return RegOptions;
			}

			// Wraps RegisterGraphWithFrontend logic in Frontend with any additional logic required to refresh editor & respective editor object state.
			// @param InMetaSound - MetaSound to register
			// @param RegOptions - Registration options utilized
			static void RegisterGraphWithFrontend(UObject& InMetaSound, Frontend::FMetaSoundAssetRegistrationOptions RegOptions = GetDefaultRegistrationOptions());

			// Determines if pin supports inspection/probe view.
			static bool CanInspectPin(const UEdGraphPin* InPin);

			// Returns a display name for a node. If the node has an empty or whitespace
			// only DisplayName, first attempts to use the asset name if class is defined
			// in an asset, and finally the NodeName is used.
			static FText GetDisplayName(const FMetasoundFrontendClassMetadata& InClassMetadata, FName InNodeName, bool bInIncludeNamespace);

			// Returns a display name for a node. If the node has an empty or whitespace
			// only DisplayName, first attempts to use the asset name if class is defined
			// in an asset, and finally the NodeName is used.
			static FText GetDisplayName(const Frontend::INodeController& InFrontendNode, bool bInIncludeNamespace);

			// Returns a display name for an input. If the input has an empty or whitespace
			// only DisplayName, then the VertexName is used. 
			static FText GetDisplayName(const Frontend::IInputController& InFrontendInput);

			// Returns a display name for a output. If the output has an empty or whitespace
			// only DisplayName, then the VertexName is used. 
			static FText GetDisplayName(const Frontend::IOutputController& InFrontendOutput);

			// Returns a display name for a variable. If the variable has an empty or whitespace
			// only DisplayName, then the VariableName is used. 
			static FText GetDisplayName(const Frontend::IVariableController& InFrontendVariable, bool bInIncludeNamespace = false);

			// Adds a new EdGraph comment node associated with the given MetaSoundFrontendGraph comment ID
			static UMetasoundEditorGraphCommentNode* CreateCommentNode(UObject& InMetaSound, bool bInSelectNewNode = true, FGuid InCommentID = FGuid::NewGuid());

			// Adds a corresponding UMetasoundEditorGraphExternalNode for the provided node handle.
			static UMetasoundEditorGraphExternalNode* AddExternalNode(UObject& InMetaSound, const FGuid& InNodeID, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode = true);

			// Adds an editor graph node that corresponds with an instance of a node that is defined by an external MetaSound node class.
			static UMetasoundEditorGraphExternalNode* AddTemplateNode(UObject& InMetaSound, const FGuid& InNodeID, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode = true);

			// Adds an externally-defined node with the given class info to both the editor and document graphs.
			// Generates analogous FNodeHandle.
			static UMetasoundEditorGraphExternalNode* AddExternalNode(UObject& InMetaSound, const FMetasoundFrontendClassMetadata& InMetadata, bool bInSelectNewNode = true);

			static Frontend::FNodeHandle AddExternalNodeHandle(UObject& InMetaSound, const FMetasoundFrontendClassName& InClassName);

			// Adds an variable editor node with the given variable node (ex. mutator, accessor) to the editor graph.
			static UMetasoundEditorGraphVariableNode* AddVariableNode(UObject& InMetaSound, const FGuid& InNodeID, bool bInSelectNewNode = true);

			// Adds an input node to the editor graph that corresponds to the provided input template node in the document with the given ID.
			static UMetasoundEditorGraphInputNode* AddInputNode(UObject& InMetaSound, const FGuid& InInputTemplateNodeID, bool bInSelectNewNode = true);

			// Adds an output node to the editor graph that corresponds to the provided output node ID.
			static UMetasoundEditorGraphOutputNode* AddOutputNode(UObject& InMetaSound, const FGuid& InNodeID, bool bInSelectNewNode = true);

			// Convenience method for walking to the outermost object and transforming to a base MetaSound
			static FMetasoundAssetBase& GetOutermostMetaSoundChecked(UObject& InSubObject);
			static const FMetasoundAssetBase& GetOutermostConstMetaSoundChecked(const UObject& InSubObject);

			// Attempts to connect Frontend node counterparts together for provided pins.  Returns true if succeeded,
			// and breaks pin link and returns false if failed.  If bConnectEdPins is set, will attempt to connect
			// the Editor Graph representation of the pins.
			static bool ConnectNodes(UEdGraphPin& InInputPin, UEdGraphPin& InOutputPin, bool bInConnectEdPins);

			// Creates a unique class input with the given default data.
			static FMetasoundFrontendClassInput CreateUniqueClassInput(
				UObject& InMetaSound,
				const FCreateNodeVertexParams& InVertexParams,
				const TArray<FMetasoundFrontendClassInputDefault>& InDefaultLiterals = { },
				const FName* InNameBase = nullptr);

			// Creates a unique class output with the given default data.
			static FMetasoundFrontendClassOutput CreateUniqueClassOutput(
				UObject& InMetaSound,
				const FCreateNodeVertexParams& InVertexParams,
				const FName* InNameBase = nullptr);

			// Disconnects pin's associated frontend vertex from any linked input
			// or output nodes, and reflects change in the Frontend graph. Does *not*
			// disconnect the EdGraph pins.
			static void DisconnectPinVertex(UEdGraphPin& InPin);

			static UMetaSoundBuilderBase& GetBuilderFromPinChecked(const UEdGraphPin& InPin);

			static TArray<FString> GetDataTypeNameCategories(const FName& InDataTypeName);

			// Get the input handle from an input pin.  Ensures pin is an input pin.
			// TODO: use IDs to connect rather than names. Likely need an UMetasoundEditorGraphPin
			static Frontend::FInputHandle GetInputHandleFromPin(const UEdGraphPin* InPin);
			static Frontend::FConstInputHandle GetConstInputHandleFromPin(const UEdGraphPin* InPin);

			static FName GetPinDataType(const UEdGraphPin* InPin);
			static FMetasoundFrontendVertexHandle GetPinVertexHandle(const FMetaSoundFrontendDocumentBuilder& InBuilder, const UEdGraphPin* InPin);
			static const FMetasoundFrontendVertex* GetPinVertex(const FMetaSoundFrontendDocumentBuilder& InBuilder, const UEdGraphPin* InPin, const FMetasoundFrontendNode** Node = nullptr);

			// Get the output handle from an output pin.  Ensures pin is an output pin.
			// TODO: use IDs to connect rather than names. Likely need an UMetasoundEditorGraphPin
			static Frontend::FOutputHandle GetOutputHandleFromPin(const UEdGraphPin* InPin);
			static Frontend::FConstOutputHandle GetConstOutputHandleFromPin(const UEdGraphPin* InPin);

			static bool IsPreviewingMetaSound(const UObject& InObject);

			static UEdGraphPin* FindReroutedOutputPin(UEdGraphPin* InPin);
			static const UEdGraphPin* FindReroutedOutputPin(const UEdGraphPin* InPin);

			// Find the "concrete" output handle associated with an output pin.  If the given output pin is on
			// a reroute node, will recursively search for the non-rerouted output its representing.
			static Frontend::FConstOutputHandle FindReroutedConstOutputHandleFromPin(const UEdGraphPin* InPin);

			// Find the "concrete" input handle associated with an output pin.  If the given input pin is on
			// a reroute node, will recursively search for all the non-rerouted input pins its representing.
			static void FindReroutedInputPins(UEdGraphPin* InPinToCheck, TArray<UEdGraphPin*>& InOutInputPins);

			// Returns the default literal stored on the respective Frontend Node's Input.
			static bool GetPinLiteral(const UEdGraphPin& InInputPin, FMetasoundFrontendLiteral& OutLiteralDefault);

			// Deletes both the editor graph & frontend nodes from respective graphs
			static bool DeleteNode(UEdGraphNode& InNode, bool bRemoveUnusedDependencies = true);

			// Returns Editor Graph associated with the given builder's MetaSound object. If the editor graph was created, initialized, and
			// bound to builder's MetaSound object, returns true (false if it already existed).  Sets (optional) pointer to the bound graph.
			static bool BindEditorGraph(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph** OutGraph = nullptr);

			// Refreshes pin state from class FrontendClassVertexMetadata
			static void RefreshPinMetadata(UEdGraphPin& InPin, bool bAdvancedView = false);

			// Adds and removes nodes, pins and connections so that the UEdGraph of the MetaSound matches the
			// FMetasoundFrontendDocument model. Validates the graph (and those referenced recursively).
			//
			// @param InBuilder - Builder to synchronize ed graph with.
			// @param OutGraph - Graph to mutate to conform to provided builder's selected build graph
			// @param bSkipIfModifyContextUnchanged - If true, recursively checks deprecated document modification for builder's document and all referenced documents, and skips EdGraph synchronization if no modifications are logged.
			// @return whether or not EditorGraph synchronization was performed.
			static bool SynchronizeGraph(FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph, bool bSkipIfModifyContextUnchanged = true);

			// Synchronizes editor nodes with frontend nodes, removing editor nodes that are not represented in the frontend, and adding editor nodes to represent missing frontend nodes.
			static void SynchronizeNodes(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph);

			// Synchronizes and reports to log whether or not an output node's associated FrontendNode ID has changed and therefore been updated through node versioning.
			//
			// @return True if the UMetasoundEditorGraphNode was altered. False otherwise.
			static bool SynchronizeOutputNodes(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph);

			// Adds and removes pins so that the UMetasoundEditorGraphNode matches the InNode.
			//
			// @return True if the UMetasoundEditorGraphNode was altered. False otherwise.
			static bool SynchronizeNodePins(
				const FMetaSoundFrontendDocumentBuilder& InBuilder,
				UMetasoundEditorGraphNode& InEditorNode,
				const FMetasoundFrontendNode& InNode,
				const FMetasoundFrontendClass& InClass,
				bool bRemoveUnusedPins = true);

			static bool SynchronizeComments(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph);

			// Adds and removes connections so that the UEdGraph of the MetaSound has the same
			// connections as the FMetasoundFrontendDocument graph.
			//
			// @return True if the UEdGraph was altered. False otherwise.
			static bool SynchronizeConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph);

			// Synchronizes literal for a given input with the EdGraph's pin value.
			static bool SynchronizePinLiteral(const FMetaSoundFrontendDocumentBuilder& InBuilder, UEdGraphPin& InPin);

			// Synchronizes pin type for a given pin with that registered with the MetaSound editor module provided.
			static bool SynchronizePinType(const IMetasoundEditorModule& InEditorModule, UEdGraphPin& InPin, const FName InDataType);

			// Synchronizes inputs, variables, and outputs for the given MetaSound.
			//
			// @return True if the UEdGraph was altered. False otherwise.
			static bool SynchronizeGraphMembers(const FMetaSoundFrontendDocumentBuilder& InBuilder, UMetasoundEditorGraph& OutGraph);

			// Function signature for visiting a node doing depth first traversal.
			//
			// Functions accept a UEdGraphNode* and return a TSet<UEdGraphNode*> which
			// represent all the children of the node. 
			using FDepthFirstVisitFunction = TFunctionRef<TSet<UEdGraphNode*> (UEdGraphNode*)>;

			// Traverse depth first starting at the InInitialNode and calling the InVisitFunction
			// for each node. 
			//
			// This implementation avoids recursive function calls to support deep
			// graphs.
			static void DepthFirstTraversal(UEdGraphNode* InInitialNode, FDepthFirstVisitFunction InVisitFunction);

		private:
			// Adds an input or output UEdGraphPin to a UMetasoundEditorGraphNode
			static UEdGraphPin* AddPinToNode(
				const IMetasoundEditorModule& EditorModule,
				const FMetaSoundFrontendDocumentBuilder& InBuilder,
				UMetasoundEditorGraphNode& InEditorNode,
				const FMetasoundFrontendClass& InClass,
				const FMetasoundFrontendNode& InNode,
				const FMetasoundFrontendVertex& InVertex,
				EEdGraphPinDirection Direction);

			// Returns true if the given input vertex and UEdGraphPin match each other.
			static bool IsMatchingInputAndPin(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FMetasoundFrontendVertex& InVertex, const UEdGraphPin& InEditorPin);

			// Returns true if the given output vertex and UEdGraphPin match each other.
			static bool IsMatchingOutputAndPin(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FMetasoundFrontendVertex& InVertex, const UEdGraphPin& InEditorPin);
		};
	} // namespace Editor
} // namespace Metasound
