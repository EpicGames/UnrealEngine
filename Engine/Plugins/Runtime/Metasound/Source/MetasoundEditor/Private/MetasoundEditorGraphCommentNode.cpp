// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorGraphCommentNode.h"

#include "Internationalization/Internationalization.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Layout/SlateRect.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundUObjectRegistry.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraphCommentNode)

class UEdGraphPin;

#define LOCTEXT_NAMESPACE "MetasoundEditor"


FMetasoundAssetBase& UMetasoundEditorGraphCommentNode::GetAssetChecked()
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	UObject* Outermost = GetOutermostObject();
	check(Outermost);

	FMetasoundAssetBase* MetaSound = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(Outermost);
	check(MetaSound);
	return *MetaSound;
}

bool UMetasoundEditorGraphCommentNode::CanUserDeleteNode() const
{
	return true;
}

void UMetasoundEditorGraphCommentNode::ConvertToFrontendComment(const UEdGraphNode_Comment& InEdNode, FMetaSoundFrontendGraphComment& OutComment)
{
	OutComment.bColorBubble = InEdNode.bColorCommentBubble != 0;
	OutComment.Color = InEdNode.CommentColor;
	OutComment.Comment = InEdNode.NodeComment;
	OutComment.Depth = InEdNode.CommentDepth;
	OutComment.FontSize = InEdNode.FontSize;
	OutComment.MoveMode = InEdNode.MoveMode == ECommentBoxMode::GroupMovement ? EMetaSoundFrontendGraphCommentMoveMode::GroupMovement : EMetaSoundFrontendGraphCommentMoveMode::NoGroupMovement;
	OutComment.Position = FIntVector2(InEdNode.NodePosX, InEdNode.NodePosY);
	OutComment.Size = FIntVector2(InEdNode.NodeWidth, InEdNode.NodeHeight);
}

void UMetasoundEditorGraphCommentNode::ConvertFromFrontendComment(const FMetaSoundFrontendGraphComment& InComment, UEdGraphNode_Comment& OutEdNode)
{
	OutEdNode.bColorCommentBubble = (uint32)InComment.bColorBubble;
	OutEdNode.CommentColor = InComment.Color;
	OutEdNode.NodeComment = InComment.Comment;
	OutEdNode.CommentDepth = InComment.Depth;
	OutEdNode.FontSize = InComment.FontSize;
	OutEdNode.MoveMode = InComment.MoveMode == EMetaSoundFrontendGraphCommentMoveMode::GroupMovement ? ECommentBoxMode::GroupMovement : ECommentBoxMode::NoGroupMovement;
	OutEdNode.NodePosX = InComment.Position.X;
	OutEdNode.NodePosY = InComment.Position.Y;
	OutEdNode.NodeWidth = InComment.Size.X;
	OutEdNode.NodeHeight = InComment.Size.Y;
}

void UMetasoundEditorGraphCommentNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FMetaSoundFrontendGraphComment Comment;
	ConvertToFrontendComment(*this, Comment);
	if (FMetaSoundFrontendGraphComment* FrontendComment = GetBuilderChecked().FindGraphComment(CommentID))
	{
		*FrontendComment = MoveTemp(Comment);
	}
}

void UMetasoundEditorGraphCommentNode::ResizeNode(const FVector2f& NewSize)
{
	using namespace Metasound::Engine;

	Super::ResizeNode(NewSize);
	if (bCanResizeNode) 
	{
		// Update both location and size, since resizing from certain corners can change the location too
		UpdateFrontendNodeLocation();
	}
}

UMetaSoundBuilderBase& UMetasoundEditorGraphCommentNode::GetBuilderChecked() const
{
	UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(GetGraph());
	return EdGraph->GetBuilderChecked();
}

FGuid UMetasoundEditorGraphCommentNode::GetCommentID() const
{
	return CommentID;
}

UObject& UMetasoundEditorGraphCommentNode::GetMetasoundChecked() const
{
	UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(GetGraph());
	return EdGraph->GetMetasoundChecked();
}

bool UMetasoundEditorGraphCommentNode::RemoveFromDocument() const
{
	return GetBuilderChecked().RemoveGraphComment(CommentID);
}

void UMetasoundEditorGraphCommentNode::SetBounds(const class FSlateRect& Rect)
{
	Super::SetBounds(Rect);
	UpdateFrontendNodeLocation();
}

void UMetasoundEditorGraphCommentNode::SetCommentID(const FGuid& InGuid)
{ 
	CommentID = InGuid; 
}

void UMetasoundEditorGraphCommentNode::OnRenameNode(const FString& NewName)
{
	Super::OnRenameNode(NewName);

	UMetaSoundBuilderBase& Builder = GetBuilderChecked();
	Builder.FindOrAddGraphComment(CommentID).Comment = NewName;
}

void UMetasoundEditorGraphCommentNode::UpdateFrontendNodeLocation()
{
	using namespace Metasound::Frontend;
	FMetaSoundFrontendGraphComment& FrontendComment = GetBuilderChecked().FindOrAddGraphComment(CommentID);
	FrontendComment.Position = FIntVector2(NodePosX, NodePosY);
	FrontendComment.Size = FIntVector2(NodeWidth, NodeHeight);
}
#undef LOCTEXT_NAMESPACE
