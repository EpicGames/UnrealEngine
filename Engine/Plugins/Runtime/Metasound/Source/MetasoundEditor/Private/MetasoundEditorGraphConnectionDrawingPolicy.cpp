// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphConnectionDrawingPolicy.h"

#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerDensity.h"
#include "Components/AudioComponent.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundTrace.h"
#include "Misc/App.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "SMetasoundPinValueInspector.h"
#include "Templates/Function.h"
#include "WaveTableSampler.h"

#define METASOUND_EDITOR_DEBUG_CONNECTIONS 0

#if METASOUND_EDITOR_DEBUG_CONNECTIONS
#include "HAL/IConsoleManager.h"
#endif // METASOUND_EDITOR_DEBUG_CONNECTIONS


namespace Metasound
{
	namespace Editor
	{
		namespace DrawingPolicyPrivate
		{
			static float EnvelopeConnectionSpeedCVar = 1.0f;
			static float EnvelopeConnectionSpacingCVar = 3.0f;
			static int32 EnvelopeConnectionMaxPointsPerConnection = 1024;

#if METASOUND_EDITOR_DEBUG_CONNECTIONS
			// Draw the bounding box for debugging
			void DrawDebugConnection(FSlateWindowElementList& InDrawElementsList, uint32 InLayerId, const FDrawConnectionData& InDrawConnectionData)
			{
				const FVector2f& TL = InDrawConnectionData.Bounds.Min;
				const FVector2f& BR = InDrawConnectionData.Bounds.Max;
				const FVector2f TR = FVector2f(InDrawConnectionData.Bounds.Max.X, InDrawConnectionData.Bounds.Min.Y);
				const FVector2f BL = FVector2f(InDrawConnectionData.Bounds.Min.X, InDrawConnectionData.Bounds.Max.Y);

				auto DrawSpaceLine = [&](const FVector2f& Point1, const FVector2f& Point2, const FLinearColor& InColor)
				{
					const FVector2f FakeTangent = (Point2 - Point1).GetSafeNormal();
					FSlateDrawElement::MakeDrawSpaceSpline(InDrawElementsList, InLayerId, Point1, FakeTangent, Point2, FakeTangent, 1.0f, ESlateDrawEffect::None);
				};

				const FLinearColor BoundsWireColor = InDrawConnectionData.bCloseToSpline ? FLinearColor::Green : FLinearColor::White;
				DrawSpaceLine(TL, TR, BoundsWireColor);
				DrawSpaceLine(TR, BR, BoundsWireColor);
				DrawSpaceLine(BR, BL, BoundsWireColor);
				DrawSpaceLine(BL, TL, BoundsWireColor);
			}

			FAutoConsoleVariableRef CVarMetaSoundEditorAudioConnectSpeed(
				TEXT("au.MetaSound.Editor.EnvelopeConnection.Speed"),
				DrawingPolicyPrivate::EnvelopeConnectionSpeedCVar,
				TEXT("Speed of lines drawn in audio connections in the MetaSoundEditor.\n")
				TEXT("Default: 1.0f"),
				ECVF_Default);

			FAutoConsoleVariableRef CVarMetaSoundEditorAudioConnectSpacing(
				TEXT("au.MetaSound.Editor.EnvelopeConnection.Spacing"),
				DrawingPolicyPrivate::EnvelopeConnectionSpacingCVar,
				TEXT("Spacing of lines drawn in audio connections in the MetaSoundEditor.\n")
				TEXT("Default: 3.0f"),
				ECVF_Default);

			FAutoConsoleVariableRef CVarEnvelopeConnectionMaxPointsPerConnection(
				TEXT("au.MetaSound.Editor.EnvelopeConnection.MaxPointsPerConnection"),
				DrawingPolicyPrivate::EnvelopeConnectionMaxPointsPerConnection,
				TEXT("Max number of draw points per connection for animated connections.\n")
				TEXT("Default: 1024"),
				ECVF_Default);
#endif // METASOUND_EDITOR_DEBUG_CONNECTIONS

			template<typename TAnalyzerType>
			void DetermineWiringStyle_Envelope(bool bIsPlaying, FGraphConnectionManager& ConnectionManager, const FGuid& NodeID, FName OutputName, UEdGraphPin* InputPin, FConnectionParams& OutParams)
			{
				using namespace Frontend;

				OutParams.bDrawBubbles = false;

				if (bIsPlaying)
				{
					OutParams.bDrawBubbles = ConnectionManager.IsTracked(NodeID, OutputName, TAnalyzerType::GetAnalyzerName());
					if (OutParams.bDrawBubbles)
					{
						const float WindowValue = ConnectionManager.UpdateValueWindow<TAnalyzerType>(NodeID, OutputName);
						OutParams.WireThickness = FMath::Clamp(WindowValue, 0.0f, 1.0f);
						FLinearColor HSV = OutParams.WireColor.LinearRGBToHSV();
						HSV.G = FMath::Lerp(HSV.G, 1.0f, WindowValue);
						OutParams.WireColor = HSV.HSVToLinearRGB();
					}
				}
			}

			template<typename TNumericType>
			void DrawConnectionSpline_Numeric(FGraphConnectionDrawingPolicy& InDrawingPolicy, const FEditor& InEditor, int32 InLayerId, const FDrawConnectionData& InData, TNumericType InDefaultValue)
			{
				if (!InData.OutputVertex || !InData.OutputNode)
				{
					return;
				}

				TNumericType Value = InDefaultValue;
				InEditor.GetConnectionManager().GetValue(InData.OutputNode->GetID(), InData.OutputVertex->Name, Value);

				FLinearColor InnerColor = Frontend::DisplayStyle::EdgeAnimation::DefaultColor;

				if (!InData.EdgeStyle || InData.EdgeStyle->LiteralColorPairs.IsEmpty())
				{
					InDrawingPolicy.DrawConnectionSpline(InLayerId, InData);
					return;
				}

				for (int32 i = 0; i < InData.EdgeStyle->LiteralColorPairs.Num() - 1; ++i)
				{
					const FMetasoundFrontendEdgeStyleLiteralColorPair& PairA = InData.EdgeStyle->LiteralColorPairs[i];

					float ValueFloat = static_cast<float>(Value);
					TNumericType ValueA = InDefaultValue;
					if (PairA.Value.TryGet(ValueA))
					{
						const float ValueAFloat = static_cast<float>(ValueA);
						if (ValueFloat > ValueAFloat)
						{
							const FMetasoundFrontendEdgeStyleLiteralColorPair& PairB = InData.EdgeStyle->LiteralColorPairs[i + 1];
							TNumericType ValueB = InDefaultValue;
							if (PairB.Value.TryGet(ValueB))
							{
								float Denom = static_cast<float>(ValueB) - ValueAFloat;
								if (FMath::IsNearlyZero(Denom))
								{
									Denom = SMALL_NUMBER;
								}
								const float Alpha = (ValueFloat - ValueAFloat) / Denom;
								InnerColor = PairA.Color + FMath::Clamp(Alpha, 0.0f, 1.0f) * (PairB.Color - PairA.Color);
							}
							break;
						}
					}

					InnerColor = PairA.Color;
				}

				// Draw the outer spline, which uses standardized color syntax
				InDrawingPolicy.DrawSpline(InData);

				// Draw the inner spline, which uses interpolated custom color syntax.
				FDrawConnectionData InnerSplineData = InData;
				InnerSplineData.Params.WireThickness *= 0.6666f;
				InnerSplineData.Params.WireColor = InnerColor;
				InDrawingPolicy.DrawAnimatedSpline(InnerSplineData);
			}
		} // namespace DrawingPolicyPrivate

		FDrawConnectionData::FDrawConnectionData(const UE::Slate::FDeprecateVector2DParameter& InStart, const UE::Slate::FDeprecateVector2DParameter& InEnd, const FEditor& Editor, const UE::Slate::FDeprecateVector2DParameter& InSplineTangent, const FConnectionParams& InParams, const UGraphEditorSettings& InSettings, const UE::Slate::FDeprecateVector2DParameter& InMousePosition)
			: P0(InStart)
			, P0Tangent(InParams.StartDirection == EGPD_Output ? UE::Slate::CastToVector2f(InSplineTangent) : UE::Slate::CastToVector2f(-InSplineTangent))
			, P1(InEnd)
			, P1Tangent(InParams.EndDirection == EGPD_Input ? UE::Slate::CastToVector2f(InSplineTangent) : UE::Slate::CastToVector2f(-InSplineTangent))
			, Params(InParams)
		{
			if (Params.AssociatedPin1)
			{
				OutputPin = Params.AssociatedPin1->Direction == EGPD_Output ? Params.AssociatedPin1 : Params.AssociatedPin2;
			}

			// The curve will include the endpoints but can extend out of a tight bounds because of the tangents
			// P0Tangent coefficient maximizes to 4/27 at a=1/3, and P1Tangent minimizes to -4/27 at a=2/3.
			constexpr float MaximumTangentContribution = 4.0f / 27.0f;
			Bounds = FBox2f(ForceInit);

			Bounds += P0;
			Bounds += P0 + MaximumTangentContribution * P0Tangent;
			Bounds += P1;
			Bounds += P1 - MaximumTangentContribution * P1Tangent;

			if (InSettings.bTreatSplinesLikePins)
			{
				QueryDistanceTriggerThresholdSquared = FMath::Square(InSettings.SplineHoverTolerance + Params.WireThickness * 0.5f);
				QueryDistanceForCloseSquared = FMath::Square(FMath::Sqrt(QueryDistanceTriggerThresholdSquared) + InSettings.SplineCloseTolerance);
				bCloseToSpline = Bounds.ComputeSquaredDistanceToPoint(InMousePosition) < QueryDistanceForCloseSquared;
			}

			ClosestPoint = FVector2f(ForceInit);
			if (InSettings.bTreatSplinesLikePins)
			{
				if (bCloseToSpline)
				{
					const int32 NumStepsToTest = 16;
					const float StepInterval = 1.0f / (float)NumStepsToTest;
					FVector2f Point1 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, 0.0f);
					for (float TestAlpha = 0.0f; TestAlpha < 1.0f; TestAlpha += StepInterval)
					{
						const FVector2f Point2 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, TestAlpha + StepInterval);

						const FVector2f ClosestPointToSegment = FMath::ClosestPointOnSegment2D(InMousePosition, Point1, Point2);
						const float DistanceSquared = (InMousePosition - ClosestPointToSegment).SizeSquared();

						if (DistanceSquared < ClosestDistanceSquared)
						{
							ClosestDistanceSquared = DistanceSquared;
							ClosestPoint = ClosestPointToSegment;
						}

						Point1 = Point2;
					}
				}
			}

			if (const FMetaSoundFrontendDocumentBuilder* Builder = Editor.GetFrontendBuilder())
			{
				if (ensure(Builder->IsValid()))
				{
					if (const UEdGraphPin* ReroutePin = FGraphBuilder::FindReroutedOutputPin(OutputPin))
					{
						OutputVertex = FGraphBuilder::GetPinVertex(*Builder, ReroutePin, &OutputNode);
						if (OutputVertex && OutputNode)
						{
							EdgeStyle = Builder->FindConstEdgeStyle(OutputNode->GetID(), OutputVertex->Name);
						}
					}
				}
			}
		}

		void FDrawConnectionData::UpdateSplineOverlap(FGraphSplineOverlapResult& OutResult) const
		{
			if (!bCloseToSpline)
			{
				return;
			}

			if (ClosestDistanceSquared < QueryDistanceTriggerThresholdSquared)
			{
				if (ClosestDistanceSquared < OutResult.GetDistanceSquared())
				{
					const float SquaredDistToPin1 = Params.AssociatedPin1 ? (P0 - ClosestPoint).SizeSquared() : TNumericLimits<float>::Max();
					const float SquaredDistToPin2 = Params.AssociatedPin2 ? (P1 - ClosestPoint).SizeSquared() : TNumericLimits<float>::Max();

					OutResult = FGraphSplineOverlapResult(Params.AssociatedPin1, Params.AssociatedPin2, ClosestDistanceSquared, SquaredDistToPin1, SquaredDistToPin2, bCloseToSpline);
				}
			}
			else if (ClosestDistanceSquared < QueryDistanceForCloseSquared)
			{
				OutResult.SetCloseToSpline(bCloseToSpline);
			}
		}

		FConnectionDrawingPolicy* FGraphConnectionDrawingPolicyFactory::CreateConnectionPolicy(
			const UEdGraphSchema* Schema,
			int32 InBackLayerID,
			int32 InFrontLayerID,
			float ZoomFactor,
			const FSlateRect& InClippingRect,
			FSlateWindowElementList& InDrawElements,
			UEdGraph* InGraphObj) const
		{
			if (Schema->IsA(UMetasoundEditorGraphSchema::StaticClass()))
			{
				return new FGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
			}

			return nullptr;
		}

		FGraphConnectionDrawingPolicy::FGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
			: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
			, GraphObj(InGraphObj)
		{
			// Don't show arrow images in the MetaSound graph
			ArrowImage = nullptr;
			ArrowRadius = FVector2f::ZeroVector;

			WireAnimationLayerID = (InFrontLayerID + InBackLayerID) / 2;

			// Numeric wires are inflated up to account for interior animation
			if (const UMetasoundEditorSettings* MetasoundSettings = GetDefault<UMetasoundEditorSettings>())
			{
				AnalyzerSettings = MetasoundSettings->AnalyzerAnimationSettings;
			}

			if (GraphObj)
			{
				MetasoundEditor = FGraphBuilder::GetEditorForGraph(*GraphObj);
			}
		}

		void FGraphConnectionDrawingPolicy::ScaleEnvelopeWireThickness(FConnectionParams& OutParams) const
		{
			if (OutParams.bDrawBubbles)
			{
				OutParams.WireThickness = FMath::Lerp(Settings->TraceReleaseWireThickness * AnalyzerSettings.WireScalarMin, AnalyzerSettings.EnvelopeWireThickness * AnalyzerSettings.WireScalarMax, OutParams.WireThickness);
			}
			else
			{
				OutParams.WireThickness = Settings->TraceReleaseWireThickness;
			}
		}

		const TSharedPtr<const FEditor> FGraphConnectionDrawingPolicy::GetEditor() const
		{
			return MetasoundEditor.Pin();
		}

		TSharedPtr<FEditor> FGraphConnectionDrawingPolicy::GetEditor()
		{
			return MetasoundEditor.Pin();
		}

		void FGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* InPinA, UEdGraphPin* InPinB, FConnectionParams& OutParams)
		{
			using namespace Frontend;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphConnectionDrawingPolicy::DetermineWiringStyle);

			TSharedPtr<FEditor> EditorPtr = MetasoundEditor.Pin();
			if (!EditorPtr)
			{
				FConnectionDrawingPolicy::DetermineWiringStyle(InPinA, InPinB, OutParams);
				return;
			}

			UEdGraphPin* InputPin = nullptr;
			UEdGraphPin* OutputPin = nullptr;
			if (InPinA && InPinA->Direction == EGPD_Input)
			{
				InputPin = InPinA;
			}
			else if (InPinB && InPinB->Direction == EGPD_Input)
			{
				InputPin = InPinB;
			}

			if (InPinA && InPinA->Direction == EGPD_Output)
			{
				OutputPin = InPinA;
			}
			else if (InPinB && InPinB->Direction == EGPD_Output)
			{
				OutputPin = InPinB;
			}

			OutParams.AssociatedPin1 = InputPin;
			OutParams.AssociatedPin2 = OutputPin;

			const UEdGraphPin* AnyPin = InputPin ? InputPin : OutputPin;

			const bool bInputOrphaned = InputPin && InputPin->bOrphanedPin;
			const bool bOutputOrphaned = OutputPin && OutputPin->bOrphanedPin;
			if (bInputOrphaned || bOutputOrphaned)
			{
				OutParams.WireColor = FLinearColor::Red;
			}
			else if (AnyPin)
			{
				OutParams.WireColor = FMetaSoundGraphPanelPinFactory::GetChecked()->GetPinColor(AnyPin->PinType);
			}

			bool bThicknessSet = false;
			if (AnyPin && AnalyzerSettings.bAnimateConnections)
			{
				if (const UEdGraphPin* ReroutePin = FGraphBuilder::FindReroutedOutputPin(OutputPin))
				{
					if (const FMetaSoundFrontendDocumentBuilder* Builder = EditorPtr->GetFrontendBuilder())
					{
						if (ensure(Builder->IsValid()))
						{
							const FMetasoundFrontendNode* OutputNode = nullptr;
							if (const FMetasoundFrontendVertex* OutputVertex = FGraphBuilder::GetPinVertex(*Builder, ReroutePin, &OutputNode))
							{
								check(OutputNode);
								const bool bIsPlaying = EditorPtr->IsPlaying();
								const FGuid& NodeID = OutputNode->GetID();
								if (AnyPin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryTrigger)
								{
									FGraphConnectionManager& ConnectionManager = EditorPtr->GetConnectionManager();
									DrawingPolicyPrivate::DetermineWiringStyle_Envelope<FVertexAnalyzerTriggerDensity>(bIsPlaying, ConnectionManager, NodeID, OutputVertex->Name, InputPin, OutParams);
									ScaleEnvelopeWireThickness(OutParams);
									bThicknessSet = true;
								}
								else if (AnyPin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryAudio)
								{
									FGraphConnectionManager& ConnectionManager = EditorPtr->GetConnectionManager();
									DrawingPolicyPrivate::DetermineWiringStyle_Envelope<FVertexAnalyzerEnvelopeFollower>(bIsPlaying, ConnectionManager, NodeID, OutputVertex->Name, InputPin, OutParams);
									ScaleEnvelopeWireThickness(OutParams);
									bThicknessSet = true;
								}
								else if (FGraphBuilder::CanInspectPin(AnyPin))
								{
									if (const FMetasoundFrontendEdgeStyle* EdgeStyle = Builder->FindConstEdgeStyle(NodeID, OutputVertex->Name))
									{
										if (!EdgeStyle->LiteralColorPairs.IsEmpty())
										{
											OutParams.WireThickness = AnalyzerSettings.NumericWireThickness;
											bThicknessSet = true;
										}
									}
								}
							}
						}
					}
				}
			}

			if (!bThicknessSet)
			{
				OutParams.WireThickness = Settings->TraceReleaseWireThickness;
			}

			const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
			if (bDeemphasizeUnhoveredPins)
			{
				const bool bIsPlaying = EditorPtr->IsPlaying();
				ApplyHoverDeemphasisMetaSound(OutputPin, InputPin, bIsPlaying, /*inout*/ OutParams.WireThickness, /*inout*/ OutParams.WireColor);
			}
		}

		void FGraphConnectionDrawingPolicy::ApplyHoverDeemphasisMetaSound(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, const bool bIsPlaying, /*inout*/ float& Thickness, /*inout*/ FLinearColor& WireColor)
		{
			//@TODO: Move these parameters into the settings object
			const float FadeInBias = 0.75f; // Time in seconds before the fading starts to occur
			const float FadeInPeriod = 0.6f; // Time in seconds after the bias before the fade is fully complete
			const float TimeFraction = FMath::SmoothStep(0.0f, FadeInPeriod, (float)(FSlateApplication::Get().GetCurrentTime() - LastHoverTimeEvent - FadeInBias));

			const float LightFraction = 0.25f;
			const FLinearColor DarkenedColor(0.0f, 0.0f, 0.0f, 0.5f);
			const FLinearColor LightenedColor(1.0f, 1.0f, 1.0f, 1.0f);

			const bool bContainsBoth = HoveredPins.Contains(InputPin) && HoveredPins.Contains(OutputPin);
			const bool bContainsOutput = HoveredPins.Contains(OutputPin);
			const bool bEmphasize = bContainsBoth || (bContainsOutput && (InputPin == nullptr));
			if (bEmphasize)
			{
				if (!bIsPlaying)
				{
					Thickness = FMath::Lerp(Thickness, Thickness * ((Thickness < 2.5f) ? 3.5f : 2.5f), TimeFraction);
				}
				WireColor = FMath::Lerp<FLinearColor>(WireColor, LightenedColor, LightFraction * TimeFraction);
			}
			else
			{
				WireColor = FMath::Lerp<FLinearColor>(WireColor, DarkenedColor, HoverDeemphasisDarkFraction * TimeFraction);
			}
		}

		void FGraphConnectionDrawingPolicy::DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Editor::FGraphConnectionDrawingPolicy::DrawConnection);

			using namespace Frontend;

			TSharedPtr<FEditor> EditorPtr = MetasoundEditor.Pin();
			if (!EditorPtr)
			{
				FConnectionDrawingPolicy::DrawConnection(LayerId, Start, End, Params);
				return;
			}

			if (!GraphObj || !EditorPtr->IsPlaying() || !AnalyzerSettings.bAnimateConnections)
			{
				FConnectionDrawingPolicy::DrawConnection(LayerId, Start, End, Params);
				return;
			}

			const FDrawConnectionData ConnectionData(Start, End, *EditorPtr, FGraphConnectionDrawingPolicy::ComputeSplineTangent(Start, End), Params, *Settings, AbsoluteMousePosition);
			if (Settings->bTreatSplinesLikePins)
			{
#if METASOUND_EDITOR_DEBUG_CONNECTIONS
				DrawingPolicyPrivate::DrawDebugConnection(DrawElementsList, ArrowLayerID, ConnectionData);
#endif // METASOUND_EDITOR_DEBUG_CONNECTIONS

				ConnectionData.UpdateSplineOverlap(SplineOverlapResult);
			}

			// AudioBuffers & Triggers only draw envelopes, not connection splines because they trace the same path
			if (Params.bDrawBubbles || MidpointImage)
			{
				// Maps distance along curve to alpha
				FInterpCurve<float> SplineReparamTable;
				const float SplineLength = MakeSplineReparamTable(ConnectionData.P0, ConnectionData.P0Tangent, ConnectionData.P1, ConnectionData.P1Tangent, SplineReparamTable);

				// Draw bubbles on the spline
				if (Params.bDrawBubbles)
				{
					const float AppTime = FPlatformTime::Seconds() - GStartTime;
					FDrawConnectionSignalData SignalParams = { AppTime, LayerId, SplineReparamTable, SplineLength, ConnectionData };

					FName DataType;
					if (const FMetasoundFrontendVertex* OutputVertex = SignalParams.ConnectionData.OutputVertex)
					{
						DataType = OutputVertex->TypeName;
					}
						
					if (DataType == GetMetasoundDataTypeName<FAudioBuffer>())
					{
						SignalParams.SpacingFactor = DrawingPolicyPrivate::EnvelopeConnectionSpacingCVar;
						SignalParams.SpeedFactor = AnalyzerSettings.EnvelopeSpeed * DrawingPolicyPrivate::EnvelopeConnectionSpeedCVar;
						const FName AnalyzerName = Frontend::FVertexAnalyzerEnvelopeFollower::GetAnalyzerName();
						DrawConnectionSignal_Envelope(SignalParams, AnalyzerName);
					}
					else if (DataType == GetMetasoundDataTypeName<FTrigger>())
					{
						SignalParams.SpacingFactor = DrawingPolicyPrivate::EnvelopeConnectionSpacingCVar;
						SignalParams.SpeedFactor = AnalyzerSettings.EnvelopeSpeed * DrawingPolicyPrivate::EnvelopeConnectionSpeedCVar;
						const FName AnalyzerName = Frontend::FVertexAnalyzerTriggerDensity::GetAnalyzerName();
						DrawConnectionSignal_Envelope(SignalParams, AnalyzerName);
					}
					else
					{
						DrawConnectionSignal(SignalParams);
					}
				}

				if (MidpointImage)
				{
					DrawConnection_MidpointImage(LayerId, SplineReparamTable, SplineLength, ConnectionData);
				}
				return;
			}

			// Draw other types
			if (ConnectionData.OutputVertex && MetasoundEditor.IsValid())
			{
				const FName DataType = ConnectionData.OutputVertex->TypeName;
				if (DataType == GetMetasoundDataTypeName<float>())
				{
					DrawingPolicyPrivate::DrawConnectionSpline_Numeric<float>(*this, *EditorPtr, LayerId, ConnectionData, 0.0f);
				}
				else if (DataType == GetMetasoundDataTypeName<int32>())
				{
					DrawingPolicyPrivate::DrawConnectionSpline_Numeric<int32>(*this, *EditorPtr, LayerId, ConnectionData, 0);
				}
				else if (DataType == GetMetasoundDataTypeName<bool>())
				{
					DrawingPolicyPrivate::DrawConnectionSpline_Numeric<bool>(*this, *EditorPtr, LayerId, ConnectionData, 0);
				}
				else 
				{
					DrawConnectionSpline(LayerId, ConnectionData);
				}
			}
			else
			{
				DrawConnectionSpline(LayerId, ConnectionData);
			}
		}

		void FGraphConnectionDrawingPolicy::DrawConnection_MidpointImage(int32 InLayerId, const FInterpCurve<float>& InSplineReparamTable, float InSplineLength, const FDrawConnectionData& InData)
		{
			check(MidpointImage);

			// Determine the spline position for the midpoint
			const float MidpointAlpha = InSplineReparamTable.Eval(InSplineLength * 0.5f, 0.f);
			const FVector2f Midpoint = FMath::CubicInterp(InData.P0, InData.P0Tangent, InData.P1, InData.P1Tangent, MidpointAlpha);

			// Approximate the slope at the midpoint (to orient the midpoint image to the spline)
			const FVector2f MidpointPlusE = FMath::CubicInterp(InData.P0, InData.P0Tangent, InData.P1, InData.P1Tangent, MidpointAlpha + KINDA_SMALL_NUMBER);
			const FVector2f MidpointMinusE = FMath::CubicInterp(InData.P0, InData.P0Tangent, InData.P1, InData.P1Tangent, MidpointAlpha - KINDA_SMALL_NUMBER);
			const FVector2f SlopeUnnormalized = MidpointPlusE - MidpointMinusE;

			// Draw the arrow
			const FVector2f MidpointDrawPos = Midpoint - MidpointRadius;
			const float AngleInRadians = SlopeUnnormalized.IsNearlyZero() ? 0.0f : FMath::Atan2(SlopeUnnormalized.Y, SlopeUnnormalized.X);

			FSlateDrawElement::MakeRotatedBox(
				DrawElementsList,
				InLayerId,
				FPaintGeometry(MidpointDrawPos, MidpointImage->ImageSize * ZoomFactor, ZoomFactor),
				MidpointImage,
				ESlateDrawEffect::None,
				AngleInRadians,
				TOptional<FVector2f>(),
				FSlateDrawElement::RelativeToElement,
				InData.Params.WireColor
			);
		}

		void FGraphConnectionDrawingPolicy::DrawConnectionSpline(int32 InLayerId, const FDrawConnectionData& InData)
		{
			FSlateDrawElement::MakeDrawSpaceSpline(
				DrawElementsList,
				InLayerId,
				InData.P0, InData.P0Tangent,
				InData.P1, InData.P1Tangent,
				InData.Params.WireThickness,
				ESlateDrawEffect::None,
				InData.Params.WireColor
			);
		}

		void FGraphConnectionDrawingPolicy::DrawConnectionSignal(const FDrawConnectionSignalData& InParams)
		{
			const float BubbleSpacing = InParams.SpacingFactor * ZoomFactor;
			float BubbleSpeed = InParams.SpeedFactor * ZoomFactor;

			const FVector2f BubbleSize = BubbleImage->ImageSize * ZoomFactor * 0.2f * InParams.ConnectionData.Params.WireThickness;

			const float BubbleOffset = FMath::Fmod(InParams.AppTime * BubbleSpeed, BubbleSpacing);
			const int32 NumBubbles = FMath::CeilToInt(InParams.SplineLength / BubbleSpacing);

			for (int32 i = 0; i < NumBubbles; ++i)
			{
				const float Distance = (i * BubbleSpacing) + BubbleOffset;
				if (Distance < InParams.SplineLength)
				{
					const float Alpha = InParams.SplineReparamTable.Eval(Distance, 0.f);
					FVector2f BubblePos = FMath::CubicInterp(InParams.ConnectionData.P0, InParams.ConnectionData.P0Tangent, InParams.ConnectionData.P1, InParams.ConnectionData.P1Tangent, Alpha);
					BubblePos -= BubbleSize * 0.5f;

					FSlateDrawElement::MakeBox(
						DrawElementsList,
						InParams.LayerId,
						FPaintGeometry(BubblePos, BubbleSize, ZoomFactor),
						BubbleImage,
						ESlateDrawEffect::None,
						InParams.ConnectionData.Params.WireColor
					);
				}
			}
		}

		void FGraphConnectionDrawingPolicy::DrawConnectionSignal_Envelope(const FDrawConnectionSignalData& InParams, FName InAnalyzerName)
		{
			using namespace Frontend;

			TSharedPtr<FEditor> Editor = GetEditor();
			if (!Editor)
			{
				return;
			}

			const float PointSpacing = FMath::Max(1.0f, InParams.SpacingFactor * ZoomFactor);
			const float PointSize = ZoomFactor * InParams.ConnectionData.Params.WireThickness;
			const float PointOffset = FMath::Fmod(InParams.AppTime * ZoomFactor, PointSpacing);
			const int32 NumPoints = FMath::Min(FMath::CeilToInt(InParams.SplineLength / PointSpacing), DrawingPolicyPrivate::EnvelopeConnectionMaxPointsPerConnection);

			TArray<float> EnvMagnitudes;
			EnvMagnitudes.Init(PointSize, NumPoints);
			if (const UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(InParams.ConnectionData.OutputPin->GetOwningNode()))
			{
				if (const UEdGraph* Graph = Node->GetGraph())
				{
					if (MetasoundEditor.IsValid())
					{
						FConstOutputHandle OutputHandle = FGraphBuilder::FindReroutedConstOutputHandleFromPin(InParams.ConnectionData.OutputPin);
						const FGuid NodeId = OutputHandle->GetOwningNodeID();
						FName OutputName = OutputHandle->GetName();

						const float SpeedFactor = FMath::Clamp(1.0f - InParams.SpeedFactor, 0.0f, 1.0f);
						const int32 NumTrackedPoints = FMath::Max(1, (int32)(SpeedFactor * NumPoints));

						FGraphConnectionManager& ConnectionManager = Editor->GetConnectionManager();
						if (const FFloatMovingWindow* Window = ConnectionManager.GetValueWindow(NodeId, OutputName, InAnalyzerName))
						{
							if (NumTrackedPoints > Window->Num())
							{
								ConnectionManager.TrackValue(NodeId, OutputName, InAnalyzerName, NumTrackedPoints);
							}

							auto SampleWindowFunction = [&Window, this](TArrayView<float> SampleEnvelope)
							{
								switch(AnalyzerSettings.EnvelopeDirection)
								{
									case EMetasoundActiveAnalyzerEnvelopeDirection::FromSourceOutput:
									{
										for (int32 i = 0; i < SampleEnvelope.Num(); ++i)
										{
											const float ClampedScalar = FMath::Clamp(Window->GetValueWrapped(i), 0.0f, 1.0f) * AnalyzerSettings.WireScalarMax;
											SampleEnvelope[i] *= ClampedScalar;
										}
									}
									break;

									case EMetasoundActiveAnalyzerEnvelopeDirection::FromDestinationInput:
									{
										for (int32 i = 0; i < SampleEnvelope.Num(); ++i)
										{
											const float ClampedScalar = FMath::Clamp(Window->GetValueWrapped(SampleEnvelope.Num() - 1 - i), 0.0f, 1.0f) * AnalyzerSettings.WireScalarMax;
											SampleEnvelope[i] *= ClampedScalar;
										}
									}
									break;

									default:
									{
										checkNoEntry();
									}
									break;
								}
							};

							if (FMath::IsNearlyEqual(SpeedFactor, 1.0f))
							{
								SampleWindowFunction(EnvMagnitudes);
							}
							else
							{
								TArray<float> EnvSampled;
								EnvSampled.Init(PointSize, NumTrackedPoints);

								SampleWindowFunction(EnvSampled);

								const float StepRatio = ((float)EnvSampled.Num()) / EnvMagnitudes.Num();
								for (int32 i = 0; i < EnvMagnitudes.Num(); ++i)
								{
									EnvMagnitudes[i] = StepRatio * i;
								}
								constexpr auto InterpMode = WaveTable::FWaveTableSampler::EInterpolationMode::Cubic;
								WaveTable::FWaveTableSampler::Interpolate(EnvSampled, EnvMagnitudes, InterpMode);
							}
						}
						else
						{
							ConnectionManager.TrackValue(NodeId, OutputName, InAnalyzerName, NumTrackedPoints);
						}
					}
				}
			}

			TArray<FVector2f> PointsUp;
			TArray<FVector2f> PointsDown;
			FVector2f LastPointPos = FVector2f::Zero();
			FVector2f LastTangent = FVector2f::Zero();

			auto AddPoint = [&](const float EnvMagnitude, const float Distance)
			{
				const float Alpha = InParams.SplineReparamTable.Eval(Distance, 0.f);
				FVector2f PointPos = FMath::CubicInterp(InParams.ConnectionData.P0, InParams.ConnectionData.P0Tangent, InParams.ConnectionData.P1, InParams.ConnectionData.P1Tangent, Alpha);

				LastTangent = FVector2f(PointPos.X - LastPointPos.X, PointPos.Y - LastPointPos.Y);
				const FVector2f Normal = FVector2f(LastTangent.Y, -1.0f * LastTangent.X).GetSafeNormal();
				LastPointPos = PointPos;

				const float HalfEnvMagnitude = EnvMagnitude * 0.5f;
				const FVector2f Top = { static_cast<float>(PointPos.X - (Normal.X * HalfEnvMagnitude)), static_cast<float>(PointPos.Y - (Normal.Y * HalfEnvMagnitude)) };
				const FVector2f Bottom = { static_cast<float>(PointPos.X + (Normal.X * HalfEnvMagnitude)), static_cast<float>(PointPos.Y + (Normal.Y * HalfEnvMagnitude)) };

				if (Top.Equals(Bottom))
				{
					PointsUp.Add(Top);
					PointsDown.Add(Bottom);
				}
				else
				{
					PointsUp.Add(Top);
					PointsUp.Add(Bottom);
					PointsDown.Add(Bottom);
					PointsDown.Add(Top);
				}
			};

			for (int32 i = 0; i < EnvMagnitudes.Num(); ++i)
			{
				const float Distance = (i * PointSpacing) + PointOffset;
				if (Distance < InParams.SplineLength)
				{
					AddPoint(EnvMagnitudes[i], Distance);
				}
			}

			bool bDrawRemainingSpline = false;
			const float TermDistance = (EnvMagnitudes.Num() * PointSpacing) + PointOffset;
			if (TermDistance < InParams.SplineLength)
			{
				AddPoint(0.0f, TermDistance);
				bDrawRemainingSpline = true;
			}
			else
			{
				AddPoint(0.0f, InParams.SplineLength);
			}

			FSlateDrawElement::MakeLines(DrawElementsList, InParams.LayerId, FPaintGeometry(), PointsUp, ESlateDrawEffect::None, InParams.ConnectionData.Params.WireColor, true, InParams.ConnectionData.Params.WireThickness);
			FSlateDrawElement::MakeLines(DrawElementsList, InParams.LayerId, FPaintGeometry(), PointsDown, ESlateDrawEffect::None, InParams.ConnectionData.Params.WireColor, true, InParams.ConnectionData.Params.WireThickness);
			if (bDrawRemainingSpline)
			{
				FDrawConnectionData FinalData = InParams.ConnectionData;
				FinalData.P0 = LastPointPos;
				FinalData.P0Tangent = LastTangent;
				DrawSpline(FinalData);
			}
		}

		void FGraphConnectionDrawingPolicy::DrawSpline(const FDrawConnectionData& InData)
		{
			FSlateDrawElement::MakeDrawSpaceSpline(
				DrawElementsList,
				WireLayerID,
				InData.P0, InData.P0Tangent,
				InData.P1, InData.P1Tangent,
				InData.Params.WireThickness,
				ESlateDrawEffect::None,
				InData.Params.WireColor
			);
		}

		void FGraphConnectionDrawingPolicy::DrawAnimatedSpline(const FDrawConnectionData& InData)
		{
			FSlateDrawElement::MakeDrawSpaceSpline(
				DrawElementsList,
				WireAnimationLayerID,
				InData.P0, InData.P0Tangent,
				InData.P1, InData.P1Tangent,
				InData.Params.WireThickness,
				ESlateDrawEffect::None,
				InData.Params.WireColor
			);
		}
	} // namespace Editor
} // namespace Metasound
