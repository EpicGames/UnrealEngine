// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ConnectionDrawingPolicy.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "Layout/ArrangedWidget.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontendController.h"
#include "Misc/CoreMiscDefines.h"
#include "Rendering/DrawElements.h"
#include "Templates/Function.h"
#include "Widgets/SWidget.h"


// Forward Declarations
class FSlateRect;
class FSlateWindowElementList;
class UEdGraph;
class UEdGraphSchema;


namespace Metasound
{
	namespace Editor
	{
		// Forward Declarations
		class FEditor;
		class FGraphConnectionManager;

		struct FDrawConnectionData
		{
			FDeprecateSlateVector2D P0;
			FDeprecateSlateVector2D P0Tangent;
			FDeprecateSlateVector2D P1;
			FDeprecateSlateVector2D P1Tangent;

			FConnectionParams Params;

			FBox2f Bounds;

			const UEdGraphPin* OutputPin = nullptr;
			const FMetasoundFrontendVertex* OutputVertex = nullptr;
			const FMetasoundFrontendNode* OutputNode = nullptr;

			bool bCloseToSpline = false;
			float ClosestDistanceSquared = TNumericLimits<float>::Max();

			// Distance to consider as an overlap
			float QueryDistanceTriggerThresholdSquared = 0.0f;

			// Distance to pass the bounding box cull test. This is used for the bCloseToSpline output that can be used as a
			// dead zone to avoid mistakes caused by missing a double-click on a connection.
			float QueryDistanceForCloseSquared = 0.0f;

			FDeprecateSlateVector2D ClosestPoint;

			const FMetasoundFrontendEdgeStyle* EdgeStyle = nullptr;

			FDrawConnectionData(const UE::Slate::FDeprecateVector2DParameter& InStart, const UE::Slate::FDeprecateVector2DParameter& InEnd, const FEditor& Editor, const UE::Slate::FDeprecateVector2DParameter& InSplineTangent, const FConnectionParams& InParams, const UGraphEditorSettings& InSettings, const UE::Slate::FDeprecateVector2DParameter& InMousePosition);

			void UpdateSplineOverlap(FGraphSplineOverlapResult& OutResult) const;
		};

		struct FGraphConnectionDrawingPolicyFactory : public FGraphPanelPinConnectionFactory
		{
		public:
			virtual ~FGraphConnectionDrawingPolicyFactory() = default;

			// FGraphPanelPinConnectionFactory
			virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(
				const UEdGraphSchema* Schema,
				int32 InBackLayerID,
				int32 InFrontLayerID,
				float ZoomFactor,
				const FSlateRect& InClippingRect,
				FSlateWindowElementList& InDrawElements,
				UEdGraph* InGraphObj) const override;
			// ~FGraphPanelPinConnectionFactory
		};

		// Draws the connections for a UMetasoundEditorGraph using a UMetasoundEditorGraphSchema
		class FGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
		{
		protected:
			// Times for one execution pair within the current graph
			struct FTimePair
			{
				double PredExecTime = 0.0;
				double ThisExecTime = 0.0;
			};

			// Map of pairings
			using FExecPairingMap = TMap<UEdGraphNode*, FTimePair>;

			// Map of nodes that preceded before a given node in the execution sequence (one entry for each pairing)
			TMap<UEdGraphNode*, FExecPairingMap> PredecessorNodes;

			UEdGraph* GraphObj = nullptr;

			FLinearColor ActiveColor;
			FLinearColor InactiveColor;

			int32 WireAnimationLayerID = 0;

			FMetasoundAnalyzerAnimationSettings AnalyzerSettings;

			TWeakPtr<FEditor> MetasoundEditor;

		public:
			FGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

			// FConnectionDrawingPolicy interface
			virtual void DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& InParams) override;
			virtual void DetermineWiringStyle(UEdGraphPin* InPinA, UEdGraphPin* InPinB, FConnectionParams& OutParams) override;
			// End of FConnectionDrawingPolicy interface

			virtual void DrawAnimatedSpline(const FDrawConnectionData& InData);
			virtual void DrawSpline(const FDrawConnectionData& InData);
			virtual void DrawConnectionSpline(int32 InLayerId, const FDrawConnectionData& InData);

			// Copied from FConnectionDrawingPolicy::ApplyHoverDeemphasis, with modification to only apply thickness emphasis when not playing
			void ApplyHoverDeemphasisMetaSound(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, const bool bIsPlaying, /*inout*/ float& Thickness, /*inout*/ FLinearColor& WireColor);

			const TSharedPtr<const FEditor> GetEditor() const;
			TSharedPtr<FEditor> GetEditor();

		protected:

			struct FDrawConnectionSignalData
			{
				const float AppTime = 0.0f;
				const int32 LayerId = 0;
				const FInterpCurve<float>& SplineReparamTable;
				const float SplineLength = 0.0f;
				const FDrawConnectionData& ConnectionData;

				float SpacingFactor = 64.0f;
				float SpeedFactor = 192.0f;
			};

			void ScaleEnvelopeWireThickness(FConnectionParams& OutParams) const;

			virtual void DrawConnectionSignal(const FDrawConnectionSignalData& InParams);
			virtual void DrawConnectionSignal_Envelope(const FDrawConnectionSignalData& InParams, FName InAnalyzerName);
			virtual void DrawConnection_MidpointImage(int32 InLayerId, const FInterpCurve<float>& SplineReparamTable, float InSplineLength, const FDrawConnectionData& InData);
		};
	} // namespace Editor
} // namespace Metasound
