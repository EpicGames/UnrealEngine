// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphConnectionManager.h"

#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerForwardValue.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerDensity.h"
#include "Analysis/MetasoundVertexAnalyzerAudioBusWriter.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundVertex.h"


namespace Metasound
{
	namespace Editor
	{
		FGraphConnectionManager::FGraphConnectionManager(const FMetasoundAssetBase& InAssetBase, const UAudioComponent& InAudioComponent, uint64 InTransmitterID, const FOperatorSettings& InOperatorSettings)
			: AudioComponent(&InAudioComponent)
			, GraphAnalyzerView(MakeUnique<Frontend::FMetasoundGraphAnalyzerView>(InAssetBase, InTransmitterID, InOperatorSettings))
		{
			using namespace Frontend;

			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerForwardBool::GetAnalyzerName());
			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerForwardFloat::GetAnalyzerName());
			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerForwardInt::GetAnalyzerName());
			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerForwardString::GetAnalyzerName());

			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerEnvelopeFollower::GetAnalyzerName());
			GraphAnalyzerView->AddAnalyzerForAllSupportedOutputs(FVertexAnalyzerTriggerDensity::GetAnalyzerName());
		}

		bool FGraphConnectionManager::GetValue(const FGuid& InNodeID, FVertexName InOutputName, float& OutValue) const
		{
			const FString AnalyzerKey = GetAnalyzerKey<Frontend::FVertexAnalyzerForwardFloat>(InNodeID, InOutputName);
			if (!AnalyzerKey.IsEmpty())
			{
				if (const Audio::FVolumeFader* Fader = ConnectionFaders.Find(AnalyzerKey))
				{
					OutValue = Fader->GetVolume();
					return true;
				}
			}

			return false;
		}

		bool FGraphConnectionManager::GetValue(const FGuid& InNodeID, FVertexName InOutputName, bool& OutValue) const
		{
			return GetPrimativeValue<Frontend::FVertexAnalyzerForwardBool, bool>(InNodeID, InOutputName, BoolConnectionValues, OutValue);
		}

		bool FGraphConnectionManager::GetValue(const FGuid& InNodeID, FVertexName InOutputName, int32& OutValue) const
		{
			return GetPrimativeValue<Frontend::FVertexAnalyzerForwardInt, int32>(InNodeID, InOutputName, IntConnectionValues, OutValue);
		}

		bool FGraphConnectionManager::GetValue(const FGuid& InNodeID, FVertexName InOutputName, FString& OutValue) const
		{
			return GetPrimativeValue<Frontend::FVertexAnalyzerForwardString, FString>(InNodeID, InOutputName, StringConnectionValues, OutValue);
		}

		const FFloatMovingWindow* FGraphConnectionManager::GetValueWindow(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName) const
		{
			return WindowedValues.Find({ InNodeID, InOutputName, InAnalyzerName });
		}

		bool FGraphConnectionManager::IsTracked(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName) const
		{
			return WindowedValues.Contains({ InNodeID, InOutputName, InAnalyzerName });
		}

		void FGraphConnectionManager::TrackValue(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, int32 InWindowSize)
		{
			FFloatMovingWindow& Window = WindowedValues.FindOrAdd({ InNodeID, InOutputName, InAnalyzerName });
			Window.SetNumZeroed(InWindowSize);
		}

		FGuid FGraphConnectionManager::AddAudioBusWriter(const FGuid& InNodeID, FVertexName InOutputName, Audio::FDeviceId InDeviceID, const UAudioBus* InAudioBus)
		{
			FGuid AnalyzerInstanceID;

			if (GraphAnalyzerView.IsValid())
			{
				const uint32 AudioBusID = InAudioBus->GetUniqueID();
				const FName AnalyzerName = Engine::FVertexAnalyzerAudioBusWriter::GetAnalyzerName();
				const FName AnalyzerMemberName = Engine::FVertexAnalyzerAudioBusWriter::GetAnalyzerMemberName(InDeviceID, AudioBusID);
				AnalyzerInstanceID = GraphAnalyzerView->AddAnalyzerForSpecifiedOutput(InNodeID, InOutputName, AnalyzerName, AnalyzerMemberName);
			}

			return AnalyzerInstanceID;
		}

		void FGraphConnectionManager::RemoveAudioBusWriter(const FGuid& InAnalyzerInstanceID)
		{
			if (GraphAnalyzerView.IsValid())
			{
				const FName AnalyzerName = Engine::FVertexAnalyzerAudioBusWriter::GetAnalyzerName();
				GraphAnalyzerView->RemoveAnalyzerInstance(AnalyzerName, InAnalyzerInstanceID);
			}
		}

		bool FGraphConnectionManager::HasAudioBusWriter(const FGuid& InAnalyzerInstanceID) const
		{
			bool bResult = false;

			if (GraphAnalyzerView.IsValid())
			{
				const FName AnalyzerName = Engine::FVertexAnalyzerAudioBusWriter::GetAnalyzerName();
				bResult = GraphAnalyzerView->HasAnalyzerInstance(AnalyzerName, InAnalyzerInstanceID);
			}

			return bResult;
		}

		void FGraphConnectionManager::Update(float InDeltaTime)
		{
			using namespace Frontend;

			if (AudioComponent.IsValid())
			{
				float FadeDuration = 0.0001f;
				auto UpdateSmoothedFloat = [this, FadeDur = &FadeDuration](const FAnalyzerAddress& AnalyzerAddress, const float& NewValue)
				{
					const FString AnalyzerKey = AnalyzerAddress.ToString();
					Audio::FVolumeFader* Fader = ConnectionFaders.Find(AnalyzerKey);
					if (!Fader)
					{
						Audio::FVolumeFader NewFader;
						NewFader.SetVolume(0.0f);
						Fader = &ConnectionFaders.Add(AnalyzerKey, MoveTemp(NewFader));
					}

					if (!FMath::IsNearlyEqual(NewValue, Fader->GetTargetVolume(), UE_KINDA_SMALL_NUMBER))
					{
						constexpr Audio::EFaderCurve Curve = Audio::EFaderCurve::Linear;

						Fader->StartFade(NewValue, *FadeDur, Curve);
					}
				};

				UpdateConnections<FVertexAnalyzerForwardFloat, float>(InDeltaTime, UpdateSmoothedFloat);
				UpdateConnections<FVertexAnalyzerForwardInt, int32>(InDeltaTime, IntConnectionValues);
				UpdateConnections<FVertexAnalyzerForwardBool, bool>(InDeltaTime, BoolConnectionValues);
				UpdateConnections<FVertexAnalyzerForwardString, FString>(InDeltaTime, StringConnectionValues);

				auto UpdateWindowedFloat = [this, &UpdateSmoothedFloat](const FAnalyzerAddress& AnalyzerAddress, const float& NewValue)
				{
					UpdateSmoothedFloat(AnalyzerAddress, NewValue);
					WindowedValues.FindOrAdd({ AnalyzerAddress.NodeID, AnalyzerAddress.OutputName, AnalyzerAddress.AnalyzerName });
				};
				UpdateConnections<FVertexAnalyzerEnvelopeFollower, float>(InDeltaTime, UpdateWindowedFloat);

				FadeDuration = 0.01f;
				UpdateConnections<FVertexAnalyzerTriggerDensity, float>(InDeltaTime, UpdateWindowedFloat);

				for (TPair<FString, Audio::FVolumeFader>& Pair : ConnectionFaders)
				{
					Pair.Value.Update(InDeltaTime);
				}
			}
		}

		float FGraphConnectionManager::UpdateValueWindow(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, FName InAnalyzerOutputName)
		{
			using namespace Frontend;

			float CurrentValue = 0.0f;

			TArray<FMetasoundAnalyzerView*> Views = GraphAnalyzerView->GetAnalyzerViews(InAnalyzerName);
			for (const FMetasoundAnalyzerView* View : Views)
			{
				check(View);
				const FAnalyzerAddress& Address = View->AnalyzerAddress;
				const bool bOutputsMatch = Address.AnalyzerMemberName == InAnalyzerOutputName
					&& Address.OutputName == InOutputName
					&& Address.NodeID == InNodeID;
				if (bOutputsMatch)
				{
					const FString AnalyzerKey = Address.ToString();
					if (Audio::FVolumeFader* Fader = ConnectionFaders.Find(AnalyzerKey))
					{
						CurrentValue = Fader->GetVolume();
						FFloatMovingWindow* Window = WindowedValues.Find({ Address.NodeID, Address.OutputName, Address.AnalyzerName });
						if (Window && !Window->IsEmpty())
						{
							Window->SetValue(CurrentValue);
							return CurrentValue;
						}
					}
				}
			}

			return CurrentValue;
		}
	} // namespace Editor
} // Metasound
