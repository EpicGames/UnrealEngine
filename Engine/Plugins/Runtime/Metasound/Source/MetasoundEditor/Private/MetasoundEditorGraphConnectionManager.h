// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "Analysis/MetasoundFrontendGraphAnalyzerView.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerForwardValue.h"
#include "Components/AudioComponent.h"
#include "DSP/Dsp.h"
#include "DSP/MultithreadedPatching.h"
#include "DSP/VolumeFader.h"
#include "HAL/Platform.h"
#include "MetasoundAssetBase.h"
#include "MetasoundOperatorSettings.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "UObject/WeakObjectPtrTemplates.h"


// Forward Declarations
class UEdGraphPin;

namespace Metasound
{
	namespace Editor
	{
		class FFloatMovingWindow
		{
			TArray<float> Data;
			int32 WriteIndex = -1;

		public:
			bool IsEmpty() const
			{
				return Data.IsEmpty();
			}

			int32 Num() const
			{
				return Data.Num();
			}

			void SetNumZeroed(int32 InNum)
			{
				check(InNum >= 0);

				Data.SetNumZeroed(InNum, EAllowShrinking::No);

				// Initialize to back of array to avoid initial
				// value being behind when copying to initial array.
				if (WriteIndex < 0)
				{
					WriteIndex = Data.Num() - 1;
				}
				else
				{
					WriteIndex %= Data.Num();
				}
			}

			void SetValue(float InValue)
			{
				check(!Data.IsEmpty());
				Data[WriteIndex] = InValue;
				WriteIndex--;
				if (WriteIndex < 0)
				{
					WriteIndex += Data.Num();
				}
			}

			float GetValueWrapped(int32 Index) const
			{
				check(!Data.IsEmpty());

				const int32 WrappedIndex = (WriteIndex + Index) % Data.Num();
				return Data[WrappedIndex];
			}
		};

		class FGraphConnectionManager
		{
			FGraphConnectionManager(const FGraphConnectionManager&) = delete;

		public:
			FGraphConnectionManager() = default;
			~FGraphConnectionManager() = default;

			FGraphConnectionManager(const FMetasoundAssetBase& InAssetBase, const UAudioComponent& InAudioComponent, uint64 InTransmitterID, const FOperatorSettings& InOperatorSettings);
			bool GetValue(const FGuid& InNodeID, FVertexName InOutputName, float& OutValue) const;
			bool GetValue(const FGuid& InNodeID, FVertexName InOutputName, bool& OutValue) const;
			bool GetValue(const FGuid& InNodeID, FVertexName InOutputName, int32& OutValue) const;
			bool GetValue(const FGuid& InNodeID, FVertexName InOutputName, FString& OutValue) const;

			const FFloatMovingWindow* GetValueWindow(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName) const;

			bool IsTracked(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName) const;

			// Marks to track (if not yet tracked) & sets the window size of cached values.
			// Retains existing values provided from update up to given size.
			void TrackValue(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, int32 InWindowSize);

			// Send the audio from the given node output to the given bus. If successful, an AnalyzerInstanceID will be returned.
			FGuid AddAudioBusWriter(const FGuid& InNodeID, FVertexName InOutputName, Audio::FDeviceId InDeviceID, const UAudioBus* InAudioBus);

			// Remove the specified analyzer. InAnalyzerInstanceID is an FGuid that was previously returned from a call to AddAudioBusWriter(...).
			void RemoveAudioBusWriter(const FGuid& InAnalyzerInstanceID);

			// Does this connection manager have the specified analyzer? InAnalyzerInstanceID is an FGuid that was previously returned from a call to AddAudioBusWriter(...).
			bool HasAudioBusWriter(const FGuid& InAnalyzerInstanceID) const;

			void Update(float InDeltaTime);

			template <typename TAnalyzerName>
			float UpdateValueWindow(const FGuid& InNodeID, FVertexName InOutputName)
			{
				using namespace Frontend;
				return UpdateValueWindow(InNodeID, InOutputName, TAnalyzerName::GetAnalyzerName(), TAnalyzerName::FOutputs::GetValue().Name);
			}

		private:
			template <typename ForwardValueAnalyzerClass>
			FString GetAnalyzerKey(const FGuid& InNodeID, FVertexName InOutputName) const
			{
				using namespace Frontend;

				const Frontend::FMetasoundGraphAnalyzerView* GraphView = GraphAnalyzerView.Get();
				if (GraphView)
				{
					TArray<const FMetasoundAnalyzerView*> Views = GraphView->GetAnalyzerViewsForOutput(InNodeID, InOutputName, ForwardValueAnalyzerClass::GetAnalyzerName());
					if (!Views.IsEmpty())
					{
						const FMetasoundAnalyzerView** View = Views.FindByPredicate([](const FMetasoundAnalyzerView* Candidate)
						{
							check(Candidate);
							return Candidate->AnalyzerAddress.AnalyzerMemberName == ForwardValueAnalyzerClass::FOutputs::GetValue().Name;
						});

						if (View)
						{
							check(*View);
							const FAnalyzerAddress& Address = (*View)->AnalyzerAddress;
							const FString AnalyzerKey = Address.ToString();
							return AnalyzerKey;
						}
					}
				}
				return { };
			}

			template<typename TVertexAnalyzerValue, typename TDataType>
			bool GetPrimativeValue(const FGuid& InNodeID, FVertexName InOutputName, const TMap<FString, TDataType>& InDataMap, TDataType& OutValue) const
			{
				const FString AnalyzerKey = GetAnalyzerKey<TVertexAnalyzerValue>(InNodeID, InOutputName);
				if (!AnalyzerKey.IsEmpty())
				{
					if (const TDataType* Value = InDataMap.Find(AnalyzerKey))
					{
						OutValue = *Value;
						return true;
					}
				}

				return false;
			}

			// Updates the value window, returning the most recent value
			float UpdateValueWindow(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, FName InAnalyzerOutputName);

			template <typename ForwardValueAnalyzerClass, typename AnalyzerOutputDataType>
			void UpdateConnections(float InDeltaTime, TFunctionRef<void(const Frontend::FAnalyzerAddress& /* AnalyzerAddress */, const AnalyzerOutputDataType& /* NewValue */)> InUpdateFunction)
			{
				using namespace Frontend;

				TArray<FMetasoundAnalyzerView*> Views = GraphAnalyzerView->GetAnalyzerViews(ForwardValueAnalyzerClass::GetAnalyzerName());
				for (FMetasoundAnalyzerView* View : Views)
				{
					check(View);
					const FAnalyzerAddress& Address = View->AnalyzerAddress;
					const bool bOutputsMatch = Address.AnalyzerMemberName == ForwardValueAnalyzerClass::FOutputs::GetValue().Name;
					if (bOutputsMatch)
					{
						AnalyzerOutputDataType Value;
						if (View->TryGetOutputData<AnalyzerOutputDataType>(Address.AnalyzerMemberName, Value))
						{
							InUpdateFunction(Address, Value);
						}
					}
				}
			}

			template <typename ForwardValueAnalyzerClass, typename AnalyzerOutputDataType>
			void UpdateConnections(float InDeltaTime, TMap<FString, AnalyzerOutputDataType>& OutMap)
			{
				UpdateConnections<ForwardValueAnalyzerClass, AnalyzerOutputDataType>(InDeltaTime, [this, &OutMap](const Frontend::FAnalyzerAddress& AnalyzerAddress, const AnalyzerOutputDataType& NewValue)
				{
					FString AnalyzerKey = AnalyzerAddress.ToString();
					OutMap.FindOrAdd(AnalyzerKey) = NewValue;
				});
			}

			struct FWindowValueKey
			{
				const FGuid NodeID;
				const FName OutputName;
				const FName AnalyzerName;

				friend FORCEINLINE uint32 GetTypeHash(const FWindowValueKey& Key)
				{
					uint32 Hash = GetTypeHash(Key.NodeID);
					Hash = HashCombineFast(Hash, GetTypeHash(Key.OutputName));
					Hash = HashCombineFast(Hash, GetTypeHash(Key.AnalyzerName));
					return Hash;
				}

				FORCEINLINE bool operator==(const FWindowValueKey& Other) const
				{
					return (NodeID == Other.NodeID) && (OutputName == Other.OutputName) && (AnalyzerName == Other.AnalyzerName);
				}

				FORCEINLINE bool operator!=(const FWindowValueKey& Other) const
				{
					return !(*this == Other);
				}
			};

			struct FAudioBufferKey : FWindowValueKey{};

			TMap<FWindowValueKey, FFloatMovingWindow> WindowedValues;

			TWeakObjectPtr<const UAudioComponent> AudioComponent;
			TMap<FString, Audio::FVolumeFader> ConnectionFaders;

			TMap<FString, int32> IntConnectionValues;
			TMap<FString, bool> BoolConnectionValues;
			TMap<FString, FString> StringConnectionValues;

			TUniquePtr<Frontend::FMetasoundGraphAnalyzerView> GraphAnalyzerView;
		};
	} // namespace Editor
} // Metasound
