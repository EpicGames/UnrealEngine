// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphInputNode.h"

#include "Algo/Count.h"
#include "Algo/Transform.h"
#include "AudioParameter.h"
#include "AudioParameterControllerInterface.h"
#include "EdGraph/EdGraphNode.h"
#include "GraphEditorSettings.h"
#include "MetasoundDataReference.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSettings.h"
#include "Misc/Guid.h"
#include "Sound/SoundWave.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraphInputNode)

#define LOCTEXT_NAMESPACE "MetaSoundEditor"

void UMetasoundEditorGraphInputNode::CacheTitle()
{
	static FText InputDisplayTitle = LOCTEXT("InputNode_Title", "Input");
	CachedTitle = InputDisplayTitle;
}

const FMetasoundEditorGraphVertexNodeBreadcrumb& UMetasoundEditorGraphInputNode::GetBreadcrumb() const
{
	return Breadcrumb;
}

void UMetasoundEditorGraphInputNode::CacheBreadcrumb()
{
	using namespace Metasound;

	Breadcrumb = { };

	// Take data from associated input as pasted graph may not be same as local graph
	// and associated input will not be copied with given node.  Need the following data
	// to associate or create new associated input.
	if (Input)
	{
		Breadcrumb.MemberName = Input->GetMemberName();

		FMetaSoundFrontendDocumentBuilder& Builder = Input->GetFrontendBuilderChecked();
		if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(Breadcrumb.MemberName))
		{
			if (const FMetasoundFrontendNode* Node = Builder.FindGraphInputNode(Breadcrumb.MemberName))
			{
				if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
				{
					Breadcrumb.ClassName = Class->Metadata.GetClassName();
					Breadcrumb.AccessType = ClassInput->AccessType;
					Breadcrumb.DataType = ClassInput->TypeName;
					Breadcrumb.VertexMetadata = ClassInput->Metadata;
					if (UMetaSoundFrontendMemberMetadata* MemberMetadata = Builder.FindMemberMetadata(Node->GetID()))
					{
						Breadcrumb.MemberMetadataPath = FSoftObjectPath(MemberMetadata);
					}
					ClassInput->IterateDefaults([this](const FGuid& PageID, const FMetasoundFrontendLiteral& Literal)
					{
						Breadcrumb.DefaultLiterals.Add(PageID, Literal);
					});
					
				}
			}
		}
	}
}

UMetasoundEditorGraphMember* UMetasoundEditorGraphInputNode::GetMember() const
{
	return Input;
}

FMetasoundFrontendClassName UMetasoundEditorGraphInputNode::GetClassName() const
{
	using namespace Metasound::Frontend;

	if (Input)
	{
		const FMetaSoundFrontendDocumentBuilder& Builder = Input->GetFrontendBuilderChecked();
		if (const FMetasoundFrontendNode* Node = Builder.FindNode(Input->NodeID))
		{
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
			{
				return Class->Metadata.GetClassName();
			}
		}
	}

	return Breadcrumb.ClassName;
}

void UMetasoundEditorGraphInputNode::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	if (Input)
	{
		if (UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Input->GetLiteral())
		{
			DefaultLiteral->UpdatePreviewInstance(InParameterName, InParameterInterface);
		}
	}
}

FText UMetasoundEditorGraphInputNode::GetDisplayName() const
{
	const FMetaSoundFrontendDocumentBuilder& Builder = Input->GetFrontendBuilderChecked();
	if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(Input->GetMemberName()))
	{
		return ClassInput->Metadata.GetDisplayName();
	}
	return FText::GetEmpty();
}

FGuid UMetasoundEditorGraphInputNode::GetNodeID() const
{
	return NodeID;
}

FLinearColor UMetasoundEditorGraphInputNode::GetNodeTitleTextColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->InputNodeTitleColor;
	}

	return Super::GetNodeTitleColor();
}

const FSlateBrush* UMetasoundEditorGraphInputNode::GetNodeTitleIconBrush() const
{
	const FSlateIcon InputIcon = FSlateIcon("MetaSoundStyle", "MetasoundEditor.Graph.Node.Class.Input");
	return InputIcon.GetIcon();
}

void UMetasoundEditorGraphInputNode::GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (ensure(Pin.Direction == EGPD_Output)) // Should never display input pin for input node hover
	{
		if (ensure(Input))
		{
			OutHoverText = Input->GetDescription().ToString();
			if (ShowNodeDebugData())
			{
				if (const FMetasoundFrontendClassVertex* Vertex = Input->GetFrontendClassVertex())
				{
					OutHoverText = FString::Format(TEXT("Description: {0}\nVertex Name: {1}\nDataType: {2}\nID: {3}"),
					{
						OutHoverText,
						Vertex->Name.ToString(),
						Vertex->TypeName.ToString(),
						Vertex->NodeID.ToString(),
					});
				}
			}
		}
	}
}

void UMetasoundEditorGraphInputNode::ReconstructNode()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	Super::ReconstructNode();
}

void UMetasoundEditorGraphInputNode::Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	Super::Validate(InBuilder, OutResult);

	if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GetMember()))
	{
		FMetasoundFrontendInterface InterfaceToValidate;
		if (Vertex->IsInterfaceMember(&InterfaceToValidate))
		{
			FText RequiredText;
			// This function checks there's a node output connected to the inspected Input, so it calls IsMemberInputRequired and considers what Outputs are connected.
			if (InterfaceToValidate.IsMemberInputRequired(Vertex->GetMemberName(), RequiredText))
			{
				if (const FMetasoundFrontendNode* Node = GetFrontendNode())
				{
					const TArray<FMetasoundFrontendVertex>& Outputs = Node->Interface.Outputs;
					if (ensure(!Outputs.IsEmpty()))
					{
						if (!InBuilder.IsNodeOutputConnected(Node->GetID(), Outputs.Last().VertexID))
						{
							OutResult.SetMessage(EMessageSeverity::Warning, *RequiredText.ToString());
						}
					}
				}
			}
		}
	}
}

FText UMetasoundEditorGraphInputNode::GetTooltipText() const
{
	//If Constructor input
	if (Input)
	{
		if (Input->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Value)
		{
			UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(GetGraph());
			if (Graph->IsPreviewing())
			{
				FText ToolTip = LOCTEXT("Metasound_ConstructorInputNodeDescription", "Editing constructor values is disabled while previewing.");
				return ToolTip;
			}
		}

		if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Input->GetLiteral())
		{
			bool bPageDefaultImplemented = false;
			const FMetaSoundFrontendDocumentBuilder& Builder = Input->GetFrontendBuilderChecked();
			const FGuid& BuildPageID = Builder.GetBuildPageID();
			Literal->IterateDefaults([&bPageDefaultImplemented, &BuildPageID](const FGuid& InDefaultPageID, FMetasoundFrontendLiteral)
			{
				bPageDefaultImplemented |= InDefaultPageID == BuildPageID;
			});

			if (!bPageDefaultImplemented)
			{
				const UMetaSoundSettings* MetaSoundSettings = GetDefault<UMetaSoundSettings>();
				check(MetaSoundSettings);

				const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
				check(EditorSettings);

				if (const FMetaSoundPageSettings* PageSettings = MetaSoundSettings->FindPageSettings(BuildPageID))
				{
					if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(Input->GetMemberName()))
					{
						const FGuid FallbackPageID = EditorSettings->ResolveAuditionPage(*ClassInput, BuildPageID);
						if (const FMetaSoundPageSettings* FallbackSettings = MetaSoundSettings->FindPageSettings(FallbackPageID))
						{
							return FText::Format(
								LOCTEXT("DefaultPageValueDisabledNotImplemented",
									"No '{0}' page default value implemented.\r\n"
									"Showing platform/platform group '{1}' fallback '{2}'.\r\n"
									"(See 'Audition' menu or 'MetaSound' Editor Preferences to change 'Audition Platform')."),
								FText::FromName(PageSettings->Name),
								FText::FromName(EditorSettings->GetAuditionPlatform()),
								FText::FromName(FallbackSettings->Name));
						}
					}
				}
			}
		}

		const FText InputDescription = Input->GetDescription();
		if (!InputDescription.IsEmpty())
		{
			return InputDescription;
		}
	}

	return Super::GetTooltipText();
}

bool UMetasoundEditorGraphInputNode::EnableInteractWidgets() const
{
	using namespace Metasound::Frontend;
	
	if (Input)
	{
		if (Input->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Value)
		{
			UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(GetGraph());
			return !Graph->IsPreviewing();
		}

		bool bPageDefaultImplemented = false;
		const FGuid& BuildPageID = Input->GetFrontendBuilderChecked().GetBuildPageID();
		if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Input->GetLiteral())
		{
			Literal->IterateDefaults([&bPageDefaultImplemented, &BuildPageID](const FGuid& InDefaultPageID, FMetasoundFrontendLiteral)
			{
				bPageDefaultImplemented |= InDefaultPageID == BuildPageID;
			});
		}
		return bPageDefaultImplemented;
	}

	return false;
}
#undef LOCTEXT_NAMESPACE
