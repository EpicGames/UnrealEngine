// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphMemberDefaults.h"

#include "Algo/Count.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "AudioDefines.h"
#include "AudioParameterControllerInterface.h"
#include "EdGraph/EdGraphNode.h"
#include "GraphEditorSettings.h"
#include "MetasoundDataReference.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSettings.h"
#include "Misc/Guid.h"
#include "Sound/SoundWave.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraphMemberDefaults)

namespace Metasound::Editor
{
	namespace MemberDefaultsPrivate
	{
		TMap<FName, const FMetaSoundPageSettings*> GetPageSettingsByName()
		{
			TMap<FName, const FMetaSoundPageSettings*> PageMap;
			if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
			{
				Settings->IteratePageSettings([&PageMap](const FMetaSoundPageSettings& PageSettings)
				{
					PageMap.Add(PageSettings.Name, &PageSettings);
				});
			}
			return PageMap;
		}

		TMap<FGuid, const FMetaSoundPageSettings*> GetPageSettingsByID()
		{
			TMap<FGuid, const FMetaSoundPageSettings*> PageMap;
			if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
			{
				Settings->IteratePageSettings([&PageMap](const FMetaSoundPageSettings& PageSettings)
				{
					PageMap.Add(PageSettings.UniqueId, &PageSettings);
				});
			}
			return PageMap;
		}

		template <typename LiteralType>
		void InitDefault(UMetasoundEditorGraphMemberDefaultLiteral& EditorLiteral, const FGuid& InPageID)
		{
			FMetasoundFrontendLiteral DefaultLiteral;
			DefaultLiteral.SetFromLiteral(Frontend::IDataTypeRegistry::Get().CreateDefaultLiteral(GetMetasoundDataTypeName<LiteralType>()));
			EditorLiteral.SetFromLiteral(DefaultLiteral, InPageID);
		}

		template <typename PageDefaultType, typename MemberType, typename LiteralType = MemberType>
		void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter, TFunctionRef<LiteralType(const MemberType&)> MemberToLiteral, const TArray<PageDefaultType>& Defaults)
		{
			for (const PageDefaultType& PageDefault : Defaults)
			{
				FMetasoundFrontendLiteral Value;
				Value.Set(MemberToLiteral(PageDefault.Value));
				Iter(PageDefault.PageID, MoveTemp(Value));
			}
		}

		template <typename PageDefaultType>
		void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter, const TArray<PageDefaultType>& Defaults)
		{
			for (const PageDefaultType& PageDefault : Defaults)
			{
				FMetasoundFrontendLiteral Value;
				Value.Set(PageDefault.Value);
				Iter(PageDefault.PageID, MoveTemp(Value));
			}
		}

		template <typename PageDefaultType>
		bool RemoveDefault(const FGuid& InPageID, TArray<PageDefaultType>& OutDefaults)
		{
			return OutDefaults.RemoveAllSwap([&InPageID](const FMetasoundEditorMemberPageDefault& PageDefault)
			{
				return PageDefault.PageID == InPageID;
			}) > 0;
		}

		template <typename PageDefaultType>
		void ResolvePageDefaults(TArray<PageDefaultType>& OutPageDefaults)
		{
			TMap<FName, const FMetaSoundPageSettings*> PageSettingsByName = MemberDefaultsPrivate::GetPageSettingsByName();
			if (OutPageDefaults.IsEmpty() || PageSettingsByName.IsEmpty() || PageSettingsByName.Num() == 1)
			{
				OutPageDefaults.SetNum(1);
				OutPageDefaults.Last().PageName = Frontend::DefaultPageName;
				OutPageDefaults.Last().PageID = Frontend::DefaultPageID;
				return;
			}

			// Find duplicates to be removed (this can happen if page renamed to an existing page by user.
			// This will result in a page with the same name but mismatched page ID to given name) and missing
			// page names.
			TArray<FName> MissingPages;
			TArray<FName> DuplicatePages;
			{
				PageSettingsByName.GetKeys(MissingPages);
				for (int32 Index = OutPageDefaults.Num() - 1; Index >= 0; --Index)
				{
					const FName PageName = OutPageDefaults[Index].PageName;
					if (MissingPages.Remove(PageName) == 0)
					{
						DuplicatePages.AddUnique(PageName);
					}
				}
			}

			auto InitPageDefault = [&PageSettingsByName](FName NewPageName, FMetasoundEditorMemberPageDefault& OutDefault)
			{
				const FMetaSoundPageSettings* NewPageSettings = PageSettingsByName.FindChecked(NewPageName);
				check(NewPageSettings);
				OutDefault.PageID = NewPageSettings->UniqueId;
				OutDefault.PageName = NewPageSettings->Name;
			};

			for (int32 Index = OutPageDefaults.Num() - 1; Index >= 0; --Index)
			{
				FMetasoundEditorMemberPageDefault& Default = OutPageDefaults[Index];

				// If user selected assigned a new name, resolve the page ID to be that of the given name
				// and remove other value if it existed
				if (const FMetaSoundPageSettings* PageSettings = PageSettingsByName.FindRef(Default.PageName))
				{
					if (Default.PageID == PageSettings->UniqueId)
					{
						const bool bIsDuplicate = DuplicatePages.RemoveSwap(Default.PageName, EAllowShrinking::No) > 0;
						if (bIsDuplicate)
						{
							OutPageDefaults.RemoveAtSwap(Index, EAllowShrinking::No);
						}
					}
					else
					{
						Default.PageID = PageSettings->UniqueId;
					}
				}

				// Otherwise if user added new default entry with the new entry ID, give
				// it a valid page ID & name.  If no additional valid pages are implementable,
				// remove it.
				else if (Default.PageID == FMetasoundEditorMemberPageDefault::GetNewEntryID())
				{
					if (MissingPages.IsEmpty())
					{
						OutPageDefaults.RemoveAtSwap(Index, EAllowShrinking::No);
					}
					else
					{
						const FName NewPageName = MissingPages.Pop();
						InitPageDefault(NewPageName, Default);
					}
				}
			}

			// Must always contains at least default value in editor
			if (MissingPages.Contains(Frontend::DefaultPageName))
			{
				PageDefaultType NewDefault;
				InitPageDefault(Frontend::DefaultPageName, NewDefault);
				OutPageDefaults.Add(MoveTemp(NewDefault));
			}

			OutPageDefaults.Shrink();
		}

		template <typename PageDefaultType>
		void SortPageDefaults(TArray<PageDefaultType>& OutPageDefaults)
		{
			TMap<FGuid, int32> PageIDToSortOrder;
			const UMetaSoundSettings* Settings = ::GetDefault<UMetaSoundSettings>();
			check(Settings);

			constexpr bool bReverse = true;
			int32 Index = Settings->GetProjectPageSettings().Num(); // no "-1" as default settings are not in project page settings, which are subsequently iterated over.
			Settings->IteratePageSettings([&PageIDToSortOrder, &Index](const FMetaSoundPageSettings& PageSettings)
			{
				PageIDToSortOrder.Add(PageSettings.UniqueId, Index);
				--Index;
			}, bReverse);

			Algo::SortBy(OutPageDefaults, [&PageIDToSortOrder, &OutPageDefaults](const PageDefaultType& PageDefault) -> int32
			{
				if (const int32* SortValue = PageIDToSortOrder.Find(PageDefault.PageID))
				{
					return *SortValue;
				}
				return FMath::Max(OutPageDefaults.Num() + 1, PageIDToSortOrder.Num() + 1);
			});
		}

		template <typename PageDefaultType, typename MemberType>
		void SetFromLiteral(const TFunctionRef<void(MemberType&)> SetMemberFunc, const FGuid& InPageID, TArray<PageDefaultType>& OutPageDefaults)
		{
			auto MatchesID = [&InPageID](const PageDefaultType& Default) { return Default.PageID == InPageID; };
			if (PageDefaultType* DefaultEntry = OutPageDefaults.FindByPredicate(MatchesID))
			{
				SetMemberFunc(DefaultEntry->Value);
			}
			else
			{
				PageDefaultType PageDefault(InPageID);
				const UMetaSoundSettings* Settings = ::GetDefault<UMetaSoundSettings>();
				check(Settings);
				if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(InPageID))
				{
					PageDefault.PageName = PageSettings->Name;
					SetMemberFunc(PageDefault.Value);
					OutPageDefaults.Add(MoveTemp(PageDefault));
					SortPageDefaults(OutPageDefaults);
				}
			}
		}

		template <typename PageDefaultType, typename MemberType>
		void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID, TArray<PageDefaultType>& OutPageDefaults)
		{
			auto SetMemberFunc = [&InLiteral](MemberType& MemberValue)
			{
				InLiteral.TryGet(MemberValue);
			};
			SetFromLiteral<PageDefaultType, MemberType>(SetMemberFunc, InPageID, OutPageDefaults);
		}

		template <typename PageDefaultType, typename MemberType, typename LiteralType>
		bool SynchronizePageDefault(
			UMetasoundEditorGraphMember* Member,
			TFunctionRef<MemberType(const LiteralType&)> LiteralToMember,
			TFunctionRef<LiteralType(const MemberType&)> MemberToLiteral,
			TArray<PageDefaultType>& OutPageDefaults)
		{
			using namespace Frontend;

			bool bModified = false;
			if (ensure(Member))
			{
				FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
				if (const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(Member->GetMemberName()))
				{
					TMap<FGuid, const FMetasoundFrontendLiteral*> FrontendDefaults;
					Algo::Transform(ClassInput->GetDefaults(), FrontendDefaults, [](const FMetasoundFrontendClassInputDefault& Default)
					{
						return TPair<FGuid, const FMetasoundFrontendLiteral*>(Default.PageID, &Default.Literal);
					});

					TMap<FGuid, const FMetaSoundPageSettings*> PageSettingsByID = MemberDefaultsPrivate::GetPageSettingsByID();
					FMetasoundFrontendLiteral Value;
					for (int32 Index = OutPageDefaults.Num() - 1; Index >= 0; --Index)
					{
						PageDefaultType& Default = OutPageDefaults[Index];
						if (const FMetaSoundPageSettings* PageSettings = PageSettingsByID.FindRef(Default.PageID))
						{
							if (const FMetasoundFrontendLiteral* FrontendLiteral = FrontendDefaults.FindRef(Default.PageID))
							{
								FMetasoundFrontendLiteral TestLiteral;
								if (FrontendLiteral->GetType() == EMetasoundFrontendLiteralType::None)
								{
									TestLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(ClassInput->TypeName));
								}
								else
								{
									TestLiteral = *FrontendLiteral;
								}
								Value.Set(MemberToLiteral(Default.Value));
								if (!TestLiteral.IsEqual(Value))
								{
									bModified = true;

									LiteralType NewValue {};
									if (!TestLiteral.TryGet(NewValue))
									{
										UE_LOGF(LogMetasoundEditor, Verbose, "Synchronizing Page Default: Setting member '%ls' (type '%ls') to literal value '%ls'. Type has changed or literal could not be set and will be set to type's default constructed value.",
											*Member->GetMemberName().ToString(),
											*Member->GetDataType().ToString(),
											*TestLiteral.ToString());
									}
									Default.Value = LiteralToMember(NewValue);
								}

								if (Default.PageName != PageSettings->Name)
								{
									bModified = true;
									Default.PageName = PageSettings->Name;
								}

								FrontendDefaults.Remove(Default.PageID);
							}
							else
							{
								OutPageDefaults.RemoveAtSwap(Index, EAllowShrinking::No);
							}
						}
						else
						{
							OutPageDefaults.RemoveAtSwap(Index, EAllowShrinking::No);
						}
					}

					for (const TPair<FGuid, const FMetasoundFrontendLiteral*>& Pair : FrontendDefaults)
					{
						if (const FMetaSoundPageSettings* PageSettings = PageSettingsByID.FindRef(Pair.Key))
						{
							bModified = true;
							PageDefaultType NewPageDefault;
							NewPageDefault.PageID = Pair.Key;
							NewPageDefault.PageName = PageSettings->Name;

							LiteralType NewValue;
							check(Pair.Value);
							ensure(Pair.Value->TryGet(NewValue));

							NewPageDefault.Value = LiteralToMember(NewValue);
							OutPageDefaults.Add(MoveTemp(NewPageDefault));
						}
					}

					OutPageDefaults.Shrink();
				}
			}

			if (bModified)
			{
				MemberDefaultsPrivate::SortPageDefaults<PageDefaultType>(OutPageDefaults);
			}

			return bModified;
		}

		template <typename PageDefaultType, typename MemberType, typename LiteralType = MemberType>
		bool SynchronizePageDefault(UMetasoundEditorGraphMember* Member, TArray<PageDefaultType>& OutPageDefaults)
		{
			auto LiteralToMember = [](const LiteralType& Literal) -> MemberType { return Literal; };
			auto MemberToLiteral = [](const MemberType& Literal) -> LiteralType { return Literal; };
			return SynchronizePageDefault<PageDefaultType, MemberType, LiteralType>(Member, LiteralToMember, MemberToLiteral, OutPageDefaults);
		}

		template <typename PageDefaultType, typename MemberType, typename LiteralType>
		bool TryFindDefault(const TArray<PageDefaultType>& PageDefaults, const FGuid* InPageID, TFunctionRef<LiteralType(const MemberType&)> MemberToLiteral, FMetasoundFrontendLiteral& OutLiteral)
		{
			const FGuid PageID = InPageID ? *InPageID : Metasound::Frontend::DefaultPageID;
			auto IsPageDefault = [&PageID](const PageDefaultType& PageDefault) { return PageDefault.PageID == PageID; };
			if (const PageDefaultType* PageDefaultPtr = PageDefaults.FindByPredicate(IsPageDefault))
			{
				OutLiteral.Set(MemberToLiteral(PageDefaultPtr->Value));
				return true;
			}

			OutLiteral = { };
			return false;
		}

		template <typename PageDefaultType, typename MemberType, typename LiteralType = MemberType>
		bool TryFindDefault(const TArray<PageDefaultType>& PageDefaults, const FGuid* InPageID, FMetasoundFrontendLiteral& OutLiteral)
		{
			auto MemberToLiteral = [](const MemberType& Member) -> LiteralType { return Member; };
			return TryFindDefault<PageDefaultType, MemberType, LiteralType>(PageDefaults, InPageID, MemberToLiteral, OutLiteral);
		}
	} // namespace MemberDefaultsPrivate
} // Metasound::Editor

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultBool::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::Boolean;
}

void UMetasoundEditorGraphMemberDefaultBool::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<bool>(*this, InPageID);
}

void UMetasoundEditorGraphMemberDefaultBool::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref) { return Ref.Value; };
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultBool, FMetasoundEditorGraphMemberDefaultBoolRef, bool>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBool::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultBool::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultBool::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultBool::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	auto SetLiteral = [&InLiteral](FMetasoundEditorGraphMemberDefaultBoolRef& OutRef)
	{
		bool bValue = false;
		InLiteral.TryGet(bValue);
		OutRef.Value = bValue;
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultBool, FMetasoundEditorGraphMemberDefaultBoolRef>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultBool::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBool::Synchronize()
{
	using namespace Metasound::Editor;
	auto LiteralToMember = [](bool bValue)
	{
		FMetasoundEditorGraphMemberDefaultBoolRef Ref;
		Ref.Value = bValue;
		return Ref;
	};

	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref)
	{
		return Ref.Value;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultBool, FMetasoundEditorGraphMemberDefaultBoolRef, bool>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBool::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref) { return Ref.Value; };
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultBool, FMetasoundEditorGraphMemberDefaultBoolRef, bool>(
		Defaults,
		InPageID,
		MemberToLiteral,
		OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultBool::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetBoolParameter(InParameterName, GetDefaultAs<bool>(PageID));
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultBoolArray::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::BooleanArray;
}

void UMetasoundEditorGraphMemberDefaultBoolArray::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<TArray<bool>>(*this, InPageID);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultBoolRef>& MemberValues)
	{
		TArray<bool> LiteralValues;
		Algo::Transform(MemberValues, LiteralValues, [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref) { return Ref.Value; });
		return LiteralValues;
	};
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultBoolArray, TArray<FMetasoundEditorGraphMemberDefaultBoolRef>, TArray<bool>>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBoolArray::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;

	auto SetLiteral = [&InLiteral](TArray<FMetasoundEditorGraphMemberDefaultBoolRef>& OutRefs)
	{
		OutRefs.Reset();
		TArray<bool> Values;
		InLiteral.TryGet(Values);
		Algo::Transform(Values, OutRefs, [](bool Value)
		{
			FMetasoundEditorGraphMemberDefaultBoolRef Ref;
			Ref.Value = Value;
			return Ref;
		});
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultBoolArray, TArray<FMetasoundEditorGraphMemberDefaultBoolRef>>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultBoolArray>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBoolArray::Synchronize()
{
	using namespace Metasound::Editor;

	auto LiteralToMember = [](const TArray<bool>& Values)
	{
		TArray<FMetasoundEditorGraphMemberDefaultBoolRef> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](bool Value) { FMetasoundEditorGraphMemberDefaultBoolRef Ref; Ref.Value = Value; return Ref; });
		return DefaultValues;
	};

	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultBoolRef>& Values)
	{
		TArray<bool> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref) { return Ref.Value; });
		return DefaultValues;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultBoolArray, TArray<FMetasoundEditorGraphMemberDefaultBoolRef>, TArray<bool>>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultBoolArray::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultBoolRef>& Refs)
	{
		TArray<bool> Values;
		Algo::Transform(Refs, Values, [](const FMetasoundEditorGraphMemberDefaultBoolRef& Ref) { return Ref.Value; });
		return Values;
	};
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultBoolArray, TArray<FMetasoundEditorGraphMemberDefaultBoolRef>, TArray<bool>>(Defaults, InPageID, MemberToLiteral, OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultBoolArray::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetBoolArrayParameter(InParameterName, GetDefaultAs<TArray<bool>>(PageID));
	}
}

void UMetasoundEditorGraphMemberDefaultFloat::ClampDefaults()
{
	using namespace Metasound::Editor;

	for (FMetasoundEditorMemberPageDefaultFloat& PageDefault : Defaults)
	{
		const float ClampedValue = FMath::Clamp(PageDefault.Value, Range.X, Range.Y);
		if (!FMath::IsNearlyEqual(PageDefault.Value, ClampedValue))
		{
			PageDefault.Value = ClampedValue;
			OnDefaultValueChanged.Broadcast(PageDefault.PageID, PageDefault.Value);
		}
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultFloat::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::Float;
}

void UMetasoundEditorGraphMemberDefaultFloat::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<float>(*this, InPageID);
}

#if WITH_EDITORONLY_DATA
bool UMetasoundEditorGraphMemberDefaultFloat::MigrateProperties()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	bool bMigrated = false;

	FMetasoundAssetBase& MetaSound = FGraphBuilder::GetOutermostMetaSoundChecked(*this);
	const FMetasoundFrontendDocument& Document = MetaSound.GetConstDocumentChecked();

	// Property migration occurs before document versioning, so check if either is true to
	// ensure widget type is cleared irrespective of document version.
	bool bIsPreset = Document.RootGraph.PresetOptions.bIsPreset;
	if (!bIsPreset)
	{
		const UStruct* Struct = Document.Template.GetScriptStruct();
		bIsPreset = Struct && Struct->IsChildOf(FPresetTemplate::StaticStruct());
	}
	// Set default input widget type from settings. Widgets do not apply to presets
	if (bIsPreset)
	{
		if (WidgetType != EMetasoundMemberDefaultWidget::None)
		{
			WidgetType = EMetasoundMemberDefaultWidget::None;
			bMigrated = true;
		}
	}

	return bMigrated;
}
#endif // WITH_EDITORONLY_DATA

void UMetasoundEditorGraphMemberDefaultFloat::Initialize()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FMetasoundAssetBase& MetaSound = FGraphBuilder::GetOutermostMetaSoundChecked(*this);
	const FMetaSoundFrontendDocumentBuilder& Builder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound.GetOwningAsset());
	const bool bIsPreset = Builder.IsPreset();

	// Set default input widget type from settings. Widgets do not apply to presets
	if (!bIsPreset)
	{
		if (const UMetasoundEditorSettings* EditorSettings = ::GetDefault<UMetasoundEditorSettings>())
		{
			WidgetType = EditorSettings->DefaultInputWidgetType;
		}
	}
	// Fixup for preset input widget clamp default value.
	// When rebuilding a preset, clamp default was defaulted to false on the parent graph 
	// even if a range was applied because a widget was in use.
	// In that case, we want to make clamp default true and clear out the widget setting.
	else
	{
		if (WidgetType != EMetasoundMemberDefaultWidget::None)
		{
			ClampDefault = true;
			WidgetType = EMetasoundMemberDefaultWidget::None;
		}
	}
}

void UMetasoundEditorGraphMemberDefaultFloat::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultFloat>(Iter, Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloat::ForceRefresh()
{
	using namespace Metasound::Editor;

	OnRangeChanged.Broadcast(Range);
	for (FMetasoundEditorMemberPageDefaultFloat& PageDefault : Defaults)
	{
		const float ClampedFloat = FMath::Clamp(PageDefault.Value, Range.X, Range.Y);
		PageDefault.Value = ClampedFloat;

	}

	// If set from literal, we force the default value to be the literal's value which may require the range to be fixed up 
	SetInitialRange();
}

void UMetasoundEditorGraphMemberDefaultFloat::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetFloatParameter(InParameterName, GetDefaultAs<float>(PageID));
	}
}

void UMetasoundEditorGraphMemberDefaultFloat::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	using namespace Metasound::Editor;

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, WidgetType)) ||
		PropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, WidgetUnitValueType)))
	{
		// Update VolumeWidgetDecibelRange based on current range (it might be stale)
		if (WidgetUnitValueType == EAudioUnitsValueType::Volume && VolumeWidgetUseLinearOutput)
		{
			VolumeWidgetDecibelRange = FVector2D(Audio::ConvertToDecibels(Range.X), Audio::ConvertToDecibels(Range.Y));
		}
		else if (WidgetUnitValueType == EAudioUnitsValueType::Frequency)
		{
			// Set to a reasonable frequency range if range is set to the default
			if (Range.Equals(FVector2D(0.0f, 1.0f)))
			{
				SetRange(FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY));
			}
		}
		else
		{
			SetInitialRange();
		}

		// If the widget type is changed to none, we need to refresh clamping the value or not, since if the widget was a slider before, the value was clamped
		if (WidgetType == EMetasoundMemberDefaultWidget::None)
		{
			ClampDefault = true;
		}
		OnClampChanged.Broadcast(ClampDefault);
	}
	else if (PropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, ClampDefault)))
	{
		SetInitialRange();
		OnClampChanged.Broadcast(ClampDefault);
	}
	else if (PropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, VolumeWidgetUseLinearOutput)))
	{
		for (FMetasoundEditorMemberPageDefaultFloat& PageDefault : Defaults)
		{
			if (VolumeWidgetUseLinearOutput)
			{
				// Range and default are currently in dB, need to change to linear
				float DbDefault = PageDefault.Value;
				VolumeWidgetDecibelRange = Range;
				Range = FVector2D(Audio::ConvertToLinear(VolumeWidgetDecibelRange.X), Audio::ConvertToLinear(VolumeWidgetDecibelRange.Y));
				PageDefault.Value = Audio::ConvertToLinear(DbDefault);
			}
			else
			{
				// Range and default are currently linear, need to change to dB
				Range = VolumeWidgetDecibelRange;
				PageDefault.Value = Audio::ConvertToDecibels(PageDefault.Value);
			}
		}
	}
	else
	{
		FName ChildPropertyName;
		if (const FEditPropertyChain::TDoubleLinkedListNode* MemberNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
		{
			if (const FProperty* ChildProperty = MemberNode->GetValue())
			{
				ChildPropertyName = ChildProperty->GetFName();
			}
		}

		if (ChildPropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, Range)))
		{
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				if (WidgetType != EMetasoundMemberDefaultWidget::None &&
					WidgetUnitValueType == EAudioUnitsValueType::Volume)
				{
					if (VolumeWidgetUseLinearOutput)
					{
						Range.X = FMath::Max(Range.X, Audio::ConvertToLinear(SAudioVolumeRadialSlider::MinDbValue));
						Range.Y = FMath::Min(Range.Y, Audio::ConvertToLinear(SAudioVolumeRadialSlider::MaxDbValue));
						if (Range.X > Range.Y)
						{
							Range.Y = Range.X;
						}
						VolumeWidgetDecibelRange = FVector2D(Audio::ConvertToDecibels(Range.X), Audio::ConvertToDecibels(Range.Y));
						OnRangeChanged.Broadcast(VolumeWidgetDecibelRange);
						ClampDefaults();
					}
				}
				else
				{
					// if Range.X > Range.Y, set Range.Y to Range.X
					if (Range.X > Range.Y)
					{
						SetRange(FVector2D(Range.X, FMath::Max(Range.X, Range.Y)));
					}
					else
					{
						ForceRefresh();
					}
				}
			}
		}
		else if (ChildPropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, VolumeWidgetDecibelRange)))
		{
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				VolumeWidgetDecibelRange.X = FMath::Max(VolumeWidgetDecibelRange.X, SAudioVolumeRadialSlider::MinDbValue);
				VolumeWidgetDecibelRange.Y = FMath::Min(VolumeWidgetDecibelRange.Y, SAudioVolumeRadialSlider::MaxDbValue);
				SetRange(FVector2D(Audio::ConvertToLinear(VolumeWidgetDecibelRange.X), Audio::ConvertToLinear(VolumeWidgetDecibelRange.Y)));
			}
		}
	}

	// Only update member on non-interactive changes to avoid refreshing the details panel mid-update
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		UMetasoundEditorGraphMember* Member = FindMember();
		if (ensure(Member))
		{
			FMetasoundFrontendDocumentModifyContext& ModifyContext = FGraphBuilder::GetOutermostMetaSoundChecked(*Member).GetModifyContext();

			// Mark all nodes as modified to refresh them on synchronization.  This ensures all corresponding widgets get updated.
			const TArray<UMetasoundEditorGraphMemberNode*> MemberNodes = Member->GetNodes();
			TSet<FGuid> NodesToRefresh;
			Algo::Transform(MemberNodes, NodesToRefresh, [](const UMetasoundEditorGraphMemberNode* MemberNode) { return MemberNode->GetNodeID(); });

			// MemberID not marked as modified as this causes detail trees to collapse.
			ModifyContext.AddNodeIDsModified({ NodesToRefresh });
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

bool UMetasoundEditorGraphMemberDefaultFloat::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloat::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultFloat::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloat::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultFloat, float>(InLiteral, InPageID, Defaults);

	// If set from literal, we force the default value to be the literal's value which may require the range to be fixed up
	SetInitialRange();
}

void UMetasoundEditorGraphMemberDefaultFloat::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultFloat>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultFloat::Synchronize()
{
	using namespace Metasound::Editor;
	const bool bModified = MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultFloat, float>(FindMember(), Defaults);
	if (bModified)
	{
		// Broadcast on value changed
		for (const FMetasoundEditorMemberPageDefaultFloat& PageDefault : Defaults)
		{
			OnDefaultValueChanged.Broadcast(PageDefault.PageID, PageDefault.Value);
		}
	}
	return bModified;
}

void UMetasoundEditorGraphMemberDefaultFloat::SetInitialRange()
{
	// If value is within current range, keep it, otherwise set range to something reasonable
	bool bDefaultsInRange = true;
	float Min = TNumericLimits<float>::Min();
	float Max = TNumericLimits<float>::Max();
	for (const FMetasoundEditorMemberPageDefaultFloat& PageDefault : Defaults)
	{
		bDefaultsInRange &= PageDefault.Value >= Range.X && PageDefault.Value <= Range.Y;
		Min = FMath::Max(PageDefault.Value, Min);
		Max = FMath::Min(PageDefault.Value, Max);
	}

	if (!bDefaultsInRange)
	{
		if (Min > Max)
		{
			Swap(Min, Max);
		}

		if (FMath::IsNearlyEqual(Min, 0.0f) && FMath::IsNearlyEqual(Max, 0.0f))
		{
			SetRange(FVector2D(0.0f, 1.0f));
		}
		else if (FMath::IsNearlyEqual(Min, Max))
		{
			if (Min > 0.0f)
			{
				SetRange(FVector2D(0.0f, Min));
			}
			else
			{
				SetRange(FVector2D(Min, 0.0f));
			}
		}
		else
		{
			SetRange(FVector2D(FMath::Min(0.0f, Min), FMath::Max(0.0f, Max)));
		}
	}
}

bool UMetasoundEditorGraphMemberDefaultFloat::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultFloat, float>(Defaults, InPageID, OutLiteral);
}

FVector2D UMetasoundEditorGraphMemberDefaultFloat::GetRange() const
{
	return Range;
}

void UMetasoundEditorGraphMemberDefaultFloat::SetRange(const FVector2D InRange)
{
	using namespace Metasound::Editor;

	if (!(Range - InRange).IsNearlyZero())
	{
		Range = InRange;
		OnRangeChanged.Broadcast(InRange);
		ClampDefaults();
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultFloatArray::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::FloatArray;
}

void UMetasoundEditorGraphMemberDefaultFloatArray::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<TArray<float>>(*this, InPageID);

}

void UMetasoundEditorGraphMemberDefaultFloatArray::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::IterateDefaults(Iter, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultFloatArray::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloatArray::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultFloatArray::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloatArray::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultFloatArray, TArray<float>>(InLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultFloatArray::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultFloatArray>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultFloatArray::Synchronize()
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultFloatArray, TArray<float>>(FindMember(), Defaults);
}

bool UMetasoundEditorGraphMemberDefaultFloatArray::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultFloatArray, TArray<float>>(Defaults, InPageID, OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultFloatArray::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetFloatArrayParameter(InParameterName, GetDefaultAs<TArray<float>>(PageID));
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultInt::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::Integer;
}

void UMetasoundEditorGraphMemberDefaultInt::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<int32>(*this, InPageID);

}

void UMetasoundEditorGraphMemberDefaultInt::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref) { return Ref.Value; };
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultInt, FMetasoundEditorGraphMemberDefaultIntRef, int32>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultInt::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultInt::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultInt::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultInt::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	auto SetLiteral = [&InLiteral](FMetasoundEditorGraphMemberDefaultIntRef& OutRef)
	{
		int32 Value = 0;
		InLiteral.TryGet(Value);
		OutRef.Value = Value;
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultInt, FMetasoundEditorGraphMemberDefaultIntRef>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultInt::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultInt::Synchronize()
{
	using namespace Metasound::Editor;
	auto LiteralToMember = [](int32 bValue)
	{
		FMetasoundEditorGraphMemberDefaultIntRef Ref;
		Ref.Value = bValue;
		return Ref;
	};

	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref)
	{
		return Ref.Value;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultInt, FMetasoundEditorGraphMemberDefaultIntRef, int32>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultInt::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref) { return Ref.Value; };
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultInt, FMetasoundEditorGraphMemberDefaultIntRef, int32>(
		Defaults,
		InPageID,
		MemberToLiteral,
		OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultInt::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetIntParameter(InParameterName, GetDefaultAs<int32>(PageID));
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultIntArray::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::IntegerArray;
}

void UMetasoundEditorGraphMemberDefaultIntArray::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<TArray<int32>>(*this, InPageID);
}

void UMetasoundEditorGraphMemberDefaultIntArray::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultIntRef>& MemberValues)
	{
		TArray<int32> LiteralValues;
		Algo::Transform(MemberValues, LiteralValues, [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref) { return Ref.Value; });
		return LiteralValues;
	};
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultIntArray, TArray<FMetasoundEditorGraphMemberDefaultIntRef>, TArray<int32>>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultIntArray::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultIntArray::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultIntArray::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultIntArray::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;

	auto SetLiteral = [&InLiteral](TArray<FMetasoundEditorGraphMemberDefaultIntRef>& OutRefs)
	{
		OutRefs.Reset();
		TArray<int32> Values;
		InLiteral.TryGet(Values);
		Algo::Transform(Values, OutRefs, [](const int32& Value)
			{
				FMetasoundEditorGraphMemberDefaultIntRef Ref;
				Ref.Value = Value;
				return Ref;
			});
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultIntArray, TArray<FMetasoundEditorGraphMemberDefaultIntRef>>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultIntArray::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultIntArray>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultIntArray::Synchronize()
{
	using namespace Metasound::Editor;

	auto LiteralToMember = [](const TArray<int32>& Values)
	{
		TArray<FMetasoundEditorGraphMemberDefaultIntRef> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](const int32& Value) { FMetasoundEditorGraphMemberDefaultIntRef Ref; Ref.Value = Value; return Ref; });
		return DefaultValues;
	};

	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultIntRef>& Values)
	{
		TArray<int32> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref) { return Ref.Value; });
		return DefaultValues;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultIntArray, TArray<FMetasoundEditorGraphMemberDefaultIntRef>, TArray<int32>>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultIntArray::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultIntRef>& Refs) -> TArray<int32>
	{
		TArray<int32> Values;
		Algo::Transform(Refs, Values, [](const FMetasoundEditorGraphMemberDefaultIntRef& Ref) { return Ref.Value; });
		return Values;
	};
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultIntArray, TArray<FMetasoundEditorGraphMemberDefaultIntRef>, TArray<int32>>(Defaults, InPageID, MemberToLiteral, OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultIntArray::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetIntArrayParameter(InParameterName, GetDefaultAs<TArray<int32>>(PageID));
	}
}

const FGuid& FMetasoundEditorMemberPageDefault::GetNewEntryID()
{
	static const FGuid NewEntryID = FGuid::NewGuid();
	return NewEntryID;
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultString::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::String;
}

void UMetasoundEditorGraphMemberDefaultString::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<FString>(*this, InPageID);
}

void UMetasoundEditorGraphMemberDefaultString::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultString>(Iter, Defaults);
}

void UMetasoundEditorGraphMemberDefaultString::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetStringParameter(InParameterName, GetDefaultAs<FString>(PageID));
	}
}

bool UMetasoundEditorGraphMemberDefaultString::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultString::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultString::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultString::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultString, FString>(InLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultString::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultString>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultString::Synchronize()
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultString, FString>(FindMember(), Defaults);
}

bool UMetasoundEditorGraphMemberDefaultString::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultString, FString>(Defaults, InPageID, OutLiteral);
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultStringArray::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::StringArray;
}

void UMetasoundEditorGraphMemberDefaultStringArray::InitDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::InitDefault<TArray<FString>>(*this, InPageID);
}

void UMetasoundEditorGraphMemberDefaultStringArray::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::IterateDefaults(Iter, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultStringArray::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultStringArray::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultStringArray::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultStringArray::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultStringArray, TArray<FString>>(InLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultStringArray::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultStringArray>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultStringArray::Synchronize()
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultStringArray, TArray<FString>>(FindMember(), Defaults);
}

bool UMetasoundEditorGraphMemberDefaultStringArray::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultStringArray, TArray<FString>>(Defaults, InPageID, OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultStringArray::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetStringArrayParameter(InParameterName, GetDefaultAs<TArray<FString>>(PageID));
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultObject::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::UObject;
}

void UMetasoundEditorGraphMemberDefaultObject::InitDefault(const FGuid& InPageID)
{
	// Can't use template as the object type's MetaSound proxy DataType is unknown
	FMetasoundFrontendLiteral DefaultLiteral;
	constexpr UObject* NullObject = nullptr;
	DefaultLiteral.Set(NullObject);
	SetFromLiteral(DefaultLiteral, InPageID);
}

void UMetasoundEditorGraphMemberDefaultObject::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultObjectRef& Ref) -> UObject* { return Ref.Object; };
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultObjectRef, FMetasoundEditorGraphMemberDefaultObjectRef, UObject*>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObject::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultObject::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultObject::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultObject::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	auto SetLiteral = [&InLiteral](FMetasoundEditorGraphMemberDefaultObjectRef& OutRef)
	{
		UObject* Value = nullptr;
		InLiteral.TryGet(Value);
		OutRef.Object = Value;
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultObjectRef, FMetasoundEditorGraphMemberDefaultObjectRef>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultObject::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObject::Synchronize()
{
	using namespace Metasound::Editor;
	auto LiteralToMember = [](UObject* Value)
	{
		FMetasoundEditorGraphMemberDefaultObjectRef Ref;
		Ref.Object = Value;
		return Ref;
	};

	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultObjectRef& Value) -> UObject*
	{
		return Value.Object;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultObjectRef, FMetasoundEditorGraphMemberDefaultObjectRef, UObject*>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObject::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const FMetasoundEditorGraphMemberDefaultObjectRef& Value) -> UObject*
	{
		return Value.Object;
	};
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultObjectRef, FMetasoundEditorGraphMemberDefaultObjectRef, UObject*>(
		Defaults,
		InPageID,
		MemberToLiteral,
		OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultObject::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetObjectParameter(InParameterName, GetDefaultAs<UObject*>(PageID));
	}
}

EMetasoundFrontendLiteralType UMetasoundEditorGraphMemberDefaultObjectArray::GetLiteralType() const
{
	return EMetasoundFrontendLiteralType::UObjectArray;
}

void UMetasoundEditorGraphMemberDefaultObjectArray::InitDefault(const FGuid& InPageID)
{
	// Can't use template as the object type's MetaSound proxy DataType is unknown
	FMetasoundFrontendLiteral DefaultLiteral;
	DefaultLiteral.Set(TArray<UObject*>());
	SetFromLiteral(DefaultLiteral, InPageID);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultObjectRef>& MemberValues)
	{
		TArray<UObject*> LiteralValues;
		Algo::Transform(MemberValues, LiteralValues, [](const FMetasoundEditorGraphMemberDefaultObjectRef& Ref) { return Ref.Object; });
		return LiteralValues;
	};
	MemberDefaultsPrivate::IterateDefaults<FMetasoundEditorMemberPageDefaultObjectArray, TArray<FMetasoundEditorGraphMemberDefaultObjectRef>, TArray<UObject*>>(Iter, MemberToLiteral, Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObjectArray::RemoveDefault(const FGuid& InPageID)
{
	using namespace Metasound::Editor;
	return MemberDefaultsPrivate::RemoveDefault(InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::ResetDefaults()
{
	Defaults.Empty();
	InitDefault(Metasound::Frontend::DefaultPageID);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::ResolvePageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::ResolvePageDefaults(Defaults);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID)
{
	using namespace Metasound::Editor;

	auto SetLiteral = [&InLiteral](TArray<FMetasoundEditorGraphMemberDefaultObjectRef>& OutRefs)
	{
		OutRefs.Reset();
		TArray<UObject*> Values;
		InLiteral.TryGet(Values);
		Algo::Transform(Values, OutRefs, [](UObject* Value)
		{
			FMetasoundEditorGraphMemberDefaultObjectRef Ref;
			Ref.Object = Value;
			return Ref;
		});
	};
	MemberDefaultsPrivate::SetFromLiteral<FMetasoundEditorMemberPageDefaultObjectArray, TArray<FMetasoundEditorGraphMemberDefaultObjectRef>>(SetLiteral, InPageID, Defaults);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::SortPageDefaults()
{
	using namespace Metasound::Editor;
	MemberDefaultsPrivate::SortPageDefaults<FMetasoundEditorMemberPageDefaultObjectArray>(Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObjectArray::Synchronize()
{
	using namespace Metasound::Editor;

	auto LiteralToMember = [](const TArray<UObject*>& Values)
	{
		TArray<FMetasoundEditorGraphMemberDefaultObjectRef> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](UObject* Value) { FMetasoundEditorGraphMemberDefaultObjectRef Ref; Ref.Object = Value; return Ref; });
		return DefaultValues;
	};

	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultObjectRef>& Values)
	{
		TArray<UObject*> DefaultValues;
		Algo::Transform(Values, DefaultValues, [](const FMetasoundEditorGraphMemberDefaultObjectRef& Ref) { return Ref.Object; });
		return DefaultValues;
	};

	return MemberDefaultsPrivate::SynchronizePageDefault<FMetasoundEditorMemberPageDefaultObjectArray, TArray<FMetasoundEditorGraphMemberDefaultObjectRef>, TArray<UObject*>>(
		FindMember(),
		LiteralToMember,
		MemberToLiteral,
		Defaults);
}

bool UMetasoundEditorGraphMemberDefaultObjectArray::TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID) const
{
	using namespace Metasound::Editor;
	auto MemberToLiteral = [](const TArray<FMetasoundEditorGraphMemberDefaultObjectRef>& Refs)
	{
		TArray<UObject*> Values;
		Algo::Transform(Refs, Values, [](const FMetasoundEditorGraphMemberDefaultObjectRef& Ref) { return Ref.Object; });
		return Values;
	};
	return MemberDefaultsPrivate::TryFindDefault<FMetasoundEditorMemberPageDefaultObjectArray, TArray<FMetasoundEditorGraphMemberDefaultObjectRef>, TArray<UObject*>>(Defaults, InPageID, MemberToLiteral, OutLiteral);
}

void UMetasoundEditorGraphMemberDefaultObjectArray::UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
{
	FGuid PageID;
	if (TryGetPreviewPageID(PageID))
	{
		InParameterInterface->SetObjectArrayParameter(InParameterName, GetDefaultAs<TArray<UObject*>>(PageID));
	}
}
