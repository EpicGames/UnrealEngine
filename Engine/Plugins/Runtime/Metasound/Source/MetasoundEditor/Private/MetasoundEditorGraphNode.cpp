// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorGraphNode.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "EdGraph/EdGraphPin.h"
#include "Editor/EditorEngine.h"
#include "Engine/Font.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphEditorActions.h"
#include "HAL/IConsoleManager.h"
#include "Logging/TokenizedMessage.h"
#include "Metasound.h"
#include "MetasoundAssetKey.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorCommands.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNodeVisualization.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorGraphValidation.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLiteral.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundUObjectRegistry.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraphNode)

#define LOCTEXT_NAMESPACE "MetaSoundEditor"

namespace Metasound::Editor
{
	namespace GraphNodePrivate
	{
		static const FString MissingConcreteOutputConnectionFormat = TEXT(
			"Reroute connection for pin '{0}' does not provide a concrete output. "
			"Resulting literal value is undefined and may result in unintended results."
		);

		int32 ShowNodeDebugData = 0;
		FAutoConsoleVariableRef CVarShowNodeDebugData(
			TEXT("au.MetaSound.Editor.Debug.ShowNodeDebugData"),
			ShowNodeDebugData,
			TEXT("If enabled, shows debug data such as node IDs, vertex IDs, vertex names, and class names when hovering over node titles and pins in the MetaSound asset editor.\n")
			TEXT("0: Disabled (default), !0: Enabled"),
			ECVF_Default);
	} // GraphNodePrivate
} // Metasound::Editor

UMetasoundEditorGraphNode::UMetasoundEditorGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMetasoundEditorGraphNode::UpdateFrontendNodeLocation(const FVector2D& InLocation)
{
	using namespace Metasound::Frontend;
	
	ensureMsgf(NodeGuid.IsValid(), TEXT("Cannot update frontend node location prior to node guid being finalized."));

	const FGuid NodeID = GetNodeID();
	UMetaSoundBuilderBase& Builder = GetBuilderChecked();
	Builder.GetBuilder().SetNodeLocation(NodeID, InLocation, &NodeGuid);
}

bool UMetasoundEditorGraphNode::ShowNodeDebugData()
{
	using namespace Metasound::Editor;

	return GraphNodePrivate::ShowNodeDebugData != 0 || FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
}

bool UMetasoundEditorGraphNode::RemoveFromDocument() const
{
	UMetaSoundBuilderBase& Builder = GetBuilderChecked();
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	Builder.RemoveNode({ GetNodeID() }, Result);
	return Result == EMetaSoundBuilderResult::Succeeded;
}

void UMetasoundEditorGraphNode::SetNodeLocation(const FVector2D& InLocation)
{
	NodePosX = UE::LWC::FloatToIntCastChecked<int32>(InLocation.X);
	NodePosY = UE::LWC::FloatToIntCastChecked<int32>(InLocation.Y);

	UpdateFrontendNodeLocation(InLocation);
}

void UMetasoundEditorGraphNode::SyncCommentFromFrontendNode()
{
	using namespace Metasound::Frontend;

	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const FMetasoundFrontendNodeStyle& Style = NodeHandle->GetNodeStyle();
	NodeComment = Style.Display.Comment;
	bCommentBubbleMakeVisible = Style.Display.bCommentVisible;
}

bool UMetasoundEditorGraphNode::SyncLocationFromFrontendNode(bool bUpdateEditorNodeID)
{
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendNode* Node = GetFrontendNode())
	{
		if (!Node->Style.Display.Locations.IsEmpty())
		{
			if (bUpdateEditorNodeID)
			{
				if (Node->Style.Display.Locations.Num() != 1)
				{
					UE_LOGF(LogMetaSound, Warning, "Style location map for node %ls should only contain one location (legacy support for multiple location values should be migrated by this point)", *Node->GetID().ToString());
				}
				for (const TPair<FGuid, FVector2D>& IDLocationPair : Node->Style.Display.Locations)
				{
					if (ensureMsgf(IDLocationPair.Key.IsValid(), TEXT("Location cannot be updated for ed node with invalid guid")))
					{
						NodeGuid = IDLocationPair.Key;
						NodePosX = UE::LWC::FloatToIntCastChecked<int32>(IDLocationPair.Value.X);
						NodePosY = UE::LWC::FloatToIntCastChecked<int32>(IDLocationPair.Value.Y);
						return true;
					}
				}
			}
			if (const FVector2D* Location = Node->Style.Display.Locations.Find(NodeGuid))
			{
				NodePosX = UE::LWC::FloatToIntCastChecked<int32>(Location->X);
				NodePosY = UE::LWC::FloatToIntCastChecked<int32>(Location->Y);

				return true;
			}
		}
	}

	return false;
}

void UMetasoundEditorGraphNode::PostLoad()
{
	Super::PostLoad();

	for (int32 Index = 0; Index < Pins.Num(); ++Index)
	{
		UEdGraphPin* Pin = Pins[Index];
		if (Pin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			if (Pin->Direction == EGPD_Input)
			{
				Pin->PinName = CreateUniquePinName("Input");
			}
			else
			{
				Pin->PinName = CreateUniquePinName("Output");
			}
			Pin->PinFriendlyName = FText::GetEmpty();
		}
	}
}

void UMetasoundEditorGraphNode::CreateInputPin()
{
	// TODO: Implement for nodes supporting variadic inputs
	ensure(false);

	FString PinName; // get from UMetaSoundPatch
	UEdGraphPin* NewPin = CreatePin(EGPD_Input, TEXT("MetasoundEditorGraphNode"), *PinName);
	if (NewPin->PinName.IsNone())
	{
		// Pin must have a name for lookup purposes but is not user-facing
// 		NewPin->PinName = 
// 		NewPin->PinFriendlyName =
	}
}

int32 UMetasoundEditorGraphNode::EstimateNodeWidth() const
{
	const FString NodeTitle = GetNodeTitle(ENodeTitleType::FullTitle).ToString();
	if (const UFont* Font = GetDefault<UEditorEngine>()->EditorFont)
	{
		return Font->GetStringSize(*NodeTitle);
	}
	else
	{
		static const int32 EstimatedCharWidth = 6;
		return NodeTitle.Len() * EstimatedCharWidth;
	}
}

UMetaSoundBuilderBase& UMetasoundEditorGraphNode::GetBuilderChecked() const
{
	using namespace Metasound::Engine;

	UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(GetGraph());
	return FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(EdGraph->GetMetasoundChecked());
}

const FMetasoundFrontendClass* UMetasoundEditorGraphNode::GetFrontendClass() const
{
	using namespace Metasound::Engine;

	if (UObject* Outermost = GetOutermostObject())
	{
		const FGuid NodeID = GetNodeID();
		const FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Outermost);
		if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID))
		{
			return Builder.FindDependency(Node->ClassID);
		}
	}

	return nullptr;
}

const FMetasoundFrontendNode* UMetasoundEditorGraphNode::GetFrontendNode() const
{
	using namespace Metasound::Engine;

	if (UObject* Outermost = GetOutermostObject())
	{
		const FGuid NodeID = GetNodeID();
		const UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*Outermost);
		return Builder.GetConstBuilder().FindNode(NodeID);
	}

	return nullptr;
}

const FMetasoundFrontendNode& UMetasoundEditorGraphNode::GetFrontendNodeChecked() const
{
	using namespace Metasound::Engine;

	UObject* Outermost = GetOutermostObject();
	check(Outermost);

	const FGuid NodeID = GetNodeID();
	const UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*Outermost);

	const FMetasoundFrontendNode* FrontendNode = Builder.GetConstBuilder().FindNode(NodeID);
	check(FrontendNode);
	return *FrontendNode;
}

const FMetasoundEditorGraphNodeBreadcrumb& UMetasoundEditorGraphNode::GetBreadcrumb() const
{
	static const FMetasoundEditorGraphNodeBreadcrumb StubCrumb;
	return StubCrumb;
}

UObject* UMetasoundEditorGraphNode::GetMetasound() const
{
	if (UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(GetGraph()))
	{
		return EdGraph->GetMetasound();
	}

	return nullptr;
}

UObject& UMetasoundEditorGraphNode::GetMetasoundChecked() const
{
	UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(GetGraph());
	return EdGraph->GetMetasoundChecked();
}

const FMetasoundFrontendClass* UMetasoundEditorGraphNode::GetNodeFrontendClass() const
{
	using namespace Metasound::Frontend;

	UObject& MetaSound = GetMetasoundChecked();
	const FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&MetaSound);

	const FMetasoundFrontendNode* Node = DocBuilder.FindNode(GetNodeID());
	if (!Node)
	{
		return nullptr;
	}

	return DocBuilder.FindDependency(Node->ClassID);
}

const UClass* UMetasoundEditorGraphNode::GetNodeAssetClass() const
{
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClass* NodeFrontendClass = GetNodeFrontendClass();
	if (!NodeFrontendClass || !IMetaSoundAssetManager::GetChecked().IsAssetClass(NodeFrontendClass->Metadata))
	{
		return nullptr;
	}

	FMetasoundAssetBase* Asset = IMetaSoundAssetManager::GetChecked().FindAsset(NodeFrontendClass->Metadata);
	if (!Asset)
	{
		return nullptr;
	}

	const IMetaSoundDocumentInterface* DocumentInterface = Cast<IMetaSoundDocumentInterface>(Asset->GetOwningAsset());
	return DocumentInterface ? &DocumentInterface->GetBaseMetaSoundUClass() : nullptr;
}

Metasound::Frontend::FConstGraphHandle UMetasoundEditorGraphNode::GetConstRootGraphHandle() const
{
	const FMetasoundAssetBase* ConstMetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&GetMetasoundChecked());
	return ConstMetasoundAsset->GetRootGraphHandle();
}

Metasound::Frontend::FGraphHandle UMetasoundEditorGraphNode::GetRootGraphHandle() const
{
	const FMetasoundAssetBase* ConstMetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&GetMetasoundChecked());
	FMetasoundAssetBase* MetasoundAsset = const_cast<FMetasoundAssetBase*>(ConstMetasoundAsset);
	check(MetasoundAsset);

	return MetasoundAsset->GetRootGraphHandle();
}

Metasound::Frontend::FConstNodeHandle UMetasoundEditorGraphNode::GetConstNodeHandle() const
{
	const FGuid NodeID = GetNodeID();
	return GetConstRootGraphHandle()->GetNodeWithID(NodeID);
}

Metasound::Frontend::FNodeHandle UMetasoundEditorGraphNode::GetNodeHandle() const
{
	const FGuid NodeID = GetNodeID();
	return GetRootGraphHandle()->GetNodeWithID(NodeID);
}

void UMetasoundEditorGraphNode::IteratePins(TUniqueFunction<void(UEdGraphPin& /* Pin */, int32 /* Index */)> InFunc, EEdGraphPinDirection InPinDirection)
{
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (InPinDirection == EGPD_MAX || Pins[PinIndex]->Direction == InPinDirection)
		{
			InFunc(*Pins[PinIndex], PinIndex);
		}
	}
}

void UMetasoundEditorGraphNode::AllocateDefaultPins()
{
	using namespace Metasound;

	ensureAlways(Pins.IsEmpty());

	if (UObject* Outermost = GetOutermostObject())
	{
		const FGuid NodeID = GetNodeID();
		const FMetaSoundFrontendDocumentBuilder& Builder = Engine::FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Outermost);
		if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID))
		{
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
			{
				constexpr bool bRemoveUnusedPins = false;
				Editor::FGraphBuilder::SynchronizeNodePins(Builder, *this, *Node, *Class, bRemoveUnusedPins);
			}
		}
	}
}

void UMetasoundEditorGraphNode::SyncChangeIDs()
{
	using namespace Metasound::Frontend;
	FConstNodeHandle NodeHandle = GetConstNodeHandle();

	MetadataChangeID = NodeHandle->GetClassMetadata().GetChangeID();
	InterfaceChangeID = NodeHandle->GetClassInterface().GetChangeID();
	StyleChangeID = NodeHandle->GetClassStyle().GetChangeID();
}

void UMetasoundEditorGraphNode::CacheTitle()
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	CachedTitle = NodeHandle->GetDisplayTitle();
}

void UMetasoundEditorGraphNode::Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundEditorGraphNode::Validate);

	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

#if WITH_EDITOR
	// Validate that non-reroute inputs are connected to "real" outputs
	if (GetBreadcrumb().ClassName != FRerouteNodeTemplate::ClassName)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin)
			{
				OutResult.SetPinOrphaned(*Pin, false);
				if (Pin->Direction == EGPD_Input)
				{
					if (!Pin->LinkedTo.IsEmpty())
					{
						UEdGraphPin* ReroutedPin = FGraphBuilder::FindReroutedOutputPin(Pin->LinkedTo.Last());
						if (ReroutedPin)
						{
							if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(ReroutedPin->GetOwningNode()))
							{
								if (ExternalNode->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
								{
									if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(InBuilder, Pin))
									{
										const FString Msg = FString::Format(*GraphNodePrivate::MissingConcreteOutputConnectionFormat, { Vertex->Name.ToString() });
										OutResult.SetMessage(EMessageSeverity::Warning, Msg);
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Validate node locations 
	// Frontend node should only have one location post ed graph migration in doc version 1.12
	const FMetasoundFrontendNode* Node = InBuilder.FindNode(GetNodeID());
	if (Node)
	{
		if (Node->Style.Display.Locations.Num() > 1)
		{
			OutResult.SetMessage(EMessageSeverity::Info, FString::Format(TEXT("Frontend node with {0} locations found when there should only be 1. Moving this node and resaving the asset should fixup node locations."), { Node->Style.Display.Locations.Num() }));
		}
	}
	else
	{
		OutResult.SetMessage(EMessageSeverity::Error, TEXT("No frontend node associated with this editor graph node. Please delete and readd it."));
	}
#endif // WITH_EDITOR
}

bool UMetasoundEditorGraphNode::ContainsClassChange() const
{
	using namespace Metasound::Frontend;
	FConstNodeHandle NodeHandle = GetConstNodeHandle();

	return InterfaceChangeID != NodeHandle->GetClassInterface().GetChangeID()
	|| StyleChangeID != NodeHandle->GetClassStyle().GetChangeID()
	|| MetadataChangeID != NodeHandle->GetClassMetadata().GetChangeID();
}

void UMetasoundEditorGraphNode::ReconstructNode()
{
	using namespace Metasound;

	if (UObject* Outermost = GetOutermostObject())
	{
		const FGuid NodeID = GetNodeID();
		const FMetaSoundFrontendDocumentBuilder& Builder = Engine::FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Outermost);
		if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID))
		{
			if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
			{
				// Don't remove unused pins here. Reconstruction can occur while duplicating or pasting nodes,
				// and subsequent steps clean-up unused pins.  This can be called mid-copy, which means the node
				// handle may be invalid.  Setting to remove unused causes premature removal and then default values
				// are lost.
				Editor::FGraphBuilder::SynchronizeNodePins(Builder, *this, *Node, *Class, false /* bRemoveUnusedPins */);
			}
		}
	}

	CacheTitle();
}

void UMetasoundEditorGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin)
	{
		const UMetasoundEditorGraphSchema* Schema = CastChecked<UMetasoundEditorGraphSchema>(GetSchema());

		TSet<UEdGraphNode*> NodeList;

		// auto-connect from dragged pin to first compatible pin on the new node
		for (int32 i = 0; i < Pins.Num(); i++)
		{
			UEdGraphPin* Pin = Pins[i];
			check(Pin);
			FPinConnectionResponse Response = Schema->CanCreateConnection(FromPin, Pin);
			if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == Response.Response) //-V1051
			{
				if (Schema->TryCreateConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}
				break;
			}
			else if (ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A == Response.Response)
			{
				// TODO: Implement default connections in GraphBuilder
				break;
			}
		}

		// Send all nodes that received a new pin connection a notification
		for (auto It = NodeList.CreateConstIterator(); It; ++It)
		{
			UEdGraphNode* Node = (*It);
			Node->NodeConnectionListChanged();
		}
	}
}

bool UMetasoundEditorGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UMetasoundEditorGraphSchema::StaticClass());
}

bool UMetasoundEditorGraphNode::CanUserDeleteNode() const
{
	return true;
}

FString UMetasoundEditorGraphNode::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Metasound");
}

FText UMetasoundEditorGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return CachedTitle;
}

void UMetasoundEditorGraphNode::GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (Pin.Direction == EGPD_Input)
	{
		// Report if connected to reroute network is not connected to concrete output 
		FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(&Pin);
		if (Pin.bOrphanedPin && InputHandle->IsValid())
		{
			OutHoverText = FString::Format(*GraphNodePrivate::MissingConcreteOutputConnectionFormat, { InputHandle->GetDisplayName().ToString() });
		}
		else
		{
			OutHoverText = InputHandle->GetTooltip().ToString();
		}

		if (ShowNodeDebugData())
		{
			OutHoverText = FString::Format(TEXT("Description: {0}\nVertex Name: {1}\nDataType: {2}\nID: {3}"),
			{
				OutHoverText,
				InputHandle->GetName().ToString(),
				InputHandle->GetDataType().ToString(),
				InputHandle->GetID().ToString(),
			});
		}
	}
	else // Pin.Direction == EGPD_Output
	{
		FConstOutputHandle OutputHandle = FGraphBuilder::FindReroutedConstOutputHandleFromPin(&Pin);
		OutHoverText = OutputHandle->GetTooltip().ToString();
		if (ShowNodeDebugData())
		{
			OutHoverText = FString::Format(TEXT("Description: {0}\nVertex Name: {1}\nDataType: {2}\nID: {3}"),
			{
				OutHoverText,
				OutputHandle->GetName().ToString(),
				OutputHandle->GetDataType().ToString(),
				OutputHandle->GetID().ToString(),
			});
		}
	}
}

void UMetasoundEditorGraphNode::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (Pin && Pin->Direction == EGPD_Input)
	{
		FMetasoundFrontendLiteral LiteralValue;
		if (FGraphBuilder::GetPinLiteral(*Pin, LiteralValue))
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = GetBuilderChecked().GetBuilder();
			FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, Pin);
			if (!ensure(InputVertexHandle.IsSet()))
			{
				return;
			}

			const FMetasoundFrontendVertex* InputVertex = DocBuilder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
			if (!ensure(InputVertex))
			{
				return;
			}

			GetMetasoundChecked().Modify();
			DocBuilder.SetNodeInputDefault(InputVertexHandle.NodeID, InputVertexHandle.VertexID, LiteralValue);
		}
	}
}

Metasound::Frontend::FDataTypeRegistryInfo UMetasoundEditorGraphNode::GetPinDataTypeInfo(const UEdGraphPin& InPin) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

 	FDataTypeRegistryInfo DataTypeInfo;

	const FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderChecked().GetConstBuilder();
	const FMetasoundFrontendVertexHandle Handle = FGraphBuilder::GetPinVertexHandle(Builder, &InPin);
	if (Handle.IsSet())
	{
		if (InPin.Direction == EGPD_Input)
		{
			const FMetasoundFrontendVertex* Vertex = Builder.FindNodeInput(Handle.NodeID, Handle.VertexID);
			if (Vertex)
			{
				IDataTypeRegistry::Get().GetDataTypeInfo(Vertex->TypeName, DataTypeInfo);
			}
		}
		else // InPin.Direction == EGPD_Output
		{
			const FMetasoundFrontendVertex* Vertex = Builder.FindNodeOutput(Handle.NodeID, Handle.VertexID);
			if (Vertex)
			{
				IDataTypeRegistry::Get().GetDataTypeInfo(Vertex->TypeName, DataTypeInfo);
			}
		}
	}

	return DataTypeInfo;
}

TSet<FString> UMetasoundEditorGraphNode::GetDisallowedPinClassNames(const UEdGraphPin& InPin) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	const IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);

	const FDataTypeRegistryInfo DataTypeInfo = GetPinDataTypeInfo(InPin);
	if (DataTypeInfo.PreferredLiteralType != Metasound::ELiteralType::UObjectProxy)
	{
		return { };
	}

	UClass* ProxyGenClass = DataTypeInfo.ProxyGeneratorClass;
	if (!ProxyGenClass)
	{
		return { };
	}

	TSet<FString> DisallowedClasses;
	const FTopLevelAssetPath ClassName = ProxyGenClass->GetClassPathName();
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (!Class->IsNative())
		{
			continue;
		}

		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		if (ClassIt->GetClassPathName() == ClassName)
		{
			continue;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (EditorModule.IsExplicitProxyClass(*ProxyGenClass) && Class->IsChildOf(ProxyGenClass))
		{
			DisallowedClasses.Add(ClassIt->GetClassPathName().ToString());
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (DataTypeInfo.bIsExplicit && Class->IsChildOf(ProxyGenClass))
		{
			DisallowedClasses.Add(ClassIt->GetClassPathName().ToString());
		}
	}

	return DisallowedClasses;
}

FString UMetasoundEditorGraphNode::GetPinMetaData(FName InPinName, FName InKey)
{
	if (InKey == "DisallowedClasses")
	{
		if (UEdGraphPin* Pin = FindPin(InPinName, EGPD_Input))
		{
			TSet<FString> DisallowedClasses = GetDisallowedPinClassNames(*Pin);
			return FString::Join(DisallowedClasses.Array(), TEXT(","));
		}

		return FString();
	}

	return Super::GetPinMetaData(InPinName, InKey);
}

void UMetasoundEditorGraphNode::OnUpdateCommentText(const FString& NewComment)
{
	using namespace Metasound::Frontend;

	if (!NodeComment.Equals(NewComment))
	{
		const FScopedTransaction Transaction(LOCTEXT("CommentCommitted", "Comment Changed"));
		Modify();
		NodeComment = NewComment;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FNodeHandle NodeHandle = GetNodeHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FMetasoundFrontendNodeStyle Style = NodeHandle->GetNodeStyle();
		Style.Display.Comment = NewComment;
		NodeHandle->SetNodeStyle(Style);
	}
}

void UMetasoundEditorGraphNode::PreSave(FObjectPreSaveContext InSaveContext)
{
	using namespace Metasound::Editor;

	Super::PreSave(InSaveContext);

#if WITH_EDITOR
	if (!InSaveContext.IsCooking())
	{
		// Required to refresh upgrade nodes that are stale when saving.
		if (TSharedPtr<FEditor> MetaSoundEditor = FGraphBuilder::GetEditorForMetasound(GetMetasoundChecked()))
		{
			if (TSharedPtr<SGraphEditor> GraphEditor = MetaSoundEditor->GetGraphEditor())
			{
				GraphEditor->RefreshNode(*this);
			}
		}
	}
#endif // WITH_EDITOR
}

void UMetasoundEditorGraphNode::PostEditImport()
{
}

void UMetasoundEditorGraphNode::PostEditChangeProperty(struct FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);
	
	if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodeComment))
	{
		UMetaSoundBuilderBase& Builder = GetBuilderChecked();
		if (const FMetasoundFrontendNode* Node = Builder.GetConstBuilder().FindNode(GetNodeID()))
		{
			if (!Node->Style.Display.Comment.Equals(NodeComment))
			{
				UObject& MetaSound = GetMetasoundChecked();
				MetaSound.Modify();
				EMetaSoundBuilderResult Result;
				Builder.SetNodeComment(Node->GetID(), NodeComment, Result);
				ensure(Result == EMetaSoundBuilderResult::Succeeded);
				Builder.SetNodeCommentVisible(Node->GetID(), bCommentBubbleMakeVisible, Result);
				ensure(Result == EMetaSoundBuilderResult::Succeeded);
			}
		}
	}
}

void UMetasoundEditorGraphNode::PostEditUndo()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UEdGraphPin::ResolveAllPinReferences();
}

void UMetasoundEditorGraphNode::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		CreateNewGuid();
	}
}

const FSlateBrush* UMetasoundEditorGraphNode::GetNodeTitleIconBrush() const
{
	using namespace Metasound::Frontend;

	const FSlateIcon DefaultIcon = FSlateIcon("MetaSoundStyle", "MetasoundEditor.Metasound.Node.Icon");

	const FMetasoundFrontendClass* NodeFrontendClass = GetNodeFrontendClass();
	if (!NodeFrontendClass)
	{
		return DefaultIcon.GetIcon();
	}

	EMetasoundFrontendClassType ClassType = NodeFrontendClass->Metadata.GetType();
	if (ClassType == EMetasoundFrontendClassType::Graph || ClassType == EMetasoundFrontendClassType::External)
	{
		const FMetaSoundAssetKey AssetKey(NodeFrontendClass->Metadata);
		if (const TScriptInterface<IMetaSoundDocumentInterface> MetaSoundDoc = IMetaSoundAssetManager::GetChecked().FindAssetAsDocumentInterface(AssetKey))
		{
			const UClass& Class = MetaSoundDoc->GetBaseMetaSoundUClass();
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(&Class))
			{
				const FAssetData AssetData(MetaSoundDoc.GetObject());
				return AssetDefinition->GetIconBrush(AssetData, AssetData.AssetClassPath.GetAssetName());
			}
		}
	}

	return DefaultIcon.GetIcon();
}

void UMetasoundEditorGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	using namespace Metasound::Editor;

	if (Context->Node)
	{
		if (GetBuilderChecked().GetConstBuilder().GetIsGraphEditable())
		{
			{
				FToolMenuSection& Section = Menu->FindOrAddSection("MetasoundGraphNodeActionsOrganization", LOCTEXT("NodeActionsOrganizationMenuHeader", "Organization"));
				Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* SubMenu)
				{
					{
						FToolMenuSection& SubMenuSection = SubMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
					}

					{
						FToolMenuSection& SubMenuSection = SubMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
					}
				}));
			}			
		}
	}
}

FText UMetasoundEditorGraphNode::GetTooltipText() const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FConstNodeHandle Node = GetConstNodeHandle();
	FText Description = Node->GetDescription();

	if (ShowNodeDebugData())
	{
		Description = FText::Format(LOCTEXT("Metasound_DebugNodeTooltipText", "Description: {0}\nClass Name: {1}\nNode ID: {2}"),
			Description,
			FText::FromString(Node->GetClassMetadata().GetClassName().ToString()),
			FText::FromString(Node->GetID().ToString())
		);
	}
	return Description;
}

FText UMetasoundEditorGraphNode::GetDisplayName() const
{
	constexpr bool bIncludeNamespace = true;
	return Metasound::Editor::FGraphBuilder::GetDisplayName(*GetConstNodeHandle(), bIncludeNamespace);
}

FString UMetasoundEditorGraphNode::GetDocumentationExcerptName() const
{
	// Default the node to searching for an excerpt named for the C++ node class name, including the U prefix.
	// This is done so that the excerpt name in the doc file can be found by find-in-files when searching for the full class name.
	return FString::Printf(TEXT("%s%s"), UMetaSoundPatch::StaticClass()->GetPrefixCPP(), *UMetaSoundPatch::StaticClass()->GetName());
}

bool UMetasoundEditorGraphNode::TryGetPinVisualizationValue(FName InPinName, bool& OutValue) const
{
	return Metasound::Editor::FGraphNodeVisualizationUtils::TryGetPinValue(*this, InPinName, OutValue);
}

bool UMetasoundEditorGraphNode::TryGetPinVisualizationValue(FName InPinName, int32& OutValue) const
{
	return Metasound::Editor::FGraphNodeVisualizationUtils::TryGetPinValue(*this, InPinName, OutValue);
}

bool UMetasoundEditorGraphNode::TryGetPinVisualizationValue(FName InPinName, float& OutValue) const
{
	return Metasound::Editor::FGraphNodeVisualizationUtils::TryGetPinValue(*this, InPinName, OutValue);
}

bool UMetasoundEditorGraphMemberNode::ClampFloatLiteral(const UMetasoundEditorGraphMemberDefaultFloat* DefaultFloatLiteral, FMetasoundFrontendLiteral& LiteralValue)
{
	bool bClampedFloatLiteral = false;
	if (DefaultFloatLiteral->ClampDefault)
	{
		float LiteralFloatValue = 0.0f;
		float ClampedFloatValue = 0.0f;

		LiteralValue.TryGet(LiteralFloatValue);
		ClampedFloatValue = FMath::Clamp(LiteralFloatValue, DefaultFloatLiteral->Range.X, DefaultFloatLiteral->Range.Y);
		bClampedFloatLiteral = !FMath::IsNearlyEqual(ClampedFloatValue, LiteralFloatValue);
		LiteralValue.Set(ClampedFloatValue);
	}
	return bClampedFloatLiteral;
}

FString UMetasoundEditorGraphMemberNode::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	using namespace Metasound::Frontend;

	if (UMetasoundEditorGraphMember* GraphMember = GetMember())
	{
		FString NameToSearch;
	
		if (!GraphMember->GetDisplayName().IsEmpty())
		{
			NameToSearch = GraphMember->GetDisplayName().ToString();
		}
		else
		{
			NameToSearch = GraphMember->GetMemberName().ToString();
		}

		return FString::Printf(TEXT("\"%s\" \"%s\""), *NameToSearch, *GraphMember->GetDataType().ToString());
	}

	return FString();
}

FLinearColor UMetasoundEditorGraphMemberNode::GetNodeTitleColor() const
{
	return FLinearColor::Transparent;
}

FLinearColor UMetasoundEditorGraphMemberNode::GetNodeTitleIconColor() const
{
	return GetNodeTitleTextColor();
}

void UMetasoundEditorGraphOutputNode::PinDefaultValueChanged(UEdGraphPin* InPin)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (InPin && InPin->Direction == EGPD_Input)
	{
		UObject& MetaSound = GetMetasoundChecked();
		MetaSound.Modify();
		
		FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(InPin);
		if (InputHandle->IsValid())
		{
			FMetasoundFrontendLiteral LiteralValue;
			if (FGraphBuilder::GetPinLiteral(*InPin, LiteralValue))
			{
				if (Output)
				{
					UMetasoundEditorGraphMemberDefaultLiteral* Literal = Output->GetLiteral();
					if (ensure(Literal))
					{
						// Clamp float literal if necessary 
						bool bClampedFloatLiteral = false;
						if (const UMetasoundEditorGraphMemberDefaultFloat* DefaultFloatLiteral = Cast<UMetasoundEditorGraphMemberDefaultFloat>(Literal))
						{
							bClampedFloatLiteral = ClampFloatLiteral(DefaultFloatLiteral, LiteralValue);
						}

						Literal->SetFromLiteral(LiteralValue);

						constexpr bool bPostTransaction = false;
						Output->UpdateFrontendDefaultLiteral(bPostTransaction);

						// Update graph node if it was clamped
						if (bClampedFloatLiteral)
						{
							FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
							if (FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound))
							{
								MetaSoundAsset->GetModifyContext().AddMemberIDsModified({ Output->GetMemberID() });
							}
						}
					}
				}
			}
		}
	}
}

void UMetasoundEditorGraphOutputNode::ReconstructNode()
{
	using namespace Metasound::Frontend;

	if (!Output)
	{
		UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(GetGraph());
		if (ensure(Graph))
		{
			Graph->IterateOutputs([&Graph, this](UMetasoundEditorGraphOutput& TestOutput)
			{
				FConstNodeHandle OutputHandle = TestOutput.GetConstNodeHandle();
				FConstInputHandle TestInput = OutputHandle->GetConstInputs().Last();
				const bool bTypeMatches = TestInput->GetDataType() == Breadcrumb.DataType;
				const bool bAccessMatches = TestInput->GetVertexAccessType() == Breadcrumb.AccessType;
				const bool bNameMatches = OutputHandle->GetNodeName() == Breadcrumb.MemberName;
				if (bTypeMatches && bAccessMatches && bNameMatches)
				{
					Output = &TestOutput;
				}
			});
		}
	}

	Super::ReconstructNode();
}

bool UMetasoundEditorGraphOutputNode::RemoveFromDocument() const
{
	if (Output)
	{
		// When removing ed graph output nodes, disconnect, but only remove the location as all
		// frontend page graphs require the graph vertex node to exist and contain matching NodeIDs
		// across all pages.
		const FGuid& NodeID = GetNodeID();
		UMetaSoundBuilderBase& Builder = GetBuilderChecked();
		FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetBuilder();

		DocBuilder.RemoveEdges(NodeID);
		const int32 NumLocationsRemoved = DocBuilder.RemoveNodeLocation(NodeID);

		if (FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&GetMetasoundChecked()))
		{
			MetaSoundAsset->GetModifyContext().AddNodeIDsModified({ NodeID });
		}

		return NumLocationsRemoved > 0;
	}

	return false;
}

bool UMetasoundEditorGraphOutputNode::EnableInteractWidgets() const
{
	using namespace Metasound::Frontend;

	bool bEnabled = true;
	GetConstNodeHandle()->IterateConstInputs([bIsEnabled = &bEnabled](FConstInputHandle InputHandle)
	{
		if (InputHandle->IsConnectionUserModifiable())
		{
			*bIsEnabled &= !InputHandle->IsConnected();
		}
	});
	return bEnabled;
}

void UMetasoundEditorGraphOutputNode::Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	Super::Validate(InBuilder, OutResult);

	if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GetMember()))
	{
		FMetasoundFrontendInterface InterfaceToValidate;
		if (Vertex->IsInterfaceMember(&InterfaceToValidate))
		{
			FText RequiredText;
			if (InterfaceToValidate.IsMemberOutputRequired(Vertex->GetMemberName(), RequiredText))
			{
				if (const FMetasoundFrontendNode* Node = InBuilder.FindNode(GetNodeID()))
				{
					const TArray<FMetasoundFrontendVertex>& Inputs = Node->Interface.Inputs;
					if (ensure(!Inputs.IsEmpty()))
					{
						const FMetaSoundFrontendDocumentBuilder& Builder = GetBuilderChecked().GetConstBuilder();
						if (!Builder.IsNodeInputConnected(Node->GetID(), Inputs.Last().VertexID))
						{
							OutResult.SetMessage(EMessageSeverity::Warning, *RequiredText.ToString());
						}
					}
				}
			}
		}
	}
}

const FMetasoundEditorGraphVertexNodeBreadcrumb& UMetasoundEditorGraphOutputNode::GetBreadcrumb() const
{
	return Breadcrumb;
}

void UMetasoundEditorGraphOutputNode::CacheBreadcrumb()
{
	if (Output)
	{
		Breadcrumb.MemberName = Output->GetMemberName();

		FMetaSoundFrontendDocumentBuilder& Builder = Output->GetFrontendBuilderChecked();
		if (const FMetasoundFrontendClassOutput* ClassOutput = Builder.FindGraphOutput(Breadcrumb.MemberName))
		{
			if (const FMetasoundFrontendNode* Node = Builder.FindGraphOutputNode(Breadcrumb.MemberName))
			{
				if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
				{
					Breadcrumb.ClassName = Class->Metadata.GetClassName();
					Breadcrumb.AccessType = ClassOutput->AccessType;
					Breadcrumb.DataType = ClassOutput->TypeName;
					if (UMetaSoundFrontendMemberMetadata* MemberMetadata = Builder.FindMemberMetadata(Node->GetID()))
					{
						Breadcrumb.MemberMetadataPath = FSoftObjectPath(MemberMetadata);
					}
				}
			}
		}

		if (const UMetasoundEditorGraphMemberDefaultLiteral* Literal = Output->GetLiteral())
		{
			FMetasoundFrontendLiteral DefaultLiteral;
			Literal->TryFindDefault(DefaultLiteral);
			Breadcrumb.DefaultLiterals.Add(Metasound::Frontend::DefaultPageID, DefaultLiteral);
		}
	}
}

FGuid UMetasoundEditorGraphOutputNode::GetNodeID() const
{
	if (Output)
	{
		return Output->NodeID;
	}
	return FGuid();
}

bool UMetasoundEditorGraphOutputNode::CanUserDeleteNode() const
{
	if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GetMember()))
	{
		FMetasoundFrontendInterface MemberInterface;
		if (Vertex->IsInterfaceMember(&MemberInterface))
		{
			FText RequiredText;
			return !MemberInterface.IsMemberOutputRequired(Vertex->GetMemberName(), RequiredText);
		}
	}
	
	return true;
}

void UMetasoundEditorGraphOutputNode::SetNodeID(FGuid InNodeID)
{
	if (ensure(Output))
	{
		Output->NodeID = InNodeID;
	}
}

FLinearColor UMetasoundEditorGraphOutputNode::GetNodeTitleTextColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->OutputNodeTitleColor;
	}

	return Super::GetNodeTitleTextColor();
}

const FSlateBrush* UMetasoundEditorGraphOutputNode::GetNodeTitleIconBrush() const
{
	FSlateIcon OutputIcon = FSlateIcon("MetaSoundStyle", "MetasoundEditor.Graph.Node.Class.Output");
	return OutputIcon.GetIcon();
}

UMetasoundEditorGraphMember* UMetasoundEditorGraphOutputNode::GetMember() const
{
	return Output;
}

void UMetasoundEditorGraphExternalNode::ReconstructNode()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	Super::ReconstructNode();
}

FMetasoundFrontendVersionNumber UMetasoundEditorGraphExternalNode::FindHighestVersionInRegistry() const
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendClass HighestVersionClass;
	FMetasoundFrontendVersionNumber HighestVersionNumber = FMetasoundFrontendVersionNumber::GetInvalid();

	Metasound::Frontend::FConstNodeHandle NodeHandle = GetConstNodeHandle();
	const FMetasoundFrontendClassMetadata& Metadata = NodeHandle->GetClassMetadata();
	if (ISearchEngine::Get().FindClassWithHighestVersion(Metadata.GetClassName(), HighestVersionClass))
	{
		HighestVersionNumber = HighestVersionClass.Metadata.GetVersion();
	}

	return HighestVersionNumber;
}

bool UMetasoundEditorGraphExternalNode::CanAutoUpdate() const
{
	using namespace Metasound::Frontend;

	FMetaSoundFrontendDocumentBuilder& DocBuilder = GetBuilderChecked().GetBuilder();
	return DocBuilder.CanAutoUpdate(GetNodeID());
}

void UMetasoundEditorGraphExternalNode::CacheBreadcrumb()
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	if (const FMetasoundFrontendClass* FrontendClass = GetFrontendClass())
	{
		const FMetasoundFrontendClassMetadata& Metadata = FrontendClass->Metadata;
		const bool bIsAssetClass = IMetaSoundAssetManager::GetChecked().IsAssetClass(Metadata);

		Breadcrumb.bIsClassNative = !bIsAssetClass;
		Breadcrumb.ClassName = Metadata.GetClassName();
		Breadcrumb.Version = Metadata.GetVersion();

		const FMetasoundFrontendNode& Node = GetFrontendNodeChecked();
		Breadcrumb.NodeConfiguration = Node.Configuration;
	}

	// Cache template node generation parameters
	if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Breadcrumb.ClassName))
	{
		if (!Breadcrumb.TemplateParams.IsSet())
		{
			Breadcrumb.TemplateParams = FNodeTemplateGenerateInterfaceParams();
		}
		Breadcrumb.TemplateParams->InputsToConnect.Reset();
		Breadcrumb.TemplateParams->OutputsToConnect.Reset();

		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin)
			{
				const FName DataType = FGraphBuilder::GetPinDataType(Pin);
				if (!DataType.IsNone())
				{
					if (Pin->Direction == EGPD_Input)
					{
						Breadcrumb.TemplateParams->InputsToConnect.Add(DataType);
					}
					else
					{
						Breadcrumb.TemplateParams->OutputsToConnect.Add(DataType);
					}
				}
			}
		}
	}
}

void UMetasoundEditorGraphExternalNode::CacheTitle()
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	constexpr bool bIncludeNamespace = false;
	FConstNodeHandle NodeHandle = GetConstNodeHandle();
	CachedTitle = FGraphBuilder::GetDisplayName(*NodeHandle, bIncludeNamespace);
}

void UMetasoundEditorGraphExternalNode::GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const
{
	using namespace Metasound::Frontend;

	if (ClassName == FRerouteNodeTemplate::ClassName)
	{
		if (!ErrorMsg.IsEmpty())
		{
			OutHoverText = ErrorMsg;
			return;
		}
	}

	Super::GetPinHoverText(Pin, OutHoverText);
}

void UMetasoundEditorGraphExternalNode::Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

#if WITH_EDITOR
	Super::Validate(InBuilder, OutResult);

	UObject* Outermost = GetOutermostObject();
	check(Outermost);

	const FMetasoundFrontendNode* Node = InBuilder.FindNode(NodeID);

	if (!Node)
	{
		OutResult.SetMessage(EMessageSeverity::Error, TEXT("Stale node not found in given MetaSound"));
		return;
	}

	const FMetasoundFrontendClass* Class = InBuilder.FindDependency(Node->ClassID);
	if (!Class)
	{
		OutResult.SetMessage(EMessageSeverity::Error, FString::Format(TEXT("Node with ID '{0}' class not found"), { *Node->ClassID.ToString() }));
		return;
	}

	const FMetasoundFrontendClassMetadata& Metadata = Class->Metadata;

	// 1. Validate external referenced graph or template node
	switch(Metadata.GetType())
	{
		case EMetasoundFrontendClassType::External:
		case EMetasoundFrontendClassType::Graph:
		{
			const FMetaSoundAssetKey AssetKey(Metadata);
			if (FMetasoundAssetBase* MetaSoundAsset = IMetaSoundAssetManager::GetChecked().TryLoadAssetFromKey(AssetKey))
			{
				if (const UMetasoundEditorGraph* NodeGraph = Cast<UMetasoundEditorGraph>(MetaSoundAsset->GetGraph()))
				{
					const EMessageSeverity::Type MaxGraphMsg = static_cast<EMessageSeverity::Type>(NodeGraph->GetHighestMessageSeverity());
					switch (MaxGraphMsg)
					{
					case EMessageSeverity::Error:
					{
						OutResult.SetMessage(MaxGraphMsg, TEXT("Referenced asset class contains error(s). Check implementation for details."));
					}
					break;

					case EMessageSeverity::PerformanceWarning:
					case EMessageSeverity::Warning:
					{
						OutResult.SetMessage(MaxGraphMsg, TEXT("Referenced asset class contains warning(s). Check implementation for details."));
					}
					break;

					case EMessageSeverity::Info:
					default:
					{
					}
					break;
					}
				}
			}
			break;
		}

		case EMetasoundFrontendClassType::Template:
		{
			const FNodeRegistryKey Key = FNodeRegistryKey(Metadata);
			if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key))
			{
				const bool bIsValidInterface = Template->IsValidNodeInterface(Node->Interface);
				if (!bIsValidInterface)
				{
					OutResult.SetMessage(EMessageSeverity::Error, FString::Format(TEXT("Cannot implement template interface for node class '{0}"), { *Metadata.GetClassName().ToString() }));
				}
				else
				{
#if WITH_EDITOR
					FString Message;
					const FMetaSoundFrontendDocumentBuilder& DocBuilder = GetBuilderChecked().GetConstBuilder();
					if (!Template->HasRequiredConnections(DocBuilder, DocBuilder.GetBuildPageID(), GetNodeID(), &Message))
					{
						OutResult.SetMessage(EMessageSeverity::Warning, Message);
					}
#endif // WITH_EDITOR
				}
			}
			else
			{
				OutResult.SetMessage(EMessageSeverity::Error, FString::Format(TEXT("Template node interface missing for node class '{0}'"), { *Metadata.GetClassName().ToString() }));
			}
			break;
		}

		default:
		{
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing EMetasoundFrontendClassType case coverage");
		}
		break;
	}

	// 2. Check if node is invalid, version is missing and cache if interface changes exist between the document's records and the registry
	FClassInterfaceUpdates InterfaceUpdates;
	const bool bUseHighestMinorVersion = false;
	if (!InBuilder.DiffAgainstRegistryInterface(NodeID, InBuilder.GetBuildPageID(), bUseHighestMinorVersion, InterfaceUpdates))
	{
		if (Node)
		{
			const FText* PromptIfMissing = nullptr;
			FString FormattedClassName;
			if (bIsClassNative)
			{
				PromptIfMissing = &Metadata.GetPromptIfMissing();
				FormattedClassName = FString::Format(TEXT("{0} {1} ({2})"), { *Metadata.GetDisplayName().ToString(), *Metadata.GetVersion().ToString(), *Metadata.GetClassName().ToString() });
			}
			else
			{
				static const FText AssetPromptIfMissing = LOCTEXT("PromptIfAssetMissing", "Asset may have not been saved, deleted or is not loaded (ex. in an unloaded plugin).");
				PromptIfMissing = &AssetPromptIfMissing;
				FormattedClassName = FString::Format(TEXT("{0} {1} ({2})"), { *Metadata.GetDisplayName().ToString(), *Metadata.GetVersion().ToString(), *Metadata.GetClassName().Name.ToString() });
			}

			const FString NewErrorMsg = FString::Format(TEXT("Class definition '{0}' not found: {1}"),
			{
				*FormattedClassName,
				*PromptIfMissing->ToString()
			});

			OutResult.SetMessage(EMessageSeverity::Error, NewErrorMsg);
		}
		else
		{
			if (bIsClassNative)
			{
				OutResult.SetMessage(EMessageSeverity::Error, FString::Format(
					TEXT("Class '{0}' definition missing for last known natively defined node."),
					{ *ClassName.ToString() }));
			}
			else
			{
				OutResult.SetMessage(EMessageSeverity::Error,
					FString::Format(TEXT("Class definition missing for asset with guid '{0}': Asset is either missing or invalid"),
					{ *ClassName.Name.ToString() }));
			}
		}
	}

	// 5. Report if node was auto-updated
	const FMetasoundFrontendNodeStyle& Style = Node->Style;
	if (Style.bMessageNodeUpdated)
	{
		OutResult.SetUpgradeMessage(FText::Format(LOCTEXT("MetaSoundNode_UpgradedMessage", "Node class '{0}' updated to version {1}"),
			Metadata.GetDisplayName(),
			FText::FromString(Metadata.GetVersion().ToString())
		));
	}
	else
	{
		OutResult.SetUpgradeMessage({ });
	}

	// 6. Reset pin state (if pin was orphaned or clear if no longer orphaned)
	for (UEdGraphPin* Pin : Pins)
	{
		bool bWasRemoved = false;
		if (Pin->Direction == EGPD_Input)
		{
			if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(InBuilder, Pin))
			{
				bWasRemoved |= InterfaceUpdates.RemovedInputs.ContainsByPredicate([&Vertex](const FMetasoundFrontendClassInput* ClassInput)
				{
					return Vertex->Name == ClassInput->Name && Vertex->TypeName == ClassInput->TypeName;
				});
			}
		}

		if (Pin->Direction == EGPD_Output)
		{
			if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(InBuilder, Pin))
			{
				bWasRemoved |= InterfaceUpdates.RemovedOutputs.ContainsByPredicate([&Vertex](const FMetasoundFrontendClassOutput* ClassOutput)
				{
					return Vertex->Name == ClassOutput->Name && Vertex->TypeName == ClassOutput->TypeName;
				});
			}

		}

		OutResult.SetPinOrphaned(*Pin, bWasRemoved);
	}

	// 7. Report if node class is deprecated
	FMetasoundFrontendClass RegisteredClass;
	if (FMetasoundFrontendRegistryContainer::Get()->GetFrontendClassFromRegistered(FNodeRegistryKey(Metadata), RegisteredClass))
	{
		if (EnumHasAnyFlags(RegisteredClass.Metadata.GetAccessFlags(), EMetasoundFrontendClassAccessFlags::Deprecated))
		{
			constexpr bool bIncludeNamespace = true;
			OutResult.SetMessage(EMessageSeverity::Warning,
				FString::Format(TEXT("Class '{0} {1}' is deprecated."),
				{
					*FGraphBuilder::GetDisplayName(RegisteredClass.Metadata, { }, bIncludeNamespace).ToString(),
					*RegisteredClass.Metadata.GetVersion().ToString()
				}));
		}
	}

	// 8. Find all available versions & report if upgrade available
	const Metasound::FNodeClassName NodeClassName = Metadata.GetClassName().ToNodeClassName();
	const TArray<FMetasoundFrontendClass> SortedClasses = ISearchEngine::Get().FindClassesWithName(NodeClassName, true /* bSortByVersion */);
	if (SortedClasses.IsEmpty())
	{
		OutResult.SetMessage(EMessageSeverity::Error,
			FString::Format(TEXT("Class '{0} {1}' not registered."),
			{
				*Metadata.GetClassName().ToString(),
				*Metadata.GetVersion().ToString()
			}));
	}
	else
	{
		const FMetasoundFrontendVersionNumber& CurrentVersion = Metadata.GetVersion();
		const FMetasoundFrontendClass& HighestRegistryClass = SortedClasses[0];
		if (HighestRegistryClass.Metadata.GetVersion() > CurrentVersion)
		{
			FMetasoundFrontendClass HighestMinorVersionClass;
			FString NodeMsg;
			EMessageSeverity::Type Severity;

			const bool bClassVersionExists = SortedClasses.ContainsByPredicate([InCurrentVersion = &CurrentVersion](const FMetasoundFrontendClass& AvailableClass)
			{
				return AvailableClass.Metadata.GetVersion() == *InCurrentVersion;
			});
			if (bClassVersionExists)
			{
				NodeMsg = FString::Format(TEXT("Node class '{0} {1}' is prior version: Eligible for upgrade to {2}"),
				{
					*Metadata.GetClassName().ToString(),
					*Metadata.GetVersion().ToString(),
					*HighestRegistryClass.Metadata.GetVersion().ToString()
				});
				Severity = EMessageSeverity::Warning;
			}
			else
			{
				NodeMsg = FString::Format(TEXT("Node class '{0} {1}' is missing and ineligible for auto-update.  Highest version '{2}' found."),
				{
					*Metadata.GetClassName().ToString(),
					*Metadata.GetVersion().ToString(),
					*HighestRegistryClass.Metadata.GetVersion().ToString()
				});
				Severity = EMessageSeverity::Error;
			}

			OutResult.SetMessage(Severity, NodeMsg);
		}
		else if (HighestRegistryClass.Metadata.GetVersion() == CurrentVersion)
		{
			if (InterfaceUpdates.ContainsChanges())
			{
				OutResult.SetMessage(EMessageSeverity::Error,
					FString::Format(TEXT("Node & registered class interface mismatch: '{0} {1}'. Class either versioned improperly, class key collision exists, or AutoUpdate disabled in 'MetaSound' Developer Settings."),
					{
						*Metadata.GetClassName().ToString(),
						*Metadata.GetVersion().ToString()
					}));
			}
		}
		else
		{
			OutResult.SetMessage(EMessageSeverity::Error,
				FString::Format(TEXT("Node with class '{0} {1}' interface version higher than that of highest minor revision ({2}) in class registry."),
				{
					*Metadata.GetClassName().ToString(),
					*Metadata.GetVersion().ToString(),
					*HighestRegistryClass.Metadata.GetVersion().ToString()
				}));
		}
	}
#endif // WITH_EDITOR
}

void UMetasoundEditorGraphExternalNode::HideUnconnectedPins(const bool InHidePins)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	UObject& MetaSound = GetMetasoundChecked();
	MetaSound.Modify();

	if (const FMetasoundFrontendNode* FrontendNode = GetFrontendNode())
	{
		FMetaSoundFrontendDocumentBuilder& Builder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&MetaSound);	
		Builder.SetNodeUnconnectedPinsHidden(GetNodeID(), InHidePins);
	}

	if (!InHidePins)
	{
		bool bIsAdvancedView = false;
		for (const UEdGraphPin* Pin : Pins)
		{
			if (Pin && Pin->bAdvancedView)
			{
				bIsAdvancedView = true;			
			}
		}

		if (!bIsAdvancedView)
		{
			AdvancedPinDisplay = ENodeAdvancedPins::NoPins;
		}
	}
	else
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}

	if (TSharedPtr<FEditor> MetaSoundEditor = FGraphBuilder::GetEditorForMetasound(GetMetasoundChecked()))
	{
		if (TSharedPtr<SGraphEditor> GraphEditor = MetaSoundEditor->GetGraphEditor())
		{
			GraphEditor->RefreshNode(*this);
		}
	}
}

const FMetasoundEditorGraphNodeBreadcrumb& UMetasoundEditorGraphExternalNode::GetBreadcrumb() const
{
	return Breadcrumb;
}

FLinearColor UMetasoundEditorGraphExternalNode::GetNodeTitleColor() const
{
	using namespace Metasound::Frontend;

	// Return the asset type color if the node is an asset type (e.g. Source, Patch)
	if (const UClass* NodeAssetClass = GetNodeAssetClass())
	{
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(NodeAssetClass))
		{
			return AssetDefinition->GetAssetColor();
		}
	}

	// Use the style color of native nodes, if specified in the node class.
	UObject& MetaSound = GetMetasoundChecked();
	const FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&MetaSound);
	if (const FMetasoundFrontendNode* Node = DocBuilder.FindNode(GetNodeID()))
	{
		const FMetasoundFrontendClass* FrontendClass = DocBuilder.FindDependency(Node->ClassID);
		const FMetasoundFrontendClassStyleDisplay& StyleDisplay = FrontendClass->Style.Display;
		if (!StyleDisplay.StyleSet.IsNone() && !StyleDisplay.Color.IsNone())
		{
			if (const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle(StyleDisplay.StyleSet))
			{
				return Style->GetSlateColor(StyleDisplay.Color, nullptr, FLinearColor::Transparent).GetSpecifiedColor();
			}
		}
	}

	// Fallback color for outdated nodes, etc.
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->NativeNodeTitleColor;
	}

	return FLinearColor::Transparent;
}

const FSlateBrush* UMetasoundEditorGraphExternalNode::GetNodeTitleIconBrush() const
{
	if (const FMetasoundFrontendClass* FrontendClass = GetFrontendClass())
	{
		const FMetasoundFrontendClassStyleDisplay StyleDisplay = FrontendClass->Style.Display;
		if (!StyleDisplay.StyleSet.IsNone() && !StyleDisplay.Icon.IsNone())
		{
			return FSlateIcon(StyleDisplay.StyleSet, StyleDisplay.Icon).GetIcon();
		}
	}

	return Super::GetNodeTitleIconBrush();
}

FText UMetasoundEditorGraphExternalNode::GetNodeTitleIconTooltipText() const
{
	FText TooltipText;
	if (!bIsClassNative)
	{
		if (const FMetasoundFrontendClass* FrontendClass = GetFrontendClass())
		{
			const FMetasoundFrontendClassMetadata& Metadata = FrontendClass->Metadata;
			const FString MetadataAuthor = Metadata.GetAuthor();

			if (!MetadataAuthor.IsEmpty())
			{
				TooltipText = FText::Format(LOCTEXT("MetaSoundNodeAuthorFormat", "Author: {0}"), FText::FromString(MetadataAuthor));
			}
		}
	}

	return TooltipText;
}

bool UMetasoundEditorGraphExternalNode::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	using namespace Metasound::Frontend;

	if (GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
	{
		OutInputPinIndex = 0;
		OutOutputPinIndex = 1;
		return true;
	}

	return Super::ShouldDrawNodeAsControlPointOnly(OutInputPinIndex, OutOutputPinIndex);
}

void UMetasoundEditorGraphVariableNode::CacheBreadcrumb()
{
	using namespace Metasound::Frontend;

	Breadcrumb = { };

	if (Variable)
	{
		if (const FMetasoundFrontendVariable* FrontendVariable = Variable->GetFrontendVariable())
		{
			Breadcrumb.MemberName = FrontendVariable->Name;
			Breadcrumb.DataType = FrontendVariable->TypeName;

			// Hack to reuse the default literals breadcrumb property for variables, which only have a single (rather than paged) literals
			Breadcrumb.DefaultLiterals = TMap<FGuid, FMetasoundFrontendLiteral>
			{
				{ DefaultPageID, FrontendVariable->Literal }
			};

			FMetasoundFrontendVertexMetadata VertexMetadata;
			VertexMetadata.SetDisplayName(FrontendVariable->DisplayName);
			VertexMetadata.SetDescription(FrontendVariable->Description);

			Breadcrumb.VertexMetadata = VertexMetadata;

			FMetaSoundFrontendDocumentBuilder& Builder = Variable->GetFrontendBuilderChecked();
			if (const FMetasoundFrontendNode* Node = Builder.FindNode(GetNodeID()))
			{
				if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
				{
					const FMetasoundFrontendClassMetadata& Metadata = Class->Metadata;
					Breadcrumb.ClassName = Metadata.GetClassName();
				}
			}
		}
	}
}

const FMetasoundEditorGraphNodeBreadcrumb& UMetasoundEditorGraphVariableNode::GetBreadcrumb() const
{
	return Breadcrumb;
}

UMetasoundEditorGraphMember* UMetasoundEditorGraphVariableNode::GetMember() const
{
	return Variable;
}

bool UMetasoundEditorGraphVariableNode::EnableInteractWidgets() const
{
	using namespace Metasound::Frontend;

	bool bEnabled = false;

	if (Variable)
	{
		FConstVariableHandle VariableHandle = Variable->GetConstVariableHandle();
		FConstNodeHandle MutatorNode = VariableHandle->FindMutatorNode();
		if (MutatorNode->IsValid())
		{
			if (MutatorNode->GetID() == NodeID)
			{
				bEnabled = true;
				MutatorNode->IterateConstInputs([bIsEnabled = &bEnabled](FConstInputHandle InputHandle)
				{
					if (InputHandle->IsConnectionUserModifiable())
					{
						// Don't enable if variable input is connected
						*bIsEnabled &= !InputHandle->IsConnected();
					}
				});
			}
		}
	}

	return bEnabled;
}

EMetasoundFrontendClassType UMetasoundEditorGraphVariableNode::GetClassType() const
{
	return ClassType;
}

FGuid UMetasoundEditorGraphVariableNode::GetNodeID() const
{
	return NodeID;
}

FName UMetasoundEditorGraphVariableNode::GetCornerIcon() const
{
	if (ClassType == EMetasoundFrontendClassType::VariableDeferredAccessor)
	{
		return TEXT("Graph.Latent.LatentIcon");
	}

	return Super::GetCornerIcon();
}

void UMetasoundEditorGraphVariableNode::GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (Variable)
	{
		OutHoverText = Variable->GetBreadcrumb().Description.ToString();
	}

	if (OutHoverText.IsEmpty())
	{
		Super::GetPinHoverText(Pin, OutHoverText);
	}
}

void UMetasoundEditorGraphVariableNode::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (Pin && Pin->Direction == EGPD_Input)
	{
		UObject& MetaSound = GetMetasoundChecked();
		MetaSound.Modify();

		FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(Pin);
		if (InputHandle->IsValid())
		{
			FMetasoundFrontendLiteral LiteralValue;
			if (FGraphBuilder::GetPinLiteral(*Pin, LiteralValue))
			{
				// If this is the mutator node, synchronize the variable default literal with this default.
				FConstNodeHandle MutatorNode = Variable->GetConstVariableHandle()->FindMutatorNode();
				if (MutatorNode->IsValid())
				{
					if (MutatorNode->GetID() == NodeID)
					{
						UMetasoundEditorGraphMemberDefaultLiteral* Literal = Variable->GetLiteral();
						if (ensure(Literal))
						{
							// Clamp float literal if necessary 
							bool bClampedFloatLiteral = false;
							if (const UMetasoundEditorGraphMemberDefaultFloat* DefaultFloatLiteral = Cast<UMetasoundEditorGraphMemberDefaultFloat>(Literal))
							{
								bClampedFloatLiteral = ClampFloatLiteral(DefaultFloatLiteral, LiteralValue);
							}
							Literal->SetFromLiteral(LiteralValue);

							constexpr bool bPostTransaction = false;
							Variable->UpdateFrontendDefaultLiteral(bPostTransaction);

							if (bClampedFloatLiteral)
							{
								// Update graph node if it was clamped
								FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
								if (FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound))
								{
									MetaSoundAsset->GetModifyContext().AddNodeIDsModified({ NodeID });
								}
							}
						}
					}
				}
			}
		}
	}
}

FLinearColor UMetasoundEditorGraphVariableNode::GetNodeTitleColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->VariableNodeTitleColor;
	}

	return Super::GetNodeTitleColor();
}

const FSlateBrush* UMetasoundEditorGraphVariableNode::GetNodeTitleIconBrush() const
{
	return FSlateIcon().GetIcon();
}

void UMetasoundEditorGraphVariableNode::SetNodeID(FGuid InNodeID)
{
	NodeID = InNodeID;
}
#undef LOCTEXT_NAMESPACE

