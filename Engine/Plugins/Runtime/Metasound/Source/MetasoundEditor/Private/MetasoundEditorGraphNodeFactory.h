// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraphUtilities.h"
#include "EdGraph/EdGraphNode.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundStandardNodesNames.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateAudioAnalyzer.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "SGraphNode.h"
#include "SMetasoundGraphNode.h"
#include "SMetasoundGraphNodeComment.h"
#include "SMetasoundSpectrumAnalyzerGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class FMetasoundGraphNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* InNode) const override
	{
		using namespace Metasound::Editor;
		using namespace Metasound::Frontend;

		if (InNode->IsA<UMetasoundEditorGraphNode>())
		{
			if (const UMetasoundEditorGraphExternalNode* Node = Cast<UMetasoundEditorGraphExternalNode>(InNode))
			{
				FConstNodeHandle NodeHandle = Node->GetConstNodeHandle();
				const FMetasoundFrontendClassName ClassName = NodeHandle->GetClassMetadata().GetClassName();
				if (ClassName == FRerouteNodeTemplate::ClassName)
				{
					return SNew(SMetaSoundGraphNodeKnot, InNode);
				}
				else if (ClassName == FAudioAnalyzerNodeTemplate::ClassName)
				{
					return SNew(SMetaSoundSpectrumAnalyzerGraphNode, InNode);
				}
			}
			return SNew(SMetaSoundGraphNode, InNode);
		}
		else if (UMetasoundEditorGraphCommentNode* CommentNode = Cast<UMetasoundEditorGraphCommentNode>(InNode))
		{
			const UEdGraphSchema* EdGraphSchema = CommentNode->GetSchema();
			if (EdGraphSchema && EdGraphSchema->IsA(UMetasoundEditorGraphSchema::StaticClass()))
			{
				return SNew(SMetasoundGraphNodeComment, CommentNode);
			}
		}

		return nullptr;
	}
};
