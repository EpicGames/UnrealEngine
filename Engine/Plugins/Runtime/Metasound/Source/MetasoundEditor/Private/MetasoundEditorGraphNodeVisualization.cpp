// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorGraphNodeVisualization.h"

#include "Misc/LazySingleton.h"

namespace Metasound::Editor
{
	FGraphNodeVisualizationRegistry& FGraphNodeVisualizationRegistry::Get()
	{
		return TLazySingleton<FGraphNodeVisualizationRegistry>::Get();
	}

	void FGraphNodeVisualizationRegistry::TearDown()
	{
		TLazySingleton<FGraphNodeVisualizationRegistry>::TearDown();
	}

	void FGraphNodeVisualizationRegistry::RegisterVisualization(FName Key, FOnCreateGraphNodeVisualizationWidget OnCreateGraphNodeVisualizationWidget)
	{
		RegisteredVisualizationDelegates.Add(Key, OnCreateGraphNodeVisualizationWidget);
	}

	TSharedPtr<SWidget> FGraphNodeVisualizationRegistry::CreateVisualizationWidget(FName Key, const FCreateGraphNodeVisualizationWidgetParams& Params)
	{
		if (const FOnCreateGraphNodeVisualizationWidget* VisualizationDelegate = RegisteredVisualizationDelegates.Find(Key))
		{
			return VisualizationDelegate->Execute(Params);
		}

		return nullptr;
	}
}
