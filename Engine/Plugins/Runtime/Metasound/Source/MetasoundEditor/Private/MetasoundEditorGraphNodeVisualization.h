// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorModule.h"

namespace Metasound::Editor
{
	class FGraphNodeVisualizationRegistry
	{
	public:
		static FGraphNodeVisualizationRegistry& Get();
		static void TearDown();

		// For the given key, register a delegate that can be used for creating node visualization widgets.
		void RegisterVisualization(FName Key, FOnCreateGraphNodeVisualizationWidget OnCreateGraphNodeVisualizationWidget);

		// Creates a visualization widget if a delegate has been registered with the given key.
		TSharedPtr<SWidget> CreateVisualizationWidget(FName Key, const FCreateGraphNodeVisualizationWidgetParams& Params);

	private:
		TMap<FName, FOnCreateGraphNodeVisualizationWidget> RegisteredVisualizationDelegates;
	};

	class FGraphNodeVisualizationUtils
	{
	public:

		template<class T>
		static bool TryGetPinValue(const UMetasoundEditorGraphNode& MetaSoundNode, FName PinName, T& OutValue)
		{
			UEdGraphPin* const* FoundNamedInputPin = MetaSoundNode.Pins.FindByPredicate([&PinName](const UEdGraphPin* InPin)
				{
					return (InPin->Direction == EGPD_Input && InPin->PinType.PinCategory != FMetaSoundGraphPanelPinFactory::PinCategoryAudio && InPin->PinName == PinName);
				});

			if (!FoundNamedInputPin)
			{
				return false;
			}

			const UEdGraphPin* Pin = *FoundNamedInputPin;
			if (Pin->LinkedTo.IsEmpty())
			{
				FMetasoundFrontendLiteral DefaultLiteral;
				if (FGraphBuilder::GetPinLiteral(*Pin, DefaultLiteral))
				{
					if (DefaultLiteral.TryGet(OutValue))
					{
						return true;
					}
				}
			}
			else
			{
				// Find connected output for the input (only ever one):
				const UEdGraphPin* SourcePin = Pin->LinkedTo.Last();
				ensure(SourcePin->Direction == EGPD_Output);

				const UEdGraphPin* ReroutedOutputPin = FGraphBuilder::FindReroutedOutputPin(SourcePin);

				if (const UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(ReroutedOutputPin->GetOwningNode()))
				{
					TSharedPtr<FEditor> Editor = FGraphBuilder::GetEditorForNode(MetaSoundNode);
					if (Editor.IsValid())
					{
						const FGuid NodeID = Node->GetNodeID();
						const FName OutputName = ReroutedOutputPin->GetFName();
						if (Editor->GetConnectionManager().GetValue(NodeID, OutputName, OutValue))
						{
							return true;
						}
					}

					if (const UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(Node))
					{
						if (const UMetasoundEditorGraphMember* Member = MemberNode->GetMember())
						{
							if (Editor.IsValid())
							{
								// For an input member, we must use this NodeID/VertexName pair with the GraphConnectionManager:
								const FGuid MemberID = Member->GetMemberID();
								const FName MemberName = Member->GetMemberName();
								if (Editor->GetConnectionManager().GetValue(MemberID, MemberName, OutValue))
								{
									return true;
								}
							}

							if (const UMetasoundEditorGraphMemberDefaultLiteral* MemberDefaultLiteral = Member->GetLiteral())
							{
								FMetaSoundFrontendDocumentBuilder& Builder = Member->GetFrontendBuilderChecked();
								FMetasoundFrontendLiteral DefaultLiteral;
								MemberDefaultLiteral->TryFindDefault(DefaultLiteral, &Builder.GetBuildPageID());
								if (DefaultLiteral.TryGet(OutValue))
								{
									return true;
								}
							}
						}
					}
				}
			}

			return false;
		}
	};
}
