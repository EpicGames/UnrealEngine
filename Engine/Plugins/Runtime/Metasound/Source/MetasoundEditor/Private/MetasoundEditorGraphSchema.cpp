// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorGraphSchema.h"

#include "AssetDefinitionRegistry.h"
#include "Algo/AnyOf.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "HAL/IConsoleManager.h"
#include "Layout/SlateRect.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetKey.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorCommands.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendGraphLinter.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundFrontendRegistryContainer.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLiteral.h"
#include "MetasoundUObjectRegistry.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateAudioAnalyzer.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"
#include "ToolMenus.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorGraphSchema)

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound::Editor
{
	// If this fires (very unlikely), find usages on the Common category, and make sure those actions have their Grouping set to a larger number than
	// Recent + UMetasoundEditorSettings::MaxRecentEntriesEver.
	static_assert(static_cast<int32>(EPrimaryContextGroup::Common) - static_cast<int32>(EPrimaryContextGroup::Recents) > UMetasoundEditorSettings::MaxRecentEntriesEver,
		"Common graph actions could be going under the most recently used actions!");

	const FText& GetContextGroupDisplayName(EPrimaryContextGroup InContextGroup)
	{
		switch (InContextGroup)
		{
			case EPrimaryContextGroup::Inputs:
				return NodeCategories::Inputs;

			case EPrimaryContextGroup::Outputs:
				return NodeCategories::Outputs;

			case EPrimaryContextGroup::Graphs:
				return NodeCategories::Graphs;

			case EPrimaryContextGroup::Functions:
				return NodeCategories::Functions;

			case EPrimaryContextGroup::Conversions:
				return NodeCategories::Conversions;

			case EPrimaryContextGroup::Variables:
				return NodeCategories::Variables;

			case EPrimaryContextGroup::Favorites:
				return NodeCategories::Favorites;

			case EPrimaryContextGroup::Recents:
				return NodeCategories::Recents;

			case EPrimaryContextGroup::Common:
			default:
			{
				return FText::GetEmpty();
			}
		}
	}

	template<typename TAction UE_REQUIRES(TIsDerivedFrom<TAction, FMetasoundGraphSchemaAction_NewVariableNode>::Value)>
	TSharedPtr<TAction> MakeVariableAction(const FText& InCategory, int32 Grouping, const FText& InDisplayName, FGuid InVariableID, const FText& InToolTip)
	{
		TSharedPtr<TAction> NewAction = MakeShared<TAction>(
			InCategory,
			InDisplayName,
			InVariableID,
			InToolTip);
		NewAction->Grouping = Grouping;
		return NewAction;
	}

	namespace SchemaPrivate
	{
		static int32 EnableAllVersionsMetaSoundNodeClassCreationCVar = 0;
		FAutoConsoleVariableRef CVarEnableAllVersionsMetaSoundNodeClassCreation(
			TEXT("au.MetaSound.EnableAllVersionsNodeClassCreation"),
			EnableAllVersionsMetaSoundNodeClassCreationCVar,
			TEXT("Enable creating nodes for major versions of deprecated MetaSound classes in the Editor.\n")
			TEXT("0: Disabled (default), !0: Enabled"),
			ECVF_Default);

		static int32 ShowUnloadedAssetInBrowserCvar = 0;
		FAutoConsoleVariableRef CVarShowUnloadedAssetInBrowser(
			TEXT("au.MetaSound.Debug.ShowUnloadedAssetInBrowser"),
			ShowUnloadedAssetInBrowserCvar,
			TEXT("Shows a '*' in the MetaSound asset picker indicating the data displayed is from tags and not from a loaded asset.\n")
			TEXT("1: Disabled (default), !0: Enabled"),
			ECVF_Default);

		static const FText CategoryDelim = LOCTEXT("MetaSoundActionsCategoryDelim", "|");
		static const FText KeywordDelim = LOCTEXT("MetaSoundKeywordDelim", " ");

		static const FText InputDisplayNameFormat = LOCTEXT("DisplayNameAddInputFormat", "Get {0}");
		static const FText InputTooltipFormat = LOCTEXT("TooltipAddInputFormat", "Adds a getter for the input '{0}' to the graph.");

		static const FText OutputDisplayNameFormat = LOCTEXT("DisplayNameAddOutputFormat", "Set {0}");
		static const FText OutputTooltipFormat = LOCTEXT("TooltipAddOutputFormat", "Adds a setter for the output '{0}' to the graph.");

		static const FText VariableAccessorDisplayNameFormat = LOCTEXT("DisplayNameAddVariableAccessorFormat", "Get {0}");
		static const FText VariableAccessorTooltipFormat = LOCTEXT("TooltipAddVariableAccessorFormat", "Adds a getter for the variable '{0}' to the graph.");

		static const FText VariableDeferredAccessorDisplayNameFormat = LOCTEXT("DisplayNameAddVariableDeferredAccessorFormat", "Get Delayed {0}");
		static const FText VariableDeferredAccessorTooltipFormat = LOCTEXT("TooltipAddVariableDeferredAccessorFormat", "Adds a delayed getter for the variable '{0}' to the graph.");

		static const FText VariableMutatorDisplayNameFormat = LOCTEXT("DisplayNameAddVariableMutatorFormat", "Set {0}");
		static const FText VariableMutatorTooltipFormat = LOCTEXT("TooltipAddVariableMutatorFormat", "Adds a setter for the variable '{0}' to the graph.");

		static const FText ClassDescriptionAndAuthorFormat = LOCTEXT("ClassDescriptionAndAuthorFormat", "{0}\nAuthor: {1}");

		bool DataTypeSupportsAssetTypes(const Metasound::Frontend::FDataTypeRegistryInfo& InRegistryInfo, const TArray<FAssetData>& InAssets)
		{
			using namespace Metasound::Frontend;

			if (!InRegistryInfo.IsDataTypeProxyParsable())
			{
				return false;
			}
			const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();
			FDataTypeRegistryInfo TestDataType;
			FName DataTypeName = InRegistryInfo.DataTypeName;
			if (InRegistryInfo.bIsArrayType)
			{
				FString DataTypeString = DataTypeName.ToString();
				if (DataTypeString.EndsWith(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER))
				{
					DataTypeString.RemoveFromEnd(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER);
					DataTypeName = FName(DataTypeString);
					if (!DataTypeRegistry.GetDataTypeInfo(DataTypeName, TestDataType))
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}
			// In type is not array type but InAssets is a multiasset array
			else if (InAssets.Num() > 1)
			{
				return false;
			}

			const bool bSuccess = Algo::AllOf(InAssets, 
				[&DataTypeRegistry, &TestDataType, &DataTypeName](const FAssetData& Asset) 
				{
					if (UClass* Class = Asset.GetClass())
					{
						if (DataTypeRegistry.GetDataTypeInfo(Class, TestDataType))
						{
							return TestDataType.DataTypeName == DataTypeName;
						}
					}
					return false;
				});
			return bSuccess;
		}

		// Connects to first pin with the same DataType
		bool TryConnectNewNodeToMatchingDataTypePin(UEdGraphNode& NewGraphNode, UEdGraphPin* FromPin)
		{
			using namespace Metasound::Frontend;

			if (!FromPin)
			{
				return false;
			}

			if (FromPin->Direction == EGPD_Input)
			{
				FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(FromPin);
				for (UEdGraphPin* Pin : NewGraphNode.Pins)
				{
					if (Pin->Direction == EGPD_Output)
					{
						FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(Pin);
						if (OutputHandle->IsValid() && 
							InputHandle->CanConnectTo(*OutputHandle).Connectable == FConnectability::EConnectable::Yes)
						{
							if (ensure(FGraphBuilder::ConnectNodes(*FromPin, *Pin, true /* bConnectEdPins */)))
							{
								return true;
							}
						}
					}
				}
			}

			if (FromPin->Direction == EGPD_Output)
			{
				FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(FromPin);
				for (UEdGraphPin* Pin : NewGraphNode.Pins)
				{
					if (Pin->Direction == EGPD_Input)
					{
						FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(Pin);
						if (InputHandle->IsValid() && 
							InputHandle->CanConnectTo(*OutputHandle).Connectable == FConnectability::EConnectable::Yes)
						{
							if (ensure(FGraphBuilder::ConnectNodes(*Pin, *FromPin, true /* bConnectEdPins */)))
							{
								return true;
							}
						}
					}
				}
			}

			return false;
		}

		struct FDataTypeActionQuery
		{
			FGraphActionMenuBuilder& ActionMenuBuilder;
			const TArray<Frontend::FConstNodeHandle>& NodeHandles;
			FInterfaceNodeFilterFunction Filter;
			EPrimaryContextGroup ContextGroup;
			const FText& DisplayNameFormat;
			const FText& TooltipFormat;
			bool bShowSelectedActions = false;
		};

		template <typename TAction>
		void GetDataTypeActions(const FDataTypeActionQuery& InQuery)
		{
			using namespace Editor;
			using namespace Frontend;

			const UMetasoundEditorSettings* Settings = GetDefault<UMetasoundEditorSettings>();

			for (const FConstNodeHandle& NodeHandle : InQuery.NodeHandles)
			{
				if (!InQuery.Filter || InQuery.Filter(NodeHandle))
				{
					constexpr bool bIncludeNamespace = true;

					const FText& GroupName = GetContextGroupDisplayName(InQuery.ContextGroup);
					const FText NodeDisplayName = FGraphBuilder::GetDisplayName(*NodeHandle, bIncludeNamespace);
					const FText Tooltip = FText::Format(InQuery.TooltipFormat, NodeDisplayName);
					const FText DisplayName = FText::Format(InQuery.DisplayNameFormat, NodeDisplayName);
					TSharedPtr<TAction> NewNodeAction = MakeShared<TAction>(GroupName, DisplayName, NodeHandle->GetID(), Tooltip, InQuery.ContextGroup);
					InQuery.ActionMenuBuilder.AddAction(NewNodeAction);

					{
						const FString Key = NewNodeAction->GetUniqueStableFavoriteKey();
						const int32 RecentIndex = Settings->FindRecentlyUsedMetaSoundsGraphAction(Key);
						if (RecentIndex != INDEX_NONE)
						{
							TSharedPtr<TAction> NewRecentAction = MakeShared<TAction>(
								NodeCategories::Recents,
								DisplayName,
								NodeHandle->GetID(),
								Tooltip,
								EPrimaryContextGroup::Recents);
							NewRecentAction->Grouping = static_cast<int32>(EPrimaryContextGroup::Recents) + RecentIndex;
							InQuery.ActionMenuBuilder.AddAction(NewRecentAction);
						}

						if (Settings->IsFavoriteMetaSoundsGraphAction(Key))
						{
							TSharedPtr<TAction> NewFavAction = MakeShared<TAction>(
								FText::FormatOrdered(INVTEXT("{0}|{1}"), NodeCategories::Favorites, GroupName),
								DisplayName,
								NodeHandle->GetID(),
								Tooltip,
								EPrimaryContextGroup::Favorites);
							InQuery.ActionMenuBuilder.AddAction(NewFavAction);
						}
					}
				}
			}
		}

		void SelectNodeInEditor(UMetasoundEditorGraph& InMetaSoundGraph, UMetasoundEditorGraphNode& InNode)
		{
			TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(InMetaSoundGraph);
			if (MetasoundEditor.IsValid())
			{
				MetasoundEditor->ClearSelectionAndSelectNode(&InNode);
			}
		}

		void SelectNodeInEditorForRename(UMetasoundEditorGraph& InMetaSoundGraph, UMetasoundEditorGraphNode& InNode)
		{
			TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(InMetaSoundGraph);
			if (MetasoundEditor.IsValid())
			{
				MetasoundEditor->ClearSelectionAndSelectNode(&InNode);
				MetasoundEditor->SetDelayedRename();
			}
		}

		void PostAddNode(UMetasoundEditorGraphNode* NewGraphNode, UObject& ParentMetasound, UMetasoundEditorGraph& MetaSoundGraph, const FVector2f& InLocation, bool bSelectNewNode = true)
		{
			using namespace Metasound::Frontend;

			NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(InLocation));
			NewGraphNode->SyncLocationFromFrontendNode();

			// Ensure referencing graphs are updated
			FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
			RegOptions.bIgnoreIfLiveAuditioning = true;
			FGraphBuilder::RegisterGraphWithFrontend(ParentMetasound, MoveTemp(RegOptions));

			if (bSelectNewNode)
			{
				SchemaPrivate::SelectNodeInEditorForRename(MetaSoundGraph, *NewGraphNode);
			}
			else
			{
				SchemaPrivate::SelectNodeInEditor(MetaSoundGraph, *NewGraphNode);
			}
		}

		UEdGraphNode* PromoteToVariable(const FName BaseName, UEdGraphPin& FromPin, const FName DataType, const FMetasoundFrontendClass& InVariableClass, const FVector2f& InLocation, bool bSelectNode)
		{
			using namespace Frontend;

			UEdGraphNode* ConnectedNode = Cast<UEdGraphNode>(FromPin.GetOwningNode());
			if (!ensure(ConnectedNode))
			{
				return nullptr;
			}

			const FMetasoundFrontendClassName& ClassName = InVariableClass.Metadata.GetClassName();

			UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ConnectedNode->GetGraph());
			FMetaSoundFrontendDocumentBuilder& DocBuilder = MetaSoundGraph->GetBuilderChecked().GetBuilder();
			const FName NodeName = DocBuilder.GenerateUniqueVariableName(BaseName.ToString());

			const FScopedTransaction Transaction(FText::Format(
				LOCTEXT("PromoteNodeVertexToGraphVariableFormat", "Promote MetaSound Node {0} to {1}"),
				FText::FromName(NodeName),
				FText::FromName(ClassName.Namespace)));

			UObject& ParentMetaSound = MetaSoundGraph->GetMetasoundChecked();
			ParentMetaSound.Modify();
			MetaSoundGraph->Modify();

			// Cache the default literal from the pin if connecting to an input
			FMetasoundFrontendLiteral DefaultValue;
			if (FromPin.Direction == EGPD_Input)
			{
				FGraphBuilder::GetPinLiteral(FromPin, DefaultValue);
			}

			const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.AddGraphVariable(NodeName, DataType);
			if (ensure(FrontendVariable))
			{
				UMetasoundEditorGraphVariable* Variable = MetaSoundGraph->FindOrAddVariable(FrontendVariable->Name);
				if (ensure(Variable))
				{
					const FMetasoundFrontendNode* NewVariableNode = DocBuilder.AddGraphVariableNode(FrontendVariable->Name, InVariableClass.Metadata.GetType());
					if (ensure(NewVariableNode))
					{
						UMetasoundEditorGraphVariableNode* NewGraphNode = FGraphBuilder::AddVariableNode(ParentMetaSound, NewVariableNode->GetID());
						if (ensure(NewGraphNode))
						{
							// Set the literal using the new value if connecting to an input
							if (FromPin.Direction == EGPD_Input)
							{
								UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Variable->GetLiteral();
								if (ensure(DefaultLiteral))
								{
									DefaultLiteral->SetFromLiteral(DefaultValue);
								}

								// Ensures the setter node value is synced with the editor literal value
								constexpr bool bPostTransaction = false;
								Variable->UpdateFrontendDefaultLiteral(bPostTransaction);
							}

							UEdGraphNode* EdGraphNode = CastChecked<UEdGraphNode>(NewGraphNode);
							if (ensure(SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*EdGraphNode, &FromPin)))
							{
								SchemaPrivate::PostAddNode(NewGraphNode, ParentMetaSound, *MetaSoundGraph, InLocation, bSelectNode);
								return EdGraphNode;
							}
						}
					}
				}
			}

			return nullptr;
		}

		bool WillAddingVariableAccessorCauseLoop(const Frontend::IVariableController& InVariable, const Frontend::IInputController& InInput)
		{
			using namespace Metasound::Frontend;

			// A variable mutator node must come before a variable accessor node,
			// or else the nodes will create a loop from the hidden variable pin. 
			// To determine if adding an accessor node will cause a loop (before actually
			// adding an accessor node), we check whether an existing mutator can
			// reach the node upstream which wants to connect to the accessor node.
			//
			// Example:
			// Will cause loop:
			// 	[AccessorNode]-->[DestinationNode]-->[Node]-->[MutatorNode] 
			// 	       ^-------------------------------------------|
			//
			// Will not cause loop
			//  [Node]-->[MutatorNode]-->[AccessorNode]-->[DestinationNode]
			//       |                                        ^ 
			//       |----------------------------------------|
			FConstNodeHandle MutatorNode = InVariable.FindMutatorNode();
			FConstNodeHandle DestinationNode = InInput.GetOwningNode();
			return FGraphLinter::IsReachableUpstream(*MutatorNode, *DestinationNode);
		}

		bool WillAddingVariableMutatorCauseLoop(const Frontend::IVariableController& InVariable, const Frontend::IOutputController& InOutput)
		{
			using namespace Metasound::Frontend;

			// A variable mutator node must come before a variable accessor node,
			// or else the nodes will create a loop from the hidden variable pin. 
			// To determine if adding a mutator node will cause a loop (before actually
			// adding a mutator node), we check whether any existing accessor can
			// reach the node downstream which wants to connect to the mutator node.
			//
			// Example:
			// Will cause loop:
			// 	[AccessorNode]-->[Node]-->[SourceNode]-->[MutatorNode] 
			// 	     ^---------------------------------------|
			//
			// Will not cause loop
			//  [SourceNode]-->[MutatorNode]-->[AccessorNode]-->[Node]
			//       |                                            ^ 
			//       |--------------------------------------------|
			TArray<FConstNodeHandle> AccessorNodes = InVariable.FindAccessorNodes();
			FConstNodeHandle SourceNode = InOutput.GetOwningNode();

			auto IsSourceNodeReachableDownstream = [&SourceNode](const FConstNodeHandle& AccessorNode)
			{
				return FGraphLinter::IsReachableDownstream(*AccessorNode, *SourceNode);
			};

			return Algo::AnyOf(AccessorNodes, IsSourceNodeReachableDownstream);
		}

		class FSchemaQueryResult : public ISchemaQueryResult
		{
			public:
				FSchemaQueryResult(const Engine::FMetaSoundAssetManager& AssetManager, Frontend::FMetaSoundClassInfo InClassInfo)
				: ClassInfo(MoveTemp(InClassInfo))
			{
				if (FMetaSoundAssetKey::IsValidType(ClassInfo.ClassType))
				{
					bIsNative = !AssetManager.ContainsKey(FMetaSoundAssetKey(ClassInfo.ToRegistryKey()));
				}
				else
				{
					bIsNative = false;
				}
			}

			virtual ~FSchemaQueryResult() = default;

			virtual FMetasoundFrontendClassMetadata GenerateClassMetadata() const override
			{
				using namespace Frontend;

				// Asset-defined node classes must be loaded (and registered) in order to generate metadata
				if (!bIsNative)
				{
					const FMetaSoundAssetKey AssetKey = ClassInfo.ToRegistryKey();
					if (FMetasoundAssetBase* Asset = IMetaSoundAssetManager::GetChecked().TryLoadAssetFromKey(AssetKey))
					{
						FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
						RegOptions.bForceReregister = false;
						UObject* MetaSoundObject = Asset->GetOwningAsset();
						check(MetaSoundObject);
						FGraphBuilder::RegisterGraphWithFrontend(*MetaSoundObject);
					}
				}

				FMetasoundFrontendClass RegisteredClass;
				const bool bFoundClass = Frontend::INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(ClassInfo.ToRegistryKey(), RegisteredClass);
				ensure(bFoundClass);
				return RegisteredClass.Metadata;
			}

			virtual bool CanConnectInputOfTypeAndAccess(FName InputTypeName, EMetasoundFrontendVertexAccessType InputAccessType) const override
			{
				const bool bCanConnect = Algo::AnyOf(ClassInfo.InterfaceInfo.Outputs, [&InputTypeName, &InputAccessType](const FMetaSoundClassVertexInfo& Output)
				{
					return IsCastable(InputTypeName,Output.TypeName) && FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(Output.AccessType, InputAccessType);
				});
				return bCanConnect;
			}

			virtual bool CanConnectOutputOfTypeAndAccess(FName OutputTypeName, EMetasoundFrontendVertexAccessType OutputAccessType) const override
			{
				const bool bCanConnect = Algo::AnyOf(ClassInfo.InterfaceInfo.Inputs, [&OutputTypeName, &OutputAccessType](const FMetaSoundClassVertexInfo& Input)
				{
					return IsCastable(OutputTypeName, Input.TypeName) && FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(OutputAccessType, Input.AccessType);
				});
				return bCanConnect;
			}

			virtual EMetasoundFrontendClassAccessFlags GetAccessFlags() const override
			{
				return ClassInfo.AccessFlags;
			}

			virtual FMetaSoundAssetKey GetAssetKey() const override
			{
				return FMetaSoundAssetKey(ClassInfo.ToRegistryKey());
			}

			virtual const TArray<FText>& GetCategoryHierarchy() const override
			{
				return ClassInfo.InterfaceInfo.SearchInfo.Hierarchy;
			}

			virtual FText GetDisplayName() const override
			{
				using namespace Frontend;

				FText DisplayName;
				auto GetAssetDisplayNameFromMetadata = [&DisplayName](const FMetaSoundClassInfo& ClassInfo)
				{
					DisplayName = ClassInfo.InterfaceInfo.SearchInfo.ClassDisplayName;
					if (DisplayName.IsEmptyOrWhitespace())
					{
						if (FMetaSoundAssetKey::IsValidType(ClassInfo.ClassType))
						{
							const FTopLevelAssetPath Path = IMetaSoundAssetManager::GetChecked().FindAssetPath(FMetaSoundAssetKey(ClassInfo.ToRegistryKey()));
							if (Path.IsValid())
							{
								DisplayName = FText::FromName(Path.GetAssetName());
							}
						}
					}
				};

				// 1. Try to get display name from metadata or asset if one can be found from the asset manager
				GetAssetDisplayNameFromMetadata(ClassInfo);

				// 2. If version is missing from the registry or from asset system, then this node
				// will not provide a useful DisplayName.  In that case, attempt to find the next highest
				// class & associated DisplayName.
				if (DisplayName.IsEmptyOrWhitespace())
				{
					FMetaSoundClassInfo HighestVersionInfo;
					if (ISearchEngine::Get().FindClassWithHighestVersion(ClassInfo.ClassName, HighestVersionInfo))
					{
						GetAssetDisplayNameFromMetadata(HighestVersionInfo);
					}
				}

				return DisplayName;
			}

			virtual const TArray<FText>& GetKeywords() const override
			{
				return ClassInfo.InterfaceInfo.SearchInfo.Keywords;
			}

			virtual FText GetTooltip() const
			{
				const FText& ClassDescription = ClassInfo.InterfaceInfo.SearchInfo.ClassDescription;
#if 0
				// TODO: Add author to asset tag/class info data and include in tooltip
				const FString& Author = ClassInfo.InterfaceInfo.SearchInfo.Author;
				return !Author.IsEmpty()
					? FText::Format(ClassDescriptionAndAuthorFormat, ClassDescription, FText::FromString(Author))
					: ClassDescription;
#else
				return ClassDescription;
#endif
			}

			virtual EMetasoundFrontendClassType GetRegistryClassType() const override
			{
				return ClassInfo.ClassType;
			}

			virtual bool IsNative() const
			{
				return bIsNative;
			}

			private:
			Frontend::FMetaSoundClassInfo ClassInfo;
			bool bIsNative = false;
		};
	} // namespace SchemaPrivate

	namespace SchemaUtils
	{
		UEdGraphNode* PromoteToInput(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode, TOptional<FMetasoundFrontendLiteral> InLiteral)
		{
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
			UObject& ParentMetasound = MetasoundGraph->GetMetasoundChecked();
			FMetaSoundFrontendDocumentBuilder& Builder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);
			FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(Builder, FromPin);
			if (!ensure(InputVertexHandle.IsSet()))
			{
				return nullptr;
			}

			const FMetasoundFrontendVertex* InputVertex = Builder.FindNodeInput(InputVertexHandle.NodeID, InputVertexHandle.VertexID);
			if (!ensure(InputVertex))
			{
				return nullptr;
			}

			const FScopedTransaction Transaction(LOCTEXT("PromoteNodeInputToGraphInput", "Promote MetaSound Node Input to Graph Input"));
			ParentMetasound.Modify();
			MetasoundGraph->Modify();

			// Default literal must ALWAYS have value for default pageID, so even though this may get called
			// from a page "higher" in the page stack, always add the literal value for the default page ID.
			auto InitDefaultLiterals = [](FMetasoundFrontendLiteral NewLiteral)
			{
				TArray<FMetasoundFrontendClassInputDefault> InitValues;
				InitValues.Add_GetRef(Metasound::Frontend::DefaultPageID).Literal = NewLiteral;
				return InitValues;
			};

			TArray<FMetasoundFrontendClassInputDefault> DefaultLiterals;
			if (InLiteral.IsSet())
			{
				DefaultLiterals = InitDefaultLiterals(*InLiteral);
			}
			else if (const FMetasoundFrontendVertexLiteral* VertexLiteral = Builder.FindNodeInputDefault(InputVertexHandle.NodeID, InputVertexHandle.VertexID))
			{
				// Since a default page ID requires an associated value and no other nodes on any page would be connected to this new input, use
				// the default page ID. If the user wants different behavior, when they connect the newly created input on a lower-indexed graph,
				// they will assign a proper page default value therein.  This in practice should cut down on duplicate page input default data.
				DefaultLiterals = InitDefaultLiterals(VertexLiteral->Value);
			}
			else if (const TArray<FMetasoundFrontendClassInputDefault>* ClassDefaults = Builder.FindNodeClassInputDefaults(InputVertexHandle.NodeID, InputVertex->Name))
			{
				if (!ensure(!ClassDefaults->IsEmpty()))
				{
					return nullptr;
				}

				DefaultLiterals = *ClassDefaults;
				const FMetasoundFrontendClassInputDefault* DefaultPageValueLiteral = DefaultLiterals.FindByPredicate([](const FMetasoundFrontendClassInputDefault& InputDefault)
				{
					return InputDefault.PageID == Metasound::Frontend::DefaultPageID;
				});

				// Code OR asset-defined classes should ALWAYS include input default value associated with default page ID by this point
				if (!ensure(DefaultPageValueLiteral))
				{
					DefaultLiterals.Add_GetRef(Metasound::Frontend::DefaultPageID).Literal = ClassDefaults->Last().Literal;
				}
			}
			else
			{
				FMetasoundFrontendLiteral DefaultValue;
				DefaultValue.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(InputVertex->TypeName));
				DefaultLiterals = InitDefaultLiterals(MoveTemp(DefaultValue));
			}

			const FCreateNodeVertexParams VertexParams =
			{
				InputVertex->TypeName,
				Builder.GetNodeInputAccessType(InputVertexHandle.NodeID, InputVertex->VertexID)
			};

			const FMetasoundFrontendNode* NewNode = nullptr;
			// Name may be different than previous name because it may already exist in the graph,
			// and CreateUniqueClassInput will add an identifier to the end to make it unique
			FName NewName; 
			{
				FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(ParentMetasound, VertexParams, DefaultLiterals, &InputVertex->Name);
				NewName = ClassInput.Name;
				NewNode = Builder.AddGraphInput(MoveTemp(ClassInput));
			}

			if (ensure(NewNode))
			{
				UMetasoundEditorGraphInput* Input = MetasoundGraph->FindOrAddInput(NewNode->GetID());
				if (ensure(Input))
				{
					if (const FMetasoundFrontendNode* NewTemplateNode = FInputNodeTemplate::CreateNode(Builder, NewName))
					{
						if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetasound, NewTemplateNode->GetID()))
						{
							UEdGraphNode* EdGraphNode = CastChecked<UEdGraphNode>(NewGraphNode);
							if (ensure(SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*EdGraphNode, FromPin)))
							{
								SchemaPrivate::PostAddNode(NewGraphNode, ParentMetasound, *MetasoundGraph, InLocation, bSelectNewNode);
								return EdGraphNode;
							}
						}
					}
				}
			}

			return nullptr;
		}
	
		UEdGraphNode* PromoteToOutput(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode)
		{
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
			UObject& ParentMetasound = MetasoundGraph->GetMetasoundChecked();
			FMetaSoundFrontendDocumentBuilder& Builder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);
			FMetasoundFrontendVertexHandle OutputVertexHandle = FGraphBuilder::GetPinVertexHandle(Builder, FromPin);
			if (!ensure(OutputVertexHandle.IsSet()))
			{
				return nullptr;
			}

			const FMetasoundFrontendVertex* OutputVertex = Builder.FindNodeOutput(OutputVertexHandle.NodeID, OutputVertexHandle.VertexID);;
			if (!ensure(OutputVertex))
			{
				return nullptr;
			}

			const FScopedTransaction Transaction(LOCTEXT("PromoteNodeOutputToGraphOutput", "Promote MetaSound Node Output to Graph Output"));
			ParentMetasound.Modify();
			MetasoundGraph->Modify();

			const FCreateNodeVertexParams VertexParams =
			{
				OutputVertex->TypeName,
				Builder.GetNodeOutputAccessType(OutputVertexHandle.NodeID, OutputVertex->VertexID)
			};
			const FMetasoundFrontendClassOutput ClassOutput = FGraphBuilder::CreateUniqueClassOutput(ParentMetasound, VertexParams, &OutputVertex->Name);
			const FMetasoundFrontendNode* OutputNode = Builder.AddGraphOutput(ClassOutput);
			if (ensure(OutputNode))
			{
				const FGuid OutputNodeID = OutputNode->GetID();
				UMetasoundEditorGraphOutput* Output = MetasoundGraph->FindOrAddOutput(OutputNodeID);
				if (ensure(Output))
				{
					if (UMetasoundEditorGraphOutputNode* NewGraphNode = FGraphBuilder::AddOutputNode(ParentMetasound, OutputNodeID))
					{
						UEdGraphNode* EdGraphNode = CastChecked<UEdGraphNode>(NewGraphNode);
						if (ensure(SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*EdGraphNode, FromPin)))
						{
							SchemaPrivate::PostAddNode(NewGraphNode, ParentMetasound, *MetasoundGraph, InLocation, bSelectNewNode);

							return EdGraphNode;
						}
					}
				}
			}

			return nullptr;
		}

		UEdGraphNode* PromoteToVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode)
		{
			using namespace Metasound;
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(FromPin);
			if (!ensure(InputHandle->IsValid()))
			{
				return nullptr;
			}

			const FName NodeName = InputHandle->GetName();
			const FName DataType = InputHandle->GetDataType();
			FMetasoundFrontendClass VariableClass;
			if (ensure(IDataTypeRegistry::Get().GetFrontendVariableAccessorClass(DataType, VariableClass)))
			{
				return SchemaPrivate::PromoteToVariable(NodeName, *FromPin, DataType, VariableClass, InLocation, bSelectNewNode);
			}

			return nullptr;
		}

		UEdGraphNode* PromoteToDeferredVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode)
		{
			using namespace Metasound;
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(FromPin);
			if (!ensure(InputHandle->IsValid()))
			{
				return nullptr;
			}

			const FName NodeName = InputHandle->GetName();
			const FName DataType = InputHandle->GetDataType();
			FMetasoundFrontendClass VariableClass;
			if (ensure(IDataTypeRegistry::Get().GetFrontendVariableDeferredAccessorClass(DataType, VariableClass)))
			{
				return SchemaPrivate::PromoteToVariable(NodeName, *FromPin, DataType, VariableClass, InLocation, bSelectNewNode);
			}

			return nullptr;
		}

		UEdGraphNode* PromoteToMutatorVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode)
		{
			using namespace Metasound;
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(FromPin);
			if (!ensure(OutputHandle->IsValid()))
			{
				return nullptr;
			}

			const FName NodeName = OutputHandle->GetName();
			const FName DataType = OutputHandle->GetDataType();
			FMetasoundFrontendClass VariableClass;
			if (ensure(IDataTypeRegistry::Get().GetFrontendVariableMutatorClass(DataType, VariableClass)))
			{
				return SchemaPrivate::PromoteToVariable(NodeName, *FromPin, DataType, VariableClass, InLocation, bSelectNewNode);
			}

			return nullptr;
		}

		UEdGraphNode* AddNewGraphInput(FName DataType, FMetasoundFrontendLiteral DefaultLiteral, UEdGraph* ParentGraph, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode)
		{
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;
			using namespace Metasound::Engine;

			UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
			UObject& ParentMetasound = MetasoundGraph->GetMetasoundChecked();

			const FScopedTransaction Transaction(LOCTEXT("AddNewGraphInputNode", "Add New MetaSound Graph Input"));
			ParentMetasound.Modify();
			ParentGraph->Modify();

			FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);

			const FCreateNodeVertexParams VertexParams =
			{
				DataType,
				EMetasoundFrontendVertexAccessType::Reference
			};

			const FMetasoundFrontendNode* NewNode = nullptr;

			TArray<FMetasoundFrontendClassInputDefault> DefaultLiterals;
			const FGuid PageID = Builder.GetBuildPageID();
			// Create default page literal if not default page
			if (PageID != DefaultPageID)
			{
				FMetasoundFrontendLiteral DefaultPageLiteral;
				DefaultPageLiteral.SetType(DefaultLiteral.GetType());
				DefaultLiterals.Emplace(DefaultPageID, MoveTemp(DefaultPageLiteral));
			}
			DefaultLiterals.Emplace(PageID, DefaultLiteral);

			// Name may be different than previous name because it may already exist in the graph,
			// and CreateUniqueClassInput will add an identifier to the end to make it unique
			FName NewName = TEXT("Input");
			{
				FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(ParentMetasound, VertexParams, DefaultLiterals, &NewName);
				NewName = ClassInput.Name;
				NewNode = Builder.AddGraphInput(MoveTemp(ClassInput));
			}

			if (ensure(NewNode))
			{
				UMetasoundEditorGraphInput* Input = MetasoundGraph->FindOrAddInput(NewNode->GetID());
				if (ensure(Input))
				{
					if (const FMetasoundFrontendNode* NewTemplateNode = FInputNodeTemplate::CreateNode(Builder, NewName))
					{
						if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetasound, NewTemplateNode->GetID()))
						{
							UEdGraphNode* EdGraphNode = CastChecked<UEdGraphNode>(NewGraphNode);
							SchemaPrivate::PostAddNode(NewGraphNode, ParentMetasound, *MetasoundGraph, InLocation, bSelectNewNode);
							return EdGraphNode;
						}
					}
				}
			}
			return nullptr;
		}
	} // namespace SchemaUtils
} // namespace Metasound::Editor

UEdGraphNode* FMetasoundGraphSchemaAction_NodeWithMultipleOutputs::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	using namespace Metasound::Editor;
	
	UEdGraphNode* ResultNode = nullptr;

	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location, bSelectNewNode);

		if (ResultNode)
		{
			// Try autowiring the rest of the pins
			for (int32 Index = 1; Index < FromPins.Num(); ++Index)
			{
				ResultNode->AutowireNewNode(FromPins[Index]);
			}
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, nullptr, Location, bSelectNewNode);
	}

	bSelectNewNode &= ResultNode != nullptr;
	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*ParentGraph);
	if (MetasoundEditor.IsValid() && bSelectNewNode)
	{
		MetasoundEditor->ClearSelectionAndSelectNode(ResultNode);
	}

	return ResultNode;
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewNode::GetIconBrush() const
{
	using namespace Metasound::Frontend;

	if (QueryResult.IsValid() && QueryResult->IsNative())
	{
		const FMetaSoundAssetKey AssetKey = QueryResult->GetAssetKey();

		FMetasoundFrontendClass FrontendClass;
		INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(AssetKey, FrontendClass);
		const FMetasoundFrontendClassStyleDisplay& StyleDisplay = FrontendClass.Style.Display;

		FSlateIcon NativeIcon = FSlateIcon("MetaSoundStyle", "MetasoundEditor.Metasound.Node.Icon");
		if (!StyleDisplay.StyleSet.IsNone() && !StyleDisplay.Icon.IsNone())
		{
			NativeIcon = FSlateIcon(StyleDisplay.StyleSet, StyleDisplay.Icon);
		}

		return NativeIcon.GetIcon();
	}

	if (QueryResult.IsValid() && QueryResult.Get())
	{
		const FMetaSoundAssetKey AssetKey = QueryResult.Get()->GetAssetKey();
		if (const TScriptInterface<IMetaSoundDocumentInterface> MetaSoundDoc = IMetaSoundAssetManager::GetChecked().FindAssetAsDocumentInterface(AssetKey))
		{
			const UClass& Class = MetaSoundDoc->GetBaseMetaSoundUClass();
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(&Class))
			{
				const FAssetData AssetData(MetaSoundDoc.GetObject());
				return AssetDefinition->GetIconBrush(AssetData, AssetData.AssetClassPath.GetAssetName());
			}
		}
	}

	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Graph");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewNode::GetIconColor() const
{
	using namespace Metasound::Frontend;

	if (QueryResult.IsValid() && QueryResult->IsNative())
	{
		const FMetaSoundAssetKey AssetKey = QueryResult->GetAssetKey();

		FMetasoundFrontendClass FrontendClass;
		INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(AssetKey, FrontendClass);
		const FMetasoundFrontendClassStyleDisplay& StyleDisplay = FrontendClass.Style.Display;
		if (!StyleDisplay.StyleSet.IsNone() && !StyleDisplay.Color.IsNone())
		{
			if (const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle(StyleDisplay.StyleSet))
			{
				static FLinearColor IconColor;
				IconColor = Style->GetSlateColor(StyleDisplay.Color, nullptr, FLinearColor::Transparent).GetSpecifiedColor();
				return IconColor;
			}
		}
	}

	if (QueryResult.IsValid() && QueryResult.Get())
	{
		const FMetaSoundAssetKey AssetKey = QueryResult.Get()->GetAssetKey();
		if (const TScriptInterface<IMetaSoundDocumentInterface> MetaSoundDoc = IMetaSoundAssetManager::GetChecked().FindAssetAsDocumentInterface(AssetKey))
		{
			if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
			{
				const FAssetData AssetData(MetaSoundDoc.GetObject());
				FName AssetClassName = AssetData.AssetClassPath.GetAssetName();
				FName ColorName(*(AssetClassName.ToString() + TEXT(".Color")));
				return MetasoundStyle->GetColor(ColorName);
			}
		}
	}

	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		if (QueryResult.IsValid())
		{
			return EditorSettings->NativeNodeTitleColor;
		}
	}

	return Super::GetIconColor();
}

FString FMetasoundGraphSchemaAction_NewNode::GetUniqueStableFavoriteKey() const
{
	return QueryResult->GetAssetKey().ClassName.ToString();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	const FScopedTransaction Transaction(LOCTEXT("AddNewNode", "Add New MetaSound Node"));
	UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetasound = MetaSoundGraph->GetMetasoundChecked();
	ParentMetasound.Modify();
	ParentGraph->Modify();

	FMetasoundFrontendClassMetadata Metadata = QueryResult->GenerateClassMetadata();
	if (UMetasoundEditorGraphExternalNode* NewGraphNode = FGraphBuilder::AddExternalNode(ParentMetasound, Metadata, bSelectNewNode))
	{
		NewGraphNode->Modify();
		NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
		NewGraphNode->SyncLocationFromFrontendNode();
		SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, FromPin);
		SchemaPrivate::SelectNodeInEditorForRename(*MetaSoundGraph, *NewGraphNode);

		if (DroppedObjectsInfo.IsSet())
		{
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(ParentMetasound);
			FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetBuilder();

			// Find the vertex to set the dropped objects on
			// Just use the first matching vertex with the correct type for now 
			auto FindFirstVertexMatchingType = [&DocBuilder](FGuid NodeID, FName TypeName, bool bFindInput /*If false, find output*/)
			{
				const TArray<const FMetasoundFrontendVertex*> Vertices = bFindInput ? 
					DocBuilder.FindUserModifiableNodeInputs(NodeID) : DocBuilder.FindUserModifiableNodeOutputs(NodeID);
				const FMetasoundFrontendVertex* const* Vertex = Vertices.FindByPredicate([&TypeName](const FMetasoundFrontendVertex* Vertex)
				{
					return Vertex->TypeName == TypeName;
				});
				return (Vertex && *Vertex) ? *Vertex : nullptr;
			};
			
			const FMetasoundFrontendVertex* FirstMatchingVertex = FindFirstVertexMatchingType(NewGraphNode->GetNodeID(), DroppedObjectsInfo->TypeName, /*bFindInput*/true);
			if (!ensure(FirstMatchingVertex))
			{
				UE_LOGF(LogMetasoundEditor, Warning, "Failed to find matching vertex on node %ls with type %ls when setting dropped objects.", *NewGraphNode->GetDisplayName().ToString(), *DroppedObjectsInfo->TypeName.ToString());
				return NewGraphNode;
			}

			// Set literal
			FMetasoundFrontendLiteral DefaultLiteral;
			if (DroppedObjectsInfo->DroppedObjects.Num() == 1)
			{
				// As single object
				DefaultLiteral.Set(DroppedObjectsInfo->DroppedObjects[0]);
			}
			else
			{
				// As array
				DefaultLiteral.Set(DroppedObjectsInfo->DroppedObjects);
			}

			// Add dropped objects to literal directly
			// If more than one object, can only set on connected input because 
			// because in graph array literal editing is not currently supported
			if (DroppedObjectsInfo->DroppedObjects.Num() == 1 && DroppedObjectsInfo->ModifierKeys.IsShiftDown())
			{
				if (FirstMatchingVertex)
				{
					DocBuilder.SetNodeInputDefault(NewGraphNode->GetNodeID(), FirstMatchingVertex->VertexID, DefaultLiteral);
				}
			}
			// Add with object(s) set on new connected input
			else 
			{
				// Setup class input
				TArray<FMetasoundFrontendClassInputDefault> DefaultLiterals;
				const FGuid PageID = DocBuilder.GetBuildPageID();
				// Create default page literal if not default page
				if (PageID != DefaultPageID)
				{
					FMetasoundFrontendLiteral DefaultPageLiteral;
					DefaultPageLiteral.SetType(DefaultLiteral.GetType());
					DefaultLiterals.Emplace(DefaultPageID, MoveTemp(DefaultPageLiteral));
				}
				DefaultLiterals.Emplace(PageID, DefaultLiteral);
				
				const FCreateNodeVertexParams VertexParams =
				{
					FirstMatchingVertex->TypeName,
					DocBuilder.GetNodeInputAccessType(NewGraphNode->GetNodeID(), FirstMatchingVertex->VertexID)
				};
				// Create class input and input node
				const FMetasoundFrontendNode* InputNode;
				FName NewName;
				{
					FMetasoundFrontendClassInput ClassInput = FGraphBuilder::CreateUniqueClassInput(ParentMetasound, VertexParams, DefaultLiterals, &FirstMatchingVertex->Name);
					NewName = ClassInput.Name;
					InputNode = DocBuilder.AddGraphInput(MoveTemp(ClassInput));
				}

				if (ensure(InputNode))
				{
					UMetasoundEditorGraphInput* Input = MetaSoundGraph->FindOrAddInput(InputNode->GetID());
					if (ensure(Input))
					{
						// Add template node 
						if (const FMetasoundFrontendNode* NewTemplateNode = FInputNodeTemplate::CreateNode(DocBuilder, NewName))
						{
							// Add ed graph input node 
							if (UMetasoundEditorGraphNode* NewInputNode = FGraphBuilder::AddInputNode(ParentMetasound, NewTemplateNode->GetID()))
							{
								// Set node locations
								NewInputNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
								NewInputNode->SyncLocationFromFrontendNode();

								// Connect input node to external node
								UEdGraphNode* EdGraphNode = CastChecked<UEdGraphNode>(NewInputNode);
								UEdGraphPin* ExternalNodeInputPin = NewGraphNode->FindPin(FirstMatchingVertex->Name.ToString());
								if (ensure(SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*EdGraphNode, ExternalNodeInputPin)))
								{
									SchemaPrivate::PostAddNode(NewGraphNode, ParentMetasound, *MetaSoundGraph, FDeprecateSlateVector2D(Location) + DisplayStyle::NodeLayout::DefaultOffsetX, bSelectNewNode);
									return EdGraphNode;
								}
							}
						}
					}
				}
			}
		}
		return NewGraphNode;
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_NewInput::FMetasoundGraphSchemaAction_NewInput(FText InNodeCategory, FText InDisplayName, FGuid InNodeID, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
	: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InToolTip), InGroup)
	, NodeID(InNodeID)
{
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewInput::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Input");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewInput::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->InputNodeTitleColor;
	}

	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewInput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetasound = MetasoundGraph->GetMetasoundChecked();

	UMetasoundEditorGraphInput* Input = MetasoundGraph->FindInput(NodeID);
	if (!ensure(Input))
	{
		return nullptr;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNodeHandle InputNodeHandle = Input->GetNodeHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (!ensure(InputNodeHandle->IsValid()))
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddNewInputNode", "Add New MetaSound Input Node"));
	ParentMetasound.Modify();
	MetasoundGraph->Modify();
	Input->Modify();

	FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);
	const FMetasoundFrontendNode* TemplateNode = FInputNodeTemplate::CreateNode(Builder, Input->GetMemberName());
	if (UMetasoundEditorGraphNode* NewGraphNode = FGraphBuilder::AddInputNode(ParentMetasound, TemplateNode->GetID()))
	{
		NewGraphNode->Modify();
		NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(InLocation));
		NewGraphNode->SyncLocationFromFrontendNode();
		SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, FromPin);
		FGraphBuilder::RegisterGraphWithFrontend(ParentMetasound);
		return NewGraphNode;
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_NewGraphInput::FMetasoundGraphSchemaAction_NewGraphInput(FName InDataType, FMetasoundFrontendLiteral InLiteral)
	: FMetasoundGraphSchemaAction_NewGraphMember(
		InDataType, 
		InLiteral,
		FText(),
		LOCTEXT("AddNewGraphInputName", "Add Graph Input"),
		LOCTEXT("AddNewGraphInputTooltip", "Adds new graph input with given input default value"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewGraphInput::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Input");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewGraphInput::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->InputNodeTitleColor;
	}

	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewGraphInput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::AddNewGraphInput(DataType, DefaultLiteral, ParentGraph, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_NewGraphOutput::FMetasoundGraphSchemaAction_NewGraphOutput(FName InDataType, FMetasoundFrontendLiteral InLiteral)
	: FMetasoundGraphSchemaAction_NewGraphMember(
		InDataType,
		InLiteral,
		FText(),
		LOCTEXT("AddNewGraphOutputName", "Add Graph Output"),
		LOCTEXT("AddNewGraphOutputTooltip", "Adds new graph output with given default value"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewGraphOutput::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Output");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewGraphOutput::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->OutputNodeTitleColor;
	}

	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewGraphOutput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;
	using namespace Metasound::Engine;

	UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetaSound = MetasoundGraph->GetMetasoundChecked();

	const FScopedTransaction Transaction(LOCTEXT("AddNewGraphOutputNode", "Add New MetaSound Graph Output"));
	ParentMetaSound.Modify();
	ParentGraph->Modify();

	FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetaSound);

	const FCreateNodeVertexParams VertexParams =
	{
		DataType,
		EMetasoundFrontendVertexAccessType::Reference
	};

	const FMetasoundFrontendNode* NewNode = nullptr;

	// Name may be different than previous name because it may already exist in the graph,
	// and CreateUniqueClassOutput will add an identifier to the end to make it unique
	FName NewName = TEXT("Output");
	const FMetasoundFrontendClassOutput ClassOutput = FGraphBuilder::CreateUniqueClassOutput(ParentMetaSound, VertexParams, &NewName);
	NewName = ClassOutput.Name;

	const FMetasoundFrontendNode* OutputNode = Builder.AddGraphOutput(ClassOutput);
	if (ensure(OutputNode))
	{
		const FGuid OutputNodeID = OutputNode->GetID();
		UMetasoundEditorGraphOutput* Output = MetasoundGraph->FindOrAddOutput(OutputNodeID);
		if (ensure(Output))
		{
			if (UMetasoundEditorGraphOutputNode* NewGraphNode = FGraphBuilder::AddOutputNode(ParentMetaSound, OutputNodeID))
			{
				const FMetasoundFrontendVertex* Vertex = Builder.FindNodeInput(OutputNodeID, NewName);
				if (ensure(Vertex))
				{
					Builder.SetNodeInputDefault(OutputNodeID, Vertex->VertexID, DefaultLiteral);
				}

				SchemaPrivate::PostAddNode(NewGraphNode, ParentMetaSound, *MetasoundGraph, InLocation, bSelectNewNode);

				return NewGraphNode;
			}
		}
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_NewGraphVariable::FMetasoundGraphSchemaAction_NewGraphVariable(FName InDataType, FMetasoundFrontendLiteral InLiteral)
	: FMetasoundGraphSchemaAction_NewGraphMember(
		InDataType,
		InLiteral,
		FText(),
		LOCTEXT("AddNewGraphVariableName", "Add Graph Variable"),
		LOCTEXT("AddNewGraphVariableTooltip", "Adds new graph variable with given default value"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewGraphVariable::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Variable");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewGraphVariable::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->VariableNodeTitleColor;
	}

	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewGraphVariable::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;
	using namespace Metasound::Engine;

	UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetaSound = MetasoundGraph->GetMetasoundChecked();

	const FScopedTransaction Transaction(LOCTEXT("AddNewGraphVariableNode", "Add New MetaSound Graph Variable"));
	ParentMetaSound.Modify();
	ParentGraph->Modify();

	FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetaSound);

	const FMetasoundFrontendNode* NewNode = nullptr;

	const FName NodeName = Builder.GenerateUniqueVariableName(TEXT("Variable"));
	FMetasoundFrontendClass VariableClass;
	if (ensure(IDataTypeRegistry::Get().GetFrontendVariableAccessorClass(DataType, VariableClass)))
	{
		const FMetasoundFrontendVariable* FrontendVariable = Builder.AddGraphVariable(NodeName, DataType);
		if (ensure(FrontendVariable))
		{
			UMetasoundEditorGraphVariable* Variable = MetasoundGraph->FindOrAddVariable(FrontendVariable->Name);
			if (ensure(Variable))
			{
				const FMetasoundFrontendNode* NewVariableNode = Builder.AddGraphVariableNode(FrontendVariable->Name, VariableClass.Metadata.GetType());
				if (ensure(NewVariableNode))
				{
					UMetasoundEditorGraphVariableNode* NewGraphNode = FGraphBuilder::AddVariableNode(ParentMetaSound, NewVariableNode->GetID());
					if (ensure(NewGraphNode))
					{
						Builder.SetGraphVariableDefault(NodeName, DefaultLiteral);
						SchemaPrivate::PostAddNode(NewGraphNode, ParentMetaSound, *MetasoundGraph, InLocation, bSelectNewNode);

						return NewGraphNode;
					}
				}
			}
		}
	}
	return nullptr;
}

FMetasoundGraphSchemaAction_PromoteToInput::FMetasoundGraphSchemaAction_PromoteToInput()
	: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(
		FText(),
		LOCTEXT("PromoteToInputName", "Promote To Graph Input"),
		LOCTEXT("PromoteToInputTooltip2", "Promotes node input to graph input"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_PromoteToInput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::PromoteToInput(ParentGraph, FromPin, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode::FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode()
	: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(
		FText(),
		LOCTEXT("PromoteToVariableGetterName", "Promote To Graph Variable"),
		LOCTEXT("PromoteToInputTooltip3", "Promotes node input to graph variable and creates a getter node"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::PromoteToVariable(ParentGraph, FromPin, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode::FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode()
	: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(
		FText(),
		LOCTEXT("PromoteToVariableDeferredGetterName", "Promote To Graph Variable (Deferred)"),
		LOCTEXT("PromoteToInputTooltip1", "Promotes node input to graph variable and creates a deferred getter node"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::PromoteToDeferredVariable(ParentGraph, FromPin, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode::FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode()
	: FMetasoundGraphSchemaAction(
		FText(),
		LOCTEXT("PromoteToVariableSetterName", "Promote To Graph Variable"),
		LOCTEXT("PromoteToVariableSetterTooltip2", "Promotes node input to graph variable and creates a setter node"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::PromoteToMutatorVariable(ParentGraph, FromPin, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_NewOutput::FMetasoundGraphSchemaAction_NewOutput(FText InNodeCategory, FText InDisplayName, FGuid InOutputNodeID, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
	: FMetasoundGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InToolTip), InGroup)
	, NodeID(InOutputNodeID)
{
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewOutput::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Output");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewOutput::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->OutputNodeTitleColor;
	}

	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewOutput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetasound = MetasoundGraph->GetMetasoundChecked();

	const UMetasoundEditorGraphOutput* Output = MetasoundGraph->FindOutput(NodeID);
	if (!ensure(Output))
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddNewOutputNode2", "Add New MetaSound Output Node"));
	ParentMetasound.Modify();
	ParentGraph->Modify();

	if (UMetasoundEditorGraphOutputNode* NewGraphNode = FGraphBuilder::AddOutputNode(ParentMetasound, Output->NodeID, bSelectNewNode))
	{
		NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
		NewGraphNode->SyncLocationFromFrontendNode();
		SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, FromPin);
		FGraphBuilder::RegisterGraphWithFrontend(ParentMetasound);
		return NewGraphNode;
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_PromoteToOutput::FMetasoundGraphSchemaAction_PromoteToOutput()
	: FMetasoundGraphSchemaAction(
		FText(),
		LOCTEXT("PromoteToOutputName", "Promote To Graph Output"),
		LOCTEXT("PromoteToOutputTooltip", "Promotes node output to graph output"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_PromoteToOutput::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& InLocation, bool bSelectNewNode /* = true */)
{
	using namespace Metasound::Editor;
	return SchemaUtils::PromoteToOutput(ParentGraph, FromPin, InLocation, bSelectNewNode);
}

FMetasoundGraphSchemaAction_NewVariableNode::FMetasoundGraphSchemaAction_NewVariableNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip)
	: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InToolTip), Metasound::Editor::EPrimaryContextGroup::Variables)
	, VariableID(InVariableID)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewVariableNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph))
	{
		if (UObject* ParentMetasound = MetasoundGraph->GetMetasound())
		{
			if (UMetasoundEditorGraphVariable* Variable = MetasoundGraph->FindVariable(VariableID))
			{
				const FScopedTransaction Transaction(LOCTEXT("AddNewVariableAccessorNode", "Add New MetaSound Variable Accessor Node"));
				ParentMetasound->Modify();
				MetasoundGraph->Modify();
				Variable->Modify();
				
				const FMetasoundFrontendNode* FrontendNode = CreateFrontendVariableNode(MetasoundGraph->GetBuilderChecked().GetBuilder());
				if (ensure(FrontendNode))
				{
					if (UMetasoundEditorGraphVariableNode* NewGraphNode = FGraphBuilder::AddVariableNode(*ParentMetasound, FrontendNode->GetID(), bSelectNewNode))
					{
						NewGraphNode->Modify();
						NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
						NewGraphNode->SyncLocationFromFrontendNode();
						SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, FromPin);
						return NewGraphNode;
					}
				}
			}
		}
	}

	return nullptr;
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewVariableNode::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Variable");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewVariableNode::GetIconColor() const
{
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		return EditorSettings->VariableNodeTitleColor;
	}

	return Super::GetIconColor();
}

FMetasoundGraphSchemaAction_NewVariableAccessorNode::FMetasoundGraphSchemaAction_NewVariableAccessorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip)
: FMetasoundGraphSchemaAction_NewVariableNode(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InVariableID), MoveTemp(InToolTip))
{
}

const FMetasoundFrontendNode* FMetasoundGraphSchemaAction_NewVariableAccessorNode::CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
{
	if (const FMetasoundFrontendVariable* Variable = DocBuilder.FindGraphVariable(VariableID))
	{
		return DocBuilder.AddGraphVariableAccessorNode(Variable->Name);
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode::FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip)
: FMetasoundGraphSchemaAction_NewVariableNode(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InVariableID), MoveTemp(InToolTip))
{
}

const FMetasoundFrontendNode* FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode::CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
{
	if (const FMetasoundFrontendVariable* Variable = DocBuilder.FindGraphVariable(VariableID))
	{
		return DocBuilder.AddGraphVariableDeferredAccessorNode(Variable->Name);
	}

	return nullptr;
}

FMetasoundGraphSchemaAction_NewVariableMutatorNode::FMetasoundGraphSchemaAction_NewVariableMutatorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip)
: FMetasoundGraphSchemaAction_NewVariableNode(MoveTemp(InNodeCategory), MoveTemp(InDisplayName), MoveTemp(InVariableID), MoveTemp(InToolTip))
{
}

const FMetasoundFrontendNode* FMetasoundGraphSchemaAction_NewVariableMutatorNode::CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
{
	if (const FMetasoundFrontendVariable* Variable = DocBuilder.FindGraphVariable(VariableID))
	{
		return DocBuilder.AddGraphVariableMutatorNode(Variable->Name);
	}

	return nullptr;
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewFromSelected::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /* = true*/)
{
	// TODO: Implement
	return nullptr;
}

FMetasoundGraphSchemaAction_NewAudioAnalyzer::FMetasoundGraphSchemaAction_NewAudioAnalyzer()
	: FMetasoundGraphSchemaAction(
		FText(),
		LOCTEXT("AddAudioAnalyzerName", "Add Audio Analyzer Node..."),
		LOCTEXT("AddAudioAnalyzerTooltip", "Analyze an audio signal (editor only)"),
		Metasound::Editor::EPrimaryContextGroup::Common)
{
	//
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewAudioAnalyzer::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	check(ParentGraph);
	UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetasound = MetaSoundGraph->GetMetasoundChecked();

	const FScopedTransaction Transaction(LOCTEXT("AddNewAudioAnalyzerNode", "Add Audio Analyzer Node"));
	ParentMetasound.Modify();
	ParentGraph->Modify();

	FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&ParentMetasound);
	check(MetaSoundAsset);

	const INodeTemplate* AudioAnalyzerTemplate = INodeTemplateRegistry::Get().FindTemplate(FAudioAnalyzerNodeTemplate::ClassName);
	if (!AudioAnalyzerTemplate)
	{
		UE_LOGF(LogMetasoundEditor, Error, "Failed to find template for class \"%ls\"", *FAudioAnalyzerNodeTemplate::ClassName.ToString());
		return nullptr;
	}

	FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);

	const FName FromVertexDataType = FGraphBuilder::GetPinDataType(FromPin);
	FNodeTemplateGenerateInterfaceParams Params;
	Params.InputsToConnect.Add(FromVertexDataType);

	const FMetasoundFrontendNode* TemplateNode = DocBuilder.AddNodeByTemplate(*AudioAnalyzerTemplate, MoveTemp(Params));
	DocBuilder.SetNodeLocation(TemplateNode->GetID(), FDeprecateSlateVector2D(Location));

	const FMetasoundFrontendVertexHandle FromVertexHandle = FGraphBuilder::GetPinVertexHandle(DocBuilder, FromPin);
	const auto VertexIsMatchingDataType = [FromVertexDataType](const FMetasoundFrontendVertex& Vertex) { return Vertex.TypeName == FromVertexDataType; };
	const FMetasoundFrontendVertex* ToVertex = TemplateNode->Interface.Inputs.FindByPredicate(VertexIsMatchingDataType);
	if (FromVertexHandle.IsSet() && ToVertex)
	{
		DocBuilder.AddEdge(FMetasoundFrontendEdge
			{
				FromVertexHandle.NodeID,
				FromVertexHandle.VertexID,
				TemplateNode->GetID(),
				ToVertex->VertexID,
			});
	}	

	const FMetasoundFrontendClass& FrontendClass = AudioAnalyzerTemplate->GetFrontendClass();

	// Proactively create the corresponding EdGraphNode so that we have something to return:
	if (UMetasoundEditorGraphExternalNode* NewGraphNode = FGraphBuilder::AddTemplateNode(ParentMetasound, TemplateNode->GetID(), FrontendClass.Metadata, bSelectNewNode))
	{
		TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(ParentMetasound);
		if (ParentEditor.IsValid() && bSelectNewNode)
		{
			ParentEditor->ClearSelectionAndSelectNode(NewGraphNode);
		}

		return NewGraphNode;
	}

	return nullptr;
}

const FLinearColor& FMetasoundGraphSchemaAction_NewAudioAnalyzer::GetIconColor() const
{
	return GetDefault<UMetasoundEditorSettings>()->AudioPinTypeColor;
}

FMetasoundGraphSchemaAction_NewReroute::FMetasoundGraphSchemaAction_NewReroute(const FLinearColor* InIconColor, bool bInShouldTransact /* = true */)
	: FMetasoundGraphSchemaAction(
		FText(),
		LOCTEXT("RerouteName", "Add Reroute Node..."),
		LOCTEXT("RerouteTooltip", "Reroute Node (reroutes wires)"),
		Metasound::Editor::EPrimaryContextGroup::Common)
	, IconColor(InIconColor ? *InIconColor : FLinearColor::White)
	, bShouldTransact(bInShouldTransact)
{
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewReroute::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /* = true*/)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	check(ParentGraph);
	UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& ParentMetasound = MetaSoundGraph->GetMetasoundChecked();

	const FScopedTransaction Transaction(LOCTEXT("AddNewRerouteNode", "Add Reroute Node"));
	ParentMetasound.Modify();
	ParentGraph->Modify();

	// Provided 'FromPin' is what to connect to, so if its an input, its the output of the generated node needs to
	// match the from pin and vice versa.
	const FName FromVertexDataType = FGraphBuilder::GetPinDataType(FromPin);
	FNodeTemplateGenerateInterfaceParams Params;
	if (FromPin->Direction == EGPD_Input)
	{
		Params.OutputsToConnect.Add(FromVertexDataType);
	}
	else
	{
		Params.InputsToConnect.Add(FromVertexDataType);
	}

	FMetaSoundFrontendDocumentBuilder& DocBuilder = MetaSoundGraph->GetBuilderChecked().GetBuilder();
	const INodeTemplate* RerouteTemplate = INodeTemplateRegistry::Get().FindTemplate(FRerouteNodeTemplate::ClassName);
	check(RerouteTemplate);

	if (const FMetasoundFrontendNode* TemplateNode = DocBuilder.AddNodeByTemplate(*RerouteTemplate, MoveTemp(Params)))
	{
		if (UMetasoundEditorGraphExternalNode* NewGraphNode = FGraphBuilder::AddTemplateNode(
			ParentMetasound,
			TemplateNode->GetID(),
			RerouteTemplate->GetFrontendClass().Metadata,
			bSelectNewNode))
		{
			NewGraphNode->Modify();
			NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
			NewGraphNode->SyncLocationFromFrontendNode();

			SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, FromPin);
			DocBuilder.GetConstDocumentChecked().Metadata.ModifyContext.AddNodeIDsModified({ NewGraphNode->GetNodeID() });

			TSharedPtr<FEditor> ParentEditor = FGraphBuilder::GetEditorForMetasound(ParentMetasound);
			if (ParentEditor.IsValid() && bSelectNewNode)
			{
				ParentEditor->ClearSelectionAndSelectNode(NewGraphNode);
			}

			return NewGraphNode;
		}
	}

	return nullptr;
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewReroute::GetIconBrush() const
{
	return &Metasound::Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.Node.Class.Reroute");
}

const FLinearColor& FMetasoundGraphSchemaAction_NewReroute::GetIconColor() const
{
	return IconColor;
}

UEdGraphNode* FMetasoundGraphSchemaAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	using namespace Metasound::Editor;

	UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
	UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();

	const FScopedTransaction Transaction(LOCTEXT("AddNewOutputNode1", "Add Comment to MetaSound Graph"));
	MetaSoundGraph->Modify();
	MetaSound.Modify();

	// Must cache bounds prior to comment creation as call selects new node and invalidates original selection
	FSlateRect Bounds;
	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*ParentGraph);
	const bool bUseBounds = MetasoundEditor.IsValid() && MetasoundEditor->GetBoundsForSelectedNodes(Bounds, 50.0f);

	if (UMetasoundEditorGraphCommentNode* NewComment = FGraphBuilder::CreateCommentNode(MetaSound, bSelectNewNode))
	{
		if (bUseBounds)
		{
			NewComment->SetBounds(Bounds);
		}
		else
		{
			NewComment->NodePosX = Location.X;
			NewComment->NodePosY = Location.Y;
			NewComment->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);
		}

		// Applies new node data to frontend doc data
		FMetaSoundFrontendGraphComment& FrontendComment = MetaSoundGraph->GetBuilderChecked().FindOrAddGraphComment(NewComment->GetCommentID());
		UMetasoundEditorGraphCommentNode::ConvertToFrontendComment(*NewComment, FrontendComment);

		return NewComment;
	}

	return nullptr;
}

const FSlateBrush* FMetasoundGraphSchemaAction_NewComment::GetIconBrush() const
{
	// TODO: Implement (Find icon & rig up)
	return Super::GetIconBrush();
}

const FLinearColor& FMetasoundGraphSchemaAction_NewComment::GetIconColor() const
{
	// TODO: Implement (Set to white when icon found)
	return Super::GetIconColor();
}

UEdGraphNode* FMetasoundGraphSchemaAction_Paste::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	using namespace Metasound::Editor;

	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*ParentGraph);
	if (MetasoundEditor.IsValid())
	{
		FDeprecateSlateVector2D LocationToPaste(Location);
		MetasoundEditor->PasteNodes(&LocationToPaste);
	}

	return nullptr;
}

UMetasoundEditorGraphSchema::UMetasoundEditorGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UMetasoundEditorGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	using namespace Metasound::Editor;

	bool bCausesLoop = false;

	if ((nullptr != InputPin) && (nullptr != OutputPin))
	{
		UEdGraphNode* InputNode = InputPin->GetOwningNode();
		UEdGraphNode* OutputNode = OutputPin->GetOwningNode();

		// Sets bCausesLoop if the input node already has a path to the output node
		//
		FGraphBuilder::DepthFirstTraversal(InputNode, [&](UEdGraphNode* Node) -> TSet<UEdGraphNode*>
			{
				TSet<UEdGraphNode*> Children;

				if (OutputNode == Node)
				{
					// If the input node can already reach the output node, then this 
					// connection will cause a loop.
					bCausesLoop = true;
				}

				if (!bCausesLoop)
				{
					// Only produce children if no loop exists to avoid wasting unnecessary CPU
					if (nullptr != Node)
					{
						Node->ForEachNodeDirectlyConnectedToOutputs([&](UEdGraphNode* ChildNode) 
							{ 
								Children.Add(ChildNode);
							}
						);
					}
				}

				return Children;
			}
		);
	}
	
	return bCausesLoop;
}

void UMetasoundEditorGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	GetCommentAction(ActionMenuBuilder);
	GetFunctionActions(ActionMenuBuilder);
}

void UMetasoundEditorGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	FActionVertexFilters VertexFilters;
	FConstGraphHandle GraphHandle = IGraphController::GetInvalidHandle();
	EMetasoundFrontendVertexAccessType OutputAccessType = EMetasoundFrontendVertexAccessType::Unset;

	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*ContextMenuBuilder.CurrentGraph);
	check(MetasoundEditor.IsValid());
	TOptional<FDroppedObjectsInfo> DroppedObjectsInfo = MetasoundEditor->GetDroppedObjectsInfo();
	// If dropped assets, set vertex type filter and add new graph member options
	if (DroppedObjectsInfo.IsSet() && !DroppedObjectsInfo->DroppedObjects.IsEmpty())
	{
		// Set filter type
		VertexFilters.InputTypeName = DroppedObjectsInfo->TypeName;
		// Value for superset of value and reference 
		VertexFilters.InputAccessType = EMetasoundFrontendVertexAccessType::Value;

		// Add member actions for drop + shift
		if (DroppedObjectsInfo->ModifierKeys.IsShiftDown())
		{
			// Set default literal from dropped objects
			FMetasoundFrontendLiteral DefaultLiteral;
			if (DroppedObjectsInfo->DroppedObjects.Num() == 1)
			{
				// As single object
				DefaultLiteral.Set(DroppedObjectsInfo->DroppedObjects[0]);
			}
			else
			{
				// As array
				DefaultLiteral.Set(DroppedObjectsInfo->DroppedObjects);
			}
			FGraphActionMenuBuilder& ActionMenuBuilder = static_cast<FGraphActionMenuBuilder&>(ContextMenuBuilder);
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewGraphInput>(DroppedObjectsInfo->TypeName, DefaultLiteral));
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewGraphVariable>(DroppedObjectsInfo->TypeName, DefaultLiteral));
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewGraphOutput>(DroppedObjectsInfo->TypeName, DefaultLiteral));
			if (DroppedObjectsInfo->DroppedObjects.Num() > 1)
			{
				// Function actions are not currently supported because 
				// there is no editing UI for arrays on node inputs
				return;
			}
		}
	} 
	else if (const UEdGraphPin* FromPin = ContextMenuBuilder.FromPin)
	{
		if (FromPin->Direction == EGPD_Input)
		{
			FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(ContextMenuBuilder.FromPin);
			OutputAccessType = InputHandle->GetVertexAccessType();

			VertexFilters.OutputTypeName = InputHandle->GetDataType();
			VertexFilters.OutputAccessType = InputHandle->GetVertexAccessType();

			// Show only input nodes as output nodes can only connected if FromPin is input
			GraphHandle = InputHandle->GetOwningNode()->GetOwningGraph();
			GetDataTypeInputNodeActions(ContextMenuBuilder, GraphHandle, [&InputHandle](FConstNodeHandle NodeHandle)
			{
				bool bHasConnectableOutput = false;
				NodeHandle->IterateConstOutputs([&](FConstOutputHandle PotentialOutputHandle)
				{
					bHasConnectableOutput |= (InputHandle->CanConnectTo(*PotentialOutputHandle).Connectable == FConnectability::EConnectable::Yes);
				});
				return bHasConnectableOutput;
			});

			FGraphActionMenuBuilder& ActionMenuBuilder = static_cast<FGraphActionMenuBuilder&>(ContextMenuBuilder);
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_PromoteToInput>());

			// Constructor outputs cannot be promoted to variables
			if (OutputAccessType != EMetasoundFrontendVertexAccessType::Value)
			{
				ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode>());
				ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode>());
			}

			const FLinearColor IconColor = GetPinTypeColor(FromPin->PinType);
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewReroute>(&IconColor));
		}

		if (FromPin->Direction == EGPD_Output)
		{
			FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(FromPin);
			VertexFilters.InputTypeName = OutputHandle->GetDataType();
			VertexFilters.InputAccessType = OutputHandle->GetVertexAccessType();

			// Show only output nodes as input nodes can only connected if FromPin is output
			GraphHandle = OutputHandle->GetOwningNode()->GetOwningGraph();
			GetDataTypeOutputNodeActions(ContextMenuBuilder, GraphHandle, [OutputHandle](FConstNodeHandle NodeHandle)
			{
				bool bHasConnectableInput = false;
				NodeHandle->IterateConstInputs([&](FConstInputHandle PotentialInputHandle)
				{
					bHasConnectableInput |= (PotentialInputHandle->CanConnectTo(*OutputHandle).Connectable == FConnectability::EConnectable::Yes);
				});
				return bHasConnectableInput;
			});

			FGraphActionMenuBuilder& ActionMenuBuilder = static_cast<FGraphActionMenuBuilder&>(ContextMenuBuilder);

			if (OutputHandle->GetDataType() == GetMetasoundDataTypeName<FAudioBuffer>())
			{
				ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewAudioAnalyzer>());
			}

			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_PromoteToOutput>());
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode>());

			const FLinearColor IconColor = GetPinTypeColor(FromPin->PinType);
			ActionMenuBuilder.AddAction(MakeShared<FMetasoundGraphSchemaAction_NewReroute>(&IconColor));
		}
	}
	else
	{
		if (MetasoundEditor.IsValid() && MetasoundEditor->CanPasteNodes())
		{
			TSharedPtr<FMetasoundGraphSchemaAction_Paste> NewAction = MakeShared<FMetasoundGraphSchemaAction_Paste>(FText::GetEmpty(), LOCTEXT("PasteHereAction", "Paste here"), FText::GetEmpty(), EPrimaryContextGroup::Common);
			ContextMenuBuilder.AddAction(NewAction);
		}

		GetCommentAction(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);
		if (UObject* Metasound = MetasoundEditor->GetMetasoundObject())
		{
			const FMetasoundAssetBase* MetasoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(Metasound);
			check(MetasoundAsset);
			GraphHandle = MetasoundAsset->GetRootGraphHandle();

			GetDataTypeInputNodeActions(ContextMenuBuilder, GraphHandle);
			GetDataTypeOutputNodeActions(ContextMenuBuilder, GraphHandle);
		}
	}

	// Add function actions
	GetFunctionActions(ContextMenuBuilder, VertexFilters, true /* bShowSelectedActions */, GraphHandle, ContextMenuBuilder.CurrentGraph);

	// Variable and conversion actions are always by reference so are incompatible with constructor outputs 
	if (OutputAccessType != EMetasoundFrontendVertexAccessType::Value)
	{
		GetVariableActions(ContextMenuBuilder, VertexFilters, true /* bShowSelectedActions */, GraphHandle);
		GetConversionActions(ContextMenuBuilder, VertexFilters);
	}
}

void UMetasoundEditorGraphSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!Context->Pin && Context->Node && Context->Node->IsA<UMetasoundEditorGraphNode>())
	{
		FToolMenuSection& Section = Menu->AddSection("MetasoundGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
		Section.AddMenuEntry(FGenericCommands::Get().Delete);
		Section.AddMenuEntry(FGenericCommands::Get().Cut);
		Section.AddMenuEntry(FGenericCommands::Get().Copy);
		Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
		Section.AddMenuEntry(FGenericCommands::Get().Rename);
		Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		Section.AddMenuEntry(FGraphEditorCommands::Get().FindReferences, LOCTEXT("FindReferencesInGraph", "Find References In Graph"),
			LOCTEXT("FindReferencesInGraph_Tooltip", "Find References to the selected Node in the current Graph"),
			FSlateIcon());
		Section.AddMenuEntry(FEditorCommands::Get().PromoteAllToInput);
		Section.AddMenuEntry(FEditorCommands::Get().PromoteAllToCommonInputs);

		FToolMenuSection& OrganizationSection = Menu->FindOrAddSection("MetasoundGraphNodeActionsOrganization", LOCTEXT("NodeActionsOrganizationMenuHeader", "Organization"));

		// Only display update ability if node is of type external
		// and node registry is reporting a major update is available.
		if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Context->Node))
		{
			FMetasoundFrontendVersionNumber HighestVersion = ExternalNode->FindHighestVersionInRegistry();
			Metasound::Frontend::FConstNodeHandle NodeHandle = ExternalNode->GetConstNodeHandle();
			const FMetasoundFrontendClassMetadata& Metadata = NodeHandle->GetClassMetadata();
			const bool bHasNewVersion = HighestVersion.IsValid() && HighestVersion > Metadata.GetVersion();

			const bool bIsAssetClass = IMetaSoundAssetManager::GetChecked().IsAssetClass(Metadata);
			if (bHasNewVersion || bIsAssetClass)
			{
				Section.AddMenuEntry(FEditorCommands::Get().UpdateNodeClass);
			}

			FMetasoundFrontendNodeStyle Style = NodeHandle->GetNodeStyle();
			if (Style.bUnconnectedPinsHidden)
			{
				OrganizationSection.AddMenuEntry(FGraphEditorCommands::Get().ShowAllPins, LOCTEXT("ShowUnconnectedPins", "Show Unconnected Pins"),
					LOCTEXT("ShowUnconnectedPins_Tooltip", "Shows all pins with no connection"),
					FSlateIcon());
			}
			else
			{
				OrganizationSection.AddMenuEntry(FGraphEditorCommands::Get().HideNoConnectionPins);
			}
		}
	}
	else if (Context->Pin && Context->Node && Context->Node->IsA<UMetasoundEditorGraphNode>())
	{
		const UMetasoundEditorGraph* EdGraph = CastChecked<UMetasoundEditorGraph>(Context->Graph);
		UMetaSoundBuilderBase& Builder = Metasound::Engine::FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(EdGraph->GetMetasoundChecked());

		if (Builder.GetConstBuilder().GetIsGraphEditable())
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("Pin Actions");

			if (Context->Pin->Direction == EGPD_Input)
			{
				Section.AddMenuEntry(FEditorCommands::Get().PromoteToInput);
				Section.AddMenuEntry(FEditorCommands::Get().PromoteToVariable);
				Section.AddMenuEntry(FEditorCommands::Get().PromoteToDeferredVariable);
			}
			else
			{
				Section.AddMenuEntry(FEditorCommands::Get().PromoteToOutput);
				Section.AddMenuEntry(FEditorCommands::Get().PromoteToVariable);
			}
		}
	}

	Super::GetContextMenuActions(Menu, Context);

	if (Context->Graph)
	{
		FToolMenuSection& Section = Menu->AddSection("MetasoundGraphSchemaGraphActions", LOCTEXT("DevActionsMenuHeader", "Dev Actions"));
		Section.AddMenuEntry(FEditorCommands::Get().CopySelectionAsConfiguratorCodeToClipboard);
	}
}

void UMetasoundEditorGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	const int32 RootNodeHeightOffset = -58;

	// Create the result node
	FGraphNodeCreator<UMetasoundEditorGraphNode> NodeCreator(Graph);
	UMetasoundEditorGraphNode* ResultRootNode = NodeCreator.CreateNode();
	ResultRootNode->NodePosY = RootNodeHeightOffset;
	NodeCreator.Finalize();
	SetNodeMetaData(ResultRootNode, FNodeMetadata::DefaultGraphNode);
}

const FPinConnectionResponse UMetasoundEditorGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	using namespace Metasound;

	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(PinA, PinB, InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatible", "Directions are not compatible"));
	}

	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop2", "Connection causes loop"));
	}

	bool bConnectingNodesWithErrors = false;
	UEdGraphNode* InputNode = InputPin->GetOwningNode();
	if (ensure(InputNode))
	{
		if (InputNode->ErrorType == EMessageSeverity::Error)
		{
			bConnectingNodesWithErrors = true;
		}
	}
	UEdGraphNode* OutputNode = InputPin->GetOwningNode();
	if (ensure(OutputNode))
	{
		if (OutputNode->ErrorType == EMessageSeverity::Error)
		{
			bConnectingNodesWithErrors = true;
		}
	}

	Frontend::FConstInputHandle InputHandle = Editor::FGraphBuilder::GetConstInputHandleFromPin(InputPin);
	Frontend::FConstOutputHandle OutputHandle = Editor::FGraphBuilder::GetConstOutputHandleFromPin(OutputPin);

	const bool bInputValid = InputHandle->IsValid();
	const bool bOutputValid = OutputHandle->IsValid();
	if (bInputValid && bOutputValid)
	{
		Frontend::FConnectability Connectability = InputHandle->CanConnectTo(*OutputHandle);
		if (Connectability.Connectable == Frontend::FConnectability::EConnectable::No)
		{
			if (Frontend::FConnectability::EReason::IncompatibleDataTypes == Connectability.Reason)
			{
				const FName InputType = InputHandle->GetDataType();
				const FName OutputType = OutputHandle->GetDataType();
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FText::Format(
					LOCTEXT("ConnectionTypeIncompatibleFormat", "Output pin of type '{0}' cannot be connected to input pin of type '{1}'"),
					FText::FromName(OutputType),
					FText::FromName(InputType)
				));
			}
			else if (Frontend::FConnectability::EReason::CausesLoop == Connectability.Reason)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop1", "Connection causes loop"));
			}
			else if (Frontend::FConnectability::EReason::IncompatibleAccessTypes == Connectability.Reason)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatibleAccessTypes0", "Cannot create connection between incompatible access types. Constructor input pins can only be connected to constructor output pins."));
			}
			else
			{
				const FName InputType = InputHandle->GetDataType();
				const FName OutputType = OutputHandle->GetDataType();
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FText::Format(
					LOCTEXT("ConnectionNotAllowed", "'{0}' is not compatible with '{1}'"),
					FText::FromName(OutputType),
					FText::FromName(InputType)
				));
			}
		}
		else if (Connectability.Connectable == Frontend::FConnectability::EConnectable::YesWithConverterNode)
		{
			if (Connectability.PossibleConverterNodeClasses.Num() == 0)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatibleAccessTypes1", "Conversion not supported between these types."));
			}
			else
			{
				const FName InputType = InputHandle->GetDataType();
				const FName OutputType = OutputHandle->GetDataType();
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, 
					FText::Format(LOCTEXT("ConversionSuccess", "Convert {0} to {1}."), 
					FText::FromName(OutputType),
					FText::FromName(InputType)
				));
			}
		}

		// Break existing connections on inputs only - multiple output connections are acceptable
		if (!InputPin->LinkedTo.IsEmpty())
		{
			ECanCreateConnectionResponse ReplyBreakOutputs;
			if (InputPin == PinA)
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
			}
			else
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
			}
			return FPinConnectionResponse(ReplyBreakOutputs, LOCTEXT("ConnectionReplace", "Replace existing connections"));
		}

		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
	}
	else if (bConnectingNodesWithErrors)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionCannotContainErrorNode", "Cannot create new connections with node containing errors."));
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionInternalError", "Internal error. Metasound node vertex handle mismatch."));
	}
}

void UMetasoundEditorGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!PinA || !PinB)
	{
		return;
	}

	//@TODO: This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	UMetasoundEditorGraph* ParentGraph = Cast<UMetasoundEditorGraph>(PinA->GetOwningNode()->GetGraph());
	if (ParentGraph->IsEditable())
	{
		UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(ParentGraph);
		UObject& ParentMetasound = MetaSoundGraph->GetMetasoundChecked();
		const FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&ParentMetasound);
		
		FName VertexDataType;
		const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(DocBuilder, PinA);
		if (ensure(Vertex))
		{
			VertexDataType = Vertex->TypeName;
		}
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddConnectNewRerouteNode", "Add & Connect {0} Reroute Node"), FText::FromName(VertexDataType)));

		ParentMetasound.Modify();
		ParentGraph->Modify();

		UEdGraphPin* OutputPin = PinA->Direction == EGPD_Output ? PinA : PinB;

		const FLinearColor* IconColor = nullptr;
		constexpr bool bShouldTransact = false;
		TSharedPtr<FMetasoundGraphSchemaAction_NewReroute> RerouteAction = MakeShared<FMetasoundGraphSchemaAction_NewReroute>(IconColor, bShouldTransact);

		UEdGraphNode* NewNode = RerouteAction->PerformAction(ParentGraph, OutputPin, GraphPosition, true);

		if (ensure(NewNode))
		{
			UEdGraphPin** RerouteOutputPtr = NewNode->Pins.FindByPredicate([](const UEdGraphPin* Candidate) { return Candidate->Direction == EGPD_Output; });

			if (ensure(*RerouteOutputPtr))
			{
				constexpr bool bShouldBreakSingleTransact = false;
				BreakSinglePinLink(PinA, PinB, bShouldBreakSingleTransact);

				UEdGraphPin* InputPin = PinA->Direction == EGPD_Input ? PinA : PinB;
				ensure(TryCreateConnection(InputPin, *RerouteOutputPtr));
			}
		}
	}
}

bool UMetasoundEditorGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!ensure(PinA && PinB))
	{
		return false;
	}

	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;
	if (!CategorizePinsByDirection(PinA, PinB, InputPin, OutputPin))
	{
		return false;
	}

	if (!ensure(InputPin && OutputPin))
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("MetasoundConnect", "Connect Pins"));

	FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(InputPin);
	FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(OutputPin);
	FConnectability Connectability = InputHandle->CanConnectTo(*OutputHandle);
	if (Connectability.Connectable == FConnectability::EConnectable::YesWithConverterNode)
	{
		UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(InputPin->GetOwningNode()->GetGraph());
		UObject& ParentMetaSound = MetaSoundGraph->GetMetasoundChecked();

		MetaSoundGraph->Modify();
		ParentMetaSound.Modify();

		if (Connectability.PossibleConverterNodeClasses.Num() == 0)
		{
			return false;
		}
		FNodeRegistryKey NodeKey = Connectability.PossibleConverterNodeClasses.Last().NodeKey;

		FMetasoundFrontendClassMetadata Metadata;
		Metadata.SetClassName(NodeKey.ClassName);
		Metadata.SetType(NodeKey.Type);
		
		if (UMetasoundEditorGraphExternalNode* NewGraphNode = FGraphBuilder::AddExternalNode(ParentMetaSound, Metadata, false))
		{
			UEdGraphNode* InputNode = InputPin->GetOwningNode();
			UEdGraphNode* OutputNode = OutputPin->GetOwningNode();

			OutputPin->Modify();
			
			FVector2f Location = FVector2f::ZeroVector;
			Location += FVector2f(InputNode->NodePosX, InputNode->NodePosY);
			Location += FVector2f(OutputNode->NodePosX, OutputNode->NodePosY);
			Location *= 0.5f;

			NewGraphNode->Modify();
			NewGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Location));
			NewGraphNode->SyncLocationFromFrontendNode();

			SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, InputPin);
			SchemaPrivate::TryConnectNewNodeToMatchingDataTypePin(*NewGraphNode, OutputPin);

			return true;
		}

		return false;
	}

	// Must mark Metasound object as modified to avoid desync issues ***before*** attempting to create a connection
	// so that transaction stack observes Frontend changes last if rolled back (i.e. undone).  UEdGraphSchema::TryCreateConnection
	// intrinsically marks the respective pin EdGraphNodes as modified.
	UEdGraphNode* PinANode = PinA->GetOwningNode();
	UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(PinANode->GetGraph());
	Graph->GetMetasoundChecked().Modify();

	// This call to parent takes care of marking respective nodes for modification.
	if (!UEdGraphSchema::TryCreateConnection(PinA, PinB))
	{
		return false;
	}

	if (!FGraphBuilder::ConnectNodes(*InputPin, *OutputPin, false /* bConnectEdPins */))
	{
		return false;
	}

	FGraphBuilder::GetOutermostMetaSoundChecked(*Graph).GetModifyContext().SetDocumentModified();

	return true;
}

void UMetasoundEditorGraphSchema::TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bInMarkAsModified) const
{
	using namespace Metasound::Editor;

	if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Pin.GetOwningNode()))
	{
		if (Node->GetPinDataTypeInfo(Pin).PreferredLiteralType == Metasound::ELiteralType::UObjectProxy)
		{
			TrySetDefaultValue(Pin, NewDefaultObject ? NewDefaultObject->GetPathName() : FString(), bInMarkAsModified);
			return;
		}
	}

	Super::TrySetDefaultObject(Pin, NewDefaultObject, bInMarkAsModified);
}

void UMetasoundEditorGraphSchema::TrySetDefaultValue(UEdGraphPin& Pin, const FString& InNewDefaultValue, bool bInMarkAsModified) const
{
	if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Pin.GetOwningNode()))
	{
		if (Node->GetPinDataTypeInfo(Pin).PreferredLiteralType == Metasound::ELiteralType::UObjectProxy)
		{
			FSoftObjectPath Path = InNewDefaultValue;
			const TSet<FString> DisallowedClassNames = Node->GetDisallowedPinClassNames(Pin);
			if (UObject* Object = Path.TryLoad())
			{
				if (UClass* Class = Object->GetClass())
				{
					if (DisallowedClassNames.Contains(Class->GetClassPathName().ToString()))
					{
						return;
					}
				}
			}
		}
	}

	return Super::TrySetDefaultValue(Pin, InNewDefaultValue, bInMarkAsModified);
}

bool UMetasoundEditorGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* InNodeToDelete) const
{
	using namespace Metasound::Editor;

	UMetasoundEditorGraph* MetaSoundGraph = Cast<UMetasoundEditorGraph>(Graph);
	if (!InNodeToDelete || !MetaSoundGraph || InNodeToDelete->GetGraph() != Graph)
	{
		return false;
	}

	UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();
	MetaSound.Modify();
	Graph->Modify();

	return FGraphBuilder::DeleteNode(*InNodeToDelete);
}

bool UMetasoundEditorGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!Pin)
	{
		return true;
	}

	UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(Pin->GetOwningNode());
	const FMetaSoundFrontendDocumentBuilder& Builder = EdNode->GetBuilderChecked().GetBuilder();
	FMetasoundFrontendVertexHandle InputVertexHandle = FGraphBuilder::GetPinVertexHandle(Builder, Pin);

	if (const FMetasoundFrontendNode* Node = Builder.FindNode(InputVertexHandle.NodeID))
	{
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			return !Class->Style.Display.bShowLiterals;
		}
	}

	// TODO: Determine if should be hidden from doc data
	return false;
}

FText UMetasoundEditorGraphSchema::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	check(Pin);

	UMetasoundEditorGraphNode* EdNode = CastChecked<UMetasoundEditorGraphNode>(Pin->GetOwningNode());
	const FMetaSoundFrontendDocumentBuilder& Builder = EdNode->GetBuilderChecked().GetBuilder();

	const FMetasoundFrontendNode* Node = Builder.FindNode(EdNode->GetNodeID());
	if (!Node)
	{
		return Super::GetPinDisplayName(Pin);
	}

	const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID);
	if (!Class)
	{
		return Super::GetPinDisplayName(Pin);
	}

	const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
	switch (ClassType)
	{
		case EMetasoundFrontendClassType::Input:
		case EMetasoundFrontendClassType::Output:
		case EMetasoundFrontendClassType::Variable:
		case EMetasoundFrontendClassType::VariableAccessor:
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
		case EMetasoundFrontendClassType::VariableMutator:
		{
			UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(EdNode);
			if (ensure(MemberNode))
			{
				UMetasoundEditorGraphMember* Member = MemberNode->GetMember();
				if (ensure(Member))
				{
					return Member->GetDisplayName();
				}
			}
		}
		break;

		case EMetasoundFrontendClassType::Literal:
		case EMetasoundFrontendClassType::External:
		{
			auto PinMatchesClassVertex = [&Pin](const FMetasoundFrontendClassVertex& OtherVertex) { return OtherVertex.Name == Pin->GetFName(); };
			const FMetasoundFrontendVertex* Vertex = nullptr;
			const FMetasoundFrontendClassVertex* ClassVertex = nullptr;
			const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(*Node);
			if (Pin->Direction == EGPD_Input)
			{
				Vertex = Builder.FindNodeInput(EdNode->GetNodeID(), Pin->GetFName());
				ClassVertex = ClassInterface.Inputs.FindByPredicate(PinMatchesClassVertex);
			}
			else
			{
				Vertex = Builder.FindNodeOutput(EdNode->GetNodeID(), Pin->GetFName());
				ClassVertex = ClassInterface.Outputs.FindByPredicate(PinMatchesClassVertex);
			}

			if (Vertex && ClassVertex)
			{
				FName Namespace, ParamName;
				ClassVertex->SplitName(Namespace, ParamName);
				const FText DisplayName = ClassVertex->Metadata.GetDisplayName();
				if (DisplayName.IsEmptyOrWhitespace())
				{
					if (Namespace.IsNone())
					{
						return FText::FromName(ParamName);
					}
					else
					{
						return FText::Format(
							LOCTEXT("ClassMetadataDisplayNameWithNamespaceFormat", "{0} ({1})"),
							FText::FromName(ParamName),
							FText::FromName(Namespace)
						);
					}
				}

				return DisplayName;
			}
		}
		break;

		case EMetasoundFrontendClassType::Template:
		{
			const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Class->Metadata.GetClassName());
			if (ensure(Template))
			{
				if (Pin->Direction == EGPD_Input)
				{
					return Template->GetInputVertexDisplayName(Builder, Builder.GetBuildPageID(), Node->GetID(), Pin->GetFName());
				}
				else
				{
					return Template->GetOutputVertexDisplayName(Builder, Builder.GetBuildPageID(), Node->GetID(), Pin->GetFName());
				}
			}
		}
		break;

		case EMetasoundFrontendClassType::Graph:
		case EMetasoundFrontendClassType::Invalid:
		default:
		{
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing EMetasoundFrontendClassType case coverage");
		}
		break;
	}

	return Super::GetPinDisplayName(Pin);
}

float UMetasoundEditorGraphSchema::GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms,
	const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	using namespace Metasound::Editor;

	// Search will surface results following the priority order of EPrimaryContextGroup (highest value == highest priority)
	// Then, for items having the same priority order, we offset their weight by their actual lexical match (BaseWeight)
	// Exception for Recently Used section: their priority order (before BaseWeight offset) is such that more recently used items have higher prio
	
	const float BaseWeight = Super::GetActionFilteredWeight(InCurrentAction, InFilterTerms, InSanitizedFilterTerms, DraggedFromPins);
	if (static_cast<int32>(EPrimaryContextGroup::Recents) <= InCurrentAction.GetGrouping() && InCurrentAction.GetGrouping() < static_cast<int32>(EPrimaryContextGroup::Common))
	{
		// Highest weight matches for Recent actions should follow most recently used order
		constexpr int32 RecentsOffset = static_cast<int32>(EPrimaryContextGroup::Recents);
		return BaseWeight + RecentsOffset * 100000.0f + (InCurrentAction.GetGrouping() - RecentsOffset) * 10000.0f;
	}
	
	// Highest weight matches should follow the display order of their categories
	return BaseWeight + InCurrentAction.GetGrouping() * 100000.0f;
}

FLinearColor UMetasoundEditorGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return Metasound::Editor::FMetaSoundGraphPanelPinFactory::GetChecked()->GetPinColor(PinType);
}

void UMetasoundEditorGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	BreakNodeLinks(TargetNode, true /* bShouldActuallyTransact */);
}

void UMetasoundEditorGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode, bool bShouldActuallyTransact) const
{
	using namespace Metasound::Editor;

	const FScopedTransaction Transaction(LOCTEXT("BreakNodeLinks", "Break Node Links"), bShouldActuallyTransact);
	UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(TargetNode.GetGraph());
	Graph->GetMetasoundChecked().Modify();
	TargetNode.Modify();

	TArray<UEdGraphPin*> Pins = TargetNode.GetAllPins();
	for (UEdGraphPin* Pin : Pins)
	{
		FGraphBuilder::DisconnectPinVertex(*Pin);
		Super::BreakPinLinks(*Pin, false /* bSendsNodeNotifcation */);
	}
	Super::BreakNodeLinks(TargetNode);
}

void UMetasoundEditorGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	const FScopedTransaction Transaction(LOCTEXT("BreakPinLinks", "Break Pin Links"));
	UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(TargetPin.GetOwningNode()->GetGraph());
	Graph->GetMetasoundChecked().Modify();
	TargetPin.Modify();

	FGraphBuilder::DisconnectPinVertex(TargetPin);
	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);
}

void UMetasoundEditorGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	BreakSinglePinLink(SourcePin, TargetPin, true);
}

void UMetasoundEditorGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin, bool bShouldTransact) const
{
	using namespace Metasound::Editor;

	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;
	if (!SourcePin || !TargetPin || !SourcePin->LinkedTo.Contains(TargetPin) || !TargetPin->LinkedTo.Contains(SourcePin))
	{
		return;
	}

	if (SourcePin->Direction == EGPD_Input)
	{
		InputPin = SourcePin;
	}
	else if (TargetPin->Direction == EGPD_Input)
	{
		InputPin = TargetPin;
	}
	else
	{
		return;
	}

	UEdGraphNode* OwningNode = InputPin->GetOwningNode();
	if (!OwningNode)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("BreakSinglePinLink", "Break Single Pin Link"), bShouldTransact);
	UMetasoundEditorGraph* Graph = CastChecked<UMetasoundEditorGraph>(OwningNode->GetGraph());
	Graph->GetMetasoundChecked().Modify();
	SourcePin->Modify();
	TargetPin->Modify();

	FGraphBuilder::DisconnectPinVertex(*InputPin);
	Super::BreakSinglePinLink(SourcePin, TargetPin);
}

bool UMetasoundEditorGraphSchema::ValidateDroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString* OutTooltipText) const
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	bool bIsValid = true;
	FString OutReasonText;

	if (!HoverGraph)
	{
		return false;
	}

	OutReasonText = LOCTEXT("AddNonMetaSoundToGraph", "Add asset reference(s) to graph. \n Drop for node with new connected input \n Shift + Drop for node with asset(s) set on node \n Alt + Drop for new input").ToString();

	const UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(HoverGraph);
	const UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();

	const FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
	check(MetaSoundAsset);

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(MetaSoundGraph);
	// For non MetaSound assets, all assets must be of the same class to be valid, so keep track of type
	UClass* CurrentNonMetaSoundClass = nullptr;
	for (const FAssetData& Data : Assets)
	{
		UClass* AssetClass = Data.GetClass();
		check(AssetClass);
		static const FString MixedTypeErrorText = LOCTEXT("NotAllSameType", "Asset(s) must all be MetaSound types or a single type.").ToString();

		// MetaSound specific checks (ex. can't cause loop)
		const bool bIsRegisteredMetaSoundClass = IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass);
		if (bIsRegisteredMetaSoundClass)
		{
			// Cannot drop MetaSound types mixed with other types 
			if (CurrentNonMetaSoundClass)
			{
				bIsValid = false;
				OutReasonText = MixedTypeErrorText;
				break;
			}
			if (UObject* DroppedObject = Data.GetAsset())
			{
				FMetasoundAssetBase* DroppedMetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(DroppedObject);
				if (!DroppedMetaSoundAsset)
				{
					bIsValid = false;
					OutReasonText = LOCTEXT("NotAllReferenceableMetaSounds", "Asset(s) not (all) MetaSound(s) that can be referenced.").ToString();
					break;
				}

				if (MetaSoundAsset->AddingReferenceCausesLoop(*DroppedMetaSoundAsset))
				{
					bIsValid = false;
					OutReasonText = LOCTEXT("AddingMetaSoundWouldCauseLoop", "Cannot add asset(s) that would create a reference loop/loops.").ToString();
					break;
				}

				const EMetasoundFrontendClassAccessFlags DroppedAccessFlags = DroppedMetaSoundAsset->GetConstDocumentChecked().RootGraph.Metadata.GetAccessFlags();
				if (!EnumHasAnyFlags(DroppedAccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
				{
					bIsValid = false;
					OutReasonText = LOCTEXT("NotAllAccessibleMetaSounds", "Asset(s) do not contain access flag 'Reference' (see 'Access Flags'). Cannot reference provided MetaSound Asset.").ToString();
					break;
				}

				FText FailureReason;
				if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(Data, &FailureReason))
				{
					bIsValid = false;
					OutReasonText = FailureReason.ToString();
					break;
				}
				OutReasonText = LOCTEXT("AddMetaSoundToGraph", "Add MetaSound reference(s) to graph.").ToString();
			}
			else
			{
				bIsValid = false;
				OutReasonText = LOCTEXT("AssetNotFound", "Asset(s) not found.").ToString();
				break;
			}
		}
		// For non MetaSounds, all assets must be of the same type
		// and that type must be a registered proxy type
		else
		{
			FDataTypeRegistryInfo Info;
			const bool bFoundType = IDataTypeRegistry::Get().GetDataTypeInfo(AssetClass, Info);
			if (bFoundType && Info.IsDataTypeProxyParsable())
			{
				if (!CurrentNonMetaSoundClass)
				{
					CurrentNonMetaSoundClass = AssetClass;
					continue;
				}
				else if (CurrentNonMetaSoundClass != AssetClass)
				{
					bIsValid = false;
					OutReasonText = MixedTypeErrorText;
					break;
				}
				//else = default for multiple non MetaSound assets
			}
			else
			{
				bIsValid = false;
				OutReasonText = LOCTEXT("NotRegisteredType", "Asset type(s) not registered for use in MetaSounds.").ToString();
			}
		}
	}

	if (OutTooltipText)
	{
		*OutTooltipText = OutReasonText;
	}
	return bIsValid;
}

void UMetasoundEditorGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;
	
	OutOkIcon = ValidateDroppedAssetsOnGraph(Assets, HoverGraph, &OutTooltipText);
	return; 
}

void UMetasoundEditorGraphSchema::GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutOkIcon = ValidateDroppedAssetsOnPin(Assets, HoverPin, &OutTooltipText);
}

void UMetasoundEditorGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Editor;

	const bool bIsValid = ValidateDroppedAssetsOnGraph(Assets, Graph);
	if (!bIsValid)
	{
		return; 
	}

	// Collect MetaSound assets
	UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(Graph);
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(MetaSoundGraph);
	TArray<FMetasoundAssetBase*> MetaSoundAssets;
	Algo::Transform(Assets, MetaSoundAssets, [&AssetReferenceFilter](const FAssetData& DroppedAsset)
	{
		if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(DroppedAsset))
		{
			if (UObject* DroppedObject = DroppedAsset.GetAsset())
			{
				return IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(DroppedObject);
			}
		}
		return static_cast<FMetasoundAssetBase*>(nullptr);
	});
	
	// Add dropped MetaSound assets as nodes
	if (MetaSoundAssets.ContainsByPredicate([](FMetasoundAssetBase* Asset) { return Asset != nullptr; }))
	{
		DroppedMetaSoundAssetsOnGraph(MetaSoundAssets, GraphPosition, MetaSoundGraph);
	}
	// Open action menu for non MetaSound assets
	else
	{
		DroppedNonMetaSoundAssetsOnGraph(Assets, GraphPosition, MetaSoundGraph);
	}
}

void UMetasoundEditorGraphSchema::DroppedMetaSoundAssetsOnGraph(const TArray<FMetasoundAssetBase*>& MetaSoundAssets, const FVector2f& GraphPosition, UMetasoundEditorGraph* MetaSoundGraph) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	bool bTransactionSucceeded = false;
	bool bModifiedObjects = false;
	check(MetaSoundGraph);
	UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();

	FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound);
	check(MetaSoundAsset);
	UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;

	FScopedTransaction Transaction(LOCTEXT("DropMetaSoundOnGraph", "Drop MetaSound On Graph"));
	for (FMetasoundAssetBase* DroppedMetaSoundAsset : MetaSoundAssets)
	{
		if (DroppedMetaSoundAsset)
		{
			const EMetasoundFrontendClassAccessFlags DroppedAccessFlags = DroppedMetaSoundAsset->GetConstDocumentChecked().RootGraph.Metadata.GetAccessFlags();
			if (!EnumHasAnyFlags(DroppedAccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
			{
				continue;
			}

			if (MetaSoundAsset->AddingReferenceCausesLoop(*DroppedMetaSoundAsset))
			{
				continue;
			}

			if (!bModifiedObjects)
			{
				MetaSound.Modify();
				MetaSoundGraph->Modify();
				bModifiedObjects = true;
			}

			TScriptInterface<IMetaSoundDocumentInterface> DroppedDocInterface(DroppedMetaSoundAsset->GetOwningAsset());
			FMetaSoundNodeHandle NodeHandle = Builder.AddNode(DroppedDocInterface, Result);
			if (ensure(Result == EMetaSoundBuilderResult::Succeeded))
			{
				Builder.SetNodeLocation(NodeHandle.NodeID, FDeprecateSlateVector2D(GraphPosition), Result);
				bTransactionSucceeded = ensure(Result == EMetaSoundBuilderResult::Succeeded);
			}
		}
	}

	if (!bTransactionSucceeded)
	{
		Transaction.Cancel();
	}
}

void UMetasoundEditorGraphSchema::DroppedNonMetaSoundAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UMetasoundEditorGraph* MetaSoundGraph) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;

	bool bModifiedObjects = false;
	check(MetaSoundGraph);
	UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();
	FScopedTransaction Transaction(LOCTEXT("DropAssetOnGraph", "Drop Asset On Graph"));

	// Collect dropped objects
	FName DataType; 
	TArray<UObject*> DroppedObjects;
	for (const FAssetData& DroppedAsset : Assets)
	{
		if (UObject* DroppedObject = DroppedAsset.GetAsset())
		{
			if (!bModifiedObjects)
			{
				MetaSound.Modify();
				MetaSoundGraph->Modify();
				bModifiedObjects = true;
			}
			FDataTypeRegistryInfo Info;
			if (IDataTypeRegistry::Get().GetDataTypeInfo(DroppedObject, Info))
			{
				if (Info.IsDataTypeProxyParsable())
				{
					if (DataType.IsNone())
					{
						DataType = Info.DataTypeName;
					}
					// Must be of same type (caller should have filtered out) 
					if (ensure(DataType == Info.DataTypeName))
					{
						DroppedObjects.Add(DroppedObject);
					}
				}
			}
		}
	}

	// Set objects and open menu 
	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*MetaSoundGraph);
	if (MetasoundEditor.IsValid() && DroppedObjects.Num() > 0)
	{
		TSharedPtr<SGraphEditor> GraphEditor = MetasoundEditor->GetGraphEditor();
		if (GraphEditor.IsValid())
		{
			if (DroppedObjects.Num() > 1)
			{
				DataType = CreateArrayTypeNameFromElementTypeName(DataType);
			}
			const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			// Add new input for alt 
			if (ModifierKeys.IsAltDown())
			{
				FMetasoundFrontendLiteral DefaultLiteral;
				if (DroppedObjects.Num() == 1)
				{
					DefaultLiteral.Set(DroppedObjects[0]);
				}
				else
				{
					DefaultLiteral.Set(DroppedObjects);
				}
				SchemaUtils::AddNewGraphInput(DataType, DefaultLiteral, MetaSoundGraph, GraphPosition, /*bSelectNewNode*/true);
			}
			// Open node menu for drop or shift + drop
			else
			{
				if (SGraphPanel* GraphPanel = GraphEditor->GetGraphPanel())
				{
					// Set dropped objects info
					FDroppedObjectsInfo DroppedObjectsInfo =
					{
						.DroppedObjects = DroppedObjects,
						.ModifierKeys = ModifierKeys,
						.TypeName = DataType
					};
					MetasoundEditor->SetDroppedObjectsInfo(MoveTemp(DroppedObjectsInfo));

					// Open and focus node menu 
					// Copied from SGraphPanel::GraphCoordToPanelCoord
					const FVector2f PanelPosition = (GraphPosition - GraphPanel->GetViewOffset()) * GraphPanel->GetZoomAmount();
					// Convert panel to screen coordinates
					const FVector2f ScreenPosition = GraphPanel->GetTickSpaceGeometry().LocalToAbsolute(PanelPosition);
					const TSharedPtr<SWidget> MenuWidget = GraphPanel->SummonContextMenu(ScreenPosition, GraphPosition, {}, {}, {});
					FSlateApplication::Get().SetKeyboardFocus(MenuWidget, EFocusCause::SetDirectly);

					MetasoundEditor->ClearDroppedObjectsInfo();
				}
			}
		}
	}
}

bool UMetasoundEditorGraphSchema::ValidateDroppedAssetsOnPin(const TArray<FAssetData>& Assets, const UEdGraphPin* Pin, FString* OutTooltipText) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	const auto SetTooltipText = [&](FString InString)
	{
		if (OutTooltipText)
		{
			*OutTooltipText = InString;
		}
	};

	if (Assets.Num() > 0)
	{
		if (Pin && Pin->Direction == EGPD_Input)
		{
			if (const UEdGraphNode* Node = Pin->GetOwningNode())
			{
				if (const UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(Pin->GetOwningNode()))
				{
					const FDataTypeRegistryInfo RegistryInfo = MetaSoundNode->GetPinDataTypeInfo(*Pin);
					const bool bAssetTypesMatch = SchemaPrivate::DataTypeSupportsAssetTypes(RegistryInfo, Assets);
					if (bAssetTypesMatch)
					{
						const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(MetaSoundNode->GetGraph());
						FText FailureReason;
						if (AssetReferenceFilter.IsValid())
						{
							for (const FAssetData& Asset : Assets)
							{
								if (!AssetReferenceFilter->PassesFilter(Asset, &FailureReason))
								{
									SetTooltipText(FailureReason.ToString());
									return false;
								}
							}
						}
						if (Assets.Num() == 1)
						{
							if (Pin->HasAnyConnections())
							{
								SetTooltipText(LOCTEXT("PinAlreadyConnected", "Pin already has a connection").ToString());
								return false;
							}
							SetTooltipText(FText::FormatOrdered(LOCTEXT("DropAsset", "Set to '{0}'"), FText::FromName(Assets[0].AssetName)).ToString());
							return true;
						}
						else if (Assets.Num() > 1)
						{
							// Multiple assets 
							SetTooltipText(LOCTEXT("DropMultipleAssetsAsInput", "Add assets as connected input").ToString());
							return true;
						}
					}
					else
					{
						SetTooltipText(FText::FormatOrdered(LOCTEXT("InvalidType", "'{0}': Type mismatch"), FText::FromName(Assets[0].AssetName)).ToString());
					}
					return false;
				}

				SetTooltipText(FText::FormatOrdered(LOCTEXT("DragDropNotSupported", "Node '{0}' does not support drag/drop"), FText::FromString(Node->GetName())).ToString());
				return false;
			}
		}
	}

	SetTooltipText(LOCTEXT("InvalidAssetsOrPin", "Invalid asset(s) or input pin").ToString());
	return false;
}

void UMetasoundEditorGraphSchema::DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	if (!ValidateDroppedAssetsOnPin(Assets, Pin))
	{
		return;
	}

	if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Pin->GetOwningNode()))
	{
		const FDataTypeRegistryInfo RegistryInfo = Node->GetPinDataTypeInfo(*Pin);
		const bool bAssetTypesMatch = SchemaPrivate::DataTypeSupportsAssetTypes(RegistryInfo, Assets);
		if (bAssetTypesMatch)
		{
			if (Assets.Num() == 1 && !RegistryInfo.bIsArrayType)
			{
				UObject* Object = Assets.Last().GetAsset();
				if (Object)
				{
					const FText TransactionText = FText::Format(LOCTEXT("ChangeDefaultObjectTransaction", "Set {0} to '{1}'"),
						Pin->GetDisplayName(),
						FText::FromName(Object->GetFName()));
					const FScopedTransaction Transaction(TransactionText);
					Node->Modify();

					constexpr bool bMarkAsModified = true;
					TrySetDefaultObject(*Pin, Object, bMarkAsModified);
				}
			}
			// Add object array as new connected input (including setting single object as array)
			else
			{
				FMetasoundFrontendLiteral DefaultLiteral;
				TArray<UObject*> Objects;
				Algo::Transform(Assets, Objects,
					[](const FAssetData& AssetData) {
						return AssetData.GetAsset();
					});
				DefaultLiteral.Set(Objects);
				SchemaUtils::PromoteToInput(Node->GetGraph(), Pin, FDeprecateSlateVector2D(GraphPosition) - DisplayStyle::NodeLayout::DefaultOffsetX, /*bSelectNewNode*/true, TOptional<FMetasoundFrontendLiteral>(MoveTemp(DefaultLiteral)));
			}
		}
	}
}

void UMetasoundEditorGraphSchema::GetConversionActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters AccessFilters, bool bShowSelectedActions) const
{
	using namespace Metasound;
	using namespace Metasound::Editor;

	QueryNodeClasses([&](TUniquePtr<ISchemaQueryResult>&& Result)
	{
		if (Result->GetRegistryClassType() != EMetasoundFrontendClassType::External)
		{
			return;
		}

		if (AccessFilters.HasInputFilters())
		{
			if (!Result->CanConnectOutputOfTypeAndAccess(AccessFilters.InputTypeName, AccessFilters.InputAccessType))
			{
				return;
			}
		}

		if (AccessFilters.HasOutputFilters())
		{
			if (!Result->CanConnectInputOfTypeAndAccess(AccessFilters.OutputTypeName, AccessFilters.OutputAccessType))
			{
				return;
			}
		}

		const TArray<FText>& CategoryHierarchy = Result->GetCategoryHierarchy();
		if (!CategoryHierarchy.IsEmpty() && !CategoryHierarchy[0].CompareTo(NodeCategories::Conversions))
		{
			const FText Tooltip = Result->GetTooltip();
			TSharedPtr<FMetasoundGraphSchemaAction_NewNode> NewNodeAction = MakeShared<FMetasoundGraphSchemaAction_NewNode>
			(
				FText::Join(SchemaPrivate::CategoryDelim, CategoryHierarchy),
				Result->GetDisplayName(),
				Tooltip,
				EPrimaryContextGroup::Conversions,
				FText::Join(SchemaPrivate::KeywordDelim, Result->GetKeywords())
			);

			NewNodeAction->QueryResult = TSharedPtr<ISchemaQueryResult>(Result.Release());
			ActionMenuBuilder.AddAction(NewNodeAction);
		}
	});
}

void UMetasoundEditorGraphSchema::GetDataTypeInputNodeActions(FGraphContextMenuBuilder& ActionMenuBuilder, Metasound::Frontend::FConstGraphHandle InGraphHandle, Metasound::Editor::FInterfaceNodeFilterFunction InFilter, bool bShowSelectedActions) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	TArray<FConstNodeHandle> Inputs = InGraphHandle->GetConstInputNodes();
	const SchemaPrivate::FDataTypeActionQuery ActionQuery
	{
		ActionMenuBuilder,
		Inputs,
		InFilter,
		EPrimaryContextGroup::Inputs,
		SchemaPrivate::InputDisplayNameFormat,
		SchemaPrivate::InputTooltipFormat,
		bShowSelectedActions
	};
	SchemaPrivate::GetDataTypeActions<FMetasoundGraphSchemaAction_NewInput>(ActionQuery);
}

void UMetasoundEditorGraphSchema::GetDataTypeOutputNodeActions(FGraphContextMenuBuilder& ActionMenuBuilder, Metasound::Frontend::FConstGraphHandle InGraphHandle, Metasound::Editor::FInterfaceNodeFilterFunction InFilter, bool bShowSelectedActions) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	TArray<FConstNodeHandle> Outputs = InGraphHandle->GetConstOutputNodes();

	// Prune and only add actions for outputs that are not already represented in the graph
	// (as there should only be one output reference node ever to avoid confusion with which
	// is handling active input)
	if (const UMetasoundEditorGraph* Graph = Cast<UMetasoundEditorGraph>(ActionMenuBuilder.CurrentGraph))
	{
		for (int32 i = Outputs.Num() - 1; i >= 0; --i)
		{
			if (UMetasoundEditorGraphOutput* Output = Graph->FindOutput(Outputs[i]->GetID()))
			{
				if (!Output->GetNodes().IsEmpty())
				{
					Outputs.RemoveAtSwap(i, EAllowShrinking::No);
				}
			}
		}
	}

	const SchemaPrivate::FDataTypeActionQuery ActionQuery
	{
		ActionMenuBuilder,
		Outputs,
		InFilter,
		EPrimaryContextGroup::Outputs,
		SchemaPrivate::OutputDisplayNameFormat,
		SchemaPrivate::OutputTooltipFormat,
		bShowSelectedActions
	};
	SchemaPrivate::GetDataTypeActions<FMetasoundGraphSchemaAction_NewOutput>(ActionQuery);
}

void UMetasoundEditorGraphSchema::GetFunctionActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters AccessFilters, bool bShowSelectedActions, Metasound::Frontend::FConstGraphHandle InGraphHandle, const UEdGraph* CurrentGraph) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	const UMetasoundEditorSettings* Settings = GetDefault<UMetasoundEditorSettings>();

	const FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
	const FMetaSoundAssetKey ParentAssetKey(InGraphHandle->GetGraphMetadata());
	QueryNodeClasses([&](TUniquePtr<ISchemaQueryResult>&& Result)
	{
		if (Result->GetRegistryClassType() != EMetasoundFrontendClassType::External)
		{
			return;
		}

		if (AccessFilters.HasInputFilters())
		{
			if (!Result->CanConnectOutputOfTypeAndAccess(AccessFilters.InputTypeName, AccessFilters.InputAccessType))
			{
				return;
			}
		}

		if (AccessFilters.HasOutputFilters())
		{
			if (!Result->CanConnectInputOfTypeAndAccess(AccessFilters.OutputTypeName, AccessFilters.OutputAccessType))
			{
				return;
			}
		}

		const EMetasoundFrontendClassAccessFlags AccessFlags = Result->GetAccessFlags();
		if (!EnumHasAnyFlags(AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
		{
			return;
		}

		bool bCausesLoop = false;
		auto IsMatchingKey = [&ParentAssetKey, &bCausesLoop](const FMetaSoundAssetKey& AssetKey)
		{
			bCausesLoop = ParentAssetKey == AssetKey;
		};
		AssetManager.IterateReferences(Result->GetAssetKey(), IsMatchingKey);
		if (bCausesLoop)
		{
			return;
		}

		const TArray<FText>& CategoryHierarchy = Result->GetCategoryHierarchy();
		if (!CategoryHierarchy.IsEmpty() && !CategoryHierarchy[0].CompareTo(Metasound::NodeCategories::Conversions))
		{
			return;
		}
		
		TOptional<FDroppedObjectsInfo> OptionalDroppedObjectsInfo = TOptional<FDroppedObjectsInfo>();
		if (CurrentGraph)
		{
			const TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*CurrentGraph);
			if (MetasoundEditor.IsValid())
			{
				OptionalDroppedObjectsInfo = MetasoundEditor->GetDroppedObjectsInfo();
			}
		}

		const FText Tooltip = Result->GetTooltip();
		const EPrimaryContextGroup ContextGroup = Result->IsNative() ? EPrimaryContextGroup::Functions : EPrimaryContextGroup::Graphs;
		TArray<FText> TextHierarchy { GetContextGroupDisplayName(ContextGroup) };
		TextHierarchy.Append(CategoryHierarchy);

		TSharedPtr<FMetasoundGraphSchemaAction_NewNode> NewNodeAction = MakeShared<FMetasoundGraphSchemaAction_NewNode>
		(
			FText::Join(SchemaPrivate::CategoryDelim, TextHierarchy),
			Result->GetDisplayName(),
			Tooltip,
			ContextGroup,
			FText::Join(SchemaPrivate::KeywordDelim, Result->GetKeywords()), 
			OptionalDroppedObjectsInfo
		);

		NewNodeAction->QueryResult = TSharedPtr<ISchemaQueryResult>(Result.Release());
		ActionMenuBuilder.AddAction(NewNodeAction);
		
		{
			const FString Key = NewNodeAction->GetUniqueStableFavoriteKey();
			const int32 RecentIndex = Settings->FindRecentlyUsedMetaSoundsGraphAction(Key);
			if (RecentIndex != INDEX_NONE)
			{
				TSharedPtr<FMetasoundGraphSchemaAction_NewNode> NewRecentAction = MakeShared<FMetasoundGraphSchemaAction_NewNode>(
					Metasound::NodeCategories::Recents,
					NewNodeAction->GetMenuDescription(),
					NewNodeAction->GetTooltipDescription(),
					EPrimaryContextGroup::Recents,
					NewNodeAction->GetKeywords(), 
					OptionalDroppedObjectsInfo);
				NewRecentAction->Grouping = static_cast<int32>(EPrimaryContextGroup::Recents) + RecentIndex;
				NewRecentAction->QueryResult = NewNodeAction->QueryResult;
				ActionMenuBuilder.AddAction(NewRecentAction);
			}

			if (Settings->IsFavoriteMetaSoundsGraphAction(Key))
			{
				TSharedPtr<FMetasoundGraphSchemaAction_NewNode> NewFavAction = MakeShared<FMetasoundGraphSchemaAction_NewNode>(
					FText::FormatOrdered(INVTEXT("{0}|{1}"), Metasound::NodeCategories::Favorites, NewNodeAction->GetCategory()),
					NewNodeAction->GetMenuDescription(),
					NewNodeAction->GetTooltipDescription(),
					EPrimaryContextGroup::Favorites,
					NewNodeAction->GetKeywords(), 
					OptionalDroppedObjectsInfo);
				NewFavAction->QueryResult = NewNodeAction->QueryResult;
				ActionMenuBuilder.AddAction(NewFavAction);
			}
		}
	});
}

void UMetasoundEditorGraphSchema::GetVariableActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters, bool bShowSelectedActions, Metasound::Frontend::FConstGraphHandle InGraphHandle) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Editor;
	using namespace Metasound::Editor::SchemaPrivate;

	TArray<FConstVariableHandle> Variables = InGraphHandle->GetVariables();

	bool bGetAccessor = true;
	bool bGetDeferredAccessor = true;
	bool bGetMutator = true;
	bool bFilterByDataType = false;
	bool bCheckForLoops = false;
	FName DataType;
	FConstInputHandle ConnectingInputHandle = IInputController::GetInvalidHandle();
	FConstOutputHandle ConnectingOutputHandle = IOutputController::GetInvalidHandle();

	// Determine which variable actions to create.
	if (const UEdGraphPin* FromPin = ActionMenuBuilder.FromPin)
	{
		bFilterByDataType = true;
		bCheckForLoops = true;

		if (FromPin->Direction == EGPD_Input)
		{
			bGetMutator = false;
			ConnectingInputHandle = FGraphBuilder::GetConstInputHandleFromPin(FromPin);
			DataType = ConnectingInputHandle->GetDataType();
		}
		else if (FromPin->Direction == EGPD_Output)
		{
			bGetAccessor = false;
			bGetDeferredAccessor = false;
			ConnectingOutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(FromPin);
			DataType = ConnectingOutputHandle->GetDataType();
		}
	}

	// Filter variable by data type.
	if (bFilterByDataType && DataType.IsValid() && !DataType.IsNone())
	{
		Variables.RemoveAllSwap([&DataType](const FConstVariableHandle& Var) { return Var->GetDataType() != DataType; });
	}

	// Create actions for each variable.
	static const FText& BaseGroupName = GetContextGroupDisplayName(EPrimaryContextGroup::Variables);
	static const FText& RecentGroupName = GetContextGroupDisplayName(EPrimaryContextGroup::Recents);
	static const FText& FavoriteGroupName = FText::FormatOrdered(INVTEXT("{0}|{1}"), GetContextGroupDisplayName(EPrimaryContextGroup::Favorites), BaseGroupName);
	const UMetasoundEditorSettings* Settings = GetDefault<UMetasoundEditorSettings>();
	for (FConstVariableHandle& Variable : Variables)
	{
		const FText VariableDisplayName = FGraphBuilder::GetDisplayName(*Variable);
		const FGuid VariableID = Variable->GetID();

		const int32 VariableRecentIndex = Settings->FindRecentlyUsedMetaSoundsGraphAction(VariableID.ToString());
		const bool bIsVariableFavorited = Settings->IsFavoriteMetaSoundsGraphAction(VariableID.ToString());

		if (bGetAccessor)
		{
			// Do not add the action if adding an accessor node would cause a loop.
			if (!(bCheckForLoops && WillAddingVariableAccessorCauseLoop(*Variable, *ConnectingInputHandle)))
			{
				FText ActionDisplayName = FText::Format(SchemaPrivate::VariableAccessorDisplayNameFormat, VariableDisplayName);
				FText ActionTooltip = FText::Format(SchemaPrivate::VariableAccessorTooltipFormat, VariableDisplayName);
				TSharedPtr<FMetasoundGraphSchemaAction_NewVariableAccessorNode> NewNodeAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableAccessorNode>(
					BaseGroupName, static_cast<int32>(EPrimaryContextGroup::Variables), ActionDisplayName, VariableID, ActionTooltip);
				ActionMenuBuilder.AddAction(NewNodeAction);
				
				if (VariableRecentIndex != INDEX_NONE)
				{
					TSharedPtr<FMetasoundGraphSchemaAction_NewVariableAccessorNode> NewRecentAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableAccessorNode>(
						RecentGroupName, static_cast<int32>(EPrimaryContextGroup::Recents) + VariableRecentIndex, ActionDisplayName, VariableID, ActionTooltip);
					ActionMenuBuilder.AddAction(NewRecentAction);
				}

				if (bIsVariableFavorited)
				{
					TSharedPtr<FMetasoundGraphSchemaAction_NewVariableAccessorNode> NewFavAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableAccessorNode>(
						FavoriteGroupName, static_cast<int32>(EPrimaryContextGroup::Favorites), ActionDisplayName, VariableID, ActionTooltip);
					ActionMenuBuilder.AddAction(NewFavAction);
				}
			}
		}

		if (bGetDeferredAccessor)
		{
			FText ActionDisplayName = FText::Format(SchemaPrivate::VariableDeferredAccessorDisplayNameFormat, VariableDisplayName);
			FText ActionTooltip = FText::Format(SchemaPrivate::VariableDeferredAccessorTooltipFormat, VariableDisplayName);
			TSharedPtr<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode> NewNodeAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode>(
				BaseGroupName, static_cast<int32>(EPrimaryContextGroup::Variables), ActionDisplayName, VariableID, ActionTooltip);
			ActionMenuBuilder.AddAction(NewNodeAction);
				
			if (VariableRecentIndex != INDEX_NONE)
			{
				TSharedPtr<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode> NewRecentAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode>(
					RecentGroupName, static_cast<int32>(EPrimaryContextGroup::Recents) + VariableRecentIndex, ActionDisplayName, VariableID, ActionTooltip);
				ActionMenuBuilder.AddAction(NewRecentAction);
			}

			if (bIsVariableFavorited)
			{
				TSharedPtr<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode> NewFavAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode>(
					FavoriteGroupName, static_cast<int32>(EPrimaryContextGroup::Favorites), ActionDisplayName, VariableID, ActionTooltip);
				ActionMenuBuilder.AddAction(NewFavAction);
			}
		}

		if (bGetMutator)
		{
			// There can only be one mutator node per a variable. Only add the new
			// mutator node action if no mutator nodes exist.
			bool bMutatorNodeAlreadyExists = Variable->FindMutatorNode()->IsValid();
			if (!bMutatorNodeAlreadyExists)
			{
				// Do not add the action if adding a mutator node would cause a loop.
				if (!(bCheckForLoops && WillAddingVariableMutatorCauseLoop(*Variable, *ConnectingOutputHandle)))
				{
					FText ActionDisplayName = FText::Format(SchemaPrivate::VariableMutatorDisplayNameFormat, VariableDisplayName);
					FText ActionTooltip = FText::Format(SchemaPrivate::VariableMutatorTooltipFormat, VariableDisplayName);
					TSharedPtr<FMetasoundGraphSchemaAction_NewVariableMutatorNode> NewNodeAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableMutatorNode>(
						BaseGroupName, static_cast<int32>(EPrimaryContextGroup::Variables), ActionDisplayName, VariableID, ActionTooltip);
					ActionMenuBuilder.AddAction(NewNodeAction);
					
					if (VariableRecentIndex != INDEX_NONE)
					{
						TSharedPtr<FMetasoundGraphSchemaAction_NewVariableMutatorNode> NewRecentAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableMutatorNode>(
							RecentGroupName, static_cast<int32>(EPrimaryContextGroup::Recents) + VariableRecentIndex, ActionDisplayName, VariableID, ActionTooltip);
						ActionMenuBuilder.AddAction(NewRecentAction);
					}

					if (bIsVariableFavorited)
					{
						TSharedPtr<FMetasoundGraphSchemaAction_NewVariableMutatorNode> NewFavAction = MakeVariableAction<FMetasoundGraphSchemaAction_NewVariableMutatorNode>(
							FavoriteGroupName, static_cast<int32>(EPrimaryContextGroup::Favorites), ActionDisplayName, VariableID, ActionTooltip);
						ActionMenuBuilder.AddAction(NewFavAction);
					}
				}
			}
		}
	}
}

void UMetasoundEditorGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	using namespace Metasound::Editor;

	if (!ActionMenuBuilder.FromPin && CurrentGraph)
	{
		TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*CurrentGraph);
		if (MetasoundEditor.IsValid())
		{
			const int32 NumSelected = MetasoundEditor->GetNumNodesSelected();
			const FText MenuDescription = NumSelected > 0 ? LOCTEXT("CreateCommentAction", "Create Comment from Selection") : LOCTEXT("AddCommentAction", "Add Comment...");
			const FText ToolTip = LOCTEXT("CreateCommentToolTip", "Creates a comment.");

			TSharedPtr<FMetasoundGraphSchemaAction_NewComment> NewAction = MakeShared<FMetasoundGraphSchemaAction_NewComment>(FText::GetEmpty(), MenuDescription, ToolTip, EPrimaryContextGroup::Common);
			ActionMenuBuilder.AddAction(NewAction);
		}
	}
}

int32 UMetasoundEditorGraphSchema::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	using namespace Metasound::Editor;

	TSharedPtr<FEditor> MetasoundEditor = FGraphBuilder::GetEditorForGraph(*Graph);
	if (MetasoundEditor.IsValid())
	{
		return MetasoundEditor->GetNumNodesSelected();
	}

	return 0;
}

TSharedPtr<FEdGraphSchemaAction> UMetasoundEditorGraphSchema::GetCreateCommentAction() const
{
	TSharedPtr<FMetasoundGraphSchemaAction_NewComment> Comment = MakeShared<FMetasoundGraphSchemaAction_NewComment>();
	return StaticCastSharedPtr<FEdGraphSchemaAction, FMetasoundGraphSchemaAction_NewComment>(Comment);
}

void UMetasoundEditorGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const
{
	if (UMetasoundEditorGraphNode* MetasoundGraphNode = Cast<UMetasoundEditorGraphNode>(Node))
	{
		MetasoundGraphNode->GetMetasoundChecked().Modify();
		UEdGraphSchema::SetNodePosition(Node, Position);
		MetasoundGraphNode->UpdateFrontendNodeLocation(FDeprecateSlateVector2D(Position));
	}
	else
	{
		UEdGraphSchema::SetNodePosition(Node, Position);
	}
}

void UMetasoundEditorGraphSchema::QueryNodeClasses(TFunctionRef<void(TUniquePtr<Metasound::Editor::ISchemaQueryResult>&&)> OnClassFound) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	const FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();

	// Gather all loaded classes
	{
		constexpr bool bIncludeUnloadedAssets = true;
		const ISearchEngine::EResultVersion Options = SchemaPrivate::EnableAllVersionsMetaSoundNodeClassCreationCVar == 0
			? ISearchEngine::EResultVersion::Highest
			: ISearchEngine::EResultVersion::All;
		TArray<FMetaSoundClassInfo> ClassInfos = ISearchEngine::Get().FindAllClasses(Options, bIncludeUnloadedAssets);
		for (const FMetaSoundClassInfo& ClassInfo: ClassInfos)
		{
			OnClassFound(MakeUnique<SchemaPrivate::FSchemaQueryResult>(AssetManager, ClassInfo));
		}
	}
}

TSharedPtr<IAssetReferenceFilter> UMetasoundEditorGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const UMetasoundEditorGraph* MetasoundGraph = Cast<const UMetasoundEditorGraph>(Graph))
	{
		if (const UObject* Metasound = MetasoundGraph->GetMetasound())
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(Metasound);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
