// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEditorGraphValidation.h"

#include "Algo/NoneOf.h"
#include "EdGraph/EdGraphNode.h"
#include "Logging/TokenizedMessage.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundLog.h"


namespace Metasound
{
	namespace Editor
	{
		FGraphNodeValidationResult::FGraphNodeValidationResult(UMetasoundEditorGraphNode& InNode)
			: Node(&InNode)
		{
			// 1. Reset ed node validation state
			if (InNode.ErrorType != EMessageSeverity::Info)
			{
				InNode.ErrorType = EMessageSeverity::Info;
				bHasDirtiedNode = true;
			}

			if (InNode.bHasCompilerMessage)
			{
				InNode.bHasCompilerMessage = false;
				bHasDirtiedNode = true;
			}

			if (!InNode.ErrorMsg.IsEmpty())
			{
				InNode.ErrorMsg.Reset();
				bHasDirtiedNode = true;
			}
		}

		bool FGraphNodeValidationResult::GetHasDirtiedNode() const
		{
			return bHasDirtiedNode;
		}

		bool FGraphNodeValidationResult::GetIsInvalid() const
		{
			return Node->ErrorType == EMessageSeverity::Error;
		}

		UMetasoundEditorGraphNode& FGraphNodeValidationResult::GetNodeChecked() const
		{
			check(Node);
			return *Node;
		}

		void FGraphNodeValidationResult::SetMessage(EMessageSeverity::Type InSeverity, const FString& InMessage)
		{
			check(Node);

			if (!Node->bHasCompilerMessage)
			{
				bHasDirtiedNode = true;
				Node->bHasCompilerMessage = true;
			}

			if (Node->ErrorMsg != InMessage)
			{
				bHasDirtiedNode = true;
				Node->ErrorMsg = InMessage;
			}

			if (Node->ErrorType != InSeverity)
			{
				bHasDirtiedNode = true;
				Node->ErrorType = InSeverity;
			}

			if (InSeverity == EMessageSeverity::Error)
			{
				UE_LOGF(LogMetaSound, Error, "%ls", *InMessage);
			}
		}

		void FGraphNodeValidationResult::SetPinOrphaned(UEdGraphPin& Pin, bool bIsOrphaned)
		{
			if (Pin.bOrphanedPin != bIsOrphaned)
			{
				Pin.bOrphanedPin = bIsOrphaned;
				bHasDirtiedNode = true;
			}
		}

		void FGraphNodeValidationResult::SetUpgradeMessage(const FText& InMessage)
		{
			if (Node)
			{
				Node->NodeUpgradeMessage = InMessage;
			}
		}

		EMessageSeverity::Type FGraphValidationResults::GetHighestMessageSeverity() const
		{
			int32 HighestMessageSeverity = static_cast<int32>(EMessageSeverity::Info);

			for (const FGraphNodeValidationResult& Result : NodeResults)
			{
				UMetasoundEditorGraphNode& Node = Result.GetNodeChecked();
				HighestMessageSeverity = FMath::Min(Node.ErrorType, HighestMessageSeverity);
			}

			return static_cast<EMessageSeverity::Type>(HighestMessageSeverity);
		}

		const TArray<FGraphNodeValidationResult>& FGraphValidationResults::GetResults() const
		{
			return NodeResults;
		}
	} // namespace Editor
} // namespace Metasound