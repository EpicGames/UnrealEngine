// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorModule.h"

#include "Algo/Transform.h"
#include "AssetFolderContextMenu.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetTypeActions_Base.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMeter.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioSpectrumPlotStyle.h"
#include "AudioVectorscopePanelStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "ContentBrowserAssetDataCore.h"
#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserMenuContexts.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/IConsoleManager.h"
#include "IDetailCustomization.h"
#include "IMetasoundEngineModule.h"
#include "ISettingsModule.h"
#include "Metasound.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundDynamicInterfaceNodeConfigurationCustomization.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDetailCustomization.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphConnectionDrawingPolicy.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphNodeFactory.h"
#include "MetasoundEditorGraphNodeVisualization.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundNodeDetailCustomization.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTime.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundUObjectRegistry.h"
#include "Modules/ModuleManager.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "PackageMigrationContext.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SMetasoundFilterFrequencyResponsePlots.h"
#include "SMetasoundInputBrowser.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UObject/NoExportTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"


DEFINE_LOG_CATEGORY(LogMetasoundEditor);

#define LOCTEXT_NAMESPACE "MetaSounds"

namespace Metasound::Editor
{
	EAssetPrimeStatus IMetasoundEditorModule::GetAssetRegistryPrimeStatus() const
	{
		return EAssetPrimeStatus::NotRequested;
	}

	EAssetScanStatus IMetasoundEditorModule::GetAssetRegistryScanStatus() const
	{
		const bool bIsScanComplete = Engine::FMetaSoundAssetManager::GetChecked().IsInitialAssetScanComplete();
		if (bIsScanComplete)
		{
			return EAssetScanStatus::Complete;
		}

		// No longer returns whether or not requested vs InProgress, but function is deprecated and should just
		// use AssetManager to know whether or not scan is in progress.
		return EAssetScanStatus::InProgress;
	}

	namespace ModulePrivate
	{
		static const FLazyName AssetToolName { "AssetTools" };

		static TOptional<EMetasoundFrontendClassAccessFlags> ShowAccessFlagSelectDialog()
		{
			TOptional<EMetasoundFrontendClassAccessFlags> TargetFlags = EMetasoundFrontendClassAccessFlags::None;
			if (!GEditor)
			{
				return TargetFlags;
			}

			TSharedRef<SWindow> DialogWindow =
				SNew(SWindow)
				.Title(LOCTEXT("MetaSoundSetAccessFlagTitle", "Set MetaSound(s) Access Flags..."))
				.SupportsMinimize(false)
				.SupportsMaximize(false)
				.SizingRule(ESizingRule::Autosized)
				.AutoCenter(EAutoCenter::PreferredWorkArea);
				
			TSharedPtr<SVerticalBox> AccessFlagVerticalBox;
			TSharedRef<SBox> DialogContent = SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(AccessFlagVerticalBox, SVerticalBox)
				];

			const UEnum* FlagsEnum = StaticEnum<EMetasoundFrontendClassAccessFlags>();
			check(FlagsEnum);
			for (EMetasoundFrontendClassAccessFlags Flag : TEnumRange<EMetasoundFrontendClassAccessFlags>())
			{
				FText FlagDisplayName;
				FlagsEnum->FindDisplayNameTextByValue(FlagDisplayName, (int64)(Flag));
				AccessFlagVerticalBox->AddSlot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
							.IsChecked_Lambda([&TargetFlags, Flag]()
							{
								if (EnumHasAllFlags(*TargetFlags, Flag))
								{
									return ECheckBoxState::Checked;
								}
								return ECheckBoxState::Unchecked;
							})
							.OnCheckStateChanged_Lambda([&TargetFlags, Flag](ECheckBoxState InNewState)
							{
								if (InNewState == ECheckBoxState::Checked)
								{
									EnumAddFlags(*TargetFlags, Flag);
								}
								else
								{
									EnumRemoveFlags(*TargetFlags, Flag);
								}
							})
							[
								SNew(STextBlock)
									.Text(FlagDisplayName)
									.Font(IDetailLayoutBuilder::GetDetailFont())
							]
					];
			}

			AccessFlagVerticalBox->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
					.Text(LOCTEXT("MetaSoundSetAccessFlagAction_Accept", "Accept"))
					.OnClicked_Lambda([&DialogWindow]()
					{
						DialogWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
					.Text(LOCTEXT("MetaSoundSetAccessFlagAction_Cancel", "Cancel"))
					.OnClicked_Lambda([&TargetFlags, &DialogWindow]()
					{
						TargetFlags.Reset();
						DialogWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			];
				
			DialogWindow->SetContent(DialogContent);
			GEditor->EditorAddModalWindow(DialogWindow);

			return TargetFlags;
		}

		void AddVersionDocumentFolderMenuEntry(UToolMenu* Menu)
		{
			if (!Menu)
			{
				return;
			}

			const UContentBrowserFolderContext* Context = Menu->FindContext<UContentBrowserFolderContext>();
			if (!Context)
			{
				return;
			}

			bool bRecursePaths = false;
			auto CreateVersioningExecutionLambda = [Context](bool bRecursePaths) -> FExecuteAction
			{
				return FExecuteAction::CreateLambda([FolderPaths = Context->GetSelectedPackagePaths(), bRecursePaths]()
				{
					using namespace Frontend;

					FNotificationInfo Info(LOCTEXT("VersioningMetaSoundsNotifyTitle", "Versioning MetaSounds..."));
					Info.bFireAndForget = false;
					Info.ExpireDuration = 0.0f;
					Info.bUseThrobber = true;
					TSharedPtr<SNotificationItem> Notify = FSlateNotificationManager::Get().AddNotification(Info);
					if (Notify.IsValid())
					{
						Notify->SetCompletionState(SNotificationItem::CS_Pending);
					}

					const IMetaSoundAssetManager::FVersionAssetResults Results = IMetaSoundAssetManager::GetChecked().VersionAssetsInFolders(FolderPaths, bRecursePaths);
					if (!Results.DocumentsFoundInPackages())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_None);
						}
						UE_LOGF(LogMetasoundEditor, Display, "No MetaSound(s) Versioned: Folder/child folder(s) contain(s) no MetaSound asset(s).");
					}
					else if (Results.PackagesToReserialize.IsEmpty())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_None);
						}
						UE_LOGF(LogMetasoundEditor, Display, "No MetaSound(s) Versioned: Folder/child folder(s) contain(s) no MetaSound asset(s) requiring versioning.");
					}
					else if (!Results.FailedPackages.IsEmpty())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_Fail);
						}
						UE_LOGF(LogMetasoundEditor, Display, "%i MetaSound(s) Reserialized but '%i' failed: See output log for details.", Results.PackagesToReserialize.Num(), Results.FailedPackages.Num());
						FEditorFileUtils::PromptToCheckoutPackages(false, Results.PackagesToReserialize);
					}
					else
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_Success);
						}
						UE_LOGF(LogMetasoundEditor, Display, "%i MetaSound(s) Successfully Reserialized.", Results.PackagesToReserialize.Num());
						FEditorFileUtils::PromptToCheckoutPackages(false, Results.PackagesToReserialize);
					}
					
					if (Notify.IsValid())
					{
						Notify->ExpireAndFadeout();
					}
				});
			};

			const FText ResaveEntryWarning = LOCTEXT("ResaveMetaSoundAssetsMenuEntry_Warning", "Does not resave if replace deprecated node classes are found with newer major versions.");
			const FText ResaveContext = LOCTEXT("ResaveMetaSoundAssetMenuEntry_Context", "Load, version MetaSound asset document and update asset tags if either are necessary.");

			const FText ResaveContextFlat = LOCTEXT("ResaveMetaSoundAssetsMenuEntry_ContextFlat", "Resaves all MetaSounds in the given folder.");
			Menu->AddMenuEntry("MetaSounds", FToolMenuEntry::InitMenuEntry(
				"VersionMetaSoundsFlat",
				LOCTEXT("ResaveMetaSoundAssetsMenuEntry", "Version MetaSounds"),
				FText::Format(LOCTEXT("ResaveMetaSoundAssetsMenuEntryTooltip", "{0} {1} {2}"), ResaveContext, ResaveContextFlat, ResaveEntryWarning),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
				FToolUIActionChoice(CreateVersioningExecutionLambda(false /* bRecursePaths */))
			));

			const FText ResaveContextRecursive = LOCTEXT("ResaveMetaSoundAssetsMenuEntry_RecurseContext", "Resaves all MetaSounds in the given folder and all sub-folders.");
			Menu->AddMenuEntry("MetaSounds", FToolMenuEntry::InitMenuEntry(
				"VersionMetaSoundsRecursively",
				LOCTEXT("ResaveMetaSoundAssetsMenuEntry_Recursive", "Version MetaSounds (Recursive)"),
				FText::Format(LOCTEXT("ResaveMetaSoundAssetsMenuEntryTooltip", "{0} {1} {2}"), ResaveContextRecursive, ResaveContextFlat, ResaveEntryWarning),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
				FToolUIActionChoice(CreateVersioningExecutionLambda(true /* bRecursePaths */))
			));

			auto CreateSetAccessFlagsExecutionLambda = [Context](bool bRecursePaths) -> FExecuteAction
			{
				return FExecuteAction::CreateLambda([bRecursePaths, FolderPaths = Context->GetSelectedPackagePaths()]()
				{
					using namespace Frontend;

					const TOptional<EMetasoundFrontendClassAccessFlags> TargetAccessFlags = ModulePrivate::ShowAccessFlagSelectDialog();
					if (!TargetAccessFlags.IsSet())
					{
						return;
					}

					FNotificationInfo Info(LOCTEXT("SetMetaSoundsAccessFlagsNotifyTitle", "Setting MetaSound(s) AccessFlags..."));
					Info.bFireAndForget = false;
					Info.ExpireDuration = 0.0f;
					Info.bUseThrobber = true;
					TSharedPtr<SNotificationItem> Notify = FSlateNotificationManager::Get().AddNotification(Info);
					if (Notify.IsValid())
					{
						Notify->SetCompletionState(SNotificationItem::CS_Pending);
					}

					const IMetaSoundAssetManager::FVersionAssetResults Results = IMetaSoundAssetManager::GetChecked().SetAccessFlagsOnAssetsInFolders(FolderPaths, *TargetAccessFlags, bRecursePaths);
					if (!Results.DocumentsFoundInPackages())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_None);
						}
						UE_LOGF(LogMetasoundEditor, Display, "No MetaSound(s) Versioned: Folder/child folder(s) contain(s) no MetaSound asset(s).");
					}
					else if (Results.PackagesToReserialize.IsEmpty())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_None);
						}
						UE_LOGF(LogMetasoundEditor, Display, "No MetaSound(s) Versioned: Folder/child folder(s) contain(s) no MetaSound asset(s) requiring versioning.");
					}
					else if (!Results.FailedPackages.IsEmpty())
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_Fail);
						}
						UE_LOGF(LogMetasoundEditor, Display, "%i MetaSound(s) Reserialized but '%i' failed: See output log for details.", Results.PackagesToReserialize.Num(), Results.FailedPackages.Num());
						FEditorFileUtils::PromptToCheckoutPackages(false, Results.PackagesToReserialize);
					}
					else
					{
						if (Notify.IsValid())
						{
							Notify->SetCompletionState(SNotificationItem::CS_Success);
						}
						UE_LOGF(LogMetasoundEditor, Display, "%i MetaSound(s) Successfully Reserialized.", Results.PackagesToReserialize.Num());
						FEditorFileUtils::PromptToCheckoutPackages(false, Results.PackagesToReserialize);
					}

					if (Notify.IsValid())
					{
						Notify->ExpireAndFadeout();
					}
				});
			};

			Menu->AddMenuEntry("MetaSounds", FToolMenuEntry::InitMenuEntry(
				"SetMetaSoundAccessFlags",
				LOCTEXT("SetAccessFlagsMetaSoundAssetsMenuEntry", "Set MetaSound Access Flags"),
				LOCTEXT("SetMetaSoundFlags", "Sets given AccessFlag values on all MetaSounds in the given folder."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
				FToolUIActionChoice(CreateSetAccessFlagsExecutionLambda(false /* bRecursePaths */))
			));

			Menu->AddMenuEntry("MetaSounds", FToolMenuEntry::InitMenuEntry(
				"SetMetaSoundAccessFlagsRecursively",
				LOCTEXT("SetAccessFlagsMetaSoundAssetsMenuEntry_Recursive", "Set MetaSound Access Flags (Recursive)"),
				LOCTEXT("SetMetaSoundFlags_RecurseContext", "Sets given AccessFlag values on all MetaSounds in the given folder (and all sub-folders)."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
				FToolUIActionChoice(CreateSetAccessFlagsExecutionLambda(true /* bRecursePaths */))
			));
		}
	} // namespace ModulePrivate

	class FSlateStyle final : public FSlateStyleSet
	{
	public:
		FSlateStyle()
			: FSlateStyleSet("MetaSoundStyle")
		{
			SetParentStyleName(FAudioWidgetsStyle::Get().GetStyleSetName());

			SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/Metasound/Content/Editor/Slate"));
			SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

			static const FVector2D Icon20x20(20.0f, 20.0f);
			static const FVector2D Icon40x40(40.0f, 40.0f);

			static const FVector2D Icon16 = FVector2D(16.0f, 16.0f);
			static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

			const FVector2D Icon15x11(15.0f, 11.0f);

			// Metasound Editor
			{
				Set("MetaSoundPatch.Color", FColor(31, 133, 31));
				Set("MetaSoundSource.Color", FColor(103, 214, 66));

				// Actions
				Set("MetasoundEditor.Play", new IMAGE_BRUSH_SVG(TEXT("Icons/play"), Icon40x40));
				Set("MetasoundEditor.Play.Small", new IMAGE_BRUSH_SVG(TEXT("Icons/play"), Icon20x20));
				Set("MetasoundEditor.Play.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/play_thumbnail"), Icon64));
				Set("MetasoundEditor.Play.Thumbnail.Hovered", new IMAGE_BRUSH_SVG(TEXT("Icons/play_thumbnail_hover"), Icon64));

				Set("MetasoundEditor.Play.Active.Valid", new IMAGE_BRUSH_SVG(TEXT("Icons/play_active_valid"), Icon40x40));
				Set("MetasoundEditor.Play.Active.Warning", new IMAGE_BRUSH_SVG(TEXT("Icons/play_active_warning"), Icon40x40));
				Set("MetasoundEditor.Play.Inactive.Valid", new IMAGE_BRUSH_SVG(TEXT("Icons/play_inactive_valid"), Icon40x40));
				Set("MetasoundEditor.Play.Inactive.Warning", new IMAGE_BRUSH_SVG(TEXT("Icons/play_inactive_warning"), Icon40x40));
				Set("MetasoundEditor.Play.Error", new IMAGE_BRUSH_SVG(TEXT("Icons/play_error"), Icon40x40));

				Set("MetasoundEditor.Stop", new IMAGE_BRUSH_SVG(TEXT("Icons/stop"), Icon40x40));

				Set("MetasoundEditor.Stop.Disabled", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_disabled"), Icon40x40));
				Set("MetasoundEditor.Stop.Active", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_active"), Icon40x40));
				Set("MetasoundEditor.Stop.Inactive", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_inactive"), Icon40x40));
				Set("MetasoundEditor.Stop.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_thumbnail"), Icon64));
				Set("MetasoundEditor.Stop.Thumbnail.Hovered", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_thumbnail_hover"), Icon64));

				Set("MetasoundEditor.Import", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_40x.png")), Icon40x40));
				Set("MetasoundEditor.Import.Small", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_40x.png")), Icon20x20));
				Set("MetasoundEditor.Export", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_40x.png")), Icon40x40));
				Set("MetasoundEditor.Export.Small", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_40x.png")), Icon20x20));
				Set("MetasoundEditor.ExportError", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_error_40x.png")), Icon40x40));
				Set("MetasoundEditor.ExportError.Small", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/build_error_40x.png")), Icon20x20));
				Set("MetasoundEditor.Settings", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/settings_40x.png")), Icon20x20));

				// Graph Editor
				Set("MetasoundEditor.Graph.Node.Body.Input", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_input_body_64x.png")), FVector2D(114.0f, 64.0f)));
				Set("MetasoundEditor.Graph.Node.Body.Default", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_default_body_64x.png")), FVector2D(64.0f, 64.0f)));

				Set("MetasoundEditor.Graph.TriggerPin.Connected", new IMAGE_BRUSH(TEXT("Graph/pin_trigger_connected"), Icon15x11));
				Set("MetasoundEditor.Graph.TriggerPin.Disconnected", new IMAGE_BRUSH(TEXT("Graph/pin_trigger_disconnected"), Icon15x11));

				Set("MetasoundEditor.Graph.Node.Class.Native", new IMAGE_BRUSH_SVG(TEXT("Icons/native_node"), FVector2D(8.0f, 16.0f)));
				Set("MetasoundEditor.Graph.Node.Class.Graph", new IMAGE_BRUSH_SVG(TEXT("Icons/graph_node"), Icon16));
				Set("MetasoundEditor.Graph.Node.Class.Input", new IMAGE_BRUSH_SVG(TEXT("Icons/input_node"), FVector2D(16.0f, 13.0f)));
				Set("MetasoundEditor.Graph.Node.Class.Output", new IMAGE_BRUSH_SVG(TEXT("Icons/output_node"), FVector2D(16.0f, 13.0f)));
				Set("MetasoundEditor.Graph.Node.Class.Reroute", new IMAGE_BRUSH_SVG(TEXT("Icons/reroute_node"), Icon16));
				Set("MetasoundEditor.Graph.Node.Class.Variable", new IMAGE_BRUSH_SVG(TEXT("Icons/variable_node"), FVector2D(16.0f, 13.0f)));

				Set("MetasoundEditor.Graph.Node.Math.Add", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_add_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Divide", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_divide_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Modulo", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_modulo_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Multiply", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_multiply_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Subtract", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_subtract_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Power", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_power_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Math.Logarithm", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_math_logarithm_40x.png")), Icon40x40));
				Set("MetasoundEditor.Graph.Node.Conversion", new FSlateImageBrush(RootToContentDir(TEXT("/Graph/node_conversion_40x.png")), Icon40x40));

				Set("MetasoundEditor.Graph.InvalidReroute", new IMAGE_BRUSH_SVG(TEXT("Icons/invalid_reroute"), Icon16));
				Set("MetasoundEditor.Graph.ConstructorPinArray", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/array_pin_rotated.png")), Icon16));
				Set("MetasoundEditor.Graph.ConstructorPinArrayDisconnected", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/array_pin_rotated_disconnected.png")), Icon16));
				Set("MetasoundEditor.Graph.ArrayPin", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/array_pin.png")), Icon16));
				Set("MetasoundEditor.Graph.ConstructorPin", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/square_pin_rotated.png")), Icon16));
				Set("MetasoundEditor.Graph.ConstructorPinDisconnected", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/square_pin_rotated_disconnected.png")), Icon16));

				// Analyzers
				Set("MetasoundEditor.Analyzers.BackgroundColor", FLinearColor(0.0075f, 0.0075f, 0.0075f, 1.0f));
				Set("MetasoundEditor.Analyzers.ForegroundColor", FLinearColor(0.025719f, 0.208333f, 0.069907f, 1.0f)); // "Audio" Green

				// Misc
				Set("MetasoundEditor.Audition", new IMAGE_BRUSH_SVG(TEXT("Icons/metasound_page"), Icon16));
				Set("MetasoundEditor.Metasound.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasound_icon"), Icon16));
				Set("MetasoundEditor.Speaker", new FSlateImageBrush(RootToContentDir(TEXT("/Icons/speaker_144x.png")), FVector2D(144.0f, 144.0f)));
				Set("MetasoundEditor.InterfacesInOut.Background", new FSlateBoxBrush(FAppStyle::Get().GetContentRootDir() / TEXT("Common/RoundedSelection_16x.png"), 4.0f / 16.0f, FLinearColor(1.f, 1.f, 1.f, 0.1f)));

				// Pages
				Set("MetasoundEditor.Page.Executing.ForegroundColor", FStyleColors::AccentGreen.GetSpecifiedColor());
				Set("MetasoundEditor.Page.Executing", new IMAGE_BRUSH_SVG(TEXT("Icons/metasound_page_exec"), Icon16));

				// Class Icons
				auto SetClassIcon = [this, InIcon16 = Icon16, InIcon64 = Icon64](const FString& ClassName)
				{
					const FString IconFileName = FString::Printf(TEXT("Icons/%s"), *ClassName.ToLower());
					const FSlateColor DefaultForeground(FStyleColors::Foreground);

					Set(*FString::Printf(TEXT("ClassIcon.%s"), *ClassName), new IMAGE_BRUSH_SVG(IconFileName, InIcon16));
					Set(*FString::Printf(TEXT("ClassThumbnail.%s"), *ClassName), new IMAGE_BRUSH_SVG(IconFileName, InIcon64));
				};

				SetClassIcon(TEXT("MetasoundPatch"));
				SetClassIcon(TEXT("MetasoundSource"));

				Set("MetasoundEditor.Metasound.Node.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasound"), Icon20x20));
				Set("MetasoundEditor.MetasoundPatch.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundpatch_icon"), Icon20x20));
				Set("MetasoundEditor.MetasoundPatch.Preset.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundpatchpreset_icon"), Icon20x20));
				Set("MetasoundEditor.MetasoundSource.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundsource_icon"), Icon20x20));
				Set("MetasoundEditor.MetasoundSource.Preset.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundsourcepreset_icon"), Icon20x20));
				Set("MetasoundEditor.MetasoundPatch.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundpatch_thumbnail"), Icon20x20));
				Set("MetasoundEditor.MetasoundPatch.Preset.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundpatchpreset_thumbnail"), Icon20x20));
				Set("MetasoundEditor.MetasoundSource.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundsource_thumbnail"), Icon20x20));
				Set("MetasoundEditor.MetasoundSource.Preset.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/metasoundsourcepreset_thumbnail"), Icon20x20));
			}

			// Audio Widgets
			{
				const FLinearColor& AnalyzerBackgroundColor = GetColor("MetasoundEditor.Analyzers.BackgroundColor");
				const FLinearColor& AnalyzerForegroundColor = GetColor("MetasoundEditor.Analyzers.ForegroundColor");

				// Add static overrides for these widget styles:
				FAudioMeterDefaultColorStyle MeterStyle;
				MeterStyle.MeterValueColor = AnalyzerForegroundColor;
				Set("AudioMeter.DefaultColorStyle", MeterStyle);

				Set("AudioOscilloscope.PanelStyle", FAudioOscilloscopePanelStyle()
					.SetWaveViewerStyle(FSampledSequenceViewerStyle()
						.SetSequenceColor(AnalyzerForegroundColor)));

				Set("AudioSpectrumPlot.Style", FAudioSpectrumPlotStyle()
					.SetCrosshairColor(FSlateColor(AnalyzerForegroundColor).UseSubduedForeground())
					.SetSpectrumColor(AnalyzerForegroundColor));

				Set("AudioVectorscope.PanelStyle", FAudioVectorscopePanelStyle()
					.SetVectorViewerStyle(FSampledSequenceVectorViewerStyle()
						.SetLineColor(AnalyzerForegroundColor)));

				Set("AudioAnalyzerRack.BackgroundColor", AnalyzerBackgroundColor);

				// Add dynamic overrides for the widget style types with settings overrides:
				AddDynamicLoadedWidgetStyle<FAudioMaterialKnobStyle>([]() { return GetDefault<UMetasoundEditorSettings>()->KnobStyleOverride; });
				AddDynamicLoadedWidgetStyle<FAudioMaterialSliderStyle>([]() { return GetDefault<UMetasoundEditorSettings>()->SliderStyleOverride; });
				AddDynamicLoadedWidgetStyle<FAudioMaterialButtonStyle>([]() { return GetDefault<UMetasoundEditorSettings>()->ButtonStyleOverride; });
				AddDynamicLoadedWidgetStyle<FAudioMaterialMeterStyle>([]() { return GetDefault<UMetasoundEditorSettings>()->MeterStyleOverride; });
			}

			FSlateStyleRegistry::RegisterSlateStyle(*this);
		}

	private:
		virtual const FSlateWidgetStyle* GetWidgetStyleInternal(const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const override
		{
			if (const auto* WidgetStyleLoader = AudioMaterialWidgetStyleLoaders.Find(DesiredTypeName))
			{
				const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
				if (EditorSettings == nullptr || !EditorSettings->bUseAudioMaterialWidgets)
				{
					ensure(!bWarnIfNotFound);

					// AudioMaterialWidgets are disabled, so explicitly return null rather than fallback to the base style.
					return nullptr;
				}
					
				if (const FSlateWidgetStyle* CustomWidgetStyle = (*WidgetStyleLoader)->LoadWidgetStyle())
				{
					return CustomWidgetStyle;
				}
			}
				
			const FSlateWidgetStyle* WidgetStyle = FSlateStyleSet::GetWidgetStyleInternal(DesiredTypeName, StyleName, DefaultStyle, bWarnIfNotFound);
			ensure(!bWarnIfNotFound || WidgetStyle != nullptr);
			return WidgetStyle;
		}

		using FGetAssetPathFunc = TFunction<FSoftObjectPath()>;

		template<std::derived_from<FSlateWidgetStyle> WidgetStyleType>
		void AddDynamicLoadedWidgetStyle(FGetAssetPathFunc GetAssetPath)
		{
			static_assert(std::is_base_of_v<FAudioMaterialWidgetStyle, WidgetStyleType>);
			AudioMaterialWidgetStyleLoaders.Add(WidgetStyleType::TypeName, MakeShared<TWidgetStyleLoader<WidgetStyleType>>(GetAssetPath));
		}

		class IWidgetStyleLoader
		{
		public:
			virtual ~IWidgetStyleLoader() {};
			virtual const FSlateWidgetStyle* LoadWidgetStyle() = 0;
		};

		template<std::derived_from<FSlateWidgetStyle> WidgetStyleType>
		class TWidgetStyleLoader : public IWidgetStyleLoader
		{
		public:
			TWidgetStyleLoader(FGetAssetPathFunc InGetAssetPath)
				: GetAssetPath(InGetAssetPath)
			{
				//
			}

			virtual const FSlateWidgetStyle* LoadWidgetStyle() override
			{
				TSoftObjectPtr<USlateWidgetStyleAsset> SoftObjectPtr(GetAssetPath());
				if (const USlateWidgetStyleAsset* WidgetStyleAsset = SoftObjectPtr.LoadSynchronous())
				{
					if (const WidgetStyleType* WidgetStyle = WidgetStyleAsset->GetStyle<WidgetStyleType>())
					{
						// Copy style:
						CachedStyle = *WidgetStyle;

						// Return pointer to our copy:
						return &CachedStyle;
					}
				}

				return nullptr;
			}

		private:
			FGetAssetPathFunc GetAssetPath;
			WidgetStyleType CachedStyle;
		};

		TMap<FName, TSharedRef<IWidgetStyleLoader>> AudioMaterialWidgetStyleLoaders;
	};

	namespace Style
	{
		FSlateIcon CreateSlateIcon(FName InName)
		{
			return { "MetaSoundStyle", InName};
		}

		const FSlateBrush& GetSlateBrushSafe(FName InName)
		{
			const ISlateStyle* MetaSoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle");
			if (ensureMsgf(MetaSoundStyle, TEXT("Missing slate style 'MetaSoundStyle'")))
			{
				const FSlateBrush* Brush = MetaSoundStyle->GetBrush(InName);
				if (ensureMsgf(Brush, TEXT("Missing brush '%s'"), *InName.ToString()))
				{
					return *Brush;
				}
			}

			if (const FSlateBrush* NoBrush = FAppStyle::GetBrush("NoBrush"))
			{
				return *NoBrush;
			}

			static const FSlateBrush NullBrush;
			return NullBrush;
		}

		const FSlateColor& GetPageExecutingColor()
		{
			auto MakeColor = []() -> FSlateColor
			{
				if (const ISlateStyle* MetaSoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
				{
					return MetaSoundStyle->GetColor("MetasoundEditor.Page.Executing.ForegroundColor");
				}

				return FStyleColors::AccentWhite;
			};
			static const FSlateColor AnalyzerColor = MakeColor();
			return AnalyzerColor;
		}
	} // namespace Style

	class FModule : public IMetasoundEditorModule
	{
		void RegisterInputDefaultClasses()
		{
			TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> NodeClass;
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				UClass* Class = *ClassIt;
				if (!Class->IsNative())
				{
					continue;
				}

				if (Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists))
				{
					continue;
				}

				if (!ClassIt->IsChildOf(UMetasoundEditorGraphMemberDefaultLiteral::StaticClass()))
				{
					continue;
				}

				if (const UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteralCDO = Class->GetDefaultObject<UMetasoundEditorGraphMemberDefaultLiteral>())
				{
					InputDefaultLiteralClassRegistry.Add(DefaultLiteralCDO->GetLiteralType(), DefaultLiteralCDO->GetClass());
				}
			}
		}

		void RegisterFolderBulkOperations()
		{
			if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
			{
				FToolMenuSection& BulkOpsSection = Menu->FindOrAddSection("PathContextBulkOperations");
				constexpr bool bOpenSubMenuOnClick = false;
				BulkOpsSection.AddSubMenu(
					"MetaSounds",
					LOCTEXT("MetaSoundBulkOpsLabel", "MetaSounds"),
					LOCTEXT("MetaSoundBulkOps_ToolTip", "Bulk operations pertaining to MetaSound UObject types"),
					FNewToolMenuDelegate::CreateStatic(&ModulePrivate::AddVersionDocumentFolderMenuEntry),
					bOpenSubMenuOnClick,
					Style::CreateSlateIcon("ClassIcon.MetasoundPatch")
				);
			}
		}

		virtual void RegisterPinType(FName InDataTypeName, FName InPinCategory, FName InPinSubCategory, const FSlateBrush* InPinConnectedIcon = nullptr, const FSlateBrush* InPinDisconnectedIcon = nullptr) override
		{
			FGraphPinParams Params;
			Params.PinCategory = InPinCategory;
			Params.PinSubCategory = InPinSubCategory;
			Params.PinConnectedIcon = InPinConnectedIcon;
			Params.PinDisconnectedIcon = InPinDisconnectedIcon;
			GraphPanelPinFactory->RegisterPin(InDataTypeName, Params);
		}

		virtual void RegisterCustomPinType(FName InDataTypeName, const FGraphPinParams& Params) override
		{
			GraphPanelPinFactory->RegisterPin(InDataTypeName, Params);
		}

		virtual void RegisterCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType, FCreateNodeConfigurationDetails InCreateDetailsFunc) override
		{
			CustomNodeConfigurationDetails.FindOrAdd(InNodeConfigurationStructType, InCreateDetailsFunc);
		}

		virtual void UnregisterCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType) override
		{
			CustomNodeConfigurationDetails.Remove(InNodeConfigurationStructType);
		}

		virtual void RegisterGraphNodeVisualization(FName InNodeClassName, FOnCreateGraphNodeVisualizationWidget OnCreateGraphNodeVisualizationWidget) override
		{
			FGraphNodeVisualizationRegistry::Get().RegisterVisualization(InNodeClassName, OnCreateGraphNodeVisualizationWidget);
		}

		virtual bool IsRestrictedMode() const override
		{
			return bIsRestrictedMode;
		}

		virtual void SetRestrictedMode(bool bInRestrictedMode) override
		{
			bIsRestrictedMode = bInRestrictedMode;
			const bool bEnableLogging = !bIsRestrictedMode;
			Frontend::DocumentTransform::SetVersioningLoggingEnabled(bEnableLogging);
		}

		void RegisterSettingsDelegates()
		{
			using namespace Engine;

			// All the following delegates are used for UX notification, audition
			// and PIE which are not desired/necessary when cooking.
			if (IsRunningCookCommandlet())
			{
				return;
			}

			const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
			
			if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
			{
				Settings->GetOnDefaultRenamedDelegate().AddLambda([]()
				{
					FNotificationInfo Info(LOCTEXT("MetaSoundSettings_CannotNameDefaultPage", "Cannot name 'Default': reserved MetaSound page name"));
					Info.bFireAndForget = true;
					Info.ExpireDuration = 2.0f;
					Info.bUseThrobber = true;
					FSlateNotificationManager::Get().AddNotification(Info);
				});
			}

			FEditorDelegates::PreBeginPIE.AddWeakLambda(EditorSettings, [this](const bool /* bSimulating */)
			{
				using namespace Metasound::Frontend;

				if (const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>())
				{
					if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
					{
						if (!EdSettings->bApplyAuditionSettingsInPIE)
						{

							Settings->OverrideTargetPageSettings({}); // Clear out page override if we want it disabled in PIE
						}
					}

					IMetaSoundAssetManager::GetChecked().ReregisterLoadedAssets();
				}
			});
			FEditorDelegates::EndPIE.AddWeakLambda(EditorSettings, [](const bool /* bSimulating */)
			{

				if (const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>())
				{
					if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
					{
						Settings->OverrideTargetPageSettings(EdSettings->GetPageForAudition());
					}
				}
			});

		}

		virtual void RegisterExplicitProxyClass(const UClass& InClass) override
		{
			using namespace Metasound::Frontend;

			const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();
			FDataTypeRegistryInfo RegistryInfo;
			ensureAlways(DataTypeRegistry.IsUObjectProxyFactory(InClass.GetDefaultObject()));

			ExplicitProxyClasses.Add(&InClass);
		}

		virtual bool IsExplicitProxyClass(const UClass& InClass) const override
		{
			return ExplicitProxyClasses.Contains(&InClass);
		}

		virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateMemberDefaultLiteralCustomization(UClass& InClass, IDetailCategoryBuilder& InDefaultCategoryBuilder) const override
		{
			const TUniquePtr<IMemberDefaultLiteralCustomizationFactory>* CustomizationFactory = LiteralCustomizationFactories.Find(&InClass);
			if (CustomizationFactory && CustomizationFactory->IsValid())
			{
				return (*CustomizationFactory)->CreateLiteralCustomization(InDefaultCategoryBuilder);
			}

			return nullptr;
		}

		virtual const FCreateNodeConfigurationDetails* FindCreateCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType) const override
		{
			return CustomNodeConfigurationDetails.Find(InNodeConfigurationStructType);
		}

		virtual const TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> FindDefaultLiteralClass(EMetasoundFrontendLiteralType InLiteralType) const override
		{
			return InputDefaultLiteralClassRegistry.FindRef(InLiteralType);
		}

		virtual const FSlateBrush* GetIconBrush(FName InDataType, const bool bIsConstructorType) const override
		{
			Frontend::FDataTypeRegistryInfo Info;
			Frontend::IDataTypeRegistry::Get().GetDataTypeInfo(InDataType, Info);

			if (Info.bIsArrayType)
			{
				return bIsConstructorType ? &Style::GetSlateBrushSafe("MetasoundEditor.Graph.ConstructorPinArray") : &Style::GetSlateBrushSafe("MetasoundEditor.Graph.ArrayPin");
			}
			else
			{
				return bIsConstructorType ? &Style::GetSlateBrushSafe("MetasoundEditor.Graph.ConstructorPin") : FAppStyle::GetBrush("Icons.BulletPoint");
			}
		}
			
		virtual bool GetCustomPinIcons(UEdGraphPin* InPin, const FSlateBrush*& PinConnectedIcon, const FSlateBrush*& PinDisconnectedIcon) const override
		{
			return GraphPanelPinFactory->GetCustomPinIcons(InPin, PinConnectedIcon, PinDisconnectedIcon);
		}

		virtual bool GetCustomPinIcons(FName InDataType, const FSlateBrush*& PinConnectedIcon, const FSlateBrush*& PinDisconnectedIcon) const override
		{
			return GraphPanelPinFactory->GetCustomPinIcons(InDataType, PinConnectedIcon, PinDisconnectedIcon);
		}

		virtual const FEdGraphPinType* FindPinType(FName InDataTypeName) const override
		{
			return GraphPanelPinFactory->FindPinType(InDataTypeName);
		}

		virtual TSharedRef<IMetaSoundGraphPanelPinFactory> GetGraphPanelPinFactory() const override
		{
			return StaticCastSharedRef<IMetaSoundGraphPanelPinFactory>(GraphPanelPinFactory.ToSharedRef());
		}

		virtual bool IsMetaSoundAssetClass(const FTopLevelAssetPath& InClassName) const override
		{
			if (const UClass* ClassObject = FindObject<const UClass>(InClassName))
			{
				return IMetasoundUObjectRegistry::Get().IsRegisteredClass(*ClassObject);
			}

			return false;
		}

		virtual void StartupModule() override
		{
			using namespace Engine;
			using namespace ModulePrivate;

			METASOUND_LLM_SCOPE;

			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
				
			PropertyModule.RegisterCustomClassLayout(
				UMetaSoundPatch::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundDetailCustomization>(UMetaSoundPatch::GetDocumentPropertyName()); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetaSoundSource::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundDetailCustomization>(UMetaSoundSource::GetDocumentPropertyName()); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundInterfacesView::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundInterfacesDetailCustomization>(); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundPagesView::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundPagesDetailCustomization>(); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundEditorGraphNode::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetaSoundNodeDetailCustomization>(); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundEditorGraphInput::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundInputDetailCustomization>(); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundEditorGraphOutput::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundOutputDetailCustomization>(); }));

			PropertyModule.RegisterCustomClassLayout(
				UMetasoundEditorGraphVariable::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundVariableDetailCustomization>(); }));

			PropertyModule.RegisterCustomPropertyTypeLayout(
				"MetasoundEditorGraphMemberDefaultBoolRef",
				FOnGetPropertyTypeCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundMemberDefaultBoolDetailCustomization>(); }));

			PropertyModule.RegisterCustomPropertyTypeLayout(
				"MetasoundEditorGraphMemberDefaultIntRef",
				FOnGetPropertyTypeCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundMemberDefaultIntDetailCustomization>(); }));

			PropertyModule.RegisterCustomPropertyTypeLayout(
				"MetasoundEditorGraphMemberDefaultObjectRef",
				FOnGetPropertyTypeCustomizationInstance::CreateLambda([]() { return MakeShared<FMetasoundMemberDefaultObjectDetailCustomization>(); }));

			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultLiteral::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultBool::StaticClass(), MakeUnique<FMetasoundBoolLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultBoolArray::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultFloat::StaticClass(), MakeUnique<FMetasoundFloatLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultFloatArray::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultInt::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultIntArray::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultObject::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultObjectArray::StaticClass(), MakeUnique<FMetasoundObjectArrayLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultString::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());
			LiteralCustomizationFactories.Add(UMetasoundEditorGraphMemberDefaultStringArray::StaticClass(), MakeUnique<FMetasoundDefaultLiteralCustomizationFactory>());

			StyleSet = MakeShared<FSlateStyle>();

			RegisterCustomNodeConfigurationDetailsCustomization(
				FMetaSoundDynamicInterfaceNodeConfiguration::StaticStruct()->GetFName(),
				[](TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
				{
					return MakeShareable(new FDynamicInterfaceNodeConfigurationCustomization(InStructProperty, InNode));
				});
			RegisteredCustomNodeConfigurationDetails.Add(FMetaSoundDynamicInterfaceNodeConfiguration::StaticStruct()->GetFName());

			RegisterInputDefaultClasses();

			GraphConnectionFactory = MakeShared<FGraphConnectionDrawingPolicyFactory>();
			FEdGraphUtilities::RegisterVisualPinConnectionFactory(GraphConnectionFactory);

			GraphNodeFactory = MakeShared<FMetasoundGraphNodeFactory>();
			FEdGraphUtilities::RegisterVisualNodeFactory(GraphNodeFactory);

			GraphPanelPinFactory = MakeShared<FMetaSoundGraphPanelPinFactory>();
			FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

			RegisterGraphNodeVisualization(
				"UE.Biquad Filter.Audio",
				FOnCreateGraphNodeVisualizationWidget::CreateStatic(&CreateMetaSoundBiquadFilterGraphNodeVisualizationWidget));

			RegisterGraphNodeVisualization(
				"UE.Ladder Filter.Audio",
				FOnCreateGraphNodeVisualizationWidget::CreateStatic(&CreateMetaSoundLadderFilterGraphNodeVisualizationWidget));

			RegisterGraphNodeVisualization(
				"UE.One-Pole High Pass Filter.Audio",
				FOnCreateGraphNodeVisualizationWidget::CreateStatic(&CreateMetaSoundOnePoleHighPassFilterGraphNodeVisualizationWidget));

			RegisterGraphNodeVisualization(
				"UE.One-Pole Low Pass Filter.Audio",
				FOnCreateGraphNodeVisualizationWidget::CreateStatic(&CreateMetaSoundOnePoleLowPassFilterGraphNodeVisualizationWidget));

			RegisterGraphNodeVisualization(
				"UE.State Variable Filter.Audio",
				FOnCreateGraphNodeVisualizationWidget::CreateStatic(&CreateMetaSoundStateVariableFilterGraphNodeVisualizationWidget));

			ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");

			SettingsModule.RegisterSettings("Editor", "ContentEditors", "MetaSound Editor",
				LOCTEXT("MetaSoundEditorSettingsName", "MetaSound Editor"),
				LOCTEXT("MetaSoundEditorSettingsDescription", "Customize MetaSound Editor."),
				GetMutableDefault<UMetasoundEditorSettings>()
			);

			// Metasound Engine registers USoundWave as a proxy class in the
			// Metasound Frontend. The frontend registration must occur before
			// the Metasound Editor registration of a USoundWave.
			IMetasoundEngineModule& MetaSoundEngineModule = FModuleManager::LoadModuleChecked<IMetasoundEngineModule>("MetasoundEngine");

			// Bind delegates for MetaSound registration in the asset registry
			MetaSoundEngineModule.GetOnGraphRegisteredDelegate().BindLambda([](UObject& InMetaSound, Engine::ERegistrationAssetContext AssetContext)
			{
				using namespace Engine;

				{
					Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
					RegOptions.bForceViewSynchronization = AssetContext == ERegistrationAssetContext::Renaming;
					FGraphBuilder::RegisterGraphWithFrontend(InMetaSound, MoveTemp(RegOptions));
				}
			});
			MetaSoundEngineModule.GetOnGraphUnregisteredDelegate().BindLambda([](UObject& InMetaSound, Engine::ERegistrationAssetContext AssetContext)
			{
				using namespace Engine;

				switch(AssetContext)
				{
					case ERegistrationAssetContext::Reloading:
					case ERegistrationAssetContext::Removing:
					case ERegistrationAssetContext::Renaming:
					{
						if (GIsEditor)
						{
							UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
							if (AssetEditorSubsystem)
							{
								// Close the editors so the internal reference to the builder doesn't suddenly 
								// upon GC enter an invalid state (pointing to a null MetaSound asset)
								AssetEditorSubsystem->CloseAllEditorsForAsset(&InMetaSound);
							}
						}
						break;
					}

					case ERegistrationAssetContext::None:
					default:
					{
						break;
					}
				}

				Metasound::IMetasoundUObjectRegistry& UObjectRegistry = Metasound::IMetasoundUObjectRegistry::Get();
				if (FMetasoundAssetBase* AssetBase = UObjectRegistry.GetObjectAsAssetBase(&InMetaSound))
				{
					AssetBase->UnregisterGraphWithFrontend();
				}
			});

			// Required to ensure logic to order nodes for presets exclusive to
			// editor is propagated to transform instances while editing in editor.
			Frontend::DocumentTransform::RegisterNodeDisplayNameProjection([](const Frontend::FNodeHandle& NodeHandle)
			{
				constexpr bool bIncludeNamespace = false;
				return FGraphBuilder::GetDisplayName(*NodeHandle, bIncludeNamespace);
			});

			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AssetToolName).Get();
			AssetTools.GetOnPackageMigration().AddRaw(this, &FModule::OnPackageMigration);

			RegisterSettingsDelegates();
			RegisterFolderBulkOperations();

			// Set page auditioning overrides
			const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
			UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>();
			if (Settings && EditorSettings)
			{
				Settings->OverrideTargetPageSettings(EditorSettings->GetPageForAudition());
			}

			// Register MetaSound Input Browser nomad tab
			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
				SMetasoundInputBrowser::TabId,
				FOnSpawnTab::CreateLambda([](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
				{
					TSharedRef<SMetasoundInputBrowser> Browser = SNew(SMetasoundInputBrowser);
					return SNew(SDockTab)
						.TabRole(ETabRole::NomadTab)
						.Label(LOCTEXT("MetaSoundInputBrowserTabTitle", "MetaSound Input Browser"))
						[
							Browser
						];
				}))
				.SetDisplayName(LOCTEXT("MetaSoundInputBrowserDisplayName", "MetaSound Input Browser"))
				.SetTooltipText(LOCTEXT("MetaSoundInputBrowserTooltip", "Bulk view and edit MetaSound input values across multiple assets"))
				.SetMenuType(ETabSpawnerMenuType::Hidden)
				.SetIcon(FSlateIcon("MetaSoundStyle", "MetasoundEditor.Metasound.Icon"));

			PropertyModule.NotifyCustomizationModuleChanged();
		}

		virtual void ShutdownModule() override
		{
			using namespace ModulePrivate;

			METASOUND_LLM_SCOPE;

			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SMetasoundInputBrowser::TabId);

			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->UnregisterSettings("Editor", "Audio", "MetaSound Editor");
			}

			if (FModuleManager::Get().IsModuleLoaded(AssetToolName))
			{
				IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AssetToolName).Get();
				AssetTools.GetOnPackageMigration().RemoveAll(this);
			}

			if (GraphConnectionFactory.IsValid())
			{
				FEdGraphUtilities::UnregisterVisualPinConnectionFactory(GraphConnectionFactory);
			}

			if (GraphNodeFactory.IsValid())
			{
				FEdGraphUtilities::UnregisterVisualNodeFactory(GraphNodeFactory);
				GraphNodeFactory.Reset();
			}

			if (GraphPanelPinFactory.IsValid())
			{
				FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);
				GraphPanelPinFactory.Reset();
			}

			for (const FName& NodeConfigStructName : RegisteredCustomNodeConfigurationDetails)
			{
				UnregisterCustomNodeConfigurationDetailsCustomization(NodeConfigStructName);
			}

			LiteralCustomizationFactories.Reset();

			FGraphNodeVisualizationRegistry::TearDown();
		}

		void OnPackageMigration(UE::AssetTools::FPackageMigrationContext& MigrationContext)
		{
			using namespace Metasound::Frontend;

			// Migration can create temporary new packages that use the same name 
			// (and therefore node registry key) as the asset migrated. 
			// So generate new class names to avoid registry key collisions. 
			if (MigrationContext.GetCurrentStep() == UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InstancedPackagesLoaded)
			{
				// Gather the new MetaSound assets
				TArray<FMetaSoundFrontendDocumentBuilder> NewMetaSoundAssetBuilders;
				for (const UE::AssetTools::FPackageMigrationContext::FMigrationPackageData& MigrationPackageData : MigrationContext.GetMigrationPackagesData())
				{
					UPackage* Package = MigrationPackageData.GetInstancedPackage();
					if (Package)
					{
						UObject* MainAsset = Package->FindAssetInPackage();
						// Only apply to MetaSound assets 
						if (IMetasoundUObjectRegistry::Get().IsRegisteredClass(MainAsset))
						{
							NewMetaSoundAssetBuilders.Add(FMetaSoundFrontendDocumentBuilder(MainAsset));
						}
					}
				}

				// Assign new class names and cache mapping with old one
				IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();
				TMap<FNodeRegistryKey, FNodeRegistryKey> OldToNewReferenceKeys;
				for (FMetaSoundFrontendDocumentBuilder& MetaSoundBuilder : NewMetaSoundAssetBuilders)
				{
					FNodeRegistryKey OldRegistryKey(MetaSoundBuilder.GetConstDocumentChecked().RootGraph);
					FNodeRegistryKey NewRegistryKey(EMetasoundFrontendClassType::External, MetaSoundBuilder.GenerateNewClassName(), OldRegistryKey.Version);
					OldToNewReferenceKeys.FindOrAdd(MoveTemp(OldRegistryKey)) = MoveTemp(NewRegistryKey);

					UObject& MetaSoundObject = MetaSoundBuilder.CastDocumentObjectChecked<UObject>();
					AssetManager.AddOrUpdateFromObject(MetaSoundObject);
				}

				// Fix up dependencies
				for (FMetaSoundFrontendDocumentBuilder& MetaSoundBuilder : NewMetaSoundAssetBuilders)
				{
					MetaSoundBuilder.UpdateDependencyRegistryData(OldToNewReferenceKeys);
				}
			}
		}

		TMap<EMetasoundFrontendLiteralType, const TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral>> InputDefaultLiteralClassRegistry;

		TMap<UClass*, TUniquePtr<IMemberDefaultLiteralCustomizationFactory>> LiteralCustomizationFactories;
		TMap<FName, FCreateNodeConfigurationDetails> CustomNodeConfigurationDetails;

		TSharedPtr<FMetasoundGraphNodeFactory> GraphNodeFactory;
		TSharedPtr<FGraphPanelPinConnectionFactory> GraphConnectionFactory;
		TSharedPtr<FMetaSoundGraphPanelPinFactory> GraphPanelPinFactory;
		TSharedPtr<FSlateStyleSet> StyleSet;

		TSet<const UClass*> ExplicitProxyClasses;
		// Captures the node configuration details registered from this module in 
		// order to track unregistration at module shutdown. It does not capture 
		// all custom registrations;
		TSet<FName> RegisteredCustomNodeConfigurationDetails;

		// Whether or not the editor is in restricted mode: can only make new presets and not modify graphs
		bool bIsRestrictedMode = false;
	};
} // namespace Metasound::Editor

IMPLEMENT_MODULE(Metasound::Editor::FModule, MetasoundEditor);

#undef LOCTEXT_NAMESPACE
