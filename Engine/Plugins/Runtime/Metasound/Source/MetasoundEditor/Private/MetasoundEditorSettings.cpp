// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorSettings.h"

#include "Internationalization/Text.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundSettings.h"
#include "PlatformInfo.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "UObject/UnrealType.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorSettings)

#define LOCTEXT_NAMESPACE "MetasoundEditorSettings"

const FName UMetasoundEditorSettings::DefaultAuditionPlatform = "Default";
const FName UMetasoundEditorSettings::EditorAuditionPlatform = "Editor";

UMetasoundEditorSettings::UMetasoundEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// pin type colors
	DefaultPinTypeColor = FLinearColor(0.750000f, 0.6f, 0.4f, 1.0f);			// light brown

	AudioPinTypeColor = FLinearColor(1.0f, 0.3f, 1.0f, 1.0f);					// magenta
	BooleanPinTypeColor = FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f);			// maroon
	FloatPinTypeColor = FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f);			// bright green
	IntPinTypeColor = FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f);		// green-blue
	ObjectPinTypeColor = FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);				// sharp blue
	StringPinTypeColor = FLinearColor(1.0f, 0.0f, 0.660537f, 1.0f);				// bright pink
	TimePinTypeColor = FLinearColor(0.3f, 1.0f, 1.0f, 1.0f);					// cyan
	TriggerPinTypeColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);					// white
	WaveTablePinTypeColor = FLinearColor(0.580392f, 0.0f, 0.827450f, 1.0f);		// purple

	NativeNodeTitleColor = FLinearColor(0.4f, 0.85f, 0.35f, 1.0f);				// pale green
	InputNodeTitleColor = FLinearColor(0.168f, 1.0f, 0.7294f);					// sea foam
	OutputNodeTitleColor = FLinearColor(1.0f, 0.878f, 0.1686f);					// yellow
	VariableNodeTitleColor = FLinearColor(0.211f, 0.513f, 0.035f);				// copper
}

void UMetasoundEditorSettings::AddRecentlyUsedMetaSoundsGraphAction(FString&& ActionKey)
{
	const int32 ExistingIndex = RecentMetaSoundsGraphActions.Find(ActionKey);
	if (ExistingIndex != INDEX_NONE)
	{
		RecentMetaSoundsGraphActions.RemoveAt(ExistingIndex, EAllowShrinking::No);
	}
	else if (RecentMetaSoundsGraphActions.Num() >= MaxRecentEntries)
	{
		RecentMetaSoundsGraphActions.RemoveAt(0, RecentMetaSoundsGraphActions.Num() - MaxRecentEntries + 1, EAllowShrinking::No);
	}
	RecentMetaSoundsGraphActions.Emplace(ActionKey);

	SaveConfig();
}

int32 UMetasoundEditorSettings::FindRecentlyUsedMetaSoundsGraphAction(const FString& ActionKey) const
{
	return RecentMetaSoundsGraphActions.Find(ActionKey);
}

void UMetasoundEditorSettings::AddFavoriteMetaSoundsGraphAction(FString&& ActionKey)
{
	bool bAlreadyFavorite = false;
	FavoriteMetaSoundsGraphActions.Emplace(ActionKey, &bAlreadyFavorite);
	if (!bAlreadyFavorite)
	{
		SaveConfig();
	}
}

void UMetasoundEditorSettings::RemoveFavoriteMetaSoundsGraphAction(const FString& ActionKey)
{
	if (FavoriteMetaSoundsGraphActions.Remove(ActionKey) > 0)
	{
		SaveConfig();
	}
}

bool UMetasoundEditorSettings::IsFavoriteMetaSoundsGraphAction(const FString& ActionKey) const
{
	return FavoriteMetaSoundsGraphActions.Contains(ActionKey);
}

#if WITH_EDITOR
void UMetasoundEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Update the page override if relevant settings have changed. 
	const FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMetasoundEditorSettings, AuditionPlatform)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMetasoundEditorSettings, AuditionPage))
	{
		if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
		{
			Settings->OverrideTargetPageSettings(GetPageForAudition());
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UMetasoundEditorSettings, MaxRecentEntries))
	{
		MaxRecentEntries = FMath::Min(MaxRecentEntries, MaxRecentEntriesEver);
	}
}

void UMetasoundEditorSettings::SetAuditionPageMode(EAuditionPageMode InMode)
{
	AuditionPageMode = InMode;
	if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
	{
		Settings->OverrideTargetPageSettings(GetPageForAudition());
	}
}

EAuditionPageMode UMetasoundEditorSettings::GetAuditionPageMode() const
{
	return AuditionPageMode;
}

void UMetasoundEditorSettings::SetAuditionPlatform(FName InPlatform)
{
	AuditionPlatform = InPlatform;
	if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
	{
		Settings->OverrideTargetPageSettings(GetPageForAudition());
	}
}

FName UMetasoundEditorSettings::GetAuditionPlatform() const
{
	return AuditionPlatform;
}

void UMetasoundEditorSettings::SetAuditionPage(FName InPage)
{
	AuditionPage = InPage;
	if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
	{
		Settings->OverrideTargetPageSettings(GetPageForAudition());
	}
}

FName UMetasoundEditorSettings::GetAuditionPage() const
{
	return AuditionPage;
}

#endif // WITH_EDITOR

Metasound::Engine::FTargetPageOverride UMetasoundEditorSettings::GetPageForAudition() const
{
	using namespace Metasound;
	using namespace Metasound::Engine;

	FTargetPageOverride AuditionOverride{ .PlatformName = AuditionPlatform, .PageID = Frontend::DefaultPageID };

	if(const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const FMetaSoundPageSettings* AuditionPageSettings = Settings->FindPageSettings(AuditionPage))
		{
			AuditionOverride.PageID = AuditionPageSettings->UniqueId;
		}
	}

	return AuditionOverride;
}

Metasound::Engine::FTargetPageOverride UMetasoundEditorSettings::ResolveAuditionPage(const TArray<FGuid>& InPageIDs) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;

	FTargetPageOverride PreviewInfo { .PlatformName = AuditionPlatform };

	if (AuditionPage == Frontend::DefaultPageName)
	{
		check(InPageIDs.Contains(Frontend::DefaultPageID));
		PreviewInfo.PageID = Frontend::DefaultPageID;
		return PreviewInfo;
	}

	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const FMetaSoundPageSettings* AuditionPageSettings = Settings->FindPageSettings(AuditionPage))
		{
			const FGuid& AuditionPageID = AuditionPageSettings->UniqueId;
			PreviewInfo.PageID = ResolveAuditionPage(InPageIDs, AuditionPageID);
		}
	}
	return PreviewInfo;

}

FGuid UMetasoundEditorSettings::ResolveAuditionPage(const FMetasoundFrontendClassInput& InClassInput, const FGuid& InAuditionPageID) const
{
	TArray<FGuid> PageIDs;
	InClassInput.IterateDefaults([&PageIDs](const FGuid& PageID, const FMetasoundFrontendLiteral&)
	{
		PageIDs.Add(PageID);
	});
	return ResolveAuditionPage(PageIDs, InAuditionPageID);
}

FGuid UMetasoundEditorSettings::ResolveAuditionPage(const TArray<FGuid>& InPageIDs, const FGuid& InAuditionPageID) const
{
	FGuid ResolvedPageID = Metasound::Frontend::DefaultPageID;
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		constexpr bool bReverse = true;
		bool bFoundMatch = false;
		bool bPageSelected = false;
		Settings->IteratePageSettings([&](const FMetaSoundPageSettings& PageSettings)
			{
				if (!bPageSelected)
				{
					bFoundMatch |= PageSettings.UniqueId == InAuditionPageID;
					if (bFoundMatch)
					{
						if (InPageIDs.Contains(PageSettings.UniqueId))
						{
							if (AuditionPlatform == EditorAuditionPlatform || !PageSettings.GetExcludeFromCook(AuditionPlatform))
							{
								bPageSelected = true;
								ResolvedPageID = PageSettings.UniqueId;
							}
						}
					}
				}
			}, bReverse);
	}
	return ResolvedPageID;
}

TArray<FName> UMetasoundEditorSettings::GetAuditionPlatformNames()
{
	TArray<FName> PlatformNames { EditorAuditionPlatform, DefaultAuditionPlatform };
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		PlatformNames.Append(Settings->GetAllPlatformNamesImplementingTargets());
	}
	return PlatformNames;
}

TArray<FName> UMetasoundEditorSettings::GetAuditionPageNames()
{
	const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
	const UMetasoundEditorSettings* EdSettings = GetDefault<UMetasoundEditorSettings>();

	if (Settings && EdSettings)
	{
		TSet<FName> AuditionPageNames;
		if (EdSettings->AuditionPlatform == EditorAuditionPlatform)
		{
			Algo::Transform(Settings->GetProjectPageSettings(), AuditionPageNames, [](const FMetaSoundPageSettings& PageSettings) { return PageSettings.Name; });
			AuditionPageNames.Add(Settings->GetDefaultPageSettings().Name);
		}
		else
		{
			const TArray<FGuid> AuditionPageIDs = Settings->GetCookedTargetPageIDs(EdSettings->AuditionPlatform);
			Algo::Transform(AuditionPageIDs, AuditionPageNames, [&Settings](const FGuid& PageID)
			{
				const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageID);
				if (ensure(PageSettings))
				{
					return PageSettings->Name;
				}

				return FName { };
			});
		}

		return AuditionPageNames.Array();
	}

	return { };
}
#undef LOCTEXT_NAMESPACE // "MetaSoundEditor"
