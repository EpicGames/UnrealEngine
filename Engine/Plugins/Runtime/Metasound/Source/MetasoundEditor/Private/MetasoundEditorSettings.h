// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioSpectrogram.h"
#include "AudioSpectrumAnalyzer.h"
#include "LoudnessMeterRackUnitSettings.h"
#include "MetasoundFrontendDocument.h"
#include "Misc/CoreDefines.h"
#include "ToolMenuMisc.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include "MetasoundEditorSettings.generated.h"

#define UE_API METASOUNDEDITOR_API

// Forward Declarations
class USlateWidgetStyleAsset;
struct FAudioMaterialKnobStyle;
struct FAudioMaterialButtonStyle;
struct FAudioMaterialSliderStyle;
struct FAudioMaterialMeterStyle;
struct FMetasoundFrontendDocument;

namespace Metasound::Engine
{
	struct FTargetPageOverride;
} // namespace Metasound::Engine


UENUM()
enum class EMetasoundActiveAnalyzerEnvelopeDirection : uint8
{
	FromSourceOutput,
	FromDestinationInput
};

UENUM()
enum class EMetasoundMemberDefaultWidget : uint8
{
	None,
	Slider,
	RadialSlider UMETA(DisplayName = "Knob"),
};

UENUM()
enum class EMetasoundBoolMemberDefaultWidget : uint8
{
	None,
	Button
};

UENUM()
enum class EMetasoundActiveDetailView : uint8
{
	Metasound,
	General
};

UENUM()
enum class EAuditionPageMode : uint8
{
	// Sets Audition Page automatically to graph page focused in asset editor
	Focused,

	// Audition Page is specified by user (does not automatically change when graph page is focused)
	User
};

USTRUCT()
struct FMetasoundAnalyzerAnimationSettings
{
	GENERATED_BODY()

	/** Whether or not animated connections are enabled. */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (DisplayName = "Animate Connections (Beta)"))
	bool bAnimateConnections = true;

	/** Thickness of default envelope analyzer wire thickness when connection analyzer is active. */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", UIMin = 1, UIMax = 2, ClampMin = 1))
	float EnvelopeWireThickness = 1.0f;

	/** Speed of default envelope analyzer drawing over wire when connection analyzer is active, where 0 is full visual history (slowest progress) and 1 is no visual history (fastest progress). */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", ClampMin = 0, ClampMax = 1))
	float EnvelopeSpeed = 0.95f;

	/** Whether analyzer envelopes draw from a source output (default) or from the destination input. From the destination input may not
	  * give the expected illusion of audio processing flowing left-to-right, but results in a waveform with earlier events on the left
	  * and later on the right (like a traditional timeline with a moving play head). */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", ClampMin = 0, ClampMax = 1))
	EMetasoundActiveAnalyzerEnvelopeDirection EnvelopeDirection = EMetasoundActiveAnalyzerEnvelopeDirection::FromSourceOutput;

	/** Thickness of default numeric analyzer wire thickness when connection analyzer is active. */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", UIMin = 1, UIMax = 10, ClampMin = 1))
	float NumericWireThickness = 5.0f;

	/** Minimum height scalar of wire signal analyzers (ex. audio, triggers). */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", UIMin = 1, UIMax = 5, ClampMin = 1))
	float WireScalarMin = 1.0f;

	/** Maximum height scalar of wire signal analyzers (ex. audio, triggers). */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (EditCondition = "bAnimateConnections", UIMin = 1, UIMax = 5, ClampMin = 1))
	float WireScalarMax = 4.5f;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UMetasoundEditorSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	// Represents auditioning any platform using the default target/cook settings
	static UE_API const FName DefaultAuditionPlatform;

	// Represents auditioning as the editor, ignoring any explicit target/cook settings
	static UE_API const FName EditorAuditionPlatform;

	/** Whether to pin the MetaSound Patch asset type when creating new assets.
	  * Requires editor restart for change to take effect.*/
	UPROPERTY(EditAnywhere, config, meta = (ConfigRestartRequired = true), Category = AssetMenu)
	bool bPinMetaSoundPatchInAssetMenu = false;

	/** Whether to pin the MetaSound Source asset type when creating new assets. 
	  * Requires editor restart for change to take effect.*/
	UPROPERTY(EditAnywhere, config, meta = (ConfigRestartRequired = true), Category = AssetMenu)
	bool bPinMetaSoundSourceInAssetMenu = true;

	/** If true, uses editor page/platform audition settings in PIE. If false, uses project's defined values
	  * (see project 'MetaSound' setting 'TargetPage', which can be manipulated via code/Blueprint.)
	  */
	UPROPERTY(EditAnywhere, config, Category = "Audition (Experimental)")
	bool bApplyAuditionSettingsInPIE = true;

	/** Default author title to use when authoring a new
	  * MetaSound.  If empty, uses machine name by default.
	  */
	UPROPERTY(EditAnywhere, config, Category = General)
	FString DefaultAuthor;

private:
	/* Currently set page audition mode. Set by the MetaSound Asset Editor. */
	UPROPERTY(EditAnywhere, config, Category = "Audition (Experimental)", meta = (DisplayName = "Page Audition Mode"))
	EAuditionPageMode AuditionPageMode = EAuditionPageMode::Focused;

	/** Name of platform to mock when previewing playback. This will limit playback
	  * to fallback only to paged data that are cooked for the given platform.
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order)
	  * If set to 'Editor', ignores cook settings and allows fallback to all page.
	  */
	UPROPERTY(EditAnywhere, config, Category = "Audition (Experimental)", meta = (DisplayName = "Page Audition Platform", GetOptions = "MetasoundEditor.MetasoundEditorSettings.GetAuditionPlatformNames"))
	FName AuditionPlatform = EditorAuditionPlatform;

	/** Name of the page to audition in editor. If unimplemented on the auditioned MetaSound, uses order of cooked pages
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order) falling back to lower index-ordered page implemented
	  * in MetaSound asset.
	  */
	UPROPERTY(EditAnywhere, config, Category = "Audition (Experimental)", meta =
	(
		EditCondition = "AuditionPageMode == EAuditionPageMode::User",
		EditConditionHides = true,
		GetOptions = "MetasoundEditor.MetasoundEditorSettings.GetAuditionPageNames")
	)
	FName AuditionPage = Metasound::Frontend::DefaultPageName;
public:
#if WITH_EDITOR
	/* Currently set page audition mode. */
	UE_API void SetAuditionPageMode(EAuditionPageMode InMode);
	UE_API EAuditionPageMode GetAuditionPageMode() const;

	/** Name of platform to mock when previewing playback. This will limit playback
	  * to fallback only to paged data that are cooked for the given platform.
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order)
	  * If set to 'Editor', ignores cook settings and allows fallback to all page.
	  */
	UE_API void SetAuditionPlatform(FName InPlatform);
	
	/** Name of platform to mock when previewing playback. This will limit playback
	  * to fallback only to paged data that are cooked for the given platform.
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order)
	  * If set to 'Editor', ignores cook settings and allows fallback to all page.
	  */
	UE_API FName GetAuditionPlatform() const;
	
	/** Name of the page to audition in editor. If unimplemented on the auditioned MetaSound, uses order of cooked pages
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order) falling back to lower index-ordered page implemented
	  * in MetaSound asset.
	  */
	UE_API void SetAuditionPage(FName InPage);
	
	/** Name of the page to audition in editor. If unimplemented on the auditioned MetaSound, uses order of cooked pages
	  * (see project 'MetaSound' Settings --> 'Page Settings' array for order) falling back to lower index-ordered page implemented
	  * in MetaSound asset.
	  */
	UE_API FName GetAuditionPage() const;
#endif


	UE_DEPRECATED(5.8, "Custom pin colors are now managed by FMetaSoundGraphPanelPinFactory. Use IMetaSoundGraphPanelPinFactory::GetPinColor instead.")
	TMap<FName, FLinearColor> CustomPinTypeColors;
	
	/** Default pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor DefaultPinTypeColor;

	/** Audio pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor AudioPinTypeColor;

	/** Boolean pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor BooleanPinTypeColor;

	/** Floating-point pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor FloatPinTypeColor;

	/** Integer pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor IntPinTypeColor;

	/** Object pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor ObjectPinTypeColor;

	/** String pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor StringPinTypeColor;

	/** Time pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor TimePinTypeColor;

	/** Trigger pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor TriggerPinTypeColor;

	/** WaveTable pin type color */
	UPROPERTY(EditAnywhere, config, Category = PinColors)
	FLinearColor WaveTablePinTypeColor;

	/** Native node class title color */
	UPROPERTY(EditAnywhere, config, Category = NodeTitleColors)
	FLinearColor NativeNodeTitleColor;

	/** Title color for references to MetaSound assets */
	UE_DEPRECATED(5.8, "Asset nodes now use their corresponding asset type color.")
	FLinearColor AssetReferenceNodeTitleColor;

	/** Input node title color */
	UPROPERTY(EditAnywhere, config, Category = NodeTitleColors)
	FLinearColor InputNodeTitleColor;

	/** Output node title color */
	UPROPERTY(EditAnywhere, config, Category = NodeTitleColors)
	FLinearColor OutputNodeTitleColor;

	/** Variable node title color */
	UPROPERTY(EditAnywhere, config, Category = NodeTitleColors)
	FLinearColor VariableNodeTitleColor;

	/** Settings for metasound output spectrogram widget */
	UPROPERTY(EditAnywhere, config, Category = Spectrogram, meta = (ShowOnlyInnerProperties))
	FSpectrogramRackUnitSettings SpectrogramSettings;

	/** Settings for metasound output spectrum analyzer widget */
	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer, meta = (ShowOnlyInnerProperties))
	FSpectrumAnalyzerRackUnitSettings SpectrumAnalyzerSettings;

	/** Settings for metasound output loudness meter widget */
	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ShowOnlyInnerProperties))
	FLoudnessMeterRackUnitSettings LoudnessMeterSettings;

	/** Widget type to show on input nodes by default */
	UPROPERTY(EditAnywhere, config, Category = General)
	EMetasoundMemberDefaultWidget DefaultInputWidgetType = EMetasoundMemberDefaultWidget::RadialSlider;

	/** Settings for visualizing analyzed MetaSound connections */
	UPROPERTY(EditAnywhere, config, Category = GraphAnimation, meta = (ShowOnlyInnerProperties))
	FMetasoundAnalyzerAnimationSettings AnalyzerAnimationSettings;

	/** Determines which details view to show in Metasounds Editor */
	UPROPERTY(Transient)
	EMetasoundActiveDetailView DetailView = EMetasoundActiveDetailView::General;

	/** Whether the AudioMaterialWidgets are used when possible in Metasound Editor*/
	UPROPERTY(EditAnywhere, config, DisplayName = "Use Audio Material Widgets", Category = "Widget Styling (Experimental)")
	bool bUseAudioMaterialWidgets = false;
	
	/**Override the Knob Style used in the Metasound Editor.*/
	UPROPERTY(EditAnywhere, config, Category = "Widget Styling (Experimental)", meta = (AllowedClasses = "/Script/SlateCore.SlateWidgetStyleAsset", EditCondition = "bUseAudioMaterialWidgets", DisplayName = "Knob Style"))
	FSoftObjectPath KnobStyleOverride;
	
	/**Override the Slider Style used in the Metasound Editor.*/
	UPROPERTY(EditAnywhere, config, Category = "Widget Styling (Experimental)", meta = (AllowedClasses = "/Script/SlateCore.SlateWidgetStyleAsset", EditCondition = "bUseAudioMaterialWidgets", DisplayName = "Slider Style"))
	FSoftObjectPath SliderStyleOverride;

	/**Override the Button Style used in the Metasound Editor.*/
	UPROPERTY(EditAnywhere, config, Category = "Widget Styling (Experimental)", meta = (AllowedClasses = "/Script/SlateCore.SlateWidgetStyleAsset", EditCondition = "bUseAudioMaterialWidgets", DisplayName = "Button Style"))
	FSoftObjectPath ButtonStyleOverride;
	
	/**Override the Meter Style used in the Metasound Editor.*/
	UPROPERTY(EditAnywhere, config, Category = "Widget Styling (Experimental)", meta = (AllowedClasses = "/Script/SlateCore.SlateWidgetStyleAsset", EditCondition = "bUseAudioMaterialWidgets", DisplayName = "Meter Style"))
	FSoftObjectPath MeterStyleOverride;

	/** MaxRecentEntries can go up to this number (limit must be known, for ordering Common actions in graph action menu above Recent ones) */
	static constexpr int32 MaxRecentEntriesEver = 200;

	/** The maximum number of Recent entries kept in the Metasound Editor action menu (ie., graph context menu). Older entries are overwritten by
	  * newer entries, and reusing one put it back at the top.
	  */
	UPROPERTY(EditAnywhere, config, Category = General, meta = (ClampMin = 1))
	int32 MaxRecentEntries = 20;

private:
	/** Higher index means more recent */
	UPROPERTY(config)
	TArray<FString> RecentMetaSoundsGraphActions;
	
	UPROPERTY(config)
	TSet<FString> FavoriteMetaSoundsGraphActions; // TODO: Implement profiles, like for blueprint favorites

public:
	void AddRecentlyUsedMetaSoundsGraphAction(FString&& ActionKey);
	int32 FindRecentlyUsedMetaSoundsGraphAction(const FString& ActionKey) const;
	void AddFavoriteMetaSoundsGraphAction(FString&& ActionKey);
	void RemoveFavoriteMetaSoundsGraphAction(const FString& ActionKey);
	bool IsFavoriteMetaSoundsGraphAction(const FString& ActionKey) const;
	
	//UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~UObject
	
	/** Returns the page override data derived from the AuditionPlatform and AuditionPage */
	UE_API Metasound::Engine::FTargetPageOverride GetPageForAudition() const;

	UE_API Metasound::Engine::FTargetPageOverride ResolveAuditionPage(const TArray<FGuid>& InPageIDs) const;
	
	/** Given the provided AuditionPageID, returns the resolved PageID from the provided array of values based on fallback logic. */
	UE_API FGuid ResolveAuditionPage(const TArray<FGuid>& InPageIDs, const FGuid& InAuditionPageID) const;

	/** Given the provided class input and AuditionPageID, returns the resolved PageID from the provided array of values based on fallback logic. */
	UE_API FGuid ResolveAuditionPage(const FMetasoundFrontendClassInput& InClassInput, const FGuid& InAuditionPageID) const;

	UFUNCTION()
	static UE_API TArray<FName> GetAuditionPageNames();

	UFUNCTION()
	static UE_API TArray<FName> GetAuditionPlatformNames();
};

#undef UE_API
