// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundEditorSubsystem.h"

#include "AudioPropertiesSheetAssetBase.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "Kismet/KismetSystemLibrary.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorBuilderListener.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFactory.h"
#include "MetasoundSettings.h"
#include "MetasoundUObjectRegistry.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "ScopedTransaction.h"
#include "Sound/SoundSourceBusSend.h"
#include "Sound/SoundSubmixSend.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundEditorSubsystem)

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


TScriptInterface<IMetaSoundDocumentInterface> UMetaSoundEditorSubsystem::BuildToAsset(
	UMetaSoundBuilderBase* InBuilder,
	const FString& Author,
	const FString& AssetName,
	const FString& PackagePath,
	EMetaSoundBuilderResult& OutResult,
	const USoundWave* TemplateSoundWave
)
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	OutResult = EMetaSoundBuilderResult::Failed;

	if (InBuilder)
	{
		// AddToRoot to avoid builder getting gc'ed during CreateAsset call below, as the builder 
		// may be unreferenced by other UObjects and it must be persistent to finish initializing.
		const bool bWasRooted = InBuilder->IsRooted();
		if (!bWasRooted)
		{
			InBuilder->AddToRoot();
		}

		// Not about to follow this lack of const correctness down a multidecade in the works rabbit hole.
		UClass& MetaSoundUClass = const_cast<UClass&>(InBuilder->GetBaseMetaSoundUClass());
		
		// Duplicate referenced preset object to preserve object properties (ex. quality settings, soundwave properties)
		UObject* NewMetaSound = nullptr;
		if (const FPresetTemplate* PresetTemplate = InBuilder->GetConstBuilder().GetConstTemplateAs<FPresetTemplate>())
		{
			NewMetaSound = IAssetTools::Get().DuplicateAsset(AssetName, PackagePath, PresetTemplate->Parent.GetObject());
		}
		else
		{
			constexpr UFactory* Factory = nullptr;
			NewMetaSound = IAssetTools::Get().CreateAsset(AssetName, PackagePath, &MetaSoundUClass, Factory);
		}

		if (NewMetaSound)
		{
			InBuilder->InitNodeLocations();
			InBuilder->SetAuthor(Author);

			// Initialize and Build
			{
				constexpr bool bForceUniqueClassName = true;
				constexpr bool bAddToRegistry = true;
				const FMetaSoundBuilderOptions BuilderOptions { FName(*AssetName), bForceUniqueClassName, bAddToRegistry, NewMetaSound };
				InBuilder->Build(BuilderOptions);
			}

			UMetaSoundBuilderBase& NewDocBuilder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*NewMetaSound);

			EMetaSoundBuilderResult InjectResult = EMetaSoundBuilderResult::Failed;
			constexpr bool bForceNodeCreation = true;
			NewDocBuilder.InjectInputTemplateNodes(bForceNodeCreation, InjectResult);

			FMetasoundAssetBase& Asset = NewDocBuilder.GetBuilder().GetMetasoundAsset();
			Asset.RebuildReferencedAssetClasses();

			// Apply template SoundWave settings 
			{
				// Template SoundWave settings only apply to sources
				// and will override settings from a template SoundWave's values
				const bool bIsSound = MetaSoundUClass.IsChildOf(USoundWave::StaticClass());
				if (TemplateSoundWave && bIsSound)
				{
					SetSoundWaveSettingsFromTemplate(*CastChecked<USoundWave>(NewMetaSound), *TemplateSoundWave);
				}
			}

			// Save happened in Create/Duplicate asset calls above but
			// need to save again after building (which can apply preset transform)
			if (ISourceControlModule::Get().IsEnabled())
			{
				constexpr bool bCheckDirty = false;
				constexpr bool bPromptToSave = false;
				TArray<UPackage*> OutermostPackagesToSave;
				OutermostPackagesToSave.Add(NewMetaSound->GetOutermost());
				FEditorFileUtils::PromptForCheckoutAndSave(OutermostPackagesToSave, bCheckDirty, bPromptToSave);
			}

			if (!bWasRooted)
			{
				InBuilder->RemoveFromRoot();
			}

			OutResult = EMetaSoundBuilderResult::Succeeded;
			return NewMetaSound;
		}

		if (!bWasRooted)
		{
			InBuilder->RemoveFromRoot();
		}
	}

	return nullptr;
}

UMetasoundEditorGraphMemberDefaultLiteral* UMetaSoundEditorSubsystem::CreateMemberMetadata(
	FMetaSoundFrontendDocumentBuilder& Builder,
	FName InMemberName,
	TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass) const
{
	return CreateMemberMetadata(Builder, LiteralClass);
}

UMetasoundEditorGraphMemberDefaultLiteral* UMetaSoundEditorSubsystem::CreateMemberMetadata(
	FMetaSoundFrontendDocumentBuilder& Builder,
	TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass) const
{
	return NewObject<UMetasoundEditorGraphMemberDefaultLiteral>(&Builder.CastDocumentObjectChecked<UObject>(), LiteralClass, FName(), RF_Transactional, nullptr);
}

bool UMetaSoundEditorSubsystem::BindMemberMetadata(
	FMetaSoundFrontendDocumentBuilder& Builder,
	UMetasoundEditorGraphMember& InMember,
	TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass,
	UMetasoundEditorGraphMemberDefaultLiteral* TemplateObject)
{
	UMetasoundEditorGraphMemberDefaultLiteral* NewLiteral = nullptr;
	const FGuid& MemberID = InMember.GetMemberID();

	if (TemplateObject)
	{
		Builder.ClearMemberMetadata(MemberID);
		NewLiteral = NewObject<UMetasoundEditorGraphMemberDefaultLiteral>(&Builder.CastDocumentObjectChecked<UObject>(), LiteralClass, FName(), RF_Transactional, TemplateObject);
	}
	else
	{
		if (UMetaSoundFrontendMemberMetadata* Literal = Builder.FindMemberMetadata(MemberID))
		{
			const UClass* ExistingClass = Literal->GetClass();
			check(ExistingClass);
			if (ExistingClass->IsChildOf(LiteralClass))
			{
				InMember.Literal = CastChecked<UMetasoundEditorGraphMemberDefaultLiteral>(Literal);
				return false;
			}
		}

		NewLiteral = CreateMemberMetadata(Builder, LiteralClass);
	}

	if (NewLiteral)
	{
		NewLiteral->MemberID = MemberID;

		Builder.SetMemberMetadata(*NewLiteral);
		InMember.Literal = NewLiteral;
		return true;
	}

	checkNoEntry();
	return false;
}

UMetaSoundBuilderBase* UMetaSoundEditorSubsystem::FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface> MetaSound, EMetaSoundBuilderResult& OutResult) const
{
	using namespace Metasound::Engine;

	if (UObject* Object = MetaSound.GetObject(); Object && Object->IsAsset())
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return &FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*Object);
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return nullptr;
}

UMetaSoundFrontendMemberMetadata* UMetaSoundEditorSubsystem::FindOrCreateGraphInputMetadata(UMetaSoundBuilderBase* InBuilder, FName InputName, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend; 
	
	if (InBuilder)
	{
		FMetaSoundNodeHandle GraphInputNodeHandle = InBuilder->FindGraphInputNode(InputName, OutResult);
		if (OutResult == EMetaSoundBuilderResult::Succeeded)
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = InBuilder->GetBuilder();
			// Look for existing metadata
			if (UMetaSoundFrontendMemberMetadata* MemberMetadata = DocBuilder.FindMemberMetadata(GraphInputNodeHandle.NodeID))
			{
				OutResult = EMetaSoundBuilderResult::Succeeded;
				return MemberMetadata;
			}
			// Create new metadata
			else
			{
				// Get literal class
				const FName TypeName = DocBuilder.FindGraphInput(InputName)->TypeName;
				TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass = GetLiteralClassForType(TypeName);

				// Create new literal and setup
				UMetasoundEditorGraphMemberDefaultLiteral* NewLiteral = CreateMemberMetadata(DocBuilder, LiteralClass);
				if (NewLiteral)
				{
					NewLiteral->MemberID = GraphInputNodeHandle.NodeID;
					NewLiteral->Initialize();
					DocBuilder.SetMemberMetadata(*NewLiteral);
					
					OutResult = EMetaSoundBuilderResult::Succeeded;
					return NewLiteral;
				}
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Display, "Failed to find graph input node for input '%ls' with builder '%ls'.", *InputName.ToString(), *InBuilder->GetName());
		}
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return nullptr;
}

FName UMetaSoundEditorSubsystem::GetBuildPage(UMetaSoundBuilderBase* Builder) const
{
	if (Builder)
	{
		const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
		const FGuid& BuildPageID = DocBuilder.GetBuildPageID();
		return UMetaSoundSettings::GetPageName(BuildPageID);
	}
	return FName();
}

TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> UMetaSoundEditorSubsystem::GetLiteralClassForType(FName TypeName) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	// Get literal class
	FDataTypeRegistryInfo DataTypeInfo;
	IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	IDataTypeRegistry::Get().GetDataTypeInfo(TypeName, DataTypeInfo);
	const EMetasoundFrontendLiteralType LiteralType = static_cast<EMetasoundFrontendLiteralType>(DataTypeInfo.PreferredLiteralType);

	TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass = EditorModule.FindDefaultLiteralClass(LiteralType);
	if (!LiteralClass)
	{
		LiteralClass = UMetasoundEditorGraphMemberDefaultLiteral::StaticClass();
	}
	return LiteralClass;
}

UMetaSoundEditorSubsystem& UMetaSoundEditorSubsystem::GetChecked()
{
	checkf(GEditor, TEXT("Cannot access UMetaSoundEditorSubsystem without editor loaded"));
	UMetaSoundEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>();
	checkf(EditorSubsystem, TEXT("Failed to find initialized 'UMetaSoundEditorSubsystem"));
	return *EditorSubsystem;
}

const UMetaSoundEditorSubsystem& UMetaSoundEditorSubsystem::GetConstChecked()
{
	checkf(GEditor, TEXT("Cannot access UMetaSoundEditorSubsystem without editor loaded"));
	UMetaSoundEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>();
	checkf(EditorSubsystem, TEXT("Failed to find initialized 'UMetaSoundEditorSubsystem"));
	return *EditorSubsystem;
}

const FString UMetaSoundEditorSubsystem::GetDefaultAuthor()
{
	FString Author = UKismetSystemLibrary::GetPlatformUserName();
	if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
	{
		if (!EditorSettings->DefaultAuthor.IsEmpty())
		{
			Author = EditorSettings->DefaultAuthor;
		}
	}
	return Author;
}

const TArray<TSharedRef<FExtender>>& UMetaSoundEditorSubsystem::GetToolbarExtenders() const
{
	return EditorToolbarExtenders;
}

void UMetaSoundEditorSubsystem::InitAsset(UObject& InNewMetaSound, UObject* InParentMetaSound, const bool bClearDocument)
{
	using namespace Metasound::Editor;

	TInstancedStruct<FMetaSoundFrontendDocumentTemplate> PresetTemplate = TInstancedStruct<FMetaSoundFrontendDocumentTemplate>::Make<FMetaSoundFrontendPresetTemplate>();
	PresetTemplate.GetMutable<FMetaSoundFrontendPresetTemplate>().Parent = InParentMetaSound;
	InitAsset(InNewMetaSound, FInitAssetArgs
	{
		.Template = MoveTemp(PresetTemplate),
		.SelectedObjects = { InParentMetaSound }
	});

	Metasound::Editor::IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<Metasound::Editor::IMetasoundEditorModule>(Metasound::Editor::IMetasoundEditorModule::ModuleName);
	if (MetaSoundEditorModule.IsRestrictedMode())
	{
		if (USoundWave* MetaSoundWave = Cast<USoundWave>(&InNewMetaSound))
		{
			if (USoundWave* ParentWave = Cast<USoundWave>(InParentMetaSound))
			{
				FGraphBuilder::RegisterGraphWithFrontend(InNewMetaSound);
				SetSoundWaveSettingsFromTemplate(*MetaSoundWave, *ParentWave);
			}
		}
	}
}

void UMetaSoundEditorSubsystem::InitAsset(UObject& InNewMetaSound, UMetaSoundEditorSubsystem::FInitAssetArgs InitAssetArgs)
{
	using namespace Metasound;
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;

	{
		TScriptInterface<IMetaSoundDocumentInterface> DocInterface = &InNewMetaSound;

		// Must force 'FinishBuilding' and use local builder as re-initializing document via
		// 'InitDocument' call below can lead to stale builder entries in the registry when renaming.
		const FMetasoundFrontendClassName InitClassName = DocInterface->GetConstDocument().RootGraph.Metadata.GetClassName();
		const FTopLevelAssetPath AssetPath = DocInterface->GetAssetPathChecked();
		IDocumentBuilderRegistry::GetChecked().FinishBuilding(InitClassName, AssetPath);

		FMetaSoundFrontendDocumentBuilder DocBuilder(DocInterface);

		TSharedRef<FDocumentModifyDelegates> ModifyDelegates = MakeShared<FDocumentModifyDelegates>(DocBuilder.GetConstDocumentChecked());
		DocBuilder.ClearDocument(ModifyDelegates);

		DocBuilder.InitDocument();
		DocBuilder.InitNodeLocations();

		constexpr bool bForceNodeCreation = true;
		FInputNodeTemplate::GetChecked().Inject(DocBuilder, bForceNodeCreation);

		const FString& Author = GetDefaultAuthor();
		DocBuilder.SetAuthor(Author);
	}

	// Must be called in new context after asset is initialized as blank document as configurations use a registered builder
	if (FMetaSoundFrontendDocumentTemplate* DocTemplate = InitAssetArgs.Template.GetMutablePtr())
	{
		FMetaSoundFrontendDocumentBuilder& DocBuilder = Frontend::IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(&InNewMetaSound);
		DocBuilder.SetTemplate(MoveTemp(InitAssetArgs.Template));
		DocTemplate->OnAssetInitialized(MoveTemp(InitAssetArgs.SelectedObjects), DocBuilder);
		DocBuilder.ConfigureDocument();
	}
}

bool UMetaSoundEditorSubsystem::IsPageAuditionPlatformCookTarget(FName InPageName) const
{
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(InPageName))
		{
			return IsPageAuditionPlatformCookTarget(PageSettings->UniqueId);
		}
	}

	return false;
}

bool UMetaSoundEditorSubsystem::IsPageAuditionPlatformCookTarget(const FGuid& InPageID) const
{
#if WITH_EDITORONLY_DATA
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
		{
			bool bIsAuditionable = false;
			auto PageIsTargetable = [&InPageID, &bIsAuditionable](const FGuid& PlatformTargetPageID)
			{
				bIsAuditionable |= PlatformTargetPageID == InPageID;
			};
			Settings->IterateCookedTargetPageIDs(EditorSettings->GetAuditionPlatform(), PageIsTargetable);
			return bIsAuditionable;
		}
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}

void UMetaSoundEditorSubsystem::RegisterGraphWithFrontend(UObject& InMetaSound, bool bInForceViewSynchronization) const
{
	using namespace Metasound::Editor;
	using namespace Metasound::Frontend;
	
	FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
	RegOptions.bForceViewSynchronization = bInForceViewSynchronization;
	FGraphBuilder::RegisterGraphWithFrontend(InMetaSound, MoveTemp(RegOptions));
}

void UMetaSoundEditorSubsystem::RegisterToolbarExtender(TSharedRef<FExtender> InExtender)
{
	EditorToolbarExtenders.AddUnique(InExtender);
}

UMetaSoundEditorBuilderListener* UMetaSoundEditorSubsystem::AddBuilderDelegateListener(UMetaSoundBuilderBase* InBuilder, EMetaSoundBuilderResult& OutResult)
{
	if (InBuilder)
	{
		UMetaSoundEditorBuilderListener* Listener = NewObject<UMetaSoundEditorBuilderListener>(GetTransientPackage(), FName(), RF_Public | RF_Transient);
		Listener->Init(InBuilder);
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return Listener;
	}

	UE_LOGF(LogMetaSound, Warning, "Add Builder Delegate Listener called with invalid builder, listener will not be created.");
	OutResult = EMetaSoundBuilderResult::Failed;
	return nullptr;
}

void UMetaSoundEditorSubsystem::SetBuildPage(UMetaSoundBuilderBase* Builder, FName PageName, EMetaSoundBuilderResult& OutResult) const
{
	if (!Builder)
	{
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}
	
	// For now, the focused page and build page are the same,
	// but in the future they may be different (ex. if the editor supports multiple tabs for editing different pages at once)
	// Only set the page, don't open the editor or change the audition page
	const FGuid* PageID = UMetaSoundSettings::GetPageID(PageName);
	if (!PageID)
	{
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}
	
	constexpr bool bOpenEditor = false;
	constexpr bool bPostTransaction = true;
	constexpr bool bSetAuditionPage = false;
	if (SetFocusedPageInternal(PageName, *PageID, *Builder, bOpenEditor, bPostTransaction, bSetAuditionPage))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
}

void UMetaSoundEditorSubsystem::SetFocusedPage(UMetaSoundBuilderBase* Builder, FName PageName, bool bOpenEditor, EMetaSoundBuilderResult& OutResult) const
{
	using namespace Metasound::Frontend;
	if (!Builder)
	{
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}

	const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
	check(Settings);
	if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageName))
	{
		constexpr bool bPostTransaction = true;
		constexpr bool bSetAuditionPage = true;
		const bool bFocusedPage = SetFocusedPageInternal(PageSettings->Name, PageSettings->UniqueId, *Builder, bOpenEditor, bPostTransaction, bSetAuditionPage);
		if (bFocusedPage)
		{
			OutResult = EMetaSoundBuilderResult::Succeeded;
			return;
		}
	}

	OutResult = EMetaSoundBuilderResult::Failed;
}

bool UMetaSoundEditorSubsystem::SetFocusedPage(UMetaSoundBuilderBase& Builder, const FGuid& InPageID, bool bOpenEditor, bool bPostTransaction) const
{
	using namespace Metasound::Frontend;

	const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
	check(Settings);

	FName PageName;
	if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(InPageID))
	{
		PageName = PageSettings->Name;
	}

	return SetFocusedPageInternal(PageName, InPageID, Builder, bOpenEditor, bPostTransaction);
}

bool UMetaSoundEditorSubsystem::SetFocusedPageInternal(FName PageName, const FGuid& InPageID, UMetaSoundBuilderBase& Builder, bool bOpenEditor, bool bPostTransaction, bool bSetAuditionPage) const
{
	using namespace Metasound::Frontend;

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("SetFocusedPageTransactionFormat", "Set Focused Page '{0}'"), FText::FromName(PageName)), bPostTransaction);
	bool bSuccess = false;
	// Must set audition target page before setting build page ID as listeners
	// to build page ID changes need to reliably be able to adjust to newly assigned
	// audition target page.
	if (bSetAuditionPage)
	{
		UMetasoundEditorSettings* EditorSettings = GetMutableDefault<UMetasoundEditorSettings>();
		check(EditorSettings);
		if (EditorSettings->GetAuditionPageMode() == EAuditionPageMode::Focused)
		{
			if (EditorSettings->GetAuditionPage() != PageName)
			{
				EditorSettings->Modify();
				EditorSettings->SetAuditionPage(PageName);
			}
		}
	}
	
	const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetConstBuilder();
	if (DocBuilder.GetBuildPageID() != InPageID)
	{
		Builder.Modify();
		UObject& MetaSound = DocBuilder.CastDocumentObjectChecked<UObject>();
		MetaSound.Modify();
		// Ed graph may be invalid if MetaSound has not been opened yet
		if (UEdGraph* EdGraph = DocBuilder.GetMetasoundAsset().GetGraph())
		{
			EdGraph->Modify();
		}
		if (Builder.GetBuilder().SetBuildPageID(InPageID))
		{
			// Reregister to ensure all future audible instances are using the new page implementation.
			RegisterGraphWithFrontend(MetaSound);
			bSuccess = true;
		}

		if (GEditor && bOpenEditor)
		{
			if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSubsystem->OpenEditorForAsset(&MetaSound);
			}
		}
	}

	return bSuccess;
}

bool UMetaSoundEditorSubsystem::UnregisterToolbarExtender(TSharedRef<FExtender> InExtender)
{
	const int32 NumRemoved = EditorToolbarExtenders.RemoveAllSwap([&InExtender](const TSharedRef<FExtender>& Extender) { return Extender == InExtender; });
	return NumRemoved > 0;
}

void UMetaSoundEditorSubsystem::SetNodeLocation(
	UMetaSoundBuilderBase* InBuilder,
	const FMetaSoundNodeHandle& InNode,
	const FVector2D& InLocation,
	EMetaSoundBuilderResult& OutResult)
{
	if (InBuilder)
	{
		InBuilder->SetNodeLocation(InNode, InLocation, OutResult);
	}
	else
	{
		OutResult = EMetaSoundBuilderResult::Failed;
	}
}

void UMetaSoundEditorSubsystem::SetSoundWaveSettingsFromTemplate(USoundWave& NewMetaSoundWave, const USoundWave& TemplateSoundWave) const
{
	// Sound 
	NewMetaSoundWave.Volume = TemplateSoundWave.Volume;
	NewMetaSoundWave.Pitch = TemplateSoundWave.Pitch;
	NewMetaSoundWave.SoundClassObject = TemplateSoundWave.SoundClassObject;

	// Attenuation 
	NewMetaSoundWave.AttenuationSettings = TemplateSoundWave.AttenuationSettings;
	NewMetaSoundWave.bDebug = TemplateSoundWave.bDebug;

	// Effects 
	NewMetaSoundWave.bEnableBusSends = TemplateSoundWave.bEnableBusSends;
	NewMetaSoundWave.SourceEffectChain = TemplateSoundWave.SourceEffectChain;
	NewMetaSoundWave.BusSends = TemplateSoundWave.BusSends;
	NewMetaSoundWave.PreEffectBusSends = TemplateSoundWave.PreEffectBusSends;

	NewMetaSoundWave.bEnableBaseSubmix = TemplateSoundWave.bEnableBaseSubmix;
	NewMetaSoundWave.SoundSubmixObject = TemplateSoundWave.SoundSubmixObject;
	NewMetaSoundWave.bEnableSubmixSends = TemplateSoundWave.bEnableSubmixSends;
	NewMetaSoundWave.SoundSubmixSends = TemplateSoundWave.SoundSubmixSends;

	// Modulation 
	NewMetaSoundWave.ModulationSettings = TemplateSoundWave.ModulationSettings;

	// Voice Management 
	NewMetaSoundWave.VirtualizationMode = TemplateSoundWave.VirtualizationMode;
	NewMetaSoundWave.bOverrideConcurrency = TemplateSoundWave.bOverrideConcurrency;
	NewMetaSoundWave.ConcurrencySet = TemplateSoundWave.ConcurrencySet;
	NewMetaSoundWave.ConcurrencyOverrides = TemplateSoundWave.ConcurrencyOverrides;

	NewMetaSoundWave.bBypassVolumeScaleForPriority = TemplateSoundWave.bBypassVolumeScaleForPriority;
	NewMetaSoundWave.Priority = TemplateSoundWave.Priority;

	//Property Sheets - keep this last so that properties in the sheet will be applied
	NewMetaSoundWave.AudioPropertiesSheet = TemplateSoundWave.AudioPropertiesSheet;

	if (NewMetaSoundWave.AudioPropertiesSheet)
	{
		NewMetaSoundWave.AudioPropertiesSheet->CopyToObjectProperties(&NewMetaSoundWave);
	}

	return;
}

#undef LOCTEXT_NAMESPACE // "MetaSoundEditor"
