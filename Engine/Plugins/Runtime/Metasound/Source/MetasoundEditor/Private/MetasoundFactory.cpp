// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFactory.h"

#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "Metasound.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundSource.h"
#include "UObject/ScriptInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFactory)


UMetaSoundBaseFactory::UMetaSoundBaseFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UMetaSoundFactory::UMetaSoundFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMetaSoundPatch::StaticClass();
}

UObject* UMetaSoundFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext)
{
	using namespace Metasound::Editor;

	UMetaSoundPatch* NewPatch = NewObject<UMetaSoundPatch>(InParent, InName, InFlags);
	check(NewPatch);

	UMetaSoundEditorSubsystem::GetChecked().InitAsset(*NewPatch, UMetaSoundEditorSubsystem::FInitAssetArgs
	{
		.Template = Template,
		.SelectedObjects = SelectedObjects
	});

	FGraphBuilder::RegisterGraphWithFrontend(*NewPatch);
	return NewPatch;
}

UMetaSoundSourceFactory::UMetaSoundSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMetaSoundSource::StaticClass();
}

UObject* UMetaSoundSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext)
{
	using namespace Metasound::Editor;

	UMetaSoundSource* NewSource = NewObject<UMetaSoundSource>(InParent, InName, InFlags);
	check(NewSource);

	UMetaSoundEditorSubsystem::GetChecked().InitAsset(*NewSource, UMetaSoundEditorSubsystem::FInitAssetArgs
	{
		.Template = Template,
		.SelectedObjects = SelectedObjects
	});


	const IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	if (MetaSoundEditorModule.IsRestrictedMode())
	{
		if (const FMetaSoundFrontendPresetTemplate* PresetTemplate = Template.GetPtr<FMetaSoundFrontendPresetTemplate>())
		{
			if (const USoundWave* ParentWave = Cast<const USoundWave>(PresetTemplate->Parent.GetObject()))
			{
				UMetaSoundEditorSubsystem::GetChecked().SetSoundWaveSettingsFromTemplate(*NewSource, *ParentWave);
			}
		}
	}

	NewSource->ConformObjectToDocument();
	FGraphBuilder::RegisterGraphWithFrontend(*NewSource);
	return NewSource;
}

