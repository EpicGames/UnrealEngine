// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeConfigurationCustomization.h"

#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundBuilderBase.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"

namespace Metasound::Editor
{
	void FMetaSoundNodeConfigurationCustomization::OnChildRowAdded(IDetailPropertyRow& ChildRow)
	{
		FInstancedStructDataDetails::OnChildRowAdded(ChildRow);

		TSharedPtr<IPropertyHandle> ChildHandle = ChildRow.GetPropertyHandle();
		if (ChildHandle)
		{
			TDelegate<void(const FPropertyChangedEvent&)> OnValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMetaSoundNodeConfigurationCustomization::OnChildPropertyChanged);
			ChildHandle->SetOnPropertyValueChangedWithData(OnValueChangedDelegate);
			ChildHandle->SetOnChildPropertyValueChangedWithData(OnValueChangedDelegate);
		}
	}

	void FMetaSoundNodeConfigurationCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
	{
		if (GraphNode.IsValid())
		{
			// Don't update interface on interactive changes (ex. dragging slider) to avoid refresh spam
			if (InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				// Update the node config interface
				FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
				const FGuid& NodeID = GraphNode->GetNodeID();
				DocBuilder.UpdateNodeInterfaceFromConfiguration(NodeID);
			}
		}
	}
} // namespace Metasound::Editor
