// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundNodeDetailCustomization.h"

#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "Components/AudioComponent.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "InstancedStructDetails.h"
#include "Internationalization/Text.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundLog.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundNodeConfigurationCustomization.h"
#include "MetasoundWave.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "SAssetDropTarget.h"
#include "SMetasoundActionMenu.h"
#include "SMetasoundGraphNode.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "MetaSoundEditor"

namespace Metasound
{
	namespace Editor
	{
		namespace MemberCustomizationPrivate
		{
			/** Set of input types which are valid registered types, but should
			 * not show up as an input type option in the MetaSound editor. */
			static const TSet<FName> HiddenInputTypeNames =
			{
				"Audio:Mono",
				"Audio:Stereo",
				GetMetasoundDataTypeName<Frontend::FAnalyzerAddress>(),
				"MetasoundParameterPack"
			};

			static const FText ConstructorPinText = LOCTEXT("ConstructorPinText", "Is Constructor Pin");
			static const FText ConstructorPinTooltip = LOCTEXT("ConstructorPinTooltip",
				"Whether this input or output is a constructor pin. Constructor values are only read on construction (on play), and are not dynamically updated at runtime.");
			
			static const FText AdvancedPinText = LOCTEXT("AdvancedPinText", "Is Advanced Pin");
			static const FText AdvancedPinTooltip = LOCTEXT("AdvancedPinTooltip",
				"Advanced Pins are hidden by default on the node when this MetaSound is used in other graphs.");


			// Retrieves the data type info if the literal property's member is found. Returns if the associated member is found, false if not.
			bool GetDataTypeFromElementPropertyHandle(TSharedPtr<IPropertyHandle> ElementPropertyHandle, Frontend::FDataTypeRegistryInfo& OutDataTypeInfo)
			{
				using namespace Frontend;

				if (!ElementPropertyHandle.IsValid())
				{
					return false;
				}

				OutDataTypeInfo = { };
				TArray<UObject*>OuterObjects;
				ElementPropertyHandle->GetOuterObjects(OuterObjects);
				if (OuterObjects.Num() == 1)
				{
					UObject* Outer = OuterObjects.Last();
					if (const UMetasoundEditorGraphMemberDefaultLiteral* DefaultLiteral = Cast<UMetasoundEditorGraphMemberDefaultLiteral>(Outer))
					{
						if (const UMetasoundEditorGraphMember* Member = DefaultLiteral->FindMember())
						{
							FName DataTypeName = Member->GetDataType();
							ensure(IDataTypeRegistry::Get().GetDataTypeInfo(DataTypeName, OutDataTypeInfo));
							if (OutDataTypeInfo.bIsArrayType)
							{
								DataTypeName = CreateElementTypeNameFromArrayTypeName(DataTypeName);
								const bool bIsHiddenType = HiddenInputTypeNames.Contains(DataTypeName);
								OutDataTypeInfo = { };
								if (!bIsHiddenType)
								{
									ensure(IDataTypeRegistry::Get().GetDataTypeInfo(DataTypeName, OutDataTypeInfo));
								}
							}

							return true;
						}
					}
				}

				return false;
			}

			// If DataType is an array type, creates & returns the array's
			// element type. Otherwise, returns this type's DataTypeName.
			FName GetPrimitiveTypeName(const Frontend::FDataTypeRegistryInfo& InDataTypeInfo)
			{
				return InDataTypeInfo.bIsArrayType
					? CreateElementTypeNameFromArrayTypeName(InDataTypeInfo.DataTypeName)
					: InDataTypeInfo.DataTypeName;
			}

			// Paste execute action for object member default values
			FExecuteAction CreateDefaultValueObjectPasteExecuteAction(TSharedPtr<IPropertyHandle> PropertyHandle)
			{
				return FExecuteAction::CreateLambda([PropertyHandle]()
				{
					const FScopedTransaction Transaction(LOCTEXT("PasteObjectArrayProperty", "Paste Property"));

					FString ClipboardValue;
					FPlatformApplicationMisc::ClipboardPaste(ClipboardValue);
					if (ClipboardValue.IsEmpty())
					{
						return;
					}
					
					Frontend::FDataTypeRegistryInfo DataTypeInfo;
					const bool bMemberFound = MemberCustomizationPrivate::GetDataTypeFromElementPropertyHandle(PropertyHandle, DataTypeInfo);
					UClass* ProxyGenClass = DataTypeInfo.ProxyGeneratorClass;
					if (!bMemberFound || !ProxyGenClass)
					{
						return;
					}
					FTopLevelAssetPath ClassPath = ProxyGenClass->GetClassPathName();

					// Try to reformat string 
					// Split into array of objects
					TArray<FString> Values;
					// Copying from other MS, still parse to verify object type 
					// or copying from BP
					if ((ClipboardValue.StartsWith("((") && ClipboardValue.EndsWith("))")) || 
						(ClipboardValue.StartsWith("(\"") && ClipboardValue.EndsWith("\")")))
					{
						// Remove first and last parentheses
						ClipboardValue.LeftChop(1).RightChop(1).ParseIntoArrayWS(Values, TEXT(","), true);
					}
					// Copying from content browser 
					else
					{
						ClipboardValue.ParseIntoArrayWS(Values, TEXT(","), true);
					}

					if (!Values.IsEmpty())
					{
						const bool bIsArray = PropertyHandle->AsArray().IsValid();
						TStringBuilder<512> Builder;
						if (bIsArray)
						{
							Builder << TEXT("(");
						}

						for (FString& Value : Values)
						{
							// Remove (Object= ) wrapper (other MetaSound case)
							if (Value.Contains(TEXT("Object=")))
							{
								Value = Value.LeftChop(2).RightChop(9);
							}

							FString ValueClassPath;

							const int32 CommaIndex = Value.Find("'");
							if (CommaIndex != INDEX_NONE)
							{
								// Validate the class path (before the first ')
								ValueClassPath = Value.Left(CommaIndex);
								// Remove beginning quote (BP case)
								if (ValueClassPath.StartsWith(TEXT("\"")))
								{
									ValueClassPath.RightChopInline(1);
								}
							}
							
							// Wrap objects in (Object=*)
							if (ValueClassPath == ClassPath.ToString())
							{
								Builder << TEXT("(Object=");
								Builder << Value;
								Builder << TEXT("),");
							}
							else
							{
								UE_LOGF(LogMetaSound, Warning, "Failed to paste object of type %ls which does not match default value type %ls", *ValueClassPath, *ClassPath.ToString());
							}
						}

						// Remove last comma
						if (Builder.Len() > 0)
						{
							Builder.RemoveSuffix(1);
						}

						if (bIsArray)
						{
							Builder << TEXT(")");
						}

						FString FormattedString = Builder.ToString();
						PropertyHandle->SetValueFromFormattedString(FormattedString, EPropertyValueSetFlags::InstanceObjects);
					}
				});
			}

			// Create copy/paste actions for member default value for object and object array types
			void CreateDefaultValueObjectCopyPasteActions(FDetailWidgetRow& InWidgetRow, TSharedPtr<IPropertyHandle> PropertyHandle)
			{
				// Copy action
				FUIAction CopyAction;
				CopyAction.ExecuteAction = FExecuteAction::CreateLambda([PropertyHandle]()
				{
					FString Value;
					if (PropertyHandle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
					{
						FPlatformApplicationMisc::ClipboardCopy(*Value);
					}
				});

				// Paste action
				TArray<UObject*> OuterObjects;
				UMetasoundEditorGraphMember* GraphMember = nullptr;
				PropertyHandle->GetOuterObjects(OuterObjects);
				if (!OuterObjects.IsEmpty())
				{
					if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Cast<UMetasoundEditorGraphMemberDefaultLiteral>(OuterObjects[0]))
					{
						GraphMember = Literal->FindMember();
					}
				}

				FUIAction PasteAction;
				// Paste only enabled if graph is editable (for variables/outputs)
				// or if graph is editable and input is not an interface member.
				PasteAction.CanExecuteAction = FCanExecuteAction::CreateLambda([MemberPtr = TWeakObjectPtr<UMetasoundEditorGraphMember>(GraphMember)]()
				{
					using namespace Frontend;

					if (!MemberPtr.IsValid())
					{
						return false;
					}

					const bool bIsGraphEditable = MemberPtr->GetOwningGraph()->IsEditable();
					if (bIsGraphEditable)
					{
						if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(MemberPtr.Get()))
						{
							if (Input->IsInterfaceMember())
							{
								return false;
							}

							if (const FDocumentVertexTemplate* DocTemplate = Input->GetFrontendBuilderChecked().GetConstTemplateAs<FDocumentVertexTemplate>())
							{
								if (const FTemplateVertexMetadata* VertexMetadata = DocTemplate->FindConstVertexMetadata(Input->NodeID))
								{
									return VertexMetadata->IsDefaultEditable();
								}
							}
						}

						return true;
					}

					return false;
				});

				PasteAction.ExecuteAction = CreateDefaultValueObjectPasteExecuteAction(PropertyHandle);

				InWidgetRow.CopyAction(CopyAction);
				InWidgetRow.PasteAction(PasteAction);
			}
		} // namespace MemberCustomizationPrivate

		FMetasoundFloatLiteralCustomization::~FMetasoundFloatLiteralCustomization()
		{
			if (FloatLiteral.IsValid())
			{
				FloatLiteral->OnClampChanged.Remove(OnClampChangedDelegateHandle);
			}
		}

		void FMetasoundFloatLiteralCustomization::CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
		{
			check(DefaultCategoryBuilder);

			TArray<UObject*> EditLiterals;
			if (UMetasoundEditorGraphMemberDefaultFloat* CastLiteral = Cast<UMetasoundEditorGraphMemberDefaultFloat>(&InLiteral); ensure(CastLiteral))
			{
				EditLiterals = { CastLiteral };
				FloatLiteral = CastLiteral;
			}
			else
			{
				FloatLiteral.Reset();
				return;
			}

			FMetasoundDefaultLiteralCustomizationBase::CustomizeDefaults(InLiteral, InDetailLayout);

			TAttribute<EVisibility> DefaultVisibility = GetDefaultVisibility();

			IDetailCategoryBuilder& EditorOptionsBuilder = InDetailLayout.EditCategory("DefaultEditorOptions");
			auto AddOptionPropRow = [&](FName PropertyName)
			{
				IDetailPropertyRow* Row = EditorOptionsBuilder.AddExternalObjectProperty(EditLiterals, PropertyName);
				if (ensure(Row))
				{
					Row->Visibility(DefaultVisibility);
					Row->IsEnabled(GetEnabled());
				}
				return Row;
			};

			if (IDetailPropertyRow* ClampRow = AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, ClampDefault)))
			{
				// Apply the clamp range to the default value if using a widget or ClampDefault is otherwise true
				// Only show clamp row if not using a widget (widgets always require a clamp and range)
				// Presets and non inputs are an exception, because they may have a widget inherited, but that doesn't apply and isn't editable,
				const bool bUsingWidget = FloatLiteral->WidgetType != EMetasoundMemberDefaultWidget::None;
				const UMetasoundEditorGraphMember* Member = FloatLiteral->FindMember();
				const bool bIsPreset = Member ? Member->GetFrontendBuilderChecked().IsPreset() : false;
				const bool bIsInput = Member ? Cast<UMetasoundEditorGraphInput>(Member) != nullptr : true;
				const bool bClampActiveWithoutWidget = !bUsingWidget || bIsPreset || !bIsInput;
				const bool bApplyRange = FloatLiteral->ClampDefault || !bClampActiveWithoutWidget;

				ClampRow->Visibility(bClampActiveWithoutWidget ? DefaultVisibility : EVisibility::Hidden);
				for (TSharedPtr<IPropertyHandle>& DefaultValueHandle : DefaultProperties)
				{
					if (DefaultValueHandle.IsValid())
					{
						if (bApplyRange)
						{
							FVector2D Range = FloatLiteral->GetRange();
							DefaultValueHandle->SetInstanceMetaData("ClampMin", FString::Printf(TEXT("%f"), Range.X));
							DefaultValueHandle->SetInstanceMetaData("ClampMax", FString::Printf(TEXT("%f"), Range.Y));
						}
						else // Stop clamping
						{
							DefaultValueHandle->SetInstanceMetaData("ClampMin", "");
							DefaultValueHandle->SetInstanceMetaData("ClampMax", "");
						}
					}
				}

				FloatLiteral->OnClampChanged.Remove(OnClampChangedDelegateHandle);
				OnClampChangedDelegateHandle = FloatLiteral->OnClampChanged.AddLambda([this](bool ClampInput)
				{
					if (FloatLiteral.IsValid())
					{
						FloatLiteral->OnClampChanged.Remove(OnClampChangedDelegateHandle);
						if (const UMetasoundEditorGraphMember* Member = FloatLiteral->FindMember())
						{
							FMetasoundAssetBase& MetasoundAsset = Editor::FGraphBuilder::GetOutermostMetaSoundChecked(*FloatLiteral);
							MetasoundAsset.GetModifyContext().AddMemberIDsModified({ Member->GetMemberID() });
						}
					}
				});

				if (bApplyRange)
				{
					AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, Range));
				}
			}

			// Enable widget options for editable inputs only
			bool bShowWidgetOptions = false;
			if (const UMetasoundEditorGraphInput* ParentMember = Cast<UMetasoundEditorGraphInput>(InLiteral.FindMember()))
			{
				if (const UMetasoundEditorGraph* OwningGraph = ParentMember->GetOwningGraph())
				{
					bShowWidgetOptions = OwningGraph->IsEditable();
				}
			}

			// add input widget properties
			if (bShowWidgetOptions)
			{
				AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, WidgetType));
				AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, WidgetOrientation));
				AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, WidgetUnitValueType));
				if (FloatLiteral->WidgetType != EMetasoundMemberDefaultWidget::None && FloatLiteral->WidgetUnitValueType == EAudioUnitsValueType::Volume)
				{
					AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, VolumeWidgetUseLinearOutput));
					if (FloatLiteral->VolumeWidgetUseLinearOutput)
					{
						AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultFloat, VolumeWidgetDecibelRange));
					}
				}
			}
		}

		FMetasoundBoolLiteralCustomization::~FMetasoundBoolLiteralCustomization()
		{
		}

		void FMetasoundBoolLiteralCustomization::CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
		{
			check(DefaultCategoryBuilder);

			TArray<UObject*> EditLiterals;
			if (UMetasoundEditorGraphMemberDefaultBool* CastLiteral = Cast<UMetasoundEditorGraphMemberDefaultBool>(&InLiteral); ensure(CastLiteral))
			{
				EditLiterals = { CastLiteral };
				BoolLiteral = CastLiteral;
			}
			else
			{
				BoolLiteral.Reset();
				return;
			}

			bool bIsTrigger = false;
			const UMetasoundEditorGraphMember* Member = BoolLiteral->FindMember();
			if (!Member)
			{
				return;
			}

			// Non-input members don't show any options for inputs, so early out if trigger.
			bIsTrigger = Member->GetDataType() == Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>();
			if (bIsTrigger && !Member->IsA<UMetasoundEditorGraphInput>())
			{
				return;
			}

			FMetasoundDefaultLiteralCustomizationBase::CustomizeDefaults(InLiteral, InDetailLayout);

			TAttribute<EVisibility> DefaultVisibility = GetDefaultVisibility();

			IDetailCategoryBuilder& EditorOptionsBuilder = InDetailLayout.EditCategory("DefaultEditorOptions");
			auto AddOptionPropRow = [&](FName PropertyName)
			{
				IDetailPropertyRow* Row = EditorOptionsBuilder.AddExternalObjectProperty(EditLiterals, PropertyName);
				if (ensure(Row))
				{
					Row->Visibility(DefaultVisibility);
				}
				return Row;
			};

			bool bShowWidgetOptions = false;

			const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
			check(EditorSettings);

			if (EditorSettings->bUseAudioMaterialWidgets)
			{
				if (!bIsTrigger)
				{
					if (const UMetasoundEditorGraph* OwningGraph = Member->GetOwningGraph())
					{
						bShowWidgetOptions = OwningGraph->IsEditable();
					}
				}
			}

			if (bShowWidgetOptions)
			{
				AddOptionPropRow(GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMemberDefaultBool, WidgetType));
			}
		}

		TAttribute<EVisibility> FMetasoundBoolLiteralCustomization::GetDefaultVisibility() const
		{
			if (UMetasoundEditorGraphMemberDefaultBool* DefaultBool = BoolLiteral.Pin().Get())
			{
				const UMetasoundEditorGraphMember* Member = DefaultBool->FindMember();
				if (Member && !Member->IsA<UMetasoundEditorGraphInput>())
				{
					const bool bIsTrigger = Member->GetDataType() == Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>();
					if (bIsTrigger)
					{
						return EVisibility::Collapsed;
					}
				}
			}

			return FMetasoundDefaultLiteralCustomizationBase::GetDefaultVisibility();
		}

		void FMetasoundObjectArrayLiteralCustomization::CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout)
		{
			check(DefaultCategoryBuilder);

			FMetasoundDefaultLiteralCustomizationBase::CustomizePageDefaultRows(InLiteral, InDetailLayout);
		}

		void FMetasoundObjectArrayLiteralCustomization::BuildDefaultValueWidget(IDetailPropertyRow& ValueRow, TSharedPtr<IPropertyHandle> ValueProperty)
		{
			if (!ValueProperty.IsValid())
			{
				return;
			}

			(*ValueRow.CustomValueWidget())
			[
				SNew(SAssetDropTarget)
					.bSupportsMultiDrop(true)
					.OnAreAssetsAcceptableForDropWithReason_Lambda([this, ValueProperty](TArrayView<FAssetData> InAssets, FText& OutReason)
					{
						Frontend::FDataTypeRegistryInfo DataTypeInfo;
						const bool bMemberFound = MemberCustomizationPrivate::GetDataTypeFromElementPropertyHandle(ValueProperty, DataTypeInfo);
						bool bCanDrop = bMemberFound;
						if (UClass* ProxyGenClass = DataTypeInfo.ProxyGeneratorClass; bCanDrop && bMemberFound)
						{
							bCanDrop = true;
							for (const FAssetData& AssetData : InAssets)
							{
								if (UClass* Class = AssetData.GetClass())
								{
									if (DataTypeInfo.bIsExplicit)
									{
										bCanDrop &= Class == DataTypeInfo.ProxyGeneratorClass;
									}
									else
									{
										bCanDrop &= Class->IsChildOf(DataTypeInfo.ProxyGeneratorClass);
									}
								}
							}
						}
						return bCanDrop;
					})
					.OnAssetsDropped_Lambda([this, ValueProperty](const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> InAssets)
					{
						TSharedPtr<IPropertyHandleArray> ArrayProperty = ValueProperty->AsArray();
						if (ArrayProperty.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("DragDropInputAssets", "Drop Asset(s) on MetaSound Input"));
							for (const FAssetData& AssetData : InAssets)
							{
								uint32 AddIndex = INDEX_NONE;
								ArrayProperty->GetNumElements(AddIndex);
								ArrayProperty->AddItem();
								TSharedPtr<IPropertyHandle> ElementHandle = ArrayProperty->GetElement(static_cast<int32>(AddIndex));
								TSharedPtr<IPropertyHandle> ObjectHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorGraphMemberDefaultObjectRef, Object));
								ObjectHandle->SetValue(AssetData.GetAsset());
							}
						}
					})
					[
						ValueProperty->CreatePropertyValueWidget()
					]
			];
		}

		FText FMetasoundMemberDefaultBoolDetailCustomization::GetPropertyNameOverride() const
		{
			using namespace MemberCustomizationPrivate;

			if (GetPrimitiveTypeName(DataTypeInfo) == Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>())
			{
				return LOCTEXT("TriggerInput_SimulateTitle", "Simulate");
			}

			return FText::GetEmpty();
		}

		TSharedRef<SWidget> FMetasoundMemberDefaultBoolDetailCustomization::CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const
		{
			using namespace Frontend;
			using namespace MemberCustomizationPrivate;

			TSharedPtr<IPropertyHandle> ValueProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorGraphMemberDefaultBoolRef, Value));
			if (ValueProperty.IsValid())
			{
				// Not a trigger, so just display as underlying literal type (bool)
				if (GetPrimitiveTypeName(DataTypeInfo) != Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>())
				{
					return ValueProperty->CreatePropertyValueWidget();
				}

				TAttribute<bool> EnablementAttribute = false;
				TAttribute<EVisibility> VisibilityAttribute = EVisibility::Visible;

				TArray<UObject*> OuterObjects;
				ValueProperty->GetOuterObjects(OuterObjects);
				if (!OuterObjects.IsEmpty())
				{
					if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Cast<UMetasoundEditorGraphMemberDefaultLiteral>(OuterObjects.Last()))
					{
						if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(Literal->FindMember()))
						{
							// Don't display trigger simulation widget if its a trigger
							// provided by an interface that does not support transmission.
							FName RouterName;
							const bool bFound = IInterfaceRegistry::Get().FindInterfaceRouter(Input->GetInterfaceVersion(), RouterName);
							if (!bFound || RouterName == Audio::IParameterTransmitter::RouterName)
							{
								EnablementAttribute = true;
								return SMetaSoundGraphNode::CreateTriggerSimulationWidget(*Literal, MoveTemp(VisibilityAttribute), MoveTemp(EnablementAttribute));
							}

							const FText DisabledToolTip = LOCTEXT("NonTransmittibleInputTriggerSimulationDisabledTooltip", "Trigger simulation disabled: Parent interface does not support being updated by game thread parameters.");
							return SMetaSoundGraphNode::CreateTriggerSimulationWidget(*Literal, MoveTemp(VisibilityAttribute), MoveTemp(EnablementAttribute), &DisabledToolTip);
						}
					}
				}
			}

			return SNullWidget::NullWidget;
		}

		TSharedRef<SWidget> FMetasoundMemberDefaultIntDetailCustomization::CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const
		{
			using namespace Frontend;
			using namespace MemberCustomizationPrivate;

			// DataType can be reset during deletion of a literal value.  Customization can repaint briefly before the literal is removed,
			// so just ignores if DataType is invalid.
			const bool bIsValidDataType = !DataTypeInfo.DataTypeName.IsNone();
			if (bIsValidDataType)
			{
				TSharedPtr<IPropertyHandle> ValueProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorGraphMemberDefaultIntRef, Value));
				if (ValueProperty.IsValid())
				{
					TSharedPtr<const IEnumDataTypeInterface> EnumInterface = IDataTypeRegistry::Get().GetEnumInterfaceForDataType(GetPrimitiveTypeName(DataTypeInfo));

					// Not an enum, so just display as underlying type (int32)
					if (!EnumInterface.IsValid())
					{
						return ValueProperty->CreatePropertyValueWidget();
					}

					auto GetAll = [Interface = EnumInterface](TArray<TSharedPtr<FString>>& OutStrings, TArray<TSharedPtr<SToolTip>>& OutTooltips, TArray<bool>&)
					{
						for (const IEnumDataTypeInterface::FGenericInt32Entry& i : Interface->GetAllEntries())
						{
							OutTooltips.Emplace(SNew(SToolTip).Text(i.Tooltip));
							OutStrings.Emplace(MakeShared<FString>(i.DisplayName.ToString()));
						}
					};
					auto GetValue = [Interface = EnumInterface, Prop = ValueProperty]()
					{
						int32 IntValue;
						if (Prop->GetValue(IntValue) != FPropertyAccess::Success)
						{
							IntValue = Interface->GetDefaultValue();
							UE_LOGF(LogMetasoundEditor, Warning, "Failed to read int Property '%ls', defaulting.", *GetNameSafe(Prop->GetProperty()));
						}
						if (TOptional<IEnumDataTypeInterface::FGenericInt32Entry> Result = Interface->FindByValue(IntValue))
						{
							return Result->DisplayName.ToString();
						}
						UE_LOGF(LogMetasoundEditor, Warning, "Failed to resolve int value '%d' to a valid enum value for enum '%ls'",
							IntValue, *Interface->GetNamespace().ToString());

						// Return default (should always succeed as we can't have empty Enums and we must have a default).
						return Interface->FindByValue(Interface->GetDefaultValue())->DisplayName.ToString();
					};
					auto SelectedValue = [Interface = EnumInterface, Prop = ValueProperty](const FString& InSelected)
					{
						TOptional<IEnumDataTypeInterface::FGenericInt32Entry> Found =
							Interface->FindEntryBy([TextSelected = FText::FromString(InSelected)](const IEnumDataTypeInterface::FGenericInt32Entry& i)
						{
							return i.DisplayName.EqualTo(TextSelected);
						});

						if (Found)
						{
							// Only save the changes if its different and we can read the old value to check that.
							int32 CurrentValue;
							bool bReadCurrentValue = Prop->GetValue(CurrentValue) == FPropertyAccess::Success;
							if ((bReadCurrentValue && CurrentValue != Found->Value) || !bReadCurrentValue)
							{
								ensure(Prop->SetValue(Found->Value) == FPropertyAccess::Success);
							}
						}
						else
						{
							UE_LOGF(LogMetasoundEditor, Warning, "Failed to Set Valid Value for Property '%ls' with Value of '%ls', writing default.",
								*GetNameSafe(Prop->GetProperty()), *InSelected);

							ensure(Prop->SetValue(Interface->GetDefaultValue()) == FPropertyAccess::Success);
						}
					};

					return PropertyCustomizationHelpers::MakePropertyComboBox(
						nullptr,
						FOnGetPropertyComboBoxStrings::CreateLambda(GetAll),
						FOnGetPropertyComboBoxValue::CreateLambda(GetValue),
						FOnPropertyComboBoxValueSelected::CreateLambda(SelectedValue)
					);
				}
			}

			return SNullWidget::NullWidget;
		}

		TSharedRef<SWidget> FMetasoundMemberDefaultObjectDetailCustomization::CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorGraphMemberDefaultObjectRef, Object));

			const IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			auto FilterAsset = [InEditorModule = &EditorModule, &InDataTypeInfo = DataTypeInfo](const FAssetData& InAsset)
			{
				if (InDataTypeInfo.ProxyGeneratorClass)
				{
					if (UClass* Class = InAsset.GetClass())
					{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						if (InEditorModule->IsExplicitProxyClass(*InDataTypeInfo.ProxyGeneratorClass))
						{
							return Class != InDataTypeInfo.ProxyGeneratorClass;
						}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
						if (InDataTypeInfo.bIsExplicit)
						{
							return Class != InDataTypeInfo.ProxyGeneratorClass;
						}

						return !Class->IsChildOf(InDataTypeInfo.ProxyGeneratorClass);
					}
				}

				return true;
			};

			auto ValidateAsset = [FilterAsset](const FAssetData& InAsset)
			{
				// A null asset reference is a valid default
				return InAsset.IsValid() ? !FilterAsset(InAsset) : true;
			};

			auto GetAssetPath = [PropertyHandle = PropertyHandle]()
			{
				UObject* Object = nullptr;
				if (PropertyHandle->GetValue(Object) == FPropertyAccess::Success)
				{
					if (Object)
					{
						return Object->GetPathName();
					}
				}
				return FString();
			};

			const bool bAllowCreate = DataTypeInfo.DataTypeName != GetMetasoundDataTypeName<FWaveAsset>();

			return SNew(SObjectPropertyEntryBox)
				.AllowClear(true)
				.AllowCreate(bAllowCreate)
				.AllowedClass(DataTypeInfo.ProxyGeneratorClass)
				.DisplayBrowse(true)
				.DisplayThumbnail(true)
				.DisplayUseSelected(true)
				.NewAssetFactories(PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses({DataTypeInfo.ProxyGeneratorClass }))
				.ObjectPath_Lambda(GetAssetPath)
				.OnShouldFilterAsset_Lambda(FilterAsset)
				.OnShouldSetAsset_Lambda(ValidateAsset)
				.PropertyHandle(PropertyHandle);
		}

		TSharedRef<SWidget> FMetasoundDefaultMemberElementDetailCustomizationBase::CreateNameWidget(TSharedPtr<IPropertyHandle> StructPropertyHandle) const
		{
			const FText PropertyName = GetPropertyNameOverride();
			if (!PropertyName.IsEmpty())
			{
				return SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(PropertyName);
			}

			return SNew(STextBlock)
				.Text(MemberCustomizationStyle::DefaultPropertyText)
				.Font(IDetailLayoutBuilder::GetDetailFont());
		}

		TSharedRef<SWidget> FMetasoundDefaultMemberElementDetailCustomizationBase::CreateValueWidget(TSharedPtr<IPropertyHandleArray> ParentPropertyHandleArray, TSharedPtr<IPropertyHandle> StructPropertyHandle) const
		{
			return CreateStructureWidget(StructPropertyHandle);
		}

		void FMetasoundDefaultMemberElementDetailCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
		{
			TSharedPtr<IPropertyHandleArray> ParentPropertyHandleArray;
			TSharedRef<IPropertyHandle> ElementPropertyHandle = StructPropertyHandle;
			{
				TSharedPtr<IPropertyHandle> ParentProperty = StructPropertyHandle->GetParentHandle();
				if (ParentProperty.IsValid() && ParentProperty->GetProperty() != nullptr)
				{
					ParentPropertyHandleArray = ParentProperty->AsArray();
					if (ParentPropertyHandleArray.IsValid())
					{
						ElementPropertyHandle = ParentProperty.ToSharedRef();
					}
				}
			}

			const bool bMemberFound = MemberCustomizationPrivate::GetDataTypeFromElementPropertyHandle(ElementPropertyHandle, DataTypeInfo);
			ensureAlways(bMemberFound);

			IDetailPropertyRow& ValueRow = ChildBuilder.AddProperty(StructPropertyHandle);
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			ValueRow.GetDefaultWidgets(NameWidget, ValueWidget);

			constexpr bool bShowChildren = false;
			ValueRow.CustomWidget(bShowChildren);
			if (ParentPropertyHandleArray.IsValid())
			{
				(*ValueRow.CustomNameWidget())
				[
					StructPropertyHandle->CreatePropertyNameWidget()
				];
			}
			else
			{
				(*ValueRow.CustomNameWidget())
				[
					CreateNameWidget(StructPropertyHandle)
				];
			}

			{
				TArray<UObject*> OuterObjects;
				StructPropertyHandle->GetOuterObjects(OuterObjects);
				TArray<TWeakObjectPtr<UMetasoundEditorGraphInput>> Inputs;
				for (UObject* Object : OuterObjects)
				{
					if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(Object))
					{
						Inputs.Add(Input);
					}
				}

				FSimpleDelegate UpdateFrontendDefaultLiteral = FSimpleDelegate::CreateLambda([InInputs = MoveTemp(Inputs)]()
				{
					for (const TWeakObjectPtr<UMetasoundEditorGraphInput>& GraphInput : InInputs)
					{
						if (GraphInput.IsValid())
						{
							constexpr bool bPostTransaction = true;
							GraphInput->UpdateFrontendDefaultLiteral(bPostTransaction);
						}
					}
				});
				StructPropertyHandle->SetOnChildPropertyValueChanged(UpdateFrontendDefaultLiteral);
			}

			(*ValueRow.CustomValueWidget())
			[
				CreateValueWidget(ParentPropertyHandleArray, StructPropertyHandle)
			];
		}

		void FMetasoundDefaultMemberElementDetailCustomizationBase::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
		{
		}

		FName FMetasoundDataTypeSelector::GetDataType() const
		{
			if (GraphMember.IsValid())
			{
				return GraphMember->GetDataType();
			}

			return FName();
		}

		void FMetasoundDataTypeSelector::OnDataTypeSelected(FName InSelectedTypeName)
		{
			FName NewDataTypeName;
			FName ArrayDataTypeName = CreateArrayTypeNameFromElementTypeName(InSelectedTypeName);

			// Update data type based on "Is Array" checkbox and support for arrays.
			// If an array type is not supported, default to the base data type.
			if (DataTypeArrayCheckbox->GetCheckedState() == ECheckBoxState::Checked)
			{
				if (Frontend::IDataTypeRegistry::Get().IsRegistered(ArrayDataTypeName))
				{
					NewDataTypeName = ArrayDataTypeName;
				}
				else
				{
					ensure(Frontend::IDataTypeRegistry::Get().IsRegistered(InSelectedTypeName));
					NewDataTypeName = InSelectedTypeName;
				}
			}
			else
			{
				if (Frontend::IDataTypeRegistry::Get().IsRegistered(InSelectedTypeName))
				{
					NewDataTypeName = InSelectedTypeName;
				}
				else
				{
					ensure(Frontend::IDataTypeRegistry::Get().IsRegistered(ArrayDataTypeName));
					NewDataTypeName = ArrayDataTypeName;
				}
			}

			if (NewDataTypeName == GraphMember->GetDataType())
			{
				return;
			}

			// Have to stop playback to avoid attempting to change live edit data on invalid input type.
			check(GEditor);
			GEditor->ResetPreviewAudioComponent();

			if (GraphMember.IsValid())
			{
				GraphMember->SetDataType(NewDataTypeName);
			}
		}

		void FMetasoundDataTypeSelector::AddDataTypeSelector(IDetailLayoutBuilder& InDetailLayout, const FText& InRowName, TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, bool bIsEnabled)
		{
			using namespace Frontend;

			if (!InGraphMember.IsValid())
			{
				return;
			}

			GraphMember = InGraphMember;

			FDataTypeRegistryInfo DataTypeInfo;
			if (!ensure(IDataTypeRegistry::Get().GetDataTypeInfo(InGraphMember->GetDataType(), DataTypeInfo)))
			{
				return;
			}

			if (DataTypeInfo.bIsArrayType)
			{
				ArrayTypeName = GraphMember->GetDataType();
				BaseTypeName = CreateElementTypeNameFromArrayTypeName(InGraphMember->GetDataType());
			}
			else
			{
				ArrayTypeName = CreateArrayTypeNameFromElementTypeName(InGraphMember->GetDataType());
				BaseTypeName = GraphMember->GetDataType();
			}

			IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);

			// Not all types have an equivalent array type. Base types without array
			// types should have the "Is Array" checkbox disabled.
			const bool bIsArrayTypeRegistered = IDataTypeRegistry::Get().IsRegistered(ArrayTypeName);
			const bool bIsArrayTypeRegisteredHidden = MemberCustomizationPrivate::HiddenInputTypeNames.Contains(ArrayTypeName);

			TArray<FName> BaseDataTypes;
			IDataTypeRegistry::Get().IterateDataTypeInfo([&BaseDataTypes](const FDataTypeRegistryInfo& RegistryInfo)
			{
				// Hide the type from the combo selector if any of the following is true
				const bool bIsHiddenType = MemberCustomizationPrivate::HiddenInputTypeNames.Contains(RegistryInfo.DataTypeName);
				const bool bHideBaseType = RegistryInfo.bIsArrayType || RegistryInfo.bIsVariable || bIsHiddenType;
				if (!bHideBaseType)
				{
					BaseDataTypes.Add(RegistryInfo.DataTypeName);
				}
			});

			BaseDataTypes.Sort([](const FName& DataTypeNameL, const FName& DataTypeNameR)
			{
				return DataTypeNameL.LexicalLess(DataTypeNameR);
			});

			Algo::Transform(BaseDataTypes, ComboOptions, [](const FName& Name) { return MakeShared<FString>(Name.ToString()); });

			InDetailLayout.EditCategory("General").AddCustomRow(InRowName)
			.IsEnabled(bIsEnabled)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(InRowName)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(DataTypeComboBox, SSearchableComboBox)
					.OptionsSource(&ComboOptions)
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
					{
						return SNew(STextBlock)
							.Text(FText::FromString(*InItem));
					})
					.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InNewName, ESelectInfo::Type InSelectInfo)
					{
						if (InSelectInfo != ESelectInfo::OnNavigation)
						{
							OnDataTypeSelected(FName(*InNewName));
						}
					})
					.bAlwaysSelectItem(true)
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							return FText::FromName(BaseTypeName);
						})
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(DataTypeArrayCheckbox, SCheckBox)
					.IsEnabled(bIsArrayTypeRegistered && !bIsArrayTypeRegisteredHidden)
					.IsChecked_Lambda([this, InGraphMember]()
					{
						return OnGetDataTypeArrayCheckState(InGraphMember);
					})
					.OnCheckStateChanged_Lambda([this, InGraphMember](ECheckBoxState InNewState)
					{
						OnDataTypeArrayChanged(InGraphMember, InNewState);
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Node_IsArray", "Is Array"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			];

			auto NameMatchesPredicate = [TypeString = BaseTypeName.ToString()](const TSharedPtr<FString>& Item) { return *Item == TypeString; };
			const TSharedPtr<FString>* SelectedItem = ComboOptions.FindByPredicate(NameMatchesPredicate);
			if (ensure(SelectedItem))
			{
				DataTypeComboBox->SetSelectedItem(*SelectedItem, ESelectInfo::Direct);
			}
		}

		ECheckBoxState FMetasoundDataTypeSelector::OnGetDataTypeArrayCheckState(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember) const
		{
			using namespace Frontend;

			if (InGraphMember.IsValid())
			{
				FDataTypeRegistryInfo DataTypeInfo;
				if (ensure(IDataTypeRegistry::Get().GetDataTypeInfo(InGraphMember->GetDataType(), DataTypeInfo)))
				{
					return DataTypeInfo.bIsArrayType ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			}

			return ECheckBoxState::Undetermined;
		}

		void FMetasoundDataTypeSelector::OnDataTypeArrayChanged(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, ECheckBoxState InNewState)
		{
			if (InGraphMember.IsValid() && DataTypeComboBox.IsValid())
			{
				TSharedPtr<FString> DataTypeRoot = DataTypeComboBox->GetSelectedItem();
				if (ensure(DataTypeRoot.IsValid()))
				{
					// Have to stop playback to avoid attempting to change live edit data on invalid input type.
					check(GEditor);
					GEditor->ResetPreviewAudioComponent();

					const FName DataType = InNewState == ECheckBoxState::Checked ? ArrayTypeName : BaseTypeName;
					InGraphMember->SetDataType(DataType);
				}
			}
		}

		void FMetaSoundNodeDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
		{
			TArray<TWeakObjectPtr<UObject>> Objects;
			InDetailLayout.GetObjectsBeingCustomized(Objects);
			if (!Objects.IsEmpty())
			{
				UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Objects.Last().Get());
				if (!Node)
				{
					return;
				}
				const UEdGraph* EdGraph = Node->GetGraph();
				const UMetasoundEditorGraph* MetasoundGraph = CastChecked<UMetasoundEditorGraph>(EdGraph);
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = MetasoundGraph->GetBuilderChecked().GetBuilder();
				// Only add configuration details customization if node has valid configuration
				if (DocBuilder.FindNodeConfiguration(Node->GetNodeID()).IsValid())
				{
					// Walk the property path to find the node configuration handle
					TSharedPtr<IPropertyHandle> DocumentHandle = InDetailLayout.AddObjectPropertyData({ MetasoundGraph->GetMetasound() }, FName(TEXT("RootMetasoundDocument")));
					if (!DocumentHandle || !DocumentHandle->IsValidHandle())
					{
						return;
					}
					TSharedPtr<IPropertyHandle> RootGraphHandle = DocumentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundFrontendDocument, RootGraph));
					if (!RootGraphHandle || !RootGraphHandle->IsValidHandle())
					{
						return;
					}
					TSharedPtr<IPropertyHandle> PagedGraphsHandle = RootGraphHandle->GetChildHandle(FName(TEXT("PagedGraphs")));
					if (!PagedGraphsHandle || !PagedGraphsHandle->IsValidHandle() || !PagedGraphsHandle->AsArray())
					{
						return;
					}

					const FGuid& PageID = DocBuilder.GetBuildPageID();
					const int32 PageIndex = DocBuilder.FindGraphPageIndex(PageID);
					uint32 NumPages = 0;
					PagedGraphsHandle->AsArray()->GetNumElements(NumPages);
					if (PageIndex < 0 || (uint32)PageIndex >= NumPages)
					{
						return;
					}
					TSharedRef<IPropertyHandle> GraphHandle = PagedGraphsHandle->AsArray()->GetElement(PageIndex);
					TSharedPtr<IPropertyHandle> NodesHandle = GraphHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundFrontendGraph, Nodes));
					if (!NodesHandle || !NodesHandle->IsValidHandle())
					{
						return;
					}

					const int32* NodeIndex = DocBuilder.FindNodeIndex(Node->GetNodeID());
					if (NodeIndex != nullptr)
					{
						TSharedRef<IPropertyHandle> NodeHandle = NodesHandle->AsArray()->GetElement(*NodeIndex);
						TSharedPtr<IPropertyHandle> ConfigurationHandle = NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundFrontendNode, Configuration));

						// Use a custom details builder to add custom child property update behavior
						// and hide the struct picker since users should only be able edit child values, not the struct type
						if (ConfigurationHandle && ConfigurationHandle->IsValidHandle())
						{
							const IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);

							// Get underlying struct name
							FName StructName;
							ConfigurationHandle->EnumerateRawData([&StructName](void* RawData, const int32 DataIndex, const int32 /*NumDatas*/)
							{
								if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
								{
									StructName = InstancedStruct->GetScriptStruct()->GetFName();
								}
								return true;
							});

							const FCreateNodeConfigurationDetails* CreateDetailsFunc = !StructName.IsNone() ?
								MetaSoundEditorModule.FindCreateCustomNodeConfigurationDetailsCustomization(StructName) : nullptr;

							TSharedRef<IDetailCustomNodeBuilder> DataDetailsBuilder = CreateDetailsFunc ?
								// User custom registered details
								(*CreateDetailsFunc)(ConfigurationHandle, TWeakObjectPtr<UMetasoundEditorGraphNode>(Node)) :
								// Default details
								MakeShared<FMetaSoundNodeConfigurationCustomization>(ConfigurationHandle, TWeakObjectPtr<UMetasoundEditorGraphNode>(Node));

							InDetailLayout.EditCategory("General").AddCustomBuilder(DataDetailsBuilder);
						}
					}
				}
			}
		}

		void FMetasoundMemberDetailCustomization::UpdateRenameDelegate(UMetasoundEditorGraphMember& InMember)
		{
			if (InMember.CanRename())
			{
				if (!RenameRequestedHandle.IsValid())
				{
					InMember.OnRenameRequested.Clear();
					RenameRequestedHandle = InMember.OnRenameRequested.AddLambda([this]()
					{
						FSlateApplication::Get().SetKeyboardFocus(NameEditableTextBox.ToSharedRef(), EFocusCause::SetDirectly);
					});
				}
			}
		}

		void FMetasoundMemberDetailCustomization::CacheMemberData(IDetailLayoutBuilder& InDetailLayout)
		{
			TArray<TWeakObjectPtr<UObject>> Objects;
			InDetailLayout.GetObjectsBeingCustomized(Objects);
			if (!Objects.IsEmpty())
			{
				GraphMember = Cast<UMetasoundEditorGraphMember>(Objects.Last().Get());

				TSharedPtr<IPropertyHandle> LiteralHandle = InDetailLayout.GetProperty(UMetasoundEditorGraphMember::GetLiteralPropertyName());
				if (ensure(GraphMember.IsValid()) && ensure(LiteralHandle.IsValid()))
				{
					// Always hide, even if no customization (LiteralObject isn't found) as this is the case
					// where the default object is not required (i.e. Default Member is default constructed)
					LiteralHandle->MarkHiddenByCustomization();
				}
			}
		}

		void FMetasoundMemberDetailCustomization::CustomizeDefaultCategory(IDetailLayoutBuilder& InDetailLayout)
		{
			if (!GraphMember.IsValid())
			{
				return;
			}

			UpdateRenameDelegate(*GraphMember);

			if (UMetasoundEditorGraphMemberDefaultLiteral* MemberDefaultLiteral = GraphMember->GetLiteral())
			{
				UClass* MemberClass = MemberDefaultLiteral->GetClass();
				check(MemberClass);

				IDetailCategoryBuilder& DefaultCategoryBuilder = GetDefaultCategoryBuilder(InDetailLayout);
				IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
				LiteralCustomization = EditorModule.CreateMemberDefaultLiteralCustomization(*MemberClass, DefaultCategoryBuilder);

				TAttribute<EVisibility> Visibility = TAttribute<EVisibility>::CreateSPLambda(AsShared(), [this]()
				{
					return GetDefaultVisibility();
				});

				if (LiteralCustomization.IsValid())
				{
					LiteralCustomization->SetDefaultVisibility(Visibility);
					LiteralCustomization->SetEnabled(GetEnabled());
					ConfigureLiteralCustomization(*LiteralCustomization);
					LiteralCustomization->CustomizeDefaults(*MemberDefaultLiteral, InDetailLayout);
				}
				else
				{
					IDetailPropertyRow* DefaultPropertyRow = DefaultCategoryBuilder.AddExternalObjectProperty(TArray<UObject*>({ MemberDefaultLiteral }), "Default");
					if (ensureMsgf(DefaultPropertyRow, TEXT("Class '%s' missing expected 'Default' member."
						"Either add/rename default member or register customization to display default value/opt out appropriately."),
						*MemberClass->GetName()))
					{
						DefaultPropertyRow->Visibility(Visibility);
						DefaultPropertyRow->IsEnabled(GetEnabled());
					}
				}
			}
		}

		void FMetasoundMemberDetailCustomization::CustomizeGeneralCategory(IDetailLayoutBuilder& InDetailLayout)
		{
			IDetailCategoryBuilder& CategoryBuilder = GetGeneralCategoryBuilder(InDetailLayout);
			const bool bIsReadOnly = IsInterfaceMember() || !IsGraphEditable(); 

			// Override row copy action if it's disabled by the edit condition 
			auto GenerateCopyPasteActions = [](FDetailWidgetRow& Row, const FString& Value)
			{
				FUIAction CopyAction(FExecuteAction::CreateLambda([Value]()
				{
					FPlatformApplicationMisc::ClipboardCopy(*Value);
				}));
				Row.CopyAction(CopyAction);

				// Create a dummy paste action
				// Needed because the custom copy action will only be set 
				// if both the copy and paste actions are bound
				// Pasting is still available directly via the text box if editable
				const static FUIAction PasteAction(FExecuteAction::CreateLambda([]() {}), FCanExecuteAction::CreateLambda([]() { return false; }));
				Row.PasteAction(PasteAction);
			};

			NameEditableTextBox = SNew(SEditableTextBox)
				.Text(this, &FMetasoundMemberDetailCustomization::GetName)
				.OnTextChanged(this, &FMetasoundMemberDetailCustomization::OnNameChanged)
				.OnTextCommitted(this, &FMetasoundMemberDetailCustomization::OnNameCommitted)
				.IsReadOnly(bIsReadOnly)
				.SelectAllTextWhenFocused(true)
				.Font(IDetailLayoutBuilder::GetDetailFont());

			static const FText MemberNameToolTipFormat = LOCTEXT("GraphMember_NameDescriptionFormat", "Name used within the MetaSounds editor(s) and transacting systems (ex. Blueprints) if applicable to reference the given {0}.");
			FDetailWidgetRow& NameRow = CategoryBuilder.AddCustomRow(LOCTEXT("GraphMember_NameProperty", "Name"))
				.EditCondition(!bIsReadOnly, nullptr)
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(GraphMember->GetGraphMemberLabel())
					.ToolTipText(FText::Format(MemberNameToolTipFormat, GraphMember->GetGraphMemberLabel()))
				]
				.ValueContent()
				[
					NameEditableTextBox.ToSharedRef()
				];
			GenerateCopyPasteActions(NameRow, GetName().ToString());

			static const FText MemberDisplayNameText = LOCTEXT("GraphMember_DisplayNameProperty", "Display Name");
			static const FText MemberDisplayNameToolTipFormat = LOCTEXT("GraphMember_DisplayNameDescriptionFormat", "Optional, localized name used within the MetaSounds editor(s) to describe the given {0}.");
			const FText MemberDisplayNameTooltipText = FText::Format(MemberDisplayNameToolTipFormat, GraphMember->GetGraphMemberLabel());

			TSharedRef<FGraphMemberEditableTextDisplayName> DisplayNameValueText = MakeShared<FGraphMemberEditableTextDisplayName>(GraphMember, MemberDisplayNameTooltipText);
			FDetailWidgetRow& DisplayNameRow = CategoryBuilder.AddCustomRow(MemberDisplayNameText)
				.EditCondition(!bIsReadOnly, nullptr)
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(MemberDisplayNameText)
					.ToolTipText(MemberDisplayNameTooltipText)
				]
				.ValueContent()
				[
					SNew(STextPropertyEditableTextBox, DisplayNameValueText)
					.WrapTextAt(500)
					.MinDesiredWidth(25.0f)
					.MaxDesiredHeight(200)
				];
			GenerateCopyPasteActions(DisplayNameRow, DisplayNameValueText->GetText(0).ToString());

			static const FText MemberDescriptionText = LOCTEXT("Member_DescriptionPropertyName", "Description");
			static const FText MemberDescriptionToolTipFormat = LOCTEXT("Member_DescriptionToolTipFormat", "Description for {0}. For example, used as a tooltip when displayed on another graph's referencing node.");
			const FText MemberDescriptionToolTipText = FText::Format(MemberDescriptionToolTipFormat, GraphMember->GetGraphMemberLabel());
			TSharedRef<FGraphMemberEditableTextDescription> DescriptionValueText = MakeShared<FGraphMemberEditableTextDescription>(GraphMember, MemberDescriptionToolTipText);
			FDetailWidgetRow& DescriptionRow = CategoryBuilder.AddCustomRow(MemberDescriptionText)
				.IsEnabled(true)
				.EditCondition(!bIsReadOnly, nullptr)
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(MemberDescriptionText)
					.ToolTipText(MemberDescriptionToolTipText)
				]
				.ValueContent()
				[
					SNew(STextPropertyEditableTextBox, DescriptionValueText)
					.WrapTextAt(500)
					.MinDesiredWidth(25.0f)
					.MaxDesiredHeight(200)
				];
			GenerateCopyPasteActions(DescriptionRow, DescriptionValueText->GetText(0).ToString());

			DataTypeSelector.AddDataTypeSelector(InDetailLayout, MemberCustomizationStyle::DataTypeNameText, GraphMember, !bIsReadOnly);
		}

		void FMetasoundMemberDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
		{
			CacheMemberData(InDetailLayout);
			if (GraphMember.IsValid())
			{
				CustomizeGeneralCategory(InDetailLayout);
				CustomizeDefaultCategory(InDetailLayout);
			}
		}

		void FMetasoundMemberDetailCustomization::OnNameChanged(const FText& InNewName)
		{
			using namespace Frontend;

			bIsNameInvalid = false;
			NameEditableTextBox->SetError(FText::GetEmpty());

			if (!ensure(GraphMember.IsValid()))
			{
				return;
			}

			FText Error;
			if (!GraphMember->CanRename(InNewName, Error))
			{
				bIsNameInvalid = true;
				NameEditableTextBox->SetError(Error);
			}
		}

		FMetaSoundFrontendDocumentBuilder* FMetasoundMemberDetailCustomization::GetBuilder() const
		{
			if (UMetasoundEditorGraphMember* MemberPtr = GraphMember.Get())
			{
				FMetaSoundFrontendDocumentBuilder& DocBuilder = MemberPtr->GetFrontendBuilderChecked();
				if (DocBuilder.IsValid())
				{
					return &DocBuilder;
				}
			}

			return nullptr;
		}

		FText FMetasoundMemberDetailCustomization::GetName() const
		{
			if (GraphMember.IsValid())
			{
				return FText::FromName(GraphMember->GetMemberName());
			}

			return FText::GetEmpty();
		}

		bool FMetasoundMemberDetailCustomization::IsGraphEditable() const
		{
			if (GraphMember.IsValid())
			{
				if (const UMetasoundEditorGraph* OwningGraph = GraphMember->GetOwningGraph())
				{
					return OwningGraph->IsEditable();
				}
			}

			return false;
		}

		FText FMetasoundMemberDetailCustomization::GetDisplayName() const
		{
			using namespace Frontend;

			if (GraphMember.IsValid())
			{
				return GraphMember->GetDisplayName();
			}

			return FText::GetEmpty();
		}

		void FMetasoundMemberDetailCustomization::OnTooltipCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
		{
			using namespace Frontend;

			if (GraphMember.IsValid())
			{
				constexpr bool bPostTransaction = true;
				GraphMember->SetDescription(InNewText, bPostTransaction);
			}
		}

		FText FMetasoundMemberDetailCustomization::GetTooltip() const
		{
			if (GraphMember.IsValid())
			{
				return GraphMember->GetDescription();
			}

			return FText::GetEmpty();
		}

		const Frontend::FTemplateVertexMetadata* FMetasoundVertexDetailCustomization::GetConstTemplateMetadata() const
		{
			using namespace Frontend;

			if (UMetasoundEditorGraphMember* MemberPtr = GraphMember.Get())
			{
				if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetBuilder())
				{
					return DocBuilder->FindConstVertexMetadata<FTemplateVertexMetadata>(MemberPtr->GetMemberID());
				}
			}

			return nullptr;
		}

		Frontend::FTemplateVertexMetadata* FMetasoundVertexDetailCustomization::GetTemplateMetadata() const
		{
			using namespace Frontend;

			if (UMetasoundEditorGraphMember* MemberPtr = GraphMember.Get())
			{
				if (const FMetaSoundFrontendDocumentBuilder* DocBuilder = GetBuilder())
				{
					return DocBuilder->FindVertexMetadata<FTemplateVertexMetadata>(MemberPtr->GetMemberID());
				}
			}

			return nullptr;
		}

		EVisibility FMetasoundVertexDetailCustomization::GetDefaultVisibility() const
		{
			using namespace Frontend;

			if (GraphMember.IsValid())
			{
				bool bIsInputConnected = false;
				FConstNodeHandle NodeHandle = CastChecked<UMetasoundEditorGraphVertex>(GraphMember)->GetConstNodeHandle();
				if (NodeHandle->IsValid())
				{
					NodeHandle->IterateConstInputs([&bIsInputConnected](FConstInputHandle InputHandle)
					{
						bIsInputConnected |= InputHandle->IsConnectionUserModifiable() && InputHandle->IsConnected();
					});
				}
				return bIsInputConnected ? EVisibility::Collapsed : EVisibility::Visible;
			}

			return EVisibility::Collapsed;
		}

		void FMetasoundMemberDetailCustomization::OnNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit)
		{
			using namespace Frontend;

			if (!bIsNameInvalid && GraphMember.IsValid())
			{
				if (GraphMember->GetMemberName() == InNewName.ToString())
				{
					return;
				}

				const FText SetMemberNameTransactionLabelFormat = LOCTEXT("Commit_RenameGraphVertexMemberNameFormat", "Set MetaSound {0} Namespace and Name from '{1}' to '{2}' (DisplayName cleared)");
				const FText TransactionLabel = FText::Format(SetMemberNameTransactionLabelFormat, GraphMember->GetGraphMemberLabel(), FText::FromName(GraphMember->GetMemberName()), InNewName);
				const FScopedTransaction Transaction(TransactionLabel);

				constexpr bool bPostTransaction = false;
				GraphMember->SetDisplayName(FText::GetEmpty(), bPostTransaction);
				GraphMember->SetMemberName(*InNewName.ToString(), bPostTransaction);
			}

			NameEditableTextBox->SetError(FText::GetEmpty());
			bIsNameInvalid = false;
		}

		void FMetasoundVertexDetailCustomization::AddConstructorPinRow(IDetailLayoutBuilder& InDetailLayout)
		{
			InDetailLayout.EditCategory("General").AddCustomRow(MemberCustomizationPrivate::ConstructorPinText)
			.IsEnabled(IsGraphEditable() && !IsInterfaceMember())
			.NameContent()
			[
				SNew(STextBlock)
				.Text(MemberCustomizationPrivate::ConstructorPinText)
				.ToolTipText(MemberCustomizationPrivate::ConstructorPinTooltip)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
			.ValueContent()
			[
				SAssignNew(ConstructorPinCheckbox, SCheckBox)
				.IsChecked_Lambda([this]()
				{
					if (UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get()))
					{
						return OnGetConstructorPinCheckboxState(Vertex);
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState)
				{
					if (UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get()))
					{
						OnConstructorPinStateChanged(Vertex, InNewState);
					}
				})
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
		}

#if WITH_EDITORONLY_DATA
		void FMetasoundVertexDetailCustomization::AddAdvancedPinRow(IDetailLayoutBuilder& InDetailLayout)
		{
			//only add row if input or output
			InDetailLayout.EditCategory("General").AddCustomRow(MemberCustomizationPrivate::AdvancedPinText)
				.IsEnabled(IsGraphEditable() && !IsInterfaceMember())
				.NameContent()
				[
					SNew(STextBlock)
						.Text(MemberCustomizationPrivate::AdvancedPinText)
						.ToolTipText(MemberCustomizationPrivate::AdvancedPinTooltip)
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			.ValueContent()
			[
				SAssignNew(AdvancedPinCheckbox, SCheckBox)
				.IsChecked_Lambda([this]()
				{
					if (UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get()))
					{
						return OnGetAdvancedPinCheckboxState(Vertex);
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState)
				{
					if (UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get()))
					{
						OnAdvancedPinStateChanged(Vertex, InNewState);
					}
				})
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
		}
#endif // WITH_EDITORONLY_DATA

		void FMetasoundVertexDetailCustomization::CustomizeGeneralCategory(IDetailLayoutBuilder& InDetailLayout)
		{
			FMetasoundMemberDetailCustomization::CustomizeGeneralCategory(InDetailLayout);
			UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get());
			if (!ensure(Vertex))
			{
				return;
			}

			// Constructor pin 
			Frontend::FDataTypeRegistryInfo DataTypeInfo;
			Frontend::IDataTypeRegistry::Get().GetDataTypeInfo(Vertex->GetDataType(), DataTypeInfo);
			if (DataTypeInfo.bIsConstructorType)
			{
				AddConstructorPinRow(InDetailLayout);
			}
			
			AddAdvancedPinRow(InDetailLayout);

			// Sort order
			IDetailCategoryBuilder& CategoryBuilder = FMetasoundMemberDetailCustomization::GetGeneralCategoryBuilder(InDetailLayout);
			TWeakObjectPtr<UMetasoundEditorGraphVertex> VertexPtr = Vertex;
			static const FText SortOrderText = LOCTEXT("Vertex_SortOrderPropertyName", "Sort Order");
			static const FText SortOrderToolTipFormat = LOCTEXT("Vertex_SortOrderToolTipFormat", "Sort Order for {0}. Used to organize pins in node view. The higher the number, the lower in the list.");
			const FText SortOrderToolTipText = FText::Format(SortOrderToolTipFormat, GraphMember->GetGraphMemberLabel());
			CategoryBuilder.AddCustomRow(SortOrderText)
				.EditCondition(IsGraphEditable(), nullptr)
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(SortOrderText)
					.ToolTipText(SortOrderToolTipText)
				]
				.ValueContent()
				[
					SNew(SNumericEntryBox<int32>)
					.Value_Lambda([VertexPtr]()
					{	
						if (VertexPtr.IsValid())
						{
							return VertexPtr->GetSortOrderIndex();
						}
						return 0;
					})
					.AllowSpin(false)
					.UndeterminedString(LOCTEXT("Vertex_SortOrder_MultipleValues", "Multiple"))
					.OnValueCommitted_Lambda([VertexPtr](int32 NewValue, ETextCommit::Type CommitInfo)
					{
						if (!VertexPtr.IsValid())
						{
							return;
						}

						const FText TransactionTitle = FText::Format(LOCTEXT("SetVertexSortOrderFormat", "Set MetaSound Graph {0} '{1}' SortOrder to {2}"),
							VertexPtr->GetGraphMemberLabel(),
							VertexPtr->GetDisplayName(),
							FText::AsNumber(NewValue));
						FScopedTransaction Transaction(TransactionTitle);

						UObject* MetaSoundObject = VertexPtr->GetOutermostObject();
						FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSoundObject);
						check(MetaSoundAsset);

						MetaSoundObject->Modify();
						MetaSoundAsset->GetGraphChecked().Modify();
						VertexPtr->Modify();

						VertexPtr->SetSortOrderIndex(NewValue);

						{
							Frontend::FMetaSoundAssetRegistrationOptions RegOptions = FGraphBuilder::GetDefaultRegistrationOptions();
							RegOptions.bForceViewSynchronization = true;
							FGraphBuilder::RegisterGraphWithFrontend(*MetaSoundObject, MoveTemp(RegOptions));
						}
					})
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}

		bool FMetasoundVertexDetailCustomization::IsInterfaceMember() const
		{
			if (GraphMember.IsValid())
			{
				return CastChecked<UMetasoundEditorGraphVertex>(GraphMember)->IsInterfaceMember();
			}

			return false;
		}

		ECheckBoxState FMetasoundVertexDetailCustomization::OnGetConstructorPinCheckboxState(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphVertex) const
		{
			if (InGraphVertex.IsValid())
			{
				return InGraphVertex->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Undetermined;
		}

#if WITH_EDITORONLY_DATA
		ECheckBoxState FMetasoundVertexDetailCustomization::OnGetAdvancedPinCheckboxState(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphVertex) const
		{
			if (InGraphVertex.IsValid())
			{
				FMetaSoundFrontendDocumentBuilder& DocumentBuilder = InGraphVertex->GetFrontendBuilderChecked();
				return DocumentBuilder.GetIsAdvancedDisplay(InGraphVertex->GetMemberName(), InGraphVertex->GetClassType()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Undetermined;
		}
#endif // WITH_EDITORONLY_DATA

		void FMetasoundVertexDetailCustomization::OnConstructorPinStateChanged(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphVertex, ECheckBoxState InNewState)
		{
			if (InGraphVertex.IsValid() && ConstructorPinCheckbox.IsValid())
			{
				EMetasoundFrontendVertexAccessType NewAccessType = InNewState == ECheckBoxState::Checked ? 
					EMetasoundFrontendVertexAccessType::Value : EMetasoundFrontendVertexAccessType::Reference;

				if (InGraphVertex->GetVertexAccessType() == NewAccessType)
				{
					return;
				}

				// Have to stop playback to avoid attempting to change live edit data on invalid input type.
				check(GEditor);
				GEditor->ResetPreviewAudioComponent();

				InGraphVertex->SetVertexAccessType(NewAccessType);
				
				if (FMetasoundAssetBase* MetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(GraphMember->GetOutermostObject()))
				{
					MetasoundAsset->GetModifyContext().AddMemberIDsModified({ GraphMember->GetMemberID() });
				}
			}
		}

#if WITH_EDITORONLY_DATA
		void FMetasoundVertexDetailCustomization::OnAdvancedPinStateChanged(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphVertex, ECheckBoxState InNewState)
		{
			if (InGraphVertex.IsValid() && AdvancedPinCheckbox.IsValid())
			{
				const bool Checked = InNewState == ECheckBoxState::Checked;
				InGraphVertex->SetIsAdvancedDisplay(Checked);

				if (FMetasoundAssetBase* MetasoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(GraphMember->GetOutermostObject()))
				{
					MetasoundAsset->GetModifyContext().AddMemberIDsModified({ GraphMember->GetMemberID() });
				}
			}
		}
#endif // WITH_EDITORONLY_DATA
		void FMetasoundInputDetailCustomization::ConfigureLiteralCustomization(FMetasoundDefaultLiteralCustomizationBase& InLiteralCustomization)
		{
			UMetasoundEditorGraphInput* InputMember = Cast<UMetasoundEditorGraphInput>(GraphMember.Get());
			if (!InputMember)
			{
				return;
			}

			TWeakObjectPtr<UMetasoundEditorGraphInput> EditorGraphInput(InputMember);
			InLiteralCustomization.SetResetOverride(FResetToDefaultOverride::Create(
				FResetToDefaultHandler::CreateLambda([EditorGraphInput](TSharedPtr<IPropertyHandle> PropertyHandle)
				{
					using namespace Frontend;

					if (!EditorGraphInput.IsValid())
					{
						return;
					}

					// Navigate from value property to parent element to get Page ID
					FGuid PageID = DefaultPageID;
					if (PropertyHandle)
					{
						if (TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle())
						{
							if (TSharedPtr<IPropertyHandle> PageIDHandle = ParentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetasoundEditorMemberPageDefault, PageID)))
							{
								PageID = LiteralCustomization::GetGuidPropertyValue(PageIDHandle);
							}
						}
					}

					UMetasoundEditorGraphMemberDefaultLiteral* Literal = EditorGraphInput->GetLiteral();
					if (ensure(Literal))
					{
						FMetaSoundFrontendDocumentBuilder& Builder = EditorGraphInput->GetFrontendBuilderChecked();
						UObject& MetaSound = Builder.CastDocumentObjectChecked<UObject>();
						const FName InputName = EditorGraphInput->GetMemberName();

						// If preset, reset to the inherited parent default (but keep it as an override rather than removing the paged value) 
						// rather than resetting to the type's default (ex. 0 for floats)
						const FMetasoundFrontendLiteral* InheritedLiteral = nullptr;
						if (const FMetaSoundFrontendPresetTemplate* PresetTemplate = Builder.GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>())
						{
							if (PresetTemplate->Parent.GetObject())
							{
								const FMetaSoundFrontendDocumentBuilder& ParentBuilder =
									IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(PresetTemplate->Parent);
								if (const FMetasoundFrontendClassInput* ParentInput = ParentBuilder.FindGraphInput(InputName))
								{
									InheritedLiteral = ParentInput->FindConstDefault(PageID);
									if (!InheritedLiteral)
									{
										InheritedLiteral = ParentInput->FindConstDefault(DefaultPageID);
									}
								}
							}
						}

						MetaSound.Modify();
						Literal->Modify();

						if (InheritedLiteral)
						{
							Literal->SetFromLiteral(*InheritedLiteral, PageID);
						}
						else
						{
							Literal->InitDefault(PageID);
						}

						constexpr bool bPostTransaction = false;
						EditorGraphInput->UpdateFrontendDefaultLiteral(bPostTransaction, &PageID);
						FGraphBuilder::RegisterGraphWithFrontend(MetaSound);
					}
				})
			));
		}
		void FMetasoundInputDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
		{
			using namespace Frontend;

			CacheMemberData(InDetailLayout);
			if (!GraphMember.IsValid())
			{
				return;
			}

			CustomizeGeneralCategory(InDetailLayout);

			UMetasoundEditorGraphMemberDefaultLiteral* MemberDefaultLiteral = GraphMember->GetLiteral();
			if (!MemberDefaultLiteral)
			{
				return;
			}

			FMetaSoundFrontendDocumentBuilder* DocBuilder = GetBuilder();
			if (!DocBuilder)
			{
				return;
			}

			TStructView<FMetaSoundFrontendDocumentTemplate> TemplateStruct = DocBuilder->GetDocumentTemplate();
			if (FDocumentVertexTemplate* Template = TemplateStruct.GetPtr<FDocumentVertexTemplate>())
			{
				const FGuid& MemberID = GraphMember->GetMemberID();
				if (FTemplateVertexMetadata* Metadata = Template->FindVertexMetadata(MemberID))
				{
					const UScriptStruct* StructClass = Template->GetVertexMetadataStruct(MemberID);
					if (const TSharedPtr<FStructOnScope> DataView = MakeShared<FStructOnScope>(StructClass, reinterpret_cast<uint8*>(Metadata)))
					{
						TArray<TSharedPtr<IPropertyHandle>> StructPropHandles = GetDefaultCategoryBuilder(InDetailLayout).AddAllExternalStructureProperties(DataView.ToSharedRef());

						const auto OnTemplatePreChange = TDelegate<void()>::CreateSPLambda(AsShared(), [this]()
						{
							if (GEditor && GraphMember.IsValid())
							{
								if (UMetasoundEditorGraphMemberDefaultLiteral* MemberDefaultLiteral = GraphMember->GetLiteral())
								{
									GEditor->BeginTransaction(LOCTEXT("GraphMemberDocTemplate_Transact", "Set GraphMember Template Vertex Metadata"));
									GraphMember->GetOutermost()->Modify();
									GraphMember->GetOutermostObject()->Modify();
									GraphMember->Modify();
									MemberDefaultLiteral->Modify();
								}
							}
						});

						const auto OnTemplatePropChanged = TDelegate<void(const FPropertyChangedEvent&)>::CreateSPLambda(AsShared(), [this](const FPropertyChangedEvent& InEvent)
						{
							if (FTemplateVertexMetadata* ThisMetadata = GetTemplateMetadata())
							{
								if (FMetaSoundFrontendDocumentBuilder* DocBuilder = GetBuilder())
								{
									if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(GraphMember.Get()))
									{
										if (const FMetasoundFrontendClassVertex* InputVertex = Input->GetFrontendClassVertex())
										{
											ThisMetadata->OnPropertyChanged(InEvent, *DocBuilder, *InputVertex);
										}
										
										if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Input->GetLiteral())
										{
											Literal->ForceRefresh();
										}

										if (UObject* Metasound = Input->GetOutermostObject())
										{
											FGraphBuilder::RegisterGraphWithFrontend(*Metasound);
										}
									}

									if (GEditor)
									{
										GEditor->EndTransaction();
									}
								}
							}
						});

						for (TSharedPtr<IPropertyHandle> StructPropHandle : StructPropHandles)
						{
// 							Row->ShouldAutoExpand(true);
							StructPropHandle->SetOnPropertyValuePreChange(OnTemplatePreChange);
							StructPropHandle->SetOnPropertyValueChangedWithData(OnTemplatePropChanged);
						}
					}
				}
			}

			Enabled = TAttribute<bool>::CreateSPLambda(AsShared(), [this]
			{
				using namespace Frontend;
				if (const FTemplateVertexMetadata* VertexMetadata = GetConstTemplateMetadata())
				{
					return VertexMetadata->IsDefaultEditable();
				}

				if (GraphMember.IsValid())
				{
					if (UMetasoundEditorGraphMemberDefaultLiteral* MemberDefaultLiteral = GraphMember->GetLiteral())
					{
						// Make default value uneditable while playing for constructor inputs
						const UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(MemberDefaultLiteral->FindMember());
						if (Input)
						{
							if (Input->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Value)
							{
								UObject* MetaSoundObject = Input->GetOutermostObject();
								if (TSharedPtr<FEditor> MetaSoundEditor = FGraphBuilder::GetEditorForMetasound(*MetaSoundObject))
								{
									return !MetaSoundEditor->IsPlaying();
								}
							}
						}
					}
				}
				return true;
			});

			CustomizeDefaultCategory(InDetailLayout);
		}

		TAttribute<bool> FMetasoundInputDetailCustomization::GetEnabled() const
		{
			return Enabled;
		}

		EVisibility FMetasoundVariableDetailCustomization::GetDefaultVisibility() const
		{
			using namespace Frontend;

			if (GraphMember.IsValid())
			{
				bool bIsInputConnected = false;
				const UMetasoundEditorGraphVariable* Variable = CastChecked<UMetasoundEditorGraphVariable>(GraphMember);
				FConstNodeHandle NodeHandle = Variable->GetConstVariableHandle()->FindMutatorNode();
				if (NodeHandle->IsValid())
				{
					NodeHandle->IterateConstInputs([&bIsInputConnected](FConstInputHandle InputHandle)
					{
						bIsInputConnected |= InputHandle->IsConnectionUserModifiable() && InputHandle->IsConnected();
					});
				}
				return bIsInputConnected ? EVisibility::Collapsed : EVisibility::Visible;
			}

			return EVisibility::Collapsed;
		}

		bool FMetaSoundNodeExtensionHandler::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
		{
			return InObjectClass == UMetasoundEditorGraphMemberDefaultObjectArray::StaticClass();
		}

		void FMetaSoundNodeExtensionHandler::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			MemberCustomizationPrivate::CreateDefaultValueObjectCopyPasteActions(InWidgetRow, PropertyHandle);
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
