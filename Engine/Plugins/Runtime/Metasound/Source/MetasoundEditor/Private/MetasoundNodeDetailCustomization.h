// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/Widget.h"
#include "Delegates/IDelegateInstance.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailCustomization.h"
#include "IDetailPropertyExtensionHandler.h"
#include "IDetailPropertyRow.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "MetasoundAssetBase.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundUObjectRegistry.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "STextPropertyEditableTextBox.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "WorkflowOrientedApp/SModeWidget.h"

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound
{
	namespace Editor
	{
		class FGraphMemberEditableTextBase : public IEditableTextProperty
		{
		protected:
			TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMember;
			FText ToolTip;

		public:
			FGraphMemberEditableTextBase(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, const FText& InToolTip)
				: GraphMember(InGraphMember)
				, ToolTip(InToolTip)
			{
			}

			virtual ~FGraphMemberEditableTextBase() = default;

			virtual bool IsMultiLineText() const override { return true; }
			virtual bool IsPassword() const override { return false; }
			virtual bool IsReadOnly() const override { return false; }
			virtual int32 GetNumTexts() const override { return 1; }
			virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const override { return true; }

			virtual FText GetToolTipText() const override
			{
				return ToolTip;
			}

			virtual bool IsDefaultValue() const override
			{
				return GetText(0).EqualTo(FText::GetEmpty());
			}

#if USE_STABLE_LOCALIZATION_KEYS
			virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const override
			{
				check(InIndex == 0);
				StaticStableTextId(GraphMember->GetPackage(), InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
			}
#endif // USE_STABLE_LOCALIZATION_KEYS

		};

		class FGraphMemberEditableTextDescription : public FGraphMemberEditableTextBase
		{
		public:
			FGraphMemberEditableTextDescription(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, const FText& InToolTip)
				: FGraphMemberEditableTextBase(InGraphMember, InToolTip)
			{
			}

			virtual ~FGraphMemberEditableTextDescription() = default;

			virtual FText GetText(const int32 InIndex) const override
			{
				check(InIndex == 0);

				if (GraphMember.IsValid())
				{
					return GraphMember->GetDescription();
				}

				return FText::GetEmpty();
			}

			virtual void SetText(const int32 InIndex, const FText& InText) override
			{
				check(InIndex == 0);

				if (GraphMember.IsValid())
				{
					GraphMember->SetDescription(InText, true);
				}
			}
		};

		class FGraphMemberEditableTextDisplayName : public FGraphMemberEditableTextBase
		{
		public:
			FGraphMemberEditableTextDisplayName(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, const FText& InToolTip)
				: FGraphMemberEditableTextBase(InGraphMember, InToolTip)
			{
			}

			virtual ~FGraphMemberEditableTextDisplayName() = default;

			virtual FText GetText(const int32 InIndex) const override
			{
				check(InIndex == 0);

				if (GraphMember.IsValid())
				{
					if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(GraphMember.Get()))
					{
						return Vertex->GetConstNodeHandle()->GetDisplayName();
					}

					if (const UMetasoundEditorGraphVariable* Variable = Cast<UMetasoundEditorGraphVariable>(GraphMember.Get()))
					{
						return Variable->GetConstVariableHandle()->GetDisplayName();
					}

					return GraphMember->GetDisplayName();
				}

				return FText::GetEmpty();
			}

			virtual void SetText(const int32 InIndex, const FText& InText) override
			{
				check(InIndex == 0);

				if (GraphMember.IsValid())
				{
					// At future points in processing, the display name may be used as a member name and needs to match the length requirements of an FName.
					// The text edit box used does not have a public function to display an error and it would be difficult to notify to the user, thus 
					// we are just silently truncating the DisplayName text here. 
					GraphMember->SetDisplayName(FText::FromString(InText.ToString().Left(NAME_SIZE - 1)), true);
				}
			}
		};

		// TODO: Move to actual style
		namespace MemberCustomizationStyle
		{
			/** Maximum size of the details title panel */
			static constexpr float DetailsTitleMaxWidth = 300.f;
			/** magic number retrieved from SGraphNodeComment::GetWrapAt() */
			static constexpr float DetailsTitleWrapPadding = 32.0f;

			static const FText DataTypeNameText = LOCTEXT("Node_DataTypeName", "Type");
			static const FText DefaultPropertyText = LOCTEXT("Node_DefaultPropertyName", "Default");
		} // namespace MemberCustomizationStyle

		class FMetasoundFloatLiteralCustomization : public FMetasoundDefaultLiteralCustomizationBase
		{
			TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultFloat> FloatLiteral;

			// Delegate for clamping the input value or not
			FDelegateHandle OnClampChangedDelegateHandle;

		public:
			FMetasoundFloatLiteralCustomization(IDetailCategoryBuilder& InDefaultCategoryBuilder)
				: FMetasoundDefaultLiteralCustomizationBase(InDefaultCategoryBuilder)
			{
			}
			virtual ~FMetasoundFloatLiteralCustomization();

			virtual void CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout) override;
		};

		class FMetasoundBoolLiteralCustomization : public FMetasoundDefaultLiteralCustomizationBase
		{
			TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultBool> BoolLiteral;

		public:
			FMetasoundBoolLiteralCustomization(IDetailCategoryBuilder& InDefaultCategoryBuilder)
				: FMetasoundDefaultLiteralCustomizationBase(InDefaultCategoryBuilder)
			{
			}
			virtual ~FMetasoundBoolLiteralCustomization();

			virtual void CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout) override;
			virtual TAttribute<EVisibility> GetDefaultVisibility() const override;
		};

		// Customization to support drag-and-drop of Proxy UObject types on underlying members that are structs.
		// Struct ownership of objects required to customize asset filters based on dynamic UObject MetaSound Registry DataTypes.
		class FMetasoundObjectArrayLiteralCustomization : public FMetasoundDefaultLiteralCustomizationBase
		{
		public:
			FMetasoundObjectArrayLiteralCustomization(IDetailCategoryBuilder& InDefaultCategoryBuilder)
				: FMetasoundDefaultLiteralCustomizationBase(InDefaultCategoryBuilder)
			{
			}

			virtual ~FMetasoundObjectArrayLiteralCustomization() = default;

			virtual void BuildDefaultValueWidget(IDetailPropertyRow& ValueRow, TSharedPtr<IPropertyHandle> ValueProperty) override;
			virtual void CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout) override;
		};

		class FMetasoundDefaultLiteralCustomizationFactory : public IMemberDefaultLiteralCustomizationFactory
		{
		public:
			virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateLiteralCustomization(IDetailCategoryBuilder& DefaultCategoryBuilder) const override
			{
				return TUniquePtr<FMetasoundDefaultLiteralCustomizationBase>(new FMetasoundDefaultLiteralCustomizationBase(DefaultCategoryBuilder));
			}
		};

		// Customization to support float widgets (ex. sliders, knobs)
		class FMetasoundFloatLiteralCustomizationFactory : public IMemberDefaultLiteralCustomizationFactory
		{
		public:
			virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateLiteralCustomization(IDetailCategoryBuilder& DefaultCategoryBuilder) const override
			{
				return TUniquePtr<FMetasoundDefaultLiteralCustomizationBase>(new FMetasoundFloatLiteralCustomization(DefaultCategoryBuilder));
			}
		};

		// Customization to support bool widgets (ex. Buttons)
		class FMetasoundBoolLiteralCustomizationFactory : public IMemberDefaultLiteralCustomizationFactory
		{
		public:
			virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateLiteralCustomization(IDetailCategoryBuilder& DefaultCategoryBuilder) const override
			{
				return TUniquePtr<FMetasoundDefaultLiteralCustomizationBase>(new FMetasoundBoolLiteralCustomization(DefaultCategoryBuilder));
			}
		};

		class FMetasoundObjectArrayLiteralCustomizationFactory : public IMemberDefaultLiteralCustomizationFactory
		{
		public:
			virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateLiteralCustomization(IDetailCategoryBuilder& DefaultCategoryBuilder) const override
			{
				return TUniquePtr<FMetasoundDefaultLiteralCustomizationBase>(new FMetasoundObjectArrayLiteralCustomization(DefaultCategoryBuilder));
			}
		};

		class FMetasoundDefaultMemberElementDetailCustomizationBase : public IPropertyTypeCustomization
		{
		public:
			//~ Begin IPropertyTypeCustomization
			virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
			virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
			//~ End IPropertyTypeCustomization

		protected:
			virtual FText GetPropertyNameOverride() const { return FText::GetEmpty(); }

			// TODO: Merge with FMetasoundDefaultLiteralCustomizationBaseFactory
			virtual TSharedRef<SWidget> CreateStructureWidget(TSharedPtr<IPropertyHandle>& PropertyHandle) const = 0;

			Frontend::FDataTypeRegistryInfo DataTypeInfo;

		private:
			TSharedRef<SWidget> CreateNameWidget(TSharedPtr<IPropertyHandle> StructPropertyHandle) const;
			TSharedRef<SWidget> CreateValueWidget(TSharedPtr<IPropertyHandleArray> ParentArrayProperty, TSharedPtr<IPropertyHandle> StructPropertyHandle) const;
		};

		class FMetasoundMemberDefaultBoolDetailCustomization : public FMetasoundDefaultMemberElementDetailCustomizationBase
		{
		protected:
			virtual FText GetPropertyNameOverride() const override;
			virtual TSharedRef<SWidget> CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const override;
		};

		class FMetasoundMemberDefaultIntDetailCustomization : public FMetasoundDefaultMemberElementDetailCustomizationBase
		{
		protected:
			virtual TSharedRef<SWidget> CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const override;
		};

		class FMetasoundMemberDefaultObjectDetailCustomization : public FMetasoundDefaultMemberElementDetailCustomizationBase
		{
		protected:
			virtual TSharedRef<SWidget> CreateStructureWidget(TSharedPtr<IPropertyHandle>& StructPropertyHandle) const override;
		};

		class FMetasoundDataTypeSelector
		{
		public:
			void AddDataTypeSelector(IDetailLayoutBuilder& InDetailLayoutBuilder, const FText& InRowName, TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, bool bInIsInterfaceMember);
			void OnDataTypeArrayChanged(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember, ECheckBoxState InNewState);

		protected:
			ECheckBoxState OnGetDataTypeArrayCheckState(TWeakObjectPtr<UMetasoundEditorGraphMember> InGraphMember) const;
			void OnDataTypeSelected(FName InSelectedTypeName);
			FName GetDataType() const;
		
		private:
			TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMember;
			TSharedPtr<SCheckBox> DataTypeArrayCheckbox;
			TSharedPtr<SSearchableComboBox> DataTypeComboBox;
			TArray<TSharedPtr<FString>> ComboOptions;

			FName BaseTypeName;
			FName ArrayTypeName;
		};

		class FMetaSoundNodeDetailCustomization : public IDetailCustomization
		{
			// IDetailCustomization interface
			virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
			// End of IDetailCustomization interface
		};

		class FMetasoundMemberDetailCustomization : public IDetailCustomization
		{
		public:
			virtual ~FMetasoundMemberDetailCustomization()
			{
				RenameRequestedHandle.Reset();
			}

		protected:
			static IDetailCategoryBuilder& GetDefaultCategoryBuilder(IDetailLayoutBuilder& InDetailLayout)
			{
				return InDetailLayout.EditCategory("DefaultValue");
			}

			static IDetailCategoryBuilder& GetGeneralCategoryBuilder(IDetailLayoutBuilder& InDetailLayout)
			{
				return InDetailLayout.EditCategory("General");
			}

			void UpdateRenameDelegate(UMetasoundEditorGraphMember& InMember);
			void CacheMemberData(IDetailLayoutBuilder& InDetailLayout);

			virtual void CustomizeDefaultCategory(IDetailLayoutBuilder& InDetailLayout);
			virtual void CustomizeGeneralCategory(IDetailLayoutBuilder& InDetailLayout);
			virtual void ConfigureLiteralCustomization(FMetasoundDefaultLiteralCustomizationBase&) { }

			virtual EVisibility GetDefaultVisibility() const { return EVisibility::Visible; }
			virtual TAttribute<bool> GetEnabled() const { return { }; }

			virtual bool IsInterfaceMember() const { return false; }

			// IDetailCustomization interface
			virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
			// End of IDetailCustomization interface

			void OnNameChanged(const FText& InNewName);
			FText GetName() const;
			bool IsGraphEditable() const;
			FText GetDisplayName() const;
			void OnTooltipCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
			FText GetTooltip() const;
			void OnNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);

		protected:
			FMetaSoundFrontendDocumentBuilder* GetBuilder() const;

			TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMember;

			TSharedPtr<SEditableTextBox> NameEditableTextBox;
			FMetasoundDataTypeSelector DataTypeSelector;

			bool bIsNameInvalid = false;

			FDelegateHandle RenameRequestedHandle;

		private:
			TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> LiteralCustomization;
		};

		class FMetasoundVertexDetailCustomization : public FMetasoundMemberDetailCustomization
		{
		public:
			virtual ~FMetasoundVertexDetailCustomization() = default;

		protected:
			virtual void CustomizeGeneralCategory(IDetailLayoutBuilder& InDetailLayout) override;
			virtual EVisibility GetDefaultVisibility() const;
			virtual bool IsInterfaceMember() const override;

			void AddConstructorPinRow(IDetailLayoutBuilder& InDetailLayout);
			ECheckBoxState OnGetConstructorPinCheckboxState(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphMember) const;
			void OnConstructorPinStateChanged(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphMember, ECheckBoxState InNewState);

			const Frontend::FTemplateVertexMetadata* GetConstTemplateMetadata() const;
			Frontend::FTemplateVertexMetadata* GetTemplateMetadata() const;

			TSharedPtr<SCheckBox> ConstructorPinCheckbox;

#if WITH_EDITORONLY_DATA
			void AddAdvancedPinRow(IDetailLayoutBuilder& InDetailLayout);
			ECheckBoxState OnGetAdvancedPinCheckboxState(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphMember) const;
			void OnAdvancedPinStateChanged(TWeakObjectPtr<UMetasoundEditorGraphVertex> InGraphMember, ECheckBoxState InNewState);

			TSharedPtr<SCheckBox> AdvancedPinCheckbox;
#endif // WITH_EDITORONLY_DATA

		};

		class FMetasoundInputDetailCustomization : public FMetasoundVertexDetailCustomization
		{
		public:
			virtual ~FMetasoundInputDetailCustomization() = default;
			
			virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
			virtual void ConfigureLiteralCustomization(FMetasoundDefaultLiteralCustomizationBase& InLiteralCustomization) override;
			virtual TAttribute<bool> GetEnabled() const override;

		private:
			TAttribute<bool> Enabled;
		};

		class FMetasoundVariableDetailCustomization : public FMetasoundMemberDetailCustomization
		{
			EVisibility GetDefaultVisibility() const override;
		};

		using FMetasoundOutputDetailCustomization = FMetasoundVertexDetailCustomization;

		class FMetaSoundNodeExtensionHandler : public IDetailPropertyExtensionHandler
		{
		public:
			// IDetailPropertyExtensionHandler interface
			virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;
			virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override;
		};
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
