// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundPinAudioInspector.h"

#include "AudioBusSubsystem.h"
#include "AudioDefines.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioOscilloscope.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetaSoundGraphPanelPinFactory.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"

namespace Metasound::Editor
{
	namespace FMetasoundPinAudioInspectorPrivate
	{
		UEdGraphPin* ResolvePinObjectAsOutput(UEdGraphPin* InPin)
		{
			if (InPin && !InPin->LinkedTo.IsEmpty())
			{
				// Swap to show connected output if input (Only ever one)
				if (InPin->Direction == EGPD_Input)
				{
					InPin = InPin->LinkedTo.Last();
					check(InPin->Direction == EGPD_Output);
				}
			}

			return InPin;
		}
	}

	FMetasoundPinAudioInspector::FMetasoundPinAudioInspector(FEdGraphPinReference InPinRef)
		: GraphPinObj(FMetasoundPinAudioInspectorPrivate::ResolvePinObjectAsOutput(InPinRef.Get()))
	{
		using namespace Audio;

		// Initialize Oscilloscope
		const FDeviceId AudioDeviceId = GEditor->GetMainAudioDeviceID();

		constexpr int32 NumChannels = 1; // Audio wires are currently mono signals

		const FAudioOscilloscopePanelStyle* OscilloscopePanelStyle = nullptr;
		if (const ISlateStyle* StyleSet = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
		{
			OscilloscopePanelStyle = &StyleSet->GetWidgetStyle<FAudioOscilloscopePanelStyle>("AudioOscilloscope.PanelStyle");
		}

		Oscilloscope = MakeShared<AudioWidgets::FAudioOscilloscope>(AudioDeviceId,
			NumChannels,
			/*InTimeWindowMs*/     10.0f,
			/*InMaxTimeWindowMs*/  10.0f,
			/*InAnalysisPeriodMs*/ 10.0f,
			/*InPanelLayoutType*/  EAudioPanelLayoutType::Basic,
			/*InOscilloscopePanelStyle*/ OscilloscopePanelStyle
		);

		PinAudioInspectorWidget = SNew(SMetasoundPinAudioInspector)
			.VisualizationWidget(Oscilloscope->GetPanelWidget());

		Oscilloscope->StartProcessing();

		// Analyze Audio Pin
		if (GraphPinObj && !GraphPinObj->LinkedTo.IsEmpty() && GraphPinObj->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryAudio)
		{
			if (FGraphConnectionManager* ConnectionManager = GetConnectionManager())
			{
				const Frontend::FConstOutputHandle OutputHandle = FGraphBuilder::FindReroutedConstOutputHandleFromPin(GraphPinObj);
				const FGuid NodeID = OutputHandle->GetOwningNodeID();
				const FName OutputName = OutputHandle->GetName();
				AnalyzerInstanceID = ConnectionManager->AddAudioBusWriter(NodeID, OutputName, AudioDeviceId, Oscilloscope->GetAudioBus());
			}
		}
	}

	FMetasoundPinAudioInspector::~FMetasoundPinAudioInspector()
	{
		// Remove audio pin analyzer
		if (AnalyzerInstanceID.IsValid())
		{
			if (FGraphConnectionManager* ConnectionManager = GetConnectionManager())
			{
				ConnectionManager->RemoveAudioBusWriter(AnalyzerInstanceID);
				AnalyzerInstanceID.Invalidate();
			}
		}

		// Stop AudioBus
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			if (Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(EditorWorld->GetAudioDeviceRaw()))
			{
				if (UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>())
				{
					const uint32 AudioBusId = Oscilloscope->GetAudioBus()->GetUniqueID();
					AudioBusSubsystem->StopAudioBus(Audio::FAudioBusKey(AudioBusId));
				}
			}
		}
	}

	TSharedPtr<SMetasoundPinAudioInspector> FMetasoundPinAudioInspector::GetWidget()
	{
		return PinAudioInspectorWidget;
	}

	const UMetasoundEditorGraphNode& FMetasoundPinAudioInspector::GetReroutedNode() const
	{
		const UMetasoundEditorGraphNode* Node = FGraphBuilder::FindReroutedOutputPin(GraphPinObj)
			? Cast<UMetasoundEditorGraphNode>(GraphPinObj->GetOwningNode())
			: nullptr;
		check(Node);

		return *Node;
	}

	FGraphConnectionManager* FMetasoundPinAudioInspector::GetConnectionManager()
	{
		const UMetasoundEditorGraphNode& Node = GetReroutedNode();
		TSharedPtr<FEditor> Editor = FGraphBuilder::GetEditorForNode(Node);

		return Editor.IsValid() ? &Editor->GetConnectionManager() : nullptr;
	}
} // namespace Metasound::Editor

#undef LOCTEXT_NAMESPACE
