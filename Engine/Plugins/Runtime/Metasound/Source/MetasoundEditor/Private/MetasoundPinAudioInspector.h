// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/MultithreadedPatching.h"
#include "EdGraph/EdGraphPin.h"
#include "MetasoundEditorGraphConnectionManager.h"
#include "SMetasoundPinAudioInspector.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDEDITOR_API

namespace AudioWidgets
{
	class FAudioOscilloscope;
}

class UMetasoundEditorGraphNode;

namespace Metasound
{
	namespace Editor
	{
		class FMetasoundPinAudioInspector
		{
		public:
			UE_API explicit FMetasoundPinAudioInspector(FEdGraphPinReference InPinRef);
			UE_API virtual ~FMetasoundPinAudioInspector();

			UE_API TSharedPtr<SMetasoundPinAudioInspector> GetWidget();
			
		private:
			UE_API const UMetasoundEditorGraphNode& GetReroutedNode() const;
			UE_API FGraphConnectionManager* GetConnectionManager();

			UEdGraphPin* GraphPinObj = nullptr;

			TSharedPtr<AudioWidgets::FAudioOscilloscope> Oscilloscope = nullptr;
			TSharedPtr<SMetasoundPinAudioInspector> PinAudioInspectorWidget = nullptr;
			FGuid AnalyzerInstanceID;
		};
	} // namespace Editor
} // namespace Metasound

#undef UE_API
