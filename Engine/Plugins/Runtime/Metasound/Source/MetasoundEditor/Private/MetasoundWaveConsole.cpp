// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Audio/AudioDebug.h"
#include "MetasoundSource.h"

#if ENABLE_AUDIO_DEBUG

namespace Metasound
{
	namespace Console
	{
		static void HandleSoloMetaSound(const TArray<FString>& InArgs, UWorld* World)
		{
			check(GEngine);
			FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
			if (!DeviceManager)
			{
				return;
			}

			FAudioDeviceHandle AudioDevice = DeviceManager->GetActiveAudioDevice();
			if (!AudioDevice)
			{
				return;
			}

			TArray<FString> MetaSoundWaves;

			const int32 WorldID = World->GetUniqueID();
			const TArray<FActiveSound*>& ActiveSounds = AudioDevice->GetActiveSounds();
			for (const FActiveSound* const& ActiveSound : ActiveSounds)
			{
				if (WorldID == ActiveSound->GetWorldID())
				{
					// Make sure that sound asset is MetaSound based
					if (Cast<UMetaSoundSource>(ActiveSound->GetSound()))
					{
						for (const TPair<UPTRINT, FWaveInstance*>& WaveInstancePair : ActiveSound->GetWaveInstances())
						{
							const FWaveInstance* WaveInstance = WaveInstancePair.Value;
							MetaSoundWaves.Add(WaveInstance->GetName());
						}
					}
				}
			}

			if (InArgs.Num() == 1)
			{
				for (const FString& Wave : MetaSoundWaves)
				{
					// we can't use FAudioDebugger::SetSoloSoundWave(...) because we don't want to mute non MetaSounds
					const bool bMute = !Wave.Contains(InArgs[0], ESearchCase::IgnoreCase);
					DeviceManager->GetDebugger().SetMuteSoundWave(*Wave, bMute);
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "You can solo ONLY ONE MetaSound!");
			}
		}

		static void HandleMuteMetaSound(const TArray<FString>& InArgs, UWorld* World)
		{
			check(GEngine);
			FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
			if (!DeviceManager)
			{
				return;
			}

			FAudioDeviceHandle AudioDevice = DeviceManager->GetActiveAudioDevice();
			if (!AudioDevice)
			{
				return;
			}

			TArray<FString> MetaSoundWaves;

			const int32 WorldID = World->GetUniqueID();
			const TArray<FActiveSound*>& ActiveSounds = AudioDevice->GetActiveSounds();
			for (const FActiveSound* const& ActiveSound : ActiveSounds)
			{
				if (WorldID == ActiveSound->GetWorldID())
				{
					// Make sure that sound asset is MetaSound based
					if (Cast<UMetaSoundSource>(ActiveSound->GetSound()))
					{
						for (const TPair<UPTRINT, FWaveInstance*>& WaveInstancePair : ActiveSound->GetWaveInstances())
						{
							const FWaveInstance* WaveInstance = WaveInstancePair.Value;
							MetaSoundWaves.Add(WaveInstance->GetName());
						}
					}
				}
			}

			for (const FString& Arg : InArgs)
			{
				for (const FString& Wave : MetaSoundWaves)
				{
					if (Wave.Contains(Arg, ESearchCase::IgnoreCase))
					{
						DeviceManager->GetDebugger().SetMuteSoundWave(*Wave, true);
					}
				}
			}
		}
	} // namespace Console
} // namespace Metasound

static FAutoConsoleCommandWithWorldAndArgs SoloMetaSound
(
	TEXT("au.MetaSound.SoloMetaSound"),
	TEXT("Mutes all other MetaSounds. Only the first argument is accepted."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&Metasound::Console::HandleSoloMetaSound)
);

static FAutoConsoleCommandWithWorldAndArgs MuteMetaSound
(
	TEXT("au.MetaSound.MuteMetaSound"),
	TEXT("Mutes all given MetaSounds."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&Metasound::Console::HandleMuteMetaSound)
);

#endif // ENABLE_AUDIO_DEBUG
