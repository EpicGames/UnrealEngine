// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetasoundActionMenu.h"

#include "Framework/Application/SlateApplication.h"
#include "EdGraphSchema_K2.h"
#include "Styling/AppStyle.h"
#include "IDocumentation.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundFrontendDocument.h"
#include "SGraphActionMenu.h"
#include "SMetasoundPalette.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

#define LOCTEXT_NAMESPACE "MetaSoundEditor"


namespace Metasound
{
	namespace Editor
	{
		class SMetasoundActionFavoriteToggle : public SCompoundWidget
		{
			SLATE_BEGIN_ARGS(SMetasoundActionFavoriteToggle) {}
			SLATE_END_ARGS()

		public:
			void Construct(const FArguments& InArgs, const FCustomExpanderData& CustomExpanderData)
			{
				Container = CustomExpanderData.WidgetContainer;
				
				if (CustomExpanderData.RowAction.IsValid() && CustomExpanderData.RowAction->GetTypeId() == SchemaUtils::MetasoundGraphSchemaActionType)
				{
					ActionPtr = StaticCastSharedPtr<FMetasoundGraphSchemaAction>(CustomExpanderData.RowAction);
				}

				ChildSlot
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Center)
						.FillWidth(1.0)
					[
						SNew(SCheckBox)
							.Visibility(this, &SMetasoundActionFavoriteToggle::IsVisible)
							.ToolTipText(this, &SMetasoundActionFavoriteToggle::GetToolTipText)
							.IsChecked(this, &SMetasoundActionFavoriteToggle::GetFavoritedState)
							.OnCheckStateChanged(this, &SMetasoundActionFavoriteToggle::OnFavoriteToggled)
							.Style(FAppStyle::Get(), "Kismet.Palette.FavoriteToggleStyle")
					]
				];
			}

		private:
			/** The action that the owning palette entry represents */
			TWeakPtr<FMetasoundGraphSchemaAction> ActionPtr;

			/** The widget that this widget is nested inside */
			TSharedPtr<SPanel> Container;
			
			EVisibility IsVisible() const
			{
				EVisibility CurrentVisibility = EVisibility::Hidden;
				if (TSharedPtr<FMetasoundGraphSchemaAction> Action = ActionPtr.Pin())
				{
					if (Action->Grouping != static_cast<int32>(EPrimaryContextGroup::Common))
					{
						if (Container->IsHovered() || GetDefault<UMetasoundEditorSettings>()->IsFavoriteMetaSoundsGraphAction(Action->GetUniqueStableFavoriteKey()))
						{
							CurrentVisibility = EVisibility::Visible;
						}
					}
				}

				return CurrentVisibility;
			}

			/**
			 * Retrieves tooltip that describes the current favorited state of the 
			 * associated action.
			 * 
			 * @return Text describing what this toggle will do when you click on it.
			 */
			FText GetToolTipText() const
			{
				if (GetFavoritedState() == ECheckBoxState::Checked)
				{
					return LOCTEXT("Unfavorite", "Click to remove this item from your favorites.");
				}
				return LOCTEXT("Favorite", "Click to add this item to your favorites.");
			}

			ECheckBoxState GetFavoritedState() const
			{
				ECheckBoxState FavoriteState = ECheckBoxState::Unchecked;
				
				if (TSharedPtr<FMetasoundGraphSchemaAction> Action = ActionPtr.Pin())
				{
					FavoriteState = GetDefault<UMetasoundEditorSettings>()->IsFavoriteMetaSoundsGraphAction(Action->GetUniqueStableFavoriteKey()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				
				return FavoriteState;
			}

			void OnFavoriteToggled(ECheckBoxState InNewState)
			{
				if (TSharedPtr<FMetasoundGraphSchemaAction> Action = ActionPtr.Pin())
				{
					if (InNewState == ECheckBoxState::Checked)
					{
						GetMutableDefault<UMetasoundEditorSettings>()->AddFavoriteMetaSoundsGraphAction(Action->GetUniqueStableFavoriteKey());
					}
					else
					{
						GetMutableDefault<UMetasoundEditorSettings>()->RemoveFavoriteMetaSoundsGraphAction(Action->GetUniqueStableFavoriteKey());
					}
				}
			}
		};
		
		void SMetasoundActionMenuExpanderArrow::Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData)
		{
			OwnerRowPtr = ActionMenuData.TableRow;
			SetIndentAmount(InArgs._IndentAmount);
			ActionPtr = ActionMenuData.RowAction;

			if (ActionPtr.IsValid())
			{
				ChildSlot
				.Padding(TAttribute<FMargin>(this, &SMetasoundActionMenuExpanderArrow::GetCustomIndentPadding))
				[
					SNew(SMetasoundActionFavoriteToggle, ActionMenuData)
				];
			}
			else
			{
				SExpanderArrow::FArguments SuperArgs;
				SuperArgs._IndentAmount = InArgs._IndentAmount;

				SExpanderArrow::Construct(SuperArgs, ActionMenuData.TableRow);
			}
		}

		FMargin SMetasoundActionMenuExpanderArrow::GetCustomIndentPadding() const
		{
			return SExpanderArrow::GetExpanderPadding();
		}

		SMetasoundActionMenu::~SMetasoundActionMenu()
		{
			OnClosedCallback.ExecuteIfBound();
			OnCloseReasonCallback.ExecuteIfBound(bActionExecuted, false, !DraggedFromPins.IsEmpty());
		}

		void SMetasoundActionMenu::Construct(const FArguments& InArgs)
		{
			using namespace Metasound::Editor;
			using namespace Metasound::Frontend;

			Graph = InArgs._Graph;
			DraggedFromPins = InArgs._DraggedFromPins;
			NewNodePosition = InArgs._NewNodePosition;
			OnClosedCallback = InArgs._OnClosedCallback;
			bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
			OnCloseReasonCallback = InArgs._OnCloseReason;

			FSlateColor TypeColor;
			const FSlateBrush* PinBrush = nullptr;
			if (!DraggedFromPins.IsEmpty())
			{
				if (const UEdGraphPin* Pin = DraggedFromPins[0])
				{
					FDataTypeRegistryInfo RegistryInfo;

					if (Pin->Direction == EGPD_Input)
					{
						FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(Pin);
						IDataTypeRegistry::Get().GetDataTypeInfo(InputHandle->GetDataType(), RegistryInfo);
					}
					else
					{
						FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(Pin);
						IDataTypeRegistry::Get().GetDataTypeInfo(OutputHandle->GetDataType(), RegistryInfo);
					}

					TypeColor = FMetaSoundGraphPanelPinFactory::GetChecked()->GetPinColor(Pin->PinType);

					if (RegistryInfo.bIsArrayType)
					{
						PinBrush = FAppStyle::GetBrush("Graph.ArrayPin.Connected");
					}
					else if (Pin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryTrigger)
					{
						if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
						{
							PinBrush = MetasoundStyle->GetBrush(TEXT("MetasoundEditor.Graph.TriggerPin.Connected"));
						}
					}

					if (!PinBrush)
					{
						PinBrush = FAppStyle::GetBrush("Graph.Pin.Connected");
					}
				}
			}

			FText MenuTitle;
			if (DraggedFromPins.IsEmpty())
			{
				MenuTitle = LOCTEXT("ContextText", "All MetaSound Node Classes");
			}
			else if (DraggedFromPins[0]->Direction == EGPD_Input)
			{
				FConstInputHandle InputHandle = FGraphBuilder::GetConstInputHandleFromPin(DraggedFromPins[0]);
				MenuTitle = FText::Format(LOCTEXT("ContextTypeFilteredText_Output", "Classes with output of type '{0}'"), FText::FromName(InputHandle->GetDataType()));
			}
			else
			{
				FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(DraggedFromPins[0]);
				MenuTitle = FText::Format(LOCTEXT("ContextTypeFilteredText_Input", "Classes with input of type '{0}'"), FText::FromName(OutputHandle->GetDataType()));
			}

			SBorder::Construct(SBorder::FArguments()
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(5)
				[
					SNew(SBox)
					.WidthOverride(400)
					.HeightOverride(400)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2, 2, 2, 5)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0, 0, 5, 0)
							[
								SNew(SImage)
								.ColorAndOpacity(TypeColor)
								.Visibility(DraggedFromPins.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible)
								.Image(PinBrush)
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(MenuTitle)
								// TODO: Move to Metasound Style
								.Font(FAppStyle::GetFontStyle("BlueprintEditor.ActionMenu.ContextDescriptionFont"))
								.ToolTip(IDocumentation::Get()->CreateToolTip(
									LOCTEXT("ActionMenuContextTextTooltip", "Describes the current context of the action list"),
									nullptr,
									TEXT("Shared/Editors/MetasoundEditor"),
									TEXT("MetasoundActionMenuContextText")))
								.WrapTextAt(280)
							]
						]
						+SVerticalBox::Slot()
						[
							SAssignNew(GraphActionMenu, SGraphActionMenu)
								.OnActionSelected(this, &SMetasoundActionMenu::OnActionSelected)
								.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SMetasoundActionMenu::OnCreateWidgetForAction))
								.OnCollectAllActions(this, &SMetasoundActionMenu::CollectAllActions)
								.OnCreateCustomRowExpander_Lambda([](const FCustomExpanderData& InCustomExpanderData)
								{
									return SNew(SMetasoundActionMenuExpanderArrow, InCustomExpanderData);
								})
								.DraggedFromPins(DraggedFromPins)
								.GraphObj(Graph)
								.AlphaSortItems(true)
								.bAllowPreselectedItemActivation(true)
						]
					]
				]
			);
		}

		void SMetasoundActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
		{
			if (!Graph)
			{
				return;
			}

			FGraphContextMenuBuilder MenuBuilder(Graph);
			if (!DraggedFromPins.IsEmpty())
			{
				MenuBuilder.FromPin = DraggedFromPins[0];
			}

			// Cannot call GetGraphContextActions() during serialization and GC due to its use of FindObject()
			if(!UE::IsSavingPackage() && !IsGarbageCollecting())
			{
				if (const UMetasoundEditorGraphSchema* Schema = GetDefault<UMetasoundEditorGraphSchema>())
				{
					Schema->GetGraphContextActions(MenuBuilder);
				}
			}

			OutAllActions.Append(MenuBuilder);
		}

		TSharedRef<SEditableTextBox> SMetasoundActionMenu::GetFilterTextBox()
		{
			return GraphActionMenu->GetFilterTextBox();
		}

		TSharedRef<SWidget> SMetasoundActionMenu::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
		{
			using namespace Metasound;
			using namespace Metasound::Frontend;

			check(InCreateData);
			InCreateData->bHandleMouseButtonDown = false;

			const FSlateBrush* IconBrush = nullptr;
			FLinearColor IconColor;
			TSharedPtr<FMetasoundGraphSchemaAction> Action = StaticCastSharedPtr<FMetasoundGraphSchemaAction>(InCreateData->Action);
			if (Action.IsValid())
			{
				IconBrush = Action->GetIconBrush();
				IconColor = Action->GetIconColor();
			}

			TSharedPtr<SHorizontalBox> WidgetBox = SNew(SHorizontalBox);
			if (IconBrush)
			{
				WidgetBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(5, 0, 0, 0)
					[
						SNew(SImage)
						.ColorAndOpacity(IconColor)
						.Image(IconBrush)
						.DesiredSizeOverride(FVector2D(16.f, 16.f))
					];
			}

			WidgetBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 0, 0)
				[
					SNew(SGraphPaletteItem, InCreateData)
				];

			return WidgetBox->AsShared();
		}

		void SMetasoundActionMenu::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType)
		{
			if (!Graph)
			{
				return;
			}

			if (InSelectionType != ESelectInfo::OnMouseClick  && InSelectionType != ESelectInfo::OnKeyPress && !SelectedAction.IsEmpty())
			{
				return;
			}

			UMetasoundEditorSettings* Settings = GetMutableDefault<UMetasoundEditorSettings>();
			for (const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedAction)
			{
				if (Action.IsValid() && Graph)
				{
					if (!bActionExecuted && (Action->GetTypeId() != FEdGraphSchemaAction_Dummy::StaticGetTypeId()))
					{
						FSlateApplication::Get().DismissAllMenus();
						bActionExecuted = true;
						Settings->AddRecentlyUsedMetaSoundsGraphAction(StaticCastSharedPtr<FMetasoundGraphSchemaAction>(Action)->GetUniqueStableFavoriteKey());
					}

					if (Action->PerformAction(Graph, DraggedFromPins, NewNodePosition) != nullptr)
					{
						NewNodePosition += Frontend::DisplayStyle::NodeLayout::DefaultOffsetX;
					}
				}
			}
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
