// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GraphEditor.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "SGraphActionMenu.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SExpanderArrow.h"

class SEditableTextBox;
class SGraphActionMenu;
class UEdGraph;
struct FCustomExpanderData;

namespace Metasound
{
	namespace Editor
	{
		// Custom expander to specify our desired padding
		class SMetasoundActionMenuExpanderArrow : public SExpanderArrow
		{
			SLATE_BEGIN_ARGS(SMetasoundActionMenuExpanderArrow)
			{
			}

			SLATE_ATTRIBUTE(float, IndentAmount)
				SLATE_END_ARGS()

		public:
			void Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData);

		private:
			FMargin GetCustomIndentPadding() const;

			TWeakPtr<FEdGraphSchemaAction> ActionPtr;
		};

		class SMetasoundActionMenu : public SBorder
		{
		public:
			/** Delegate for the OnCloseReason event which is always raised when the SMetasoundActionMenu closes */
			DECLARE_DELEGATE_ThreeParams(FClosedReason, bool /*bActionExecuted*/, bool /*bContextSensitiveChecked*/, bool /*bGraphPinContext*/);

			SLATE_BEGIN_ARGS(SMetasoundActionMenu)
				: _Graph(static_cast<UEdGraph*>(nullptr))
				, _NewNodePosition(FVector2f::ZeroVector)
				, _AutoExpandActionMenu(false)
			{
			}
				SLATE_ARGUMENT(UEdGraph*, Graph)
				SLATE_ARGUMENT(FDeprecateSlateVector2D, NewNodePosition)
				SLATE_ARGUMENT(TArray<UEdGraphPin*>, DraggedFromPins)
				SLATE_ARGUMENT(SGraphEditor::FActionMenuClosed, OnClosedCallback)
				SLATE_ARGUMENT(bool, AutoExpandActionMenu)
				SLATE_EVENT(FClosedReason, OnCloseReason)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs);

			~SMetasoundActionMenu();

			TSharedRef<SEditableTextBox> GetFilterTextBox();

		protected:
			void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType);

			TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

			/** Callback used to populate all actions list in SGraphActionMenu */
			void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);

		private:
			UEdGraph* Graph = nullptr;
			bool bAutoExpandActionMenu = false;
			bool bActionExecuted = false;

			TArray<UEdGraphPin*> DraggedFromPins;
			FDeprecateSlateVector2D NewNodePosition = FVector2f::ZeroVector;

			SGraphEditor::FActionMenuClosed OnClosedCallback;
			FClosedReason OnCloseReasonCallback;

			TSharedPtr<SGraphActionMenu> GraphActionMenu;
		};
	} // namespace Editor
} // namespace Metasound
