// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundFilterFrequencyResponsePlots.h"

#include "Algo/Count.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SMetasoundFilterFrequencyResponsePlots"

namespace Metasound::Editor
{
	namespace SMetasoundFilterFrequencyResponsePlotsPrivate
	{
		inline bool IsConnectedAudioOutputPin(const UEdGraphPin* Pin)
		{
			const bool bIsPinConnected = !Pin->LinkedTo.IsEmpty();
			const bool bIsAudioOutputPin = Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryAudio;
			return (bIsAudioOutputPin && bIsPinConnected);
		}
	} // namespace SMetasoundFilterFrequencyResponsePlotsPrivate

	TSharedRef<SWidget> CreateMetaSoundBiquadFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams)
	{
		const FName FilterType(TEXT("Type"));
		const FName CutoffFrequency(TEXT("Cutoff Frequency"));
		const FName Bandwidth(TEXT("Bandwidth"));
		const FName GainDb(TEXT("Gain"));

		return SNew(SBox)
			.MinDesiredHeight(125.0f)
			[
				SNew(SMetaSoundBiquadFilterFrequencyResponsePlot)
					.FilterType_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<Audio::EBiquadFilter::Type>, FilterType)
					.CutoffFrequency_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, CutoffFrequency)
					.Bandwidth_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, Bandwidth)
					.GainDb_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, GainDb)
			];
	}

	TSharedRef<SWidget> CreateMetaSoundLadderFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams)
	{
		const FName CutoffFrequency(TEXT("Cutoff Frequency"));
		const FName Resonance(TEXT("Resonance"));

		return SNew(SBox)
			.MinDesiredHeight(125.0f)
			[
				SNew(SMetaSoundLadderFilterFrequencyResponsePlot)
					.CutoffFrequency_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, CutoffFrequency)
					.Resonance_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, Resonance)
			];
	}

	TSharedRef<SWidget> CreateMetaSoundOnePoleHighPassFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams)
	{
		const FName CutoffFrequency(TEXT("Cutoff Frequency"));

		return SNew(SBox)
			.MinDesiredHeight(125.0f)
			[
				SNew(SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot)
					.CutoffFrequency_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, CutoffFrequency)
			];
	}

	TSharedRef<SWidget> CreateMetaSoundOnePoleLowPassFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams)
	{
		const FName CutoffFrequency(TEXT("Cutoff Frequency"));

		return SNew(SBox)
			.MinDesiredHeight(125.0f)
			[
				SNew(SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot)
					.CutoffFrequency_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, CutoffFrequency)
			];
	}

	TSharedRef<SWidget> CreateMetaSoundStateVariableFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams)
	{
		const FName CutoffFrequency(TEXT("Cutoff Frequency"));
		const FName Resonance(TEXT("Resonance"));
		const FName BandStopControl(TEXT("Band Stop Control"));

		return SNew(SBox)
			.MinDesiredHeight(125.0f)
			[
				SNew(SMetaSoundStateVariableFilterFrequencyResponsePlot, InParams.MetaSoundNode)
					.CutoffFrequency_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, CutoffFrequency)
					.Resonance_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, Resonance)
					.BandStopControl_UObject(InParams.MetaSoundNode, &UMetasoundEditorGraphNode::GetPinVisualizationValue<float>, BandStopControl)
			];
	}


	const float SMetaSoundFilterFrequencyResponsePlot::SampleRate = 48000.0f;

	void SMetaSoundFilterFrequencyResponsePlot::Construct()
	{
		ChildSlot
			[
				SAssignNew(FrequencyResponsePlot, SAudioSpectrumPlot)
					.Clipping(EWidgetClipping::ClipToBounds)
					.ViewMinSoundLevel(-24.0f)
					.TiltExponent_Lambda([] { return 0.0f; }) // Binding this property has the effect of hiding its context menu entry (Tilting the spectrum is not desired here).
					.FrequencyAxisPixelBucketMode_Lambda([]() { return EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Sample; }) // Binding this property has the effect of hiding its context menu entry (PixelBucketMode is not much use here).
					.OnGetAudioSpectrumData(this, &SMetaSoundFilterFrequencyResponsePlot::GetAudioSpectrumData)
					.Style(FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"), "AudioSpectrumPlot.Style")
			];

		ContextMenuExtension = FrequencyResponsePlot->AddContextMenuExtension(EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMetaSoundFilterFrequencyResponsePlot::ExtendSpectrumPlotContextMenu));
	}

	SMetaSoundFilterFrequencyResponsePlot::~SMetaSoundFilterFrequencyResponsePlot()
	{
		if (ContextMenuExtension.IsValid())
		{
			FrequencyResponsePlot->RemoveContextMenuExtension(ContextMenuExtension.ToSharedRef());
		}
	}

	void SMetaSoundFilterFrequencyResponsePlot::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		bHasFilterParams = UpdateFilterParams();
	}

	FAudioPowerSpectrumData SMetaSoundFilterFrequencyResponsePlot::GetAudioSpectrumData()
	{
		if (!bHasFilterParams || !FrequencyResponsePlot.IsValid())
		{
			return FAudioPowerSpectrumData();
		}

		// Fill array of frequencies (in Hz) to be plotted (these may be log spaced or linear, as defined by the FrequencyResponsePlot ScaleInfo transform):
		CenterFrequencies.Reset();
		float LocalX = 0.0f;
		const float LocalXEnd = FrequencyResponsePlot->GetPaintSpaceGeometry().GetLocalSize().X;
		const FAudioSpectrumPlotScaleInfo ScaleInfo = FrequencyResponsePlot->GetScaleInfo();
		while (LocalX <= LocalXEnd)
		{
			const float Frequency = ScaleInfo.LocalXToFrequency(LocalX);
			CenterFrequencies.Add(Frequency);
			LocalX += 1.0f;
		}

		// Create array of complex z values for all desired frequencies (interleaved real and imaginary):
		const int32 NumFrequencies = CenterFrequencies.Num();
		TArray<float> ComplexValues;
		ComplexValues.SetNumUninitialized(2 * NumFrequencies);
		const float HzToOmega = UE_TWO_PI / SampleRate;
		for (int32 Index = 0; Index < NumFrequencies; Index++)
		{
			const float Omega = HzToOmega * CenterFrequencies[Index];
			FMath::SinCos(&ComplexValues[2 * Index + 1], &ComplexValues[2 * Index + 0], Omega);
		}

		// Get the frequency response for all of the frequencies:
		ArrayCalculateFilterResponseInPlace(ComplexValues);

		// Store frequency response as squared magnitudes:
		SquaredMagnitudes.SetNumUninitialized(NumFrequencies);
		Audio::ArrayComplexToPower(ComplexValues, SquaredMagnitudes);

		return FAudioPowerSpectrumData
		{
			.CenterFrequencies = CenterFrequencies,
			.SquaredMagnitudes = SquaredMagnitudes,
		};
	}



	void SMetaSoundBiquadFilterFrequencyResponsePlot::Construct(const FArguments& InArgs)
	{
		SMetaSoundFilterFrequencyResponsePlot::Construct();

		FilterType = InArgs._FilterType;
		CutoffFrequency = InArgs._CutoffFrequency;
		Bandwidth = InArgs._Bandwidth;
		GainDb = InArgs._GainDb;
	}

	bool SMetaSoundBiquadFilterFrequencyResponsePlot::UpdateFilterParams()
	{
		// Try and get all required filter params:
		const TOptional<Audio::EBiquadFilter::Type> FilterTypeValue = FilterType.Get();
		const TOptional<float> CutoffFrequencyValue = CutoffFrequency.Get();
		const TOptional<float> BandwidthValue = Bandwidth.Get();
		const TOptional<float> GainDbValue = GainDb.Get();
		if (!FilterTypeValue.IsSet() || !CutoffFrequencyValue.IsSet() || !BandwidthValue.IsSet() || !GainDbValue.IsSet())
		{
			return false;
		}

		const float MaxCutoffFrequency = 0.5f * SampleRate;
		const float CurrentFrequency = FMath::Clamp(*CutoffFrequencyValue, 0.f, MaxCutoffFrequency);
		const float CurrentBandwidth = FMath::Max(*BandwidthValue, 0.f);
		const float CurrentFilterGainDb = FMath::Clamp(*GainDbValue, -90.0f, 20.0f);

		// Update the filter with the filter params:
		if (Filter.GetNumChannels() == 0)
		{
			constexpr int32 NumChannels = 1;
			Filter.Init(SampleRate, NumChannels, static_cast<Audio::EBiquadFilter::Type>(*FilterTypeValue), CurrentFrequency, CurrentBandwidth, CurrentFilterGainDb);
		}
		else
		{
			Filter.SetParams(static_cast<Audio::EBiquadFilter::Type>(*FilterTypeValue), CurrentFrequency, CurrentBandwidth, CurrentFilterGainDb);
		}

		return true;
	}

	void SMetaSoundBiquadFilterFrequencyResponsePlot::ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		Filter.ArrayCalculateResponseInPlace(InOutComplexValues);
	}



	void SMetaSoundLadderFilterFrequencyResponsePlot::Construct(const FArguments& InArgs)
	{
		SMetaSoundFilterFrequencyResponsePlot::Construct();

		constexpr int32 NumChannels = 1;
		Filter.Init(SampleRate, NumChannels);

		CutoffFrequency = InArgs._CutoffFrequency;
		Resonance = InArgs._Resonance;
	}

	bool SMetaSoundLadderFilterFrequencyResponsePlot::UpdateFilterParams()
	{
		// Try and get all required filter params:
		const TOptional<float> CutoffFrequencyValue = CutoffFrequency.Get();
		const TOptional<float> ResonanceValue = Resonance.Get();
		if (!CutoffFrequencyValue.IsSet() || !ResonanceValue.IsSet())
		{
			return false;
		}

		const float MaxCutoffFrequency = 0.5f * SampleRate;
		const float CurrentFrequency = FMath::Clamp(*CutoffFrequencyValue, 0.f, MaxCutoffFrequency);
		const float CurrentResonance = FMath::Clamp(*ResonanceValue, 1.0f, 10.0f);

		// Update the filter with the filter params:
		Filter.SetQ(CurrentResonance);
		Filter.SetFrequency(CurrentFrequency);

		Filter.Update();

		return true;
	}

	void SMetaSoundLadderFilterFrequencyResponsePlot::ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		Filter.ArrayCalculateResponseInPlace(InOutComplexValues);
	}



	void SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot::Construct(const FArguments& InArgs)
	{
		SMetaSoundFilterFrequencyResponsePlot::Construct();

		constexpr int32 NumChannels = 1;
		Filter.Init(SampleRate, NumChannels);

		CutoffFrequency = InArgs._CutoffFrequency;
	}

	bool SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot::UpdateFilterParams()
	{
		// Try and get all required filter params:
		const TOptional<float> Frequency = CutoffFrequency.Get();
		if (!Frequency.IsSet())
		{
			return false;
		}

		const float ClampedFreq = FMath::Clamp(0.0f, *Frequency, SampleRate);

		// Update the filter with the filter params:
		Filter.StartFrequencyInterpolation(ClampedFreq);

		return true;
	}

	void SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot::ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		Filter.ArrayCalculateResponseInPlace(InOutComplexValues);
	}



	void SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot::Construct(const FArguments& InArgs)
	{
		SMetaSoundFilterFrequencyResponsePlot::Construct();

		constexpr int32 NumChannels = 1;
		Filter.Init(SampleRate, NumChannels);

		CutoffFrequency = InArgs._CutoffFrequency;
	}

	bool SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot::UpdateFilterParams()
	{
		// Try and get all required filter params:
		const TOptional<float> Frequency = CutoffFrequency.Get();
		if (!Frequency.IsSet())
		{
			return false;
		}

		const float ClampedFreq = FMath::Clamp(0.0f, *Frequency, SampleRate);

		// Update the filter with the filter params:
		Filter.StartFrequencyInterpolation(ClampedFreq);

		return true;
	}

	void SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot::ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		Filter.ArrayCalculateResponseInPlace(InOutComplexValues);
	}



	const FName SMetaSoundStateVariableFilterFrequencyResponsePlot::LowPassFilter(TEXT("Low Pass Filter"));
	const FName SMetaSoundStateVariableFilterFrequencyResponsePlot::HighPassFilter(TEXT("High Pass Filter"));
	const FName SMetaSoundStateVariableFilterFrequencyResponsePlot::BandPass(TEXT("Band Pass"));
	const FName SMetaSoundStateVariableFilterFrequencyResponsePlot::BandStop(TEXT("Band Stop"));

	SMetaSoundStateVariableFilterFrequencyResponsePlot::SMetaSoundStateVariableFilterFrequencyResponsePlot()
		: DisplayedFilterResponse(LowPassFilter)
	{
		constexpr int32 NumChannels = 1;
		Filter.Init(SampleRate, NumChannels);
	}

	void SMetaSoundStateVariableFilterFrequencyResponsePlot::Construct(const FArguments& InArgs, UMetasoundEditorGraphNode* InMetaSoundNode)
	{
		SMetaSoundFilterFrequencyResponsePlot::Construct();

		CutoffFrequency = InArgs._CutoffFrequency;
		Resonance = InArgs._Resonance;
		BandStopControl = InArgs._BandStopControl;

		MetaSoundNode = InMetaSoundNode;
	}

	bool SMetaSoundStateVariableFilterFrequencyResponsePlot::UpdateFilterParams()
	{
		using namespace SMetasoundFilterFrequencyResponsePlotsPrivate;

		const TStrongObjectPtr<UMetasoundEditorGraphNode> GraphNode = MetaSoundNode.Pin();
		if (!GraphNode.IsValid())
		{
			return false;
		}

		// If there are connected audio outputs but the current selection is not connected, auto-select a new filter type:
		UEdGraphPin** FirstConnectedAudioOutputPin = GraphNode->Pins.FindByPredicate(IsConnectedAudioOutputPin);
		if (FirstConnectedAudioOutputPin != nullptr)
		{
			for (UEdGraphPin* Pin : GraphNode->Pins)
			{
				const bool bIsPinConnected = !Pin->LinkedTo.IsEmpty();
				if (Pin->PinName == DisplayedFilterResponse && !bIsPinConnected)
				{
					DisplayedFilterResponse = (*FirstConnectedAudioOutputPin)->PinName;
				}
			}
		}

		// Set the filter type on the filter:
		if (DisplayedFilterResponse == LowPassFilter)
		{
			Filter.SetFilterType(Audio::EFilter::LowPass);
		}
		else if (DisplayedFilterResponse == HighPassFilter)
		{
			Filter.SetFilterType(Audio::EFilter::HighPass);
		}
		else if (DisplayedFilterResponse == BandPass)
		{
			Filter.SetFilterType(Audio::EFilter::BandPass);
		}
		else if (DisplayedFilterResponse == BandStop)
		{
			Filter.SetFilterType(Audio::EFilter::BandStop);
		}

		// Try and get all required filter params:
		const TOptional<float> CutoffFrequencyValue = CutoffFrequency.Get();
		const TOptional<float> ResonanceValue = Resonance.Get();
		const TOptional<float> BandStopControlValue = BandStopControl.Get();
		if (!CutoffFrequencyValue.IsSet() || !ResonanceValue.IsSet() || !BandStopControl.IsSet())
		{
			return false;
		}

		const float MaxCutoffFrequency = 0.5f * SampleRate;
		const float CurrentFrequency = FMath::Clamp(*CutoffFrequencyValue, 0.f, MaxCutoffFrequency);
		const float CurrentResonance = FMath::Clamp(*ResonanceValue, 0.f, 10.f);
		const float CurrentBandStopControl = FMath::Clamp(*BandStopControlValue, 0.f, 1.f);

		// Update the filter with the filter params:
		Filter.SetQ(CurrentResonance);
		Filter.SetFrequency(CurrentFrequency);
		Filter.SetBandStopControl(CurrentBandStopControl);

		Filter.Update();

		return true;
	}

	void SMetaSoundStateVariableFilterFrequencyResponsePlot::ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		Filter.ArrayCalculateResponseInPlace(InOutComplexValues);
	}

	void SMetaSoundStateVariableFilterFrequencyResponsePlot::ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder)
	{
		using namespace SMetasoundFilterFrequencyResponsePlotsPrivate;

		const TStrongObjectPtr<UMetasoundEditorGraphNode> GraphNode = MetaSoundNode.Pin();
		if (GraphNode.IsValid())
		{
			// Display filter response selection submenu if no outputs are connected, or more than one output is connected:
			const int NumConnectedAudioOutputPins = Algo::CountIf(GraphNode->Pins, IsConnectedAudioOutputPin);
			if (NumConnectedAudioOutputPins == 0 || NumConnectedAudioOutputPins > 1)
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("DisplayedFilterResponse", "Displayed Filter Response"),
					FText(),
					FNewMenuDelegate::CreateSP(this, &SMetaSoundStateVariableFilterFrequencyResponsePlot::BuildFilterOutputSubMenu));
			}
		}
	}

	void SMetaSoundStateVariableFilterFrequencyResponsePlot::BuildFilterOutputSubMenu(FMenuBuilder& SubMenu)
	{
		using namespace SMetasoundFilterFrequencyResponsePlotsPrivate;

		const TStrongObjectPtr<UMetasoundEditorGraphNode> GraphNode = MetaSoundNode.Pin();
		if (GraphNode.IsValid())
		{
			// Add menu entries for all connected audio output pins. If there are no connected audio output pins then add entries for all audio output pins:
			const bool bHasConnectedAudioOutputPins = GraphNode->Pins.ContainsByPredicate(IsConnectedAudioOutputPin);
			for (UEdGraphPin* Pin : GraphNode->Pins)
			{
				const bool bIsAudioOutputPin = Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryAudio;
				if (bIsAudioOutputPin)
				{
					const bool bIsPinConnected = !Pin->LinkedTo.IsEmpty();
					if (bIsPinConnected || !bHasConnectedAudioOutputPins)
					{
						SubMenu.AddMenuEntry(
							FText::FromName(Pin->PinName),
							FText::FromString(Pin->PinToolTip),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSPLambda(this, [this, PinName = Pin->PinName]() { DisplayedFilterResponse = PinName; }),
								FCanExecuteAction(),
								FIsActionChecked::CreateSPLambda(this, [this, PinName = Pin->PinName]() { return DisplayedFilterResponse == PinName; })
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}
				}
			}
		}
	}

}

#undef LOCTEXT_NAMESPACE
