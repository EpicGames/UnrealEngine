// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/Filter.h"
#include "DSP/InterpolatedOnePole.h"
#include "SAudioSpectrumPlot.h"
#include "SMetasoundGraphNode.h"

namespace Metasound::Editor
{
	struct FCreateGraphNodeVisualizationWidgetParams;

	TSharedRef<SWidget> CreateMetaSoundBiquadFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams);
	TSharedRef<SWidget> CreateMetaSoundLadderFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams);
	TSharedRef<SWidget> CreateMetaSoundOnePoleHighPassFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams);
	TSharedRef<SWidget> CreateMetaSoundOnePoleLowPassFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams);
	TSharedRef<SWidget> CreateMetaSoundStateVariableFilterGraphNodeVisualizationWidget(const FCreateGraphNodeVisualizationWidgetParams& InParams);

	/**
	* Abstract class implementing shared functionality for any Metasound basic filter. Allows for displaying a frequency response plot of the filter.
	*/
	class SMetaSoundFilterFrequencyResponsePlot : public SCompoundWidget
	{
	public:
		void Construct();
		virtual ~SMetaSoundFilterFrequencyResponsePlot();

	protected:
		// Update function for derived classes to update their state each frame. Implementations should return true if they have the required information to plot a frequency response, or false otherwise.
		virtual bool UpdateFilterParams() = 0;

		// Derived classes should apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats).
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const = 0;

		// Derived classes can optionally add items to the spectrum plot menu by overriding this member function.
		virtual void ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder) {};

		// Begin SWidget overrides.
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		// End SWidget overrides.

		static const float SampleRate;
		
	private:
		FAudioPowerSpectrumData GetAudioSpectrumData();

		TSharedPtr<SAudioSpectrumPlot> FrequencyResponsePlot;
		TSharedPtr<const FExtensionBase> ContextMenuExtension;
		TArray<float> CenterFrequencies;
		TArray<float> SquaredMagnitudes;
		bool bHasFilterParams = false;
	};


	class SMetaSoundBiquadFilterFrequencyResponsePlot : public SMetaSoundFilterFrequencyResponsePlot
	{
	public:
		SLATE_BEGIN_ARGS(SMetaSoundBiquadFilterFrequencyResponsePlot)
			{}
			SLATE_ATTRIBUTE(TOptional<Audio::EBiquadFilter::Type>, FilterType)
			SLATE_ATTRIBUTE(TOptional<float>, CutoffFrequency)
			SLATE_ATTRIBUTE(TOptional<float>, Bandwidth)
			SLATE_ATTRIBUTE(TOptional<float>, GainDb)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	protected:
		virtual bool UpdateFilterParams() override;
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const override;

	private:
		Audio::FBiquadFilter Filter;

		TAttribute<TOptional<Audio::EBiquadFilter::Type>> FilterType;
		TAttribute<TOptional<float>> CutoffFrequency;
		TAttribute<TOptional<float>> Bandwidth;
		TAttribute<TOptional<float>> GainDb;
	};


	class SMetaSoundLadderFilterFrequencyResponsePlot : public SMetaSoundFilterFrequencyResponsePlot
	{
	public:
		SLATE_BEGIN_ARGS(SMetaSoundLadderFilterFrequencyResponsePlot)
			{}
			SLATE_ATTRIBUTE(TOptional<float>, CutoffFrequency)
			SLATE_ATTRIBUTE(TOptional<float>, Resonance)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	protected:
		virtual bool UpdateFilterParams() override;
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const override;

	private:
		Audio::FLadderFilter Filter;

		TAttribute<TOptional<float>> CutoffFrequency;
		TAttribute<TOptional<float>> Resonance;
	};


	class SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot : public SMetaSoundFilterFrequencyResponsePlot
	{
	public:
		SLATE_BEGIN_ARGS(SMetaSoundOnePoleHighPassFilterFrequencyResponsePlot)
			{}
			SLATE_ATTRIBUTE(TOptional<float>, CutoffFrequency)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	protected:
		virtual bool UpdateFilterParams() override;
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const override;

	private:
		Audio::FInterpolatedHPF Filter;
		TAttribute<TOptional<float>> CutoffFrequency;
	};


	class SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot : public SMetaSoundFilterFrequencyResponsePlot
	{
	public:
		SLATE_BEGIN_ARGS(SMetaSoundOnePoleLowPassFilterFrequencyResponsePlot)
			{}
			SLATE_ATTRIBUTE(TOptional<float>, CutoffFrequency)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	protected:
		virtual bool UpdateFilterParams() override;
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const override;

	private:
		Audio::FInterpolatedLPF Filter;
		TAttribute<TOptional<float>> CutoffFrequency;
	};


	class SMetaSoundStateVariableFilterFrequencyResponsePlot : public SMetaSoundFilterFrequencyResponsePlot
	{
	public:
		SMetaSoundStateVariableFilterFrequencyResponsePlot();

		SLATE_BEGIN_ARGS(SMetaSoundStateVariableFilterFrequencyResponsePlot)
			{}
			SLATE_ATTRIBUTE(TOptional<float>, CutoffFrequency)
			SLATE_ATTRIBUTE(TOptional<float>, Resonance)
			SLATE_ATTRIBUTE(TOptional<float>, BandStopControl)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, UMetasoundEditorGraphNode* InMetaSoundNode);

	protected:
		virtual bool UpdateFilterParams() override;
		virtual void ArrayCalculateFilterResponseInPlace(TArrayView<float> InOutComplexValues) const override;
		virtual void ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder) override;

	private:
		void BuildFilterOutputSubMenu(FMenuBuilder& SubMenu);

		static const FName LowPassFilter;
		static const FName HighPassFilter;
		static const FName BandPass;
		static const FName BandStop;

		TWeakObjectPtr<UMetasoundEditorGraphNode> MetaSoundNode;
		Audio::FStateVariableFilter Filter;
		FName DisplayedFilterResponse;

		TAttribute<TOptional<float>> CutoffFrequency;
		TAttribute<TOptional<float>> Resonance;
		TAttribute<TOptional<float>> BandStopControl;
	};
}
