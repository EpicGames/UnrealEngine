// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetasoundGraphEnumPin.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "ScopedTransaction.h"

namespace Metasound
{
	namespace Editor
	{
		void SMetasoundGraphEnumPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
		{
			SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
			CacheAccessType();
		}

		TSharedRef<SWidget> SMetasoundGraphEnumPin::GetDefaultValueWidget()
		{
			//Get list of enum indexes
			TArray< TSharedPtr<int32> > ComboItems;
			GenerateComboBoxIndexes(ComboItems);

			//Create widget
			
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(ComboBox, SPinComboBox)
						.ComboItemList(ComboItems)
						.VisibleText(this, &SMetasoundGraphEnumPin::OnGetText)
						.OnSelectionChanged(this, &SMetasoundGraphEnumPin::ComboBoxSelectionChanged)
						.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
						.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
						.OnGetDisplayName(this, &SMetasoundGraphEnumPin::OnGetFriendlyName)
						.OnGetTooltip(this, &SMetasoundGraphEnumPin::OnGetTooltip)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					CreateResetToDefaultWidget()
				];
		}

		TSharedPtr<const Frontend::IEnumDataTypeInterface>
		SMetasoundGraphEnumPin::FindEnumInterfaceFromPin(UEdGraphPin* InPin)
		{
			using namespace Frontend;
			if (const UMetasoundEditorGraphNode* MetasoundEditorNode = Cast<UMetasoundEditorGraphNode>(InPin->GetOwningNode()))
			{
				const UMetaSoundBuilderBase& Builder = MetasoundEditorNode->GetBuilderChecked();
				if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(Builder.GetConstBuilder(), InPin))
				{
					const FName DataType = Vertex->TypeName;
					return IDataTypeRegistry::Get().GetEnumInterfaceForDataType(DataType);
				}
			}
			return nullptr;
		}

		FString SMetasoundGraphEnumPin::OnGetText() const
		{
			using namespace Frontend;

			if (const TSharedPtr<const IEnumDataTypeInterface> EnumInterface = FindEnumInterfaceFromPin(GraphPinObj); EnumInterface.IsValid())
			{
				if (GetPinObj())
				{
					const int32 SelectedValue = FCString::Atoi(*GraphPinObj->GetDefaultAsString());	// Enums are currently serialized as ints (the value of the enum).
					if (TOptional<IEnumDataTypeInterface::FGenericInt32Entry> Result = EnumInterface->FindByValue(SelectedValue); Result.IsSet() )
					{
						return Result->DisplayName.ToString();
					}
				}
			}
			return {};
		}

		void SMetasoundGraphEnumPin::GenerateComboBoxIndexes(TArray<TSharedPtr<int32>>& OutComboBoxIndexes)
		{
			using namespace Frontend;
			if (const TSharedPtr<const IEnumDataTypeInterface> EnumInterface = FindEnumInterfaceFromPin(GraphPinObj); EnumInterface.IsValid())
			{
				const TArray<IEnumDataTypeInterface::FGenericInt32Entry>& Entries = EnumInterface->GetAllEntries();
				for (int32 i = 0; i < Entries.Num(); ++i)
				{
					OutComboBoxIndexes.Add(MakeShared<int32>(i));
				}
			}
		}

		void SMetasoundGraphEnumPin::ComboBoxSelectionChanged(TSharedPtr<int32> NewSelection, ESelectInfo::Type SelectInfo)
		{
			using namespace Frontend;

			if (const TSharedPtr<const IEnumDataTypeInterface> EnumInterface = FindEnumInterfaceFromPin(SGraphPin::GetPinObj()); EnumInterface.IsValid())
			{
				const TArray<IEnumDataTypeInterface::FGenericInt32Entry>& Entries = EnumInterface->GetAllEntries();

				if (NewSelection.IsValid() && Entries.IsValidIndex(*NewSelection))
				{
					const int32 EnumValue = Entries[*NewSelection].Value;
					const FString EnumValueString = FString::FromInt(EnumValue);
					if (GraphPinObj->GetDefaultAsString() != EnumValueString)
					{
						const FScopedTransaction Transaction(NSLOCTEXT("MetaSoundEditor", "ChangeEnumPinValue", "Change MetaSound Node Default Input Enum Value"));
						GraphPinObj->Modify();

						if (UMetasoundEditorGraphNode* MetaSoundNode = Cast<UMetasoundEditorGraphNode>(GraphPinObj->GetOwningNode()))
						{
							if (UMetasoundEditorGraph* Graph = Cast<UMetasoundEditorGraph>(MetaSoundNode->GetGraph()))
							{
								Graph->Modify();
								Graph->GetMetasoundChecked().Modify();
							}
						}

						//Set new selection
						if (ensure(GraphPinObj->GetSchema()))
						{
							GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, EnumValueString);
						}
					}
				}
			}
		}

		FText SMetasoundGraphEnumPin::OnGetFriendlyName(int32 EnumIndex)
		{
			using namespace Frontend;

			if (const TSharedPtr<const IEnumDataTypeInterface> Interface = FindEnumInterfaceFromPin(GetPinObj()); Interface.IsValid())
			{
				const TArray<IEnumDataTypeInterface::FGenericInt32Entry>& Entries = Interface->GetAllEntries();
				check(Entries.IsValidIndex(EnumIndex));

				return Entries[EnumIndex].DisplayName;
			}
			return {};
		}

		FText SMetasoundGraphEnumPin::OnGetTooltip(int32 EnumIndex)
		{
			using namespace Frontend;

			if (const TSharedPtr<const IEnumDataTypeInterface> Interface = FindEnumInterfaceFromPin(GraphPinObj))
			{
				const TArray<IEnumDataTypeInterface::FGenericInt32Entry>& Entries = Interface->GetAllEntries();
				check(Entries.IsValidIndex(EnumIndex));

				return Entries[EnumIndex].Tooltip;
			}
			return {};
		}
	} // namespace Editor
} // namespace Metasound
