// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SGraphPin.h"
#include "SGraphPinComboBox.h"
#include "SMetasoundGraphPin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"


// Forward Declarations
namespace Metasound
{
	namespace Frontend
	{
		struct IEnumDataTypeInterface;
	} // namespace Frontend

	namespace Editor
	{
		class SMetasoundGraphEnumPin : public TMetasoundGraphPin<SGraphPin>
		{
			SLATE_BEGIN_ARGS(SMetasoundGraphEnumPin)
			{
			}
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

			static TSharedPtr<const Metasound::Frontend::IEnumDataTypeInterface> FindEnumInterfaceFromPin(UEdGraphPin* InPin);

		protected:
			virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

		private:
			TSharedPtr<SPinComboBox> ComboBox;
	
			FString OnGetText() const;

			void GenerateComboBoxIndexes(TArray<TSharedPtr<int32>>& OutComboBoxIndexes);

			void ComboBoxSelectionChanged(TSharedPtr<int32> NewSelection, ESelectInfo::Type SelectInfo);

			FText OnGetFriendlyName(int32 EnumIndex);

			FText OnGetTooltip(int32 EnumIndex);
		};
	} // namespace Editor
} // namespace Metasound
