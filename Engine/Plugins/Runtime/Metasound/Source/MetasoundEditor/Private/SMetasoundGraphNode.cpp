// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundGraphNode.h"

#include "AudioMaterialSlate/SAudioMaterialButton.h"
#include "AudioMaterialSlate/SAudioMaterialLabeledKnob.h"
#include "AudioMaterialSlate/SAudioMaterialLabeledSlider.h"
#include "AudioParameterControllerInterface.h"
#include "AudioWidgetsEnums.h"
#include "Components/AudioComponent.h"
#include "GraphEditorSettings.h"
#include "IAudioParameterTransmitter.h"
#include "IDocumentation.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinExec.h"
#include "KismetPins/SGraphPinInteger.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinObject.h"
#include "KismetPins/SGraphPinString.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraphNodeVisualization.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundSettings.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "NodeFactory.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "PropertyCustomizationHelpers.h"
#include "SAudioRadialSlider.h"
#include "SAudioSlider.h"
#include "ScopedTransaction.h"
#include "SGraphNode.h"
#include "SGraphPinComboBox.h"
#include "Styling/AppStyle.h"
#include "SLevelOfDetailBranchNode.h"
#include "SMetasoundGraphEnumPin.h"
#include "SMetasoundGraphPin.h"
#include "SMetasoundPinValueInspector.h"
#include "SPinTypeSelector.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateStyleRegistry.h"
#include "TutorialMetaData.h"
#include "UObject/ScriptInterface.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


#define LOCTEXT_NAMESPACE "MetasoundEditor"

namespace Metasound
{
	namespace Editor
	{
		SMetaSoundGraphNode::~SMetaSoundGraphNode()
		{
			// Clean up input widgets
			UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			if (UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(&Node))
			{
				if (UMetasoundEditorGraphMember* GraphMember = MemberNode->GetMember())
				{
					// This may hit if the asset editor is closed while interacting with a widget 
					// (ex. Ctrl-W is pressed mid drag before the value is committed)
					if (bIsInputWidgetTransacting)
					{
						GEditor->EndTransaction();
						if (UMetasoundEditorGraph* Graph = GraphMember->GetOwningGraph())
						{
							constexpr bool bPostTransaction = false;
							GraphMember->UpdateFrontendDefaultLiteral(bPostTransaction);
							FGraphBuilder::GetOutermostMetaSoundChecked(*Graph).GetModifyContext().AddMemberIDsModified({ GraphMember->GetMemberID() });
						}
					}

					if (UMetasoundEditorGraphMemberDefaultFloat* DefaultFloat = Cast<UMetasoundEditorGraphMemberDefaultFloat>(GraphMember->GetLiteral()))
					{
						DefaultFloat->OnDefaultValueChanged.Remove(InputSliderOnValueChangedDelegateHandle);
						DefaultFloat->OnRangeChanged.Remove(InputSliderOnRangeChangedDelegateHandle);

					}
					else if (UMetasoundEditorGraphMemberDefaultBool* DefaultBool = Cast<UMetasoundEditorGraphMemberDefaultBool>(GraphMember->GetLiteral()))
					{
						DefaultBool->OnDefaultStateChanged.Remove(InputButtonOnStateChangedDelegateHandle);
					}
				}
			}
		}

		bool SMetaSoundGraphNode::IsVariableAccessor() const
		{
			return ClassType == EMetasoundFrontendClassType::VariableAccessor
				|| ClassType == EMetasoundFrontendClassType::VariableDeferredAccessor;
		}

		bool SMetaSoundGraphNode::IsVariableMutator() const
		{
			return ClassType == EMetasoundFrontendClassType::VariableMutator;
		}

		const FSlateBrush* SMetaSoundGraphNode::GetShadowBrush(bool bSelected) const
		{
			if (IsVariableAccessor() || IsVariableMutator())
			{
				return bSelected ? FAppStyle::GetBrush(TEXT("Graph.VarNode.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.VarNode.Shadow"));
			}

			return SGraphNode::GetShadowBrush(bSelected);
		}

		void SMetaSoundGraphNode::Construct(const FArguments& InArgs, class UEdGraphNode* InNode)
		{
			GraphNode = InNode;
			Frontend::FConstNodeHandle NodeHandle = GetMetaSoundNode().GetConstNodeHandle();
			ClassType = NodeHandle->GetClassMetadata().GetType();

			SetCursor(EMouseCursor::CardinalCross);
			UpdateGraphNode();
		}

		void SMetaSoundGraphNode::ExecuteTrigger(UMetasoundEditorGraphMemberDefaultLiteral& Literal)
		{
			UMetasoundEditorGraphMember* Member = Literal.FindMember();
			if (!ensure(Member))
			{
				return;
			}

			if (UMetasoundEditorGraph* Graph = Member->GetOwningGraph())
			{
				if (!Graph->IsPreviewing())
				{
					TSharedPtr<FEditor> MetaSoundEditor = FGraphBuilder::GetEditorForMetasound(Graph->GetMetasoundChecked());
					if (!MetaSoundEditor.IsValid())
					{
						return;
					}
					MetaSoundEditor->Play();
				}
			}

			if (UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent())
			{
				PreviewComponent->SetTriggerParameter(Member->GetMemberName());
			}
		}

		TAttribute<EVisibility> SMetaSoundGraphNode::GetSimulationVisibilityAttribute() const
		{
			return TAttribute<EVisibility>::CreateSPLambda(AsShared(), [this]()
			{
				using namespace Frontend;

				if (const UMetasoundEditorGraphMemberNode* Node = Cast<UMetasoundEditorGraphMemberNode>(&GetMetaSoundNode()))
				{
					if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(Node->GetMember()))
					{
						if (const UMetasoundEditorGraph* Graph = Vertex->GetOwningGraph())
						{
							if (!Graph->IsPreviewing())
							{
								return EVisibility::Hidden;
							}
						}

						// Don't enable trigger simulation widget if its a trigger provided by an interface
						// that does not support transmission.
						FName RouterName;
						if (IInterfaceRegistry::Get().FindInterfaceRouter(Vertex->GetInterfaceVersion(), RouterName)
							&& RouterName != Audio::IParameterTransmitter::RouterName)
						{
							return EVisibility::Hidden;
						}
						else if (const UMetasoundEditorGraphMemberDefaultLiteral* Literal = Vertex->GetLiteral())
						{
							if (!Literal)
							{
								return EVisibility::Hidden;
							}
						}
					}
				}

				return EVisibility::Visible;
			});
		}

		TSharedRef<SWidget> SMetaSoundGraphNode::CreateTriggerSimulationWidget(UMetasoundEditorGraphMemberDefaultLiteral& InputLiteral, TAttribute<EVisibility>&& InVisibility, TAttribute<bool>&& InEnablement, const FText* InToolTip)
		{
			const FText ToolTip = InToolTip
				? *InToolTip
				: LOCTEXT("MetasoundGraphNode_TriggerTestToolTip", "Executes trigger if currently previewing MetaSound.");

			TSharedPtr<SButton> SimulationButton;
			TSharedRef<SWidget> SimulationWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(7.0f, 0.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SAssignNew(SimulationButton, SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([LiteralPtr = TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral>(&InputLiteral)]()
				{
					if (LiteralPtr.IsValid())
					{
						ExecuteTrigger(*LiteralPtr.Get());
					}
					return FReply::Handled();
				})
				.ToolTipText(ToolTip)
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowDown"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				.Visibility(MoveTemp(InVisibility))
			];

			SimulationButton->SetEnabled(MoveTemp(InEnablement));

			return SimulationWidget;
		}

		void SMetaSoundGraphNode::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
		{
			TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
				LOCTEXT("MetasoundGraphNode_AddPinInputButton", "Add Input"),
				LOCTEXT("MetasoundGraphNode_AddPinInputButton_Tooltip", "Add an input to the parent Metasound node.")
			);

			FMargin AddPinPadding = Settings->GetOutputPinPadding();
			AddPinPadding.Top += 6.0f;

			InputBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(AddPinPadding)
			[
				AddPinButton
			];
		}

		void SMetaSoundGraphNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
		{
			TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
				LOCTEXT("MetasoundGraphNode_AddPinOutputButton", "Add Output"),
				LOCTEXT("MetasoundGraphNode_AddPinOutputButton_Tooltip", "Add an output to the parent Metasound node.")
			);

			FMargin AddPinPadding = Settings->GetOutputPinPadding();
			AddPinPadding.Top += 6.0f;

			OutputBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(AddPinPadding)
			[
				AddPinButton
			];
		}

		UMetasoundEditorGraphNode& SMetaSoundGraphNode::GetMetaSoundNode() const
		{
			return *CastChecked<UMetasoundEditorGraphNode>(GraphNode);
		}

		TSharedPtr<SGraphPin> SMetaSoundGraphNode::CreatePinWidget(UEdGraphPin* InPin) const
		{
			TSharedPtr<SGraphPin> PinWidget;

			if (const UMetasoundEditorGraphSchema* GraphSchema = Cast<const UMetasoundEditorGraphSchema>(InPin->GetSchema()))
			{
				// Don't show default value field for container types
				if (InPin->PinType.ContainerType != EPinContainerType::None)
				{
					PinWidget = SNew(SMetasoundGraphPin, InPin);
				}
				else
				{
					PinWidget = FMetaSoundGraphPanelPinFactory::GetChecked()->CreatePin(InPin);
				}
			}

			if (!PinWidget.IsValid())
			{
				PinWidget = SNew(SMetasoundGraphPin, InPin);
			}

			const FSlateBrush* PinConnectedIcon = nullptr;
			const FSlateBrush* PinDisconnectedIcon = nullptr;
			if (FMetaSoundGraphPanelPinFactory::GetChecked()->GetCustomPinIcons(InPin, PinConnectedIcon, PinDisconnectedIcon))
			{
				PinWidget->SetCustomPinIcon(PinConnectedIcon, PinDisconnectedIcon);
			}

			return PinWidget;
		}

		void SMetaSoundGraphNode::CreateStandardPinWidget(UEdGraphPin* InPin)
		{
			check(InPin);

			// Set pin hidden if the node has unconnected pins hidden
			const FGuid NodeID = GetMetaSoundNode().GetNodeID();
			const FMetaSoundFrontendDocumentBuilder& Builder = GetMetaSoundNode().GetBuilderChecked().GetConstBuilder();
			if (const FMetasoundFrontendNode* FrontendNode = Builder.FindNode(NodeID))
			{
				const FMetasoundFrontendNodeStyle& Style = FrontendNode->Style;
				InPin->SafeSetHidden(Style.bUnconnectedPinsHidden);

				if (const FMetasoundFrontendClass* FrontendClass = Builder.FindDependency(FrontendNode->ClassID))
				{
					if (InPin->Direction == EGPD_Input)
					{
						const FMetasoundFrontendVertex* InputVertex = FGraphBuilder::GetPinVertex(Builder, InPin);
						auto IsPinMetadata = [&InputVertex](const FMetasoundFrontendClassVertex& Vertex) { return Vertex.Name == InputVertex->Name; };
						if (const FMetasoundFrontendClassVertex* ClassVertex = FrontendClass->GetInterfaceForNode(*FrontendNode).Inputs.FindByPredicate(IsPinMetadata))
						{
							if (ClassVertex->Metadata.bIsAdvancedDisplay != InPin->bAdvancedView)
							{
								FGraphBuilder::RefreshPinMetadata(*InPin, ClassVertex->Metadata.bIsAdvancedDisplay);
							}
						}
					}
					else if (InPin->Direction == EGPD_Output)
					{
						const FMetasoundFrontendVertex* OutputVertex = FGraphBuilder::GetPinVertex(Builder, InPin);
						auto IsPinMetadata = [&OutputVertex](const FMetasoundFrontendClassVertex& Vertex) { return Vertex.Name == OutputVertex->Name; };
						if (const FMetasoundFrontendClassVertex* ClassVertex = FrontendClass->GetInterfaceForNode(*FrontendNode).Outputs.FindByPredicate(IsPinMetadata))
						{
							if (ClassVertex->Metadata.bIsAdvancedDisplay != InPin->bAdvancedView)
							{
								FGraphBuilder::RefreshPinMetadata(*InPin, ClassVertex->Metadata.bIsAdvancedDisplay);
							}
						}
					}

					const bool bShowPin = ShouldPinBeHidden(InPin);
					if (bShowPin)
					{
						TSharedPtr<SGraphPin> NewPin = CreatePinWidget(InPin);
						check(NewPin.IsValid());

						if (InPin->Direction == EGPD_Input)
						{
							if (!FrontendClass->Style.Display.bShowInputNames)
							{
								NewPin->SetShowLabel(false);
							}
						}
						else if (InPin->Direction == EGPD_Output)
						{
							if (!FrontendClass->Style.Display.bShowOutputNames)
							{
								NewPin->SetShowLabel(false);
							}
						}

						AddPin(NewPin.ToSharedRef());
					}
				}
			}
		}

		void SMetaSoundGraphNode::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
		{
			if (MainBox.IsValid())
			{
				UMetasoundEditorGraphNode& MetaSoundNode = GetMetaSoundNode();

				const FName NodeClassName = MetaSoundNode.GetBreadcrumb().ClassName.GetFullName();
				const FCreateGraphNodeVisualizationWidgetParams CreateParams{ .MetaSoundNode = &MetaSoundNode };
				if (TSharedPtr<SWidget> VisualizationWidget = FGraphNodeVisualizationRegistry::Get().CreateVisualizationWidget(NodeClassName, CreateParams))
				{
					MainBox->AddSlot()
						.Padding(1.0f, 0.0f)
						[
							VisualizationWidget.ToSharedRef()
						];
				}
			}
		}

		TSharedRef<SWidget> SMetaSoundGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
		{
			TSharedRef<SWidget> DefaultTitle = SGraphNode::CreateTitleWidget(NodeTitle);

			const UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			const FMetasoundFrontendClass* FrontendClass = Node.GetFrontendClass();
			if (!FrontendClass || !FrontendClass->Style.Display.bShowName)
			{
				return SNullWidget::NullWidget;
			}

			const TSharedPtr<SHorizontalBox> TitleBoxWidget =
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SGraphNode::CreateTitleWidget(NodeTitle)
				];

			InlineEditableText->SetColorAndOpacity(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &SMetaSoundGraphNode::GetNodeTitleColorOverride)));

			return TitleBoxWidget.ToSharedRef();
		}

		void SMetaSoundGraphNode::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
		{
			FName CornerIcon = GetMetaSoundNode().GetCornerIcon();
			if (CornerIcon != NAME_None)
			{

				if (const FSlateBrush* Brush = FAppStyle::GetBrush(CornerIcon))
				{
					FOverlayBrushInfo OverlayInfo = { Brush };

					// Logic copied from SGraphNodeK2Base
					OverlayInfo.OverlayOffset.X = (WidgetSize.X - (OverlayInfo.Brush->ImageSize.X / 2.f)) - 3.f;
					OverlayInfo.OverlayOffset.Y = (OverlayInfo.Brush->ImageSize.Y / -2.f) + 2.f;
					Brushes.Add(MoveTemp(OverlayInfo));
				}
			}
		}

		void SMetaSoundGraphNode::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
		{
			using namespace Metasound::Engine;

			UMetasoundEditorGraphNode& EdNode = GetMetaSoundNode();
			UObject& MetaSound = EdNode.GetMetasoundChecked();
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
			if (const FMetasoundFrontendNode* Node = Builder.GetConstBuilder().FindNode(EdNode.GetNodeID()))
			{
				if (bInCommentBubbleVisible != Node->Style.Display.bCommentVisible)
				{
					const FScopedTransaction Transaction(LOCTEXT("GraphNodeCommentToggled", "Graph Node Comment Bubble Toggled"));
					MetaSound.Modify();
					EMetaSoundBuilderResult Result;
					EdNode.bCommentBubbleVisible = bInCommentBubbleVisible;
					Builder.SetNodeCommentVisible(Node->GetID(), bInCommentBubbleVisible, Result);
				}
			}
		}

		void SMetaSoundGraphNode::OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo)
		{
			using namespace Metasound::Engine;

			FString NewCommentString = NewComment.ToString();
			UMetasoundEditorGraphNode& EdNode = GetMetaSoundNode();
			UObject& MetaSound = EdNode.GetMetasoundChecked();
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
			if (const FMetasoundFrontendNode* Node = Builder.GetConstBuilder().FindNode(EdNode.GetNodeID()))
			{
				if (!Node->Style.Display.Comment.Equals(NewCommentString))
				{
					const FScopedTransaction Transaction(LOCTEXT("GraphNodeCommentChanged", "Graph Node Comment Changed"));
					MetaSound.Modify();
					EMetaSoundBuilderResult Result;
					EdNode.NodeComment = MoveTemp(NewCommentString);
					Builder.SetNodeComment(Node->GetID(), EdNode.NodeComment, Result);
				}
			}
		}

		void SMetaSoundGraphNode::OnAdvancedViewChanged(const ECheckBoxState NewCheckedState)
		{
			if (NewCheckedState == ECheckBoxState::Checked)
			{
				if (UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(&GetMetaSoundNode()))
				{
					ExternalNode->HideUnconnectedPins(false);
				}
			}
			SGraphNode::OnAdvancedViewChanged(NewCheckedState);
		}

		FLinearColor SMetaSoundGraphNode::GetNodeTitleColorOverride() const
		{
			const UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			FLinearColor TitleColor = GraphNode->IsDeprecated() ? FLinearColor::Red : Node.GetNodeTitleTextColor();
			if (ClassType != EMetasoundFrontendClassType::External)
			{
				TitleColor.A = FadeCurve.GetLerp();
			}

			return TitleColor;
		}

		void SMetaSoundGraphNode::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
		{
			SGraphNode::SetDefaultTitleAreaWidget(DefaultTitleAreaWidget);

			const UMetasoundEditorGraphNode& Node = GetMetaSoundNode();

			FMetasoundFrontendClassStyleDisplay StyleDisplay;
			if (const FMetasoundFrontendClass* FrontendClass = Node.GetFrontendClass())
			{
				StyleDisplay = FrontendClass->Style.Display;
			}

			if (!StyleDisplay.bShowName)
			{
				DefaultTitleAreaWidget->SetVisibility(EVisibility::Collapsed);
				return;
			}

			DefaultTitleAreaWidget->ClearChildren();

			const bool bShowTitleIcon = Node.ShouldShowTitleIcon();
			const TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

			const TSharedRef<SWidget> TitleTextBlock =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(bShowTitleIcon ? HAlign_Left : HAlign_Center)
				[
					CreateTitleWidget(NodeTitle)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(bShowTitleIcon ? HAlign_Left : HAlign_Center)
				[
					NodeTitle.ToSharedRef()
				];

			TSharedRef<SWidget> TitleContent =
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 6, 0)
				[
					CreateTitleIcon()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					TitleTextBlock
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					CreateTitleRightWidget()
				];

			const TSharedRef<SWidget> MainTitleArea =
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Graph.Node.ColorSpill"))
				.Padding(FMargin(10.f, 5.f, 10.f, 3.f))
				.BorderBackgroundColor(Node.GetNodeTitleColor())
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					TitleContent
				];

			DefaultTitleAreaWidget->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				MainTitleArea
			];

			DefaultTitleAreaWidget->AddSlot()
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.Visibility(EVisibility::HitTestInvisible)
				.BorderImage( FAppStyle::GetBrush( "Graph.Node.TitleHighlight" ) )
				.BorderBackgroundColor( this, &SGraphNode::GetNodeTitleIconColor )
				[
					SNew(SSpacer)
					.Size(FVector2D(20,20))
				]
			];
		}

		TSharedRef<SWidget> SMetaSoundGraphNode::CreateTitleIcon() const
		{
			const UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			if (!Node.ShouldShowTitleIcon())
			{
				return SNullWidget::NullWidget;
			}

			const FText& TooltipText = Node.GetNodeTitleIconTooltipText();
			const bool bShowTooltip = !TooltipText.IsEmpty();

			TSharedPtr<SToolTip> ToolTip;
			if (bShowTooltip)
			{
				ToolTip =
					SNew(SToolTip)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, 0, 0, 2)
						[
							SNew(STextBlock)
							.Text(TooltipText)
						]
					];
			}

			return SNew(SBox)
				.WidthOverride(20.f)
				.HeightOverride(20.f)
				.ToolTip(ToolTip)
				[
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.BorderImage_Lambda([]()
					{
						static const FSlateRoundedBoxBrush RoundedBoxBrush(FLinearColor::White, 3.f);
						return &RoundedBoxBrush;
					})
					.BorderBackgroundColor(FLinearColor(0, 0, 0, 0.33f))
					[
						SNew(SBox)
						.WidthOverride(16.f)
						.HeightOverride(16.f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(Node.GetNodeTitleIconBrush())
							.ColorAndOpacity(FLinearColor::White)
						]
					]
				];
		}

		void SMetaSoundGraphNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
		{
			SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

			UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			Node.GetMetasoundChecked().Modify();
			Node.UpdateFrontendNodeLocation(FVector2D(NewPosition));
			Node.SyncLocationFromFrontendNode();
		}

		const FSlateBrush* SMetaSoundGraphNode::GetNodeBodyBrush() const
		{
			// TODO: Add tweak & add custom bodies
			if (GraphNode)
			{
				switch (ClassType)
				{
					case EMetasoundFrontendClassType::Variable:
					case EMetasoundFrontendClassType::VariableAccessor:
					case EMetasoundFrontendClassType::VariableDeferredAccessor:
					case EMetasoundFrontendClassType::VariableMutator:
					{
						return FAppStyle::GetBrush("Graph.VarNode.Body");
					}

					case EMetasoundFrontendClassType::Input:
					case EMetasoundFrontendClassType::Output:
					default:
					{
					}
					break;
				}
			}

			return FAppStyle::GetBrush("Graph.Node.Body");
		}

		FText SMetaSoundGraphNode::GetNodeTooltip() const
		{
			FText TooltipText = FText::GetEmpty();
			if (GraphNode)
			{
				// Don't show if pin already displaying interactive tooltip
				for (UEdGraphPin* Pin : GraphNode->GetAllPins())
				{
					TSharedPtr<SGraphPin> PinWidget = FindWidgetForPin(Pin);

					if (PinWidget.IsValid() && PinWidget->HasInteractiveTooltip())
					{
						return TooltipText;
					}
				}

				TooltipText = GraphNode->GetTooltipText();
				if (TooltipText.IsEmpty())
				{
					TooltipText = GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
				}
			}

			return TooltipText;
		}

		EVisibility SMetaSoundGraphNode::IsAddPinButtonVisible() const
		{
			EVisibility DefaultVisibility = SGraphNode::IsAddPinButtonVisible();
			if (DefaultVisibility == EVisibility::Visible)
			{
				if (!GetMetaSoundNode().CanAddInputPin())
				{
					return EVisibility::Collapsed;
				}
			}

			return DefaultVisibility;
		}

		FReply SMetaSoundGraphNode::OnAddPin()
		{
			GetMetaSoundNode().CreateInputPin();

			return FReply::Handled();
		}

		FName SMetaSoundGraphNode::GetLiteralDataType() const
		{
			using namespace Frontend;

			FName TypeName;

			// Just take last type.  If more than one, all types are the same.
			const UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			Node.GetConstNodeHandle()->IterateConstOutputs([InTypeName = &TypeName](FConstOutputHandle OutputHandle)
			{
				*InTypeName = OutputHandle->GetDataType();
			});

			return TypeName;
		}

		TSharedRef<SWidget> SMetaSoundGraphNode::CreateTitleRightWidget()
		{
			using namespace Frontend;

			const FName TypeName = GetLiteralDataType();
			if (TypeName == Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>())
			{
				if (UMetasoundEditorGraphMemberNode* Node = Cast<UMetasoundEditorGraphMemberNode>(&GetMetaSoundNode()))
				{
					if (UMetasoundEditorGraphInput* Input = Cast<UMetasoundEditorGraphInput>(Node->GetMember()))
					{
						if (UMetasoundEditorGraphMemberDefaultLiteral* Literal = Input->GetLiteral())
						{
							TAttribute<EVisibility> SimVisibility = GetSimulationVisibilityAttribute();
							TAttribute<bool> SimEnablement = true;
							return CreateTriggerSimulationWidget(*Literal, MoveTemp(SimVisibility), MoveTemp(SimEnablement));
						}
					}
				}
			}

			return SGraphNode::CreateTitleRightWidget();
		}

		UMetasoundEditorGraphMember* SMetaSoundGraphNode::GetMetaSoundMember()
		{
			if (UMetasoundEditorGraphMemberNode* MemberNode = GetMetaSoundMemberNode())
			{
				return MemberNode->GetMember();
			}

			return nullptr;
		}

		TAttribute<bool> SMetaSoundGraphNode::GetInputWidgetEnabled() const
		{
			return TAttribute<bool>::Create([this]()
			{
				if (const UMetasoundEditorGraphMemberNode* Node = GetMetaSoundMemberNode())
				{
					return Node->EnableInteractWidgets();
				}
				return false;
			});
		}

		FText SMetaSoundGraphNode::GetInputWidgetTooltip() const
		{
			if (const UMetasoundEditorGraphMemberNode* Node = GetMetaSoundMemberNode())
			{
				return Node->GetTooltipText();
			}
			return FText();
		}

		UMetasoundEditorGraphMemberNode* SMetaSoundGraphNode::GetMetaSoundMemberNode() const
		{
			return Cast<UMetasoundEditorGraphMemberNode>(&GetMetaSoundNode());
		}

		TSharedPtr<SWidget> SMetaSoundGraphNode::CreateInputNodeContentArea(const FMetaSoundFrontendDocumentBuilder& InBuilder, TSharedRef<SHorizontalBox> ContentBox)
		{
			using namespace Engine;

			TSharedPtr<SWidget> OuterContentBox;
			TWeakObjectPtr<UMetasoundEditorGraphInput> GraphMember;
			{
				GraphMember = Cast<UMetasoundEditorGraphInput>(GetMetaSoundMember());
				if (!GraphMember.IsValid())
				{
					return OuterContentBox;
				}
			}

			const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>();
			const UMetaSoundSettings* MetaSoundSettings = GetDefault<UMetaSoundSettings>();
			if (!EditorSettings || !MetaSoundSettings)
			{
				return OuterContentBox;
			}

			const UMetasoundEditorGraph* OwningGraph = GraphMember->GetOwningGraph();
			if (!OwningGraph || !OwningGraph->IsEditable() || GraphMember->GetVertexAccessType() == EMetasoundFrontendVertexAccessType::Unset)
			{
				return OuterContentBox;
			}

			const bool bUseAudioMaterialWidgets = EditorSettings->bUseAudioMaterialWidgets;
			TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultFloat> DefaultFloat = Cast<UMetasoundEditorGraphMemberDefaultFloat>(GraphMember->GetLiteral());
			if (DefaultFloat.IsValid() && DefaultFloat->WidgetType != EMetasoundMemberDefaultWidget::None)
			{
				constexpr float WidgetPadding = 3.0f;
				static const FVector2D SliderDesiredSizeVertical = FVector2D(30.0f, 250.0f);
				static const FVector2D RadialSliderDesiredSize = FVector2D(56.0f, 87.0f);

				auto OnValueChangedLambda = [DefaultFloat, GraphMember, this](float Value)
				{
					BeginOrUpdateValueTransaction(GraphMember, [this, DefaultFloat, Value](const FGuid& BuildPageID, UMetasoundEditorGraphMember& Member)
					{
						if (DefaultFloat.IsValid() && FloatInputWidget.IsValid())
						{
							DefaultFloat->Modify();

							{
								FMetasoundFrontendLiteral OutputLiteral;
								OutputLiteral.Set(FloatInputWidget->GetOutputValue(Value));
								DefaultFloat->SetFromLiteral(OutputLiteral, BuildPageID);
							}

							constexpr bool bPostTransaction = false;
							Member.UpdateFrontendDefaultLiteral(bPostTransaction, &BuildPageID);
						}
					});
				};

				auto OnValueCommittedLambda = [this, GraphMember, DefaultFloat](float Value)
				{
					FinalizeValueTransaction(GraphMember, [&](const FGuid& BuildPageID, UMetasoundEditorGraphMember& Member, bool bPostTransaction)
					{
						if (DefaultFloat.IsValid() && FloatInputWidget.IsValid())
						{
							DefaultFloat->Modify();
							{
								FMetasoundFrontendLiteral OutputLiteral;
								OutputLiteral.Set(FloatInputWidget->GetOutputValue(Value));
								DefaultFloat->SetFromLiteral(OutputLiteral, BuildPageID);
							}

							Member.UpdateFrontendDefaultLiteral(bPostTransaction);
							DefaultFloat->OnDefaultValueChanged.Broadcast(BuildPageID, Value);
						}
					});
				};

				if (DefaultFloat->WidgetType == EMetasoundMemberDefaultWidget::Slider)
				{
					if (bUseAudioMaterialWidgets)
					{
						SAssignNew(FloatInputWidget, SAudioMaterialLabeledSlider)
							.Owner(GraphMember->GetOwningGraph())
							.Style(FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"), "AudioMaterialSlider.Style")
							.AudioUnitsValueType(DefaultFloat->WidgetUnitValueType)
							.bUseLinearOutput(DefaultFloat->VolumeWidgetUseLinearOutput)
							.OnValueChanged_Lambda(OnValueChangedLambda)
							.OnValueCommitted_Lambda(OnValueCommittedLambda);
					}
					else
					{
						// Create slider 
						if (DefaultFloat->WidgetUnitValueType == EAudioUnitsValueType::Frequency)
						{
							SAssignNew(FloatInputWidget, SAudioFrequencySlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnValueCommitted_Lambda(OnValueCommittedLambda);
						}
						else if (DefaultFloat->WidgetUnitValueType == EAudioUnitsValueType::Volume)
						{
							SAssignNew(FloatInputWidget, SAudioVolumeSlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnValueCommitted_Lambda(OnValueCommittedLambda);
							StaticCastSharedPtr<SAudioVolumeSlider>(FloatInputWidget)->SetUseLinearOutput(DefaultFloat->VolumeWidgetUseLinearOutput);
						}
						else
						{
							SAssignNew(FloatInputWidget, SAudioSlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnValueCommitted_Lambda(OnValueCommittedLambda);
							FloatInputWidget->SetShowUnitsText(false);
						}
					}
					// Slider layout 
					if (DefaultFloat->WidgetOrientation == Orient_Vertical)
					{
						SAssignNew(OuterContentBox, SVerticalBox)
							+ SVerticalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.AutoHeight()
							[
								ContentBox
							]
						+ SVerticalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Top)
							.Padding(WidgetPadding, 0.0f, WidgetPadding, WidgetPadding)
							.AutoHeight()
							[
								FloatInputWidget.ToSharedRef()
							];

							if (!bUseAudioMaterialWidgets)
							{
								FloatInputWidget->SetDesiredSizeOverride(SliderDesiredSizeVertical);
							}
					}
					else // horizontal orientation
					{
						UMetasoundEditorGraphMemberNode* MemberNode = GetMetaSoundMemberNode();
						TSharedPtr<SWidget> Slot1;
						TSharedPtr<SWidget> Slot2;
						if (MemberNode->IsA<UMetasoundEditorGraphInputNode>())
						{
							Slot1 = FloatInputWidget;
							Slot2 = ContentBox;
						}
						else
						{
							Slot1 = ContentBox;
							Slot2 = FloatInputWidget;
						}

						SAssignNew(OuterContentBox, SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Center)
							.Padding(WidgetPadding, 0.0f, WidgetPadding, 0.0f)
							.AutoWidth()
							[
								Slot1.ToSharedRef()
							]
						+ SHorizontalBox::Slot()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Fill)
							.AutoWidth()
							[
								Slot2.ToSharedRef()
							];

							if (!bUseAudioMaterialWidgets)
							{
								FloatInputWidget->SetDesiredSizeOverride(FVector2D(SliderDesiredSizeVertical.Y, SliderDesiredSizeVertical.X));
							}
						}

						if (bUseAudioMaterialWidgets)
						{
							// safe downcast because the ptr was just assigned above 
							StaticCastSharedPtr<SAudioMaterialLabeledSlider>(FloatInputWidget)->SetOrientation(DefaultFloat->WidgetOrientation);
						}
						else
						{
							// safe downcast because the ptr was just assigned above 
							StaticCastSharedPtr<SAudioSliderBase>(FloatInputWidget)->SetOrientation(DefaultFloat->WidgetOrientation);
						}
				}
				else if (DefaultFloat->WidgetType == EMetasoundMemberDefaultWidget::RadialSlider)
				{
					auto OnRadialSliderMouseCaptureBeginLambda = [this, GraphMember]()
					{
						BeginOrUpdateValueTransaction(GraphMember, [](const FGuid&, UMetasoundEditorGraphMember&) { });
					};

					auto OnRadialSliderMouseCaptureEndLambda = [this, GraphMember, DefaultFloat]()
					{
						FinalizeValueTransaction(GraphMember, [this, DefaultFloat](const FGuid& BuildPageID, UMetasoundEditorGraphMember& Member, bool bPostTransaction)
						{
							if (DefaultFloat.IsValid())
							{
								DefaultFloat->Modify();
								const float FinalValue = DefaultFloat->GetDefaultAs<float>(BuildPageID);

								{
									FMetasoundFrontendLiteral OutputLiteral;
									OutputLiteral.Set(FinalValue);
									DefaultFloat->SetFromLiteral(OutputLiteral, BuildPageID);
								}

								Member.UpdateFrontendDefaultLiteral(bPostTransaction);
								DefaultFloat->OnDefaultValueChanged.Broadcast(BuildPageID, FinalValue);
							}
						});
					};

					if (bUseAudioMaterialWidgets)
					{
						SAssignNew(FloatInputWidget, SAudioMaterialLabeledKnob)
							.Owner(GraphMember->GetOwningGraph())
							.Style(FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"), "AudioMaterialKnob.Style")
							.OnValueChanged_Lambda(OnValueChangedLambda)
							.AudioUnitsValueType(DefaultFloat->WidgetUnitValueType)
							.bUseLinearOutput(DefaultFloat->VolumeWidgetUseLinearOutput)
							.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
							.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);
					}
					else
					{
						// Create slider 
						if (DefaultFloat->WidgetUnitValueType == EAudioUnitsValueType::Frequency)
						{
							SAssignNew(FloatInputWidget, SAudioFrequencyRadialSlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
								.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);
						}
						else if (DefaultFloat->WidgetUnitValueType == EAudioUnitsValueType::Volume)
						{
							SAssignNew(FloatInputWidget, SAudioVolumeRadialSlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
								.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);
							StaticCastSharedPtr<SAudioVolumeRadialSlider>(FloatInputWidget)->SetUseLinearOutput(DefaultFloat->VolumeWidgetUseLinearOutput);
						}
						else
						{
							SAssignNew(FloatInputWidget, SAudioRadialSlider)
								.OnValueChanged_Lambda(OnValueChangedLambda)
								.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
								.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);
							FloatInputWidget->SetShowUnitsText(false);
						}
					}
					// Only vertical layout for radial slider
					SAssignNew(OuterContentBox, SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.AutoHeight()
						[
							ContentBox
						]
					+ SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Top)
						.Padding(WidgetPadding, 0.0f, WidgetPadding, WidgetPadding)
						.AutoHeight()
						[
							FloatInputWidget.ToSharedRef()
						];
						FloatInputWidget->SetDesiredSizeOverride(RadialSliderDesiredSize);
				}

				const FMetasoundFrontendClassInput* ClassInput = InBuilder.FindGraphInput(GraphMember->GetMemberName());
				FGuid ResolvedPageID = Frontend::DefaultPageID;
				if (ensure(ClassInput))
				{
					ResolvedPageID = EditorSettings->ResolveAuditionPage(*ClassInput, InBuilder.GetBuildPageID());
				}

				FloatInputWidget->SetOutputRange(DefaultFloat->GetRange());
				FloatInputWidget->SetUnitsTextReadOnly(true);
				FloatInputWidget->SetSliderValue(FloatInputWidget->GetSliderValue(DefaultFloat->GetDefaultAs<float>(ResolvedPageID)));
				FloatInputWidget->SetEnabled(GetInputWidgetEnabled());
				FloatInputWidget->SetToolTipText(GetInputWidgetTooltip());
				// Setup & clear delegate if necessary (ex. if was just saved)
				if (InputSliderOnValueChangedDelegateHandle.IsValid())
				{
					DefaultFloat->OnDefaultValueChanged.Remove(InputSliderOnValueChangedDelegateHandle);
					InputSliderOnValueChangedDelegateHandle.Reset();
				}

				InputSliderOnValueChangedDelegateHandle = DefaultFloat->OnDefaultValueChanged.AddLambda([this, Widget = FloatInputWidget](const FGuid& PageID, float Value)
				{
					using namespace Metasound::Engine;

					if (Widget.IsValid())
					{
						UMetasoundEditorGraphNode& EdNode = GetMetaSoundNode();
						UObject& MetaSound = EdNode.GetMetasoundChecked();
						UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);

						if (Builder.GetConstBuilder().GetBuildPageID() == PageID)
						{
							const float SliderValue = Widget->GetSliderValue(Value);
							Widget->SetSliderValue(SliderValue);
						}
					}
				});

				if (InputSliderOnRangeChangedDelegateHandle.IsValid())
				{
					DefaultFloat->OnRangeChanged.Remove(InputSliderOnRangeChangedDelegateHandle);
					InputSliderOnRangeChangedDelegateHandle.Reset();
				}

				InputSliderOnRangeChangedDelegateHandle = DefaultFloat->OnRangeChanged.AddLambda([Widget = FloatInputWidget](FVector2D Range)
				{
					if (Widget.IsValid())
					{
						Widget->SetOutputRange(Range);
					}
				});
			}
			else
			{
				TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultBool> DefaultBool = Cast<UMetasoundEditorGraphMemberDefaultBool>(GraphMember->GetLiteral());
				if (DefaultBool.IsValid())
				{
					if (bUseAudioMaterialWidgets)
					{
						bool bIsNotTriggerNode = GraphMember->GetDataType() != Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>();

						if ((bIsNotTriggerNode && DefaultBool.IsValid()) && DefaultBool->WidgetType != EMetasoundBoolMemberDefaultWidget::None)
						{
							constexpr float WidgetPadding = 3.0f;
							static const FVector2D ButtonDesiredSize = FVector2D(56.0f, 87.0f);

							auto OnBoolValueChangedLambda = [this, GraphMember, DefaultBool](bool Value)
							{
								BeginOrUpdateValueTransaction(GraphMember, [this, DefaultBool, Value](const FGuid& BuildPageID, UMetasoundEditorGraphMember& Member)
								{
									if (DefaultBool.IsValid())
									{
										DefaultBool->Modify();

										{
											FMetasoundFrontendLiteral OutputLiteral;
											OutputLiteral.Set(Value);
											DefaultBool->SetFromLiteral(OutputLiteral, BuildPageID);
										}

										constexpr bool bPostTransaction = false;
										Member.UpdateFrontendDefaultLiteral(bPostTransaction, &BuildPageID);
									}
								});
							};

							auto OnBoolMouseCaptureEndLambda = [this, GraphMember, DefaultBool]()
							{
								FinalizeValueTransaction(GraphMember, [this, DefaultBool](const FGuid& BuildPageID, UMetasoundEditorGraphMember& Member, bool bPostTransaction)
								{
									if (DefaultBool.IsValid())
									{
										DefaultBool->Modify();
										const bool FinalValue = DefaultBool->GetDefaultAs<bool>(BuildPageID);

										{
											FMetasoundFrontendLiteral OutputLiteral;
											OutputLiteral.Set(FinalValue);
											DefaultBool->SetFromLiteral(OutputLiteral, BuildPageID);
										}

										Member.UpdateFrontendDefaultLiteral(bPostTransaction);
										DefaultBool->OnDefaultStateChanged.Broadcast(FinalValue, BuildPageID);
									}
								});
							};

							SAssignNew(MaterialButtonWidget, SAudioMaterialButton)
								.AudioMaterialButtonStyle(FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"), "AudioMaterialButton.Style")
								.OnBooleanValueChanged_Lambda(OnBoolValueChangedLambda)
								.OnMouseCaptureEnd_Lambda(OnBoolMouseCaptureEndLambda)
								.bIsPressedAttribute(DefaultBool->GetDefaultAs<bool>());

							SAssignNew(OuterContentBox, SVerticalBox)
							+ SVerticalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.AutoHeight()
							[
								ContentBox
							]
							+ SVerticalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Top)
							.Padding(WidgetPadding, 0.0f, WidgetPadding, WidgetPadding)
							.AutoHeight()
							[
								MaterialButtonWidget.ToSharedRef()
							];

							MaterialButtonWidget->SetDesiredSizeOverride(ButtonDesiredSize);
							MaterialButtonWidget->SetEnabled(GetInputWidgetEnabled());

							// Setup & clear delegate if necessary (ex. if was just saved)
							if (InputButtonOnStateChangedDelegateHandle.IsValid())
							{
								DefaultBool->OnDefaultStateChanged.Remove(InputButtonOnStateChangedDelegateHandle);
								InputButtonOnStateChangedDelegateHandle.Reset();
							}

							InputButtonOnStateChangedDelegateHandle = DefaultBool->OnDefaultStateChanged.AddLambda([this, Widget = MaterialButtonWidget](bool bValue, const FGuid& InPageID)
							{
								UMetasoundEditorGraphNode& EdNode = GetMetaSoundNode();
								UObject& MetaSound = EdNode.GetMetasoundChecked();
								UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);
								const FGuid& BuildPageID = Builder.GetConstBuilder().GetBuildPageID();
								if (Widget.IsValid() && BuildPageID == InPageID)
								{
									Widget->SetPressedState(bValue);
								}
							});
						}
					}
				}
			}

			return OuterContentBox;
		}

		TSharedRef<SWidget> SMetaSoundGraphNode::CreateNodeContentArea()
		{
			using namespace Engine;
			using namespace Frontend;

			FConstNodeHandle NodeHandle = GetMetaSoundNode().GetConstNodeHandle();
			const FMetasoundFrontendClassStyleDisplay& StyleDisplay = NodeHandle->GetClassStyle().Display;
			TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

			UMetasoundEditorGraphNode& EdNode = GetMetaSoundNode();
			UObject& MetaSound = EdNode.GetMetasoundChecked();
			UMetaSoundBuilderBase& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSound);

			TSharedPtr<SWidget> InputContent = CreateInputNodeContentArea(Builder.GetConstBuilder(), ContentBox);
	
			// Gives more space for user to grab a bit easier as variables do not have any title area nor icon
			const float GrabPadding = IsVariableMutator() ? 28.0f : 0.0f;

			const EVerticalAlignment PinNodeAlignInput = (!StyleDisplay.bShowInputNames && NodeHandle->GetNumInputs() == 1) ? VAlign_Center : VAlign_Top;
			ContentBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(PinNodeAlignInput)
			.FillWidth(1.0f)
			.Padding(0.0f, 0.0f, GrabPadding, 0.0f)
			[
				SAssignNew(LeftNodeBox, SVerticalBox)
			];

			if (!StyleDisplay.ImageName.IsNone())
			{
				const FSlateBrush& ImageBrush = Metasound::Editor::Style::GetSlateBrushSafe(StyleDisplay.ImageName);
				ContentBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(&ImageBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.DesiredSizeOverride(FVector2D(20, 20))
				];
			}

			const EVerticalAlignment PinNodeAlignOutput = (!StyleDisplay.bShowInputNames && NodeHandle->GetNumOutputs() == 1) ? VAlign_Center : VAlign_Top;
			ContentBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(PinNodeAlignOutput)
				.Padding(GrabPadding, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				];

			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(0,3))
				[
					InputContent.IsValid() ? InputContent.ToSharedRef() : ContentBox
				];
		}

		void SMetaSoundGraphNode::BeginOrUpdateValueTransaction(TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMemberPtr, TFunctionRef<void(const FGuid&, UMetasoundEditorGraphMember&)> SetValue)
		{
			UMetasoundEditorGraphMember* GraphMember = GraphMemberPtr.Pin().Get();
			if (!GraphMember)
			{
				return;
			}

			FMetaSoundFrontendDocumentBuilder& Builder = GraphMember->GetFrontendBuilderChecked();
			Builder.CastDocumentObjectChecked<UObject>().Modify();
			const FGuid BuildPageID = Builder.GetBuildPageID();

			if (!bIsInputWidgetTransacting)
			{
				const UMetaSoundSettings* MetaSoundSettings = GetDefault<UMetaSoundSettings>();
				check(MetaSoundSettings);
				const FMetaSoundPageSettings* PageSettings = MetaSoundSettings->FindPageSettings(BuildPageID);

				GEditor->BeginTransaction(FText::Format(LOCTEXT("MetaSoundGraphNode_SetMemberDefault", "Set MetaSound {0} '{1}' Default (Page: {2})"),
					GraphMember->GetGraphMemberLabel(),
					GraphMember->GetDisplayName(),
					PageSettings ? FText::FromName(PageSettings->Name) : LOCTEXT("MetaSoundGraphPage_Unknown", "Unknown")
				));
				bIsInputWidgetTransacting = true;
			}

			SetValue(BuildPageID, *GraphMember);
		}

		void SMetaSoundGraphNode::FinalizeValueTransaction(TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMemberPtr, TFunctionRef<void(const FGuid&, UMetasoundEditorGraphMember&, bool)> SetValue)
		{
			UMetasoundEditorGraphMember* GraphMember = GraphMemberPtr.Pin().Get();
			if (!GraphMember)
			{
				return;
			}

			FMetaSoundFrontendDocumentBuilder& Builder = GraphMember->GetFrontendBuilderChecked();
			const FGuid& BuildPageID = Builder.GetBuildPageID();
			UObject& MetaSound = Builder.CastDocumentObjectChecked<UObject>();
			MetaSound.Modify();

			bool bPostTransaction = false;
			if (!bIsInputWidgetTransacting)
			{
				bPostTransaction = true;
				UE_LOGF(LogMetaSound, Warning, "Unmatched MetaSound editor widget transaction.");
			}

			SetValue(BuildPageID, *GraphMember, bPostTransaction);

			if (bIsInputWidgetTransacting)
			{
				GEditor->EndTransaction();
				bIsInputWidgetTransacting = false;
			}

			if (UMetasoundEditorGraph* Graph = GraphMember->GetOwningGraph())
			{
				FMetasoundFrontendDocumentModifyContext& ModifyContext = FGraphBuilder::GetOutermostMetaSoundChecked(*Graph).GetModifyContext();
				ModifyContext.AddMemberIDsModified({ GraphMember->GetMemberID() });
				ModifyContext.AddNodeIDModified(GetMetaSoundNode().GetNodeID());

				// Only inputs require registration as changes to default values on other types (i.e. variables, outputs) are not external
				// graph API changes that may have an effect on other open MetaSound asset(s) visible state(s) (ex. presets, referenced node
				// defaults, etc.)
				if (GraphMember->IsA<UMetasoundEditorGraphInput>())
				{
					FGraphBuilder::RegisterGraphWithFrontend(Builder.CastDocumentObjectChecked<UObject>());
				}
			}
		}

		TSharedPtr<SGraphPin> SMetaSoundGraphNodeKnot::CreatePinWidget(UEdGraphPin* Pin) const
		{
			return SNew(SMetaSoundGraphPinKnot, Pin);
		}

		void SMetaSoundGraphNodeKnot::Construct(const FArguments& InArgs, class UEdGraphNode* InNode)
		{
			GraphNode = InNode;
			SetCursor(EMouseCursor::CardinalCross);
			UpdateGraphNode();
		}

		void SMetaSoundGraphNodeKnot::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
		{
			SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

			UMetasoundEditorGraphNode& Node = GetMetaSoundNode();
			Node.GetMetasoundChecked().Modify();
			Node.UpdateFrontendNodeLocation(FVector2D(NewPosition));
			Node.SyncLocationFromFrontendNode();
		}

		UMetasoundEditorGraphNode& SMetaSoundGraphNodeKnot::GetMetaSoundNode()
		{
			return *CastChecked<UMetasoundEditorGraphNode>(GraphNode);
		}

		const UMetasoundEditorGraphNode& SMetaSoundGraphNodeKnot::GetMetaSoundNode() const
		{
			check(GraphNode);
			return *Cast<UMetasoundEditorGraphNode>(GraphNode);
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE // MetasoundEditor
