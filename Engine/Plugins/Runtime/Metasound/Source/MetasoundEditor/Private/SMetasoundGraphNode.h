// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "MetasoundFrontendDocument.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphNodeKnot.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"


// Forward Declarations
class SAudioInputWidget;
class SAudioMaterialButton;
class SGraphPin;
class SVerticalBox;
class UMetasoundEditorGraphMember;
class UMetasoundEditorGraphMemberDefaultLiteral;
class UMetasoundEditorGraphMemberNode;
class UMetasoundEditorGraphNode;


namespace Metasound::Editor
{
	class SMetaSoundGraphNode : public SGraphNode
	{
		public:
		SLATE_BEGIN_ARGS(SMetaSoundGraphNode)
		{
		}

		SLATE_END_ARGS()
		virtual ~SMetaSoundGraphNode();

		void Construct(const FArguments& InArgs, class UEdGraphNode* InNode);

	protected:
		bool IsVariableAccessor() const;
		bool IsVariableMutator() const;

		// SGraphNode Interface
		virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
		virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
		virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
		virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin) const override;
		virtual void CreateStandardPinWidget(UEdGraphPin* InPin) override;
		virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
		virtual TSharedRef<SWidget> CreateNodeContentArea() override;
		virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;
		virtual const FSlateBrush* GetNodeBodyBrush() const override;
		virtual FText GetNodeTooltip() const override;
		virtual TSharedRef<SWidget> CreateTitleRightWidget() override;
		virtual EVisibility IsAddPinButtonVisible() const override;
		virtual FReply OnAddPin() override;
		virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
		virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
		virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
		virtual void OnCommentBubbleToggled(bool bInCommentBubbleVisible) override;
		virtual void OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo) override;
		virtual void OnAdvancedViewChanged(const ECheckBoxState NewCheckedState) override;

		FLinearColor GetNodeTitleColorOverride() const;

		FName GetLiteralDataType() const;
		UMetasoundEditorGraphNode& GetMetaSoundNode() const;

	public:
		static void ExecuteTrigger(UMetasoundEditorGraphMemberDefaultLiteral& Literal);
		static TSharedRef<SWidget> CreateTriggerSimulationWidget(UMetasoundEditorGraphMemberDefaultLiteral& Literal, TAttribute<EVisibility>&& InVisibility, TAttribute<bool>&& InEnablement, const FText* InToolTip = nullptr);

	private:
		void BeginOrUpdateValueTransaction(TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMemberPtr, TFunctionRef<void(const FGuid& /*BuildPageID*/, UMetasoundEditorGraphMember& /*GraphMember*/)> SetValue);
		void FinalizeValueTransaction(TWeakObjectPtr<UMetasoundEditorGraphMember> GraphMemberPtr, TFunctionRef<void(const FGuid& /*BuildPageID*/, UMetasoundEditorGraphMember& /*GraphMember*/, bool /*bPostTransaction*/)> SetValue);

		TSharedRef<SWidget> CreateTitleIcon() const;

		// Returns attribute that returns whether or not the input widget should be enabled.
		TAttribute<bool> GetInputWidgetEnabled() const;

		// Returns the input widget tooltip text.
		FText GetInputWidgetTooltip() const;

		// If this node represents a graph member node, returns corresponding member.
		UMetasoundEditorGraphMember* GetMetaSoundMember();

		// If this node represents a graph member node, returns cast node.
		UMetasoundEditorGraphMemberNode* GetMetaSoundMemberNode() const;

		TAttribute<EVisibility> GetSimulationVisibilityAttribute() const;

		// Input node should be moved to own implementation
		TSharedPtr<SWidget> CreateInputNodeContentArea(const FMetaSoundFrontendDocumentBuilder& InBuilder, TSharedRef<SHorizontalBox> ContentBox);

		// Slider widget for float input
		TSharedPtr<SAudioInputWidget> FloatInputWidget;

		// Button Widget for bool input.
		TSharedPtr<SAudioMaterialButton> MaterialButtonWidget;

		// Handle for on state changed delegate for Button 
		FDelegateHandle InputButtonOnStateChangedDelegateHandle;

		// Handle for on value changed delegate for input slider 
		FDelegateHandle InputSliderOnValueChangedDelegateHandle;
		
		// Handle for on input slider range changed  
		FDelegateHandle InputSliderOnRangeChangedDelegateHandle;

		// Whether the input widget is currently transacting 
		// for keeping track of transaction state across delegates to only commit transaction on value commit
		bool bIsInputWidgetTransacting = false;

		EMetasoundFrontendClassType ClassType;
	};

	class SMetaSoundGraphNodeKnot : public SGraphNodeKnot
	{
	public:
		SLATE_BEGIN_ARGS(SMetaSoundGraphNode)
		{
		}

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, class UEdGraphNode* InNode);

		virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
		virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty) override;

		UMetasoundEditorGraphNode& GetMetaSoundNode();
		const UMetasoundEditorGraphNode& GetMetaSoundNode() const;
	};
} // namespace Metasound::Editor
