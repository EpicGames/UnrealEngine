// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundGraphNodeComment.h"

#include "EdGraphNode_Comment.h"
#include "Framework/Application/SlateApplication.h"
#include "MetasoundEditorGraphCommentNode.h"
#include "MetasoundEditorGraphNode.h"

namespace Metasound
{
	namespace Editor
	{
		void SMetasoundGraphNodeComment::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
		{
			SGraphNodeComment::MoveTo(NewPosition, NodeFilter, bMarkDirty);

			// Update frontend node position for current node 
			UEdGraphNode* Node = GetNodeObj();
			if (UMetasoundEditorGraphCommentNode* MetaSoundCommentNode = Cast<UMetasoundEditorGraphCommentNode>(Node))
			{
				MetaSoundCommentNode->GetMetasoundChecked().Modify();
				MetaSoundCommentNode->UpdateFrontendNodeLocation();
			}
			
			// Update Frontend node positions for unselected nodes that are dragged along with the comment box
			// partially copied from SGraphNodeComment::MoveTo
			// Don't drag note content if either of the shift keys are down.
			FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();
			if (!KeysState.IsShiftDown())
			{
				UMetasoundEditorGraphCommentNode* CommentNode = Cast<UMetasoundEditorGraphCommentNode>(GraphNode);
				if (CommentNode && CommentNode->MoveMode == ECommentBoxMode::GroupMovement)
				{
					FVector2f PositionDelta = NewPosition - GetPosition2f();

					// Now update any nodes which are touching the comment but *not* selected
					// Selected nodes will be moved as part of the normal selection code
					TSharedPtr< SGraphPanel > Panel = GetOwnerPanel();
					for (FCommentNodeSet::TConstIterator NodeIt(CommentNode->GetNodesUnderComment()); NodeIt; ++NodeIt)
					{
						if (UMetasoundEditorGraphNode* MetasoundGraphNode = Cast<UMetasoundEditorGraphNode>(*NodeIt))
						{
							FVector2f MetasoundNodePosition = FVector2f(MetasoundGraphNode->NodePosX, MetasoundGraphNode->NodePosY);
							MetasoundNodePosition += PositionDelta;
							MetasoundGraphNode->GetMetasoundChecked().Modify();
							MetasoundGraphNode->UpdateFrontendNodeLocation(FVector2D(MetasoundNodePosition));
						}
						else if (UMetasoundEditorGraphCommentNode* MetasoundCommentNode = Cast<UMetasoundEditorGraphCommentNode>(*NodeIt))
						{
							MetasoundCommentNode->GetMetasoundChecked().Modify();
							MetasoundCommentNode->UpdateFrontendNodeLocation();
						}
					}
				}
			}
		}

	} // namespace Editor
} // namespace Metasound
