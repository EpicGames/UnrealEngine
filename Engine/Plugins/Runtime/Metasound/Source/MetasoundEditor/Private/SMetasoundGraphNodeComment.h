// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphNodeComment.h"

namespace Metasound
{
	namespace Editor
	{
		class SMetasoundGraphNodeComment : public SGraphNodeComment
		{
		public:
			// SNodePanel::SNode interface
			virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
			// End of SNodePanel::SNode interface
		};
	} // namespace Editor
} // namespace Metasound
