// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundGraphPin.h"

#include "Logging/TokenizedMessage.h"
#include "MetasoundEditorModule.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "SPinTypeSelector.h"


namespace Metasound
{
	namespace Editor
	{
		bool SMetaSoundGraphPinKnot::CanInspectPin(const UEdGraphPin* InPin)
		{
			return false;
		}

		void SMetaSoundGraphPinKnot::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
		{
			SGraphPinKnot::Construct(SGraphPinKnot::FArguments(), InPin);

			TSharedRef<SWidget> PinWidgetRef = SPinTypeSelector::ConstructPinTypeImage(
				MakeAttributeSP(this, &SMetaSoundGraphPinKnot::GetPinIcon),
				MakeAttributeSP(this, &SMetaSoundGraphPinKnot::GetPinColor),
				MakeAttributeSP(this, &SMetaSoundGraphPinKnot::GetSecondaryPinIcon),
				MakeAttributeSP(this, &SMetaSoundGraphPinKnot::GetSecondaryPinColor));
			PinImage = PinWidgetRef;

			CacheAccessType();
			CacheHasRequiredConnections();
		}

		void SMetaSoundGraphPinKnot::CacheHasRequiredConnections()
		{
			using namespace Frontend;

			bHasRequiredConnections = false;
			if (UEdGraphPin* Pin = SGraphPinKnot::GetPinObj())
			{
				if (const UMetasoundEditorGraphExternalNode* OwningNode = Cast<UMetasoundEditorGraphExternalNode>(Pin->GetOwningNode()))
				{
					if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FRerouteNodeTemplate::GetRegistryKey()))
					{
						const FMetaSoundFrontendDocumentBuilder& DocBuilder = GetBuilderChecked().GetConstBuilder();
						bHasRequiredConnections = Template->HasRequiredConnections(DocBuilder, DocBuilder.GetBuildPageID(), OwningNode->GetNodeID());
					}
				}
			}
		}

		FSlateColor SMetaSoundGraphPinKnot::GetPinColor() const
		{
			if (const UEdGraphPin* Pin = SGraphPinKnot::GetPinObj())
			{
				if (Pin->Direction == EGPD_Output)
				{
					if (!bHasRequiredConnections || Pin->GetOwningNode()->ErrorType <= static_cast<uint32>(EMessageSeverity::Warning))
					{
						return FLinearColor::Yellow;
					}
				}
			}

			return SGraphPinKnot::GetPinColor();
		}

		const FSlateBrush* SMetaSoundGraphPinKnot::GetPinIcon() const
		{
			using namespace Metasound::Frontend;

			if (const UEdGraphPin* Pin = SGraphPinKnot::GetPinObj())
			{
				if (!bHasRequiredConnections || Pin->GetOwningNode()->ErrorType <= static_cast<uint32>(EMessageSeverity::Warning))
				{
					return &Editor::Style::GetSlateBrushSafe("MetasoundEditor.Graph.InvalidReroute");
				}
			}

			const bool bIsConstructorPin = TMetasoundGraphPin<SGraphPinKnot>::AccessType == EMetasoundFrontendVertexAccessType::Value;
			if (bIsConstructorPin)
			{
				const bool bIsConnected = IsConnected();
				if (IsArray())
				{
					const FName BrushName = bIsConnected ? "MetasoundEditor.Graph.ConstructorPinArray" : "MetasoundEditor.Graph.ConstructorPinArrayDisconnected";
					return &Editor::Style::GetSlateBrushSafe(BrushName);
				}
				else
				{
					const FName BrushName = bIsConnected ? "MetasoundEditor.Graph.ConstructorPin" : "MetasoundEditor.Graph.ConstructorPinDisconnected";
					return &Editor::Style::GetSlateBrushSafe(BrushName);
				}
			}

			return SGraphPinKnot::GetPinIcon();
		}
	} // namespace Editor
} // namespace Metasound
