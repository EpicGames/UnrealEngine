// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraph/EdGraphPin.h"
#include "Styling/AppStyle.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinInteger.h"
#include "KismetPins/SGraphPinObject.h"
#include "KismetPins/SGraphPinString.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphMemberDefaults.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontendController.h"
#include "MetasoundPinAudioInspector.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "SGraphPin.h"
#include "SGraphNodeKnot.h"
#include "SMetasoundPinValueInspector.h"
#include "SPinTypeSelector.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"


namespace Metasound
{
	namespace Editor
	{
		template <typename ParentPinType>
		class TMetasoundGraphPin : public ParentPinType
		{
			TSharedPtr<SMetasoundPinValueInspector> PinInspector;

			TSharedPtr<FMetasoundPinAudioInspector> PinAudioInspector;

			// Cached builder pointer for fast access
			mutable TWeakObjectPtr<UMetaSoundBuilderBase> BuilderPtr;

		protected:
			virtual bool CanInspectPin(const UEdGraphPin* InPin)
			{
				return FGraphBuilder::CanInspectPin(InPin);
			}

			static TWeakPtr<FPinValueInspectorTooltip> OpenPinInspector(UEdGraphPin& InPin, TSharedPtr<SMetasoundPinValueInspector>& OutPinInspector)
			{
				TSharedPtr<SMetasoundPinValueInspector> NewPinInspector = SNew(SMetasoundPinValueInspector);
				TWeakPtr<FPinValueInspectorTooltip> NewTooltip = FPinValueInspectorTooltip::SummonTooltip(&InPin, NewPinInspector);
				if (NewTooltip.IsValid())
				{
					OutPinInspector = NewPinInspector;
					return NewTooltip;
				}

				return nullptr;
			}

			void UpdatePinInspector(
				UEdGraphPin& InPin,
				const bool bIsHoveringPin,
				TSharedPtr<SMetasoundPinValueInspector>& OutPinInspector,
				TWeakPtr<FPinValueInspectorTooltip>& OutInspectorTooltip,
				TFunctionRef<void(FVector2f&)> InGetTooltipLocation)
			{
				if (bIsHoveringPin)
				{
					const bool bCanInspectPin = CanInspectPin(&InPin);
					if (bCanInspectPin)
					{
						if (OutPinInspector.IsValid())
						{
							const UEdGraphPin* InspectedPin = OutPinInspector->GetPinRef().Get();
							if (InspectedPin == &InPin)
							{
								OutPinInspector->UpdateMessage();
							}
						}
						else
						{
							OutInspectorTooltip = OpenPinInspector(InPin, OutPinInspector);
							TSharedPtr<FPinValueInspectorTooltip> NewTooltip = OutInspectorTooltip.Pin();
							if (NewTooltip.IsValid())
							{
								FVector2f TooltipLocation;
								InGetTooltipLocation(TooltipLocation);
								NewTooltip->MoveTooltip(TooltipLocation);
							}
						}

						return;
					}
				}

				if (OutPinInspector.IsValid())
				{
					TSharedPtr<FPinValueInspectorTooltip> InspectorTooltip = OutInspectorTooltip.Pin();
					if (InspectorTooltip.IsValid())
					{
						if (InspectorTooltip->TooltipCanClose())
						{
							constexpr bool bForceDismiss = true;
							InspectorTooltip->TryDismissTooltip(bForceDismiss);
							OutInspectorTooltip.Reset();
							OutPinInspector.Reset();
						}
					}
					else
					{
						OutPinInspector.Reset();
					}
				}
			}

			static TWeakPtr<FPinValueInspectorTooltip> OpenPinAudioInspector(UEdGraphPin& InPin, TSharedPtr<FMetasoundPinAudioInspector>& OutPinAudioInspector)
			{
				TSharedPtr<FMetasoundPinAudioInspector> NewPinAudioInspector = MakeShared<FMetasoundPinAudioInspector>(&InPin);
				TWeakPtr<FPinValueInspectorTooltip> NewTooltip = FPinValueInspectorTooltip::SummonTooltip(&InPin, NewPinAudioInspector->GetWidget());

				if (NewTooltip.IsValid())
				{
					OutPinAudioInspector = NewPinAudioInspector;
					return NewTooltip;
				}

				return nullptr;
			}

			void UpdatePinAudioInspector(UEdGraphPin& InPin,
				const bool bIsHoveringPin,
				TSharedPtr<FMetasoundPinAudioInspector>& OutPinAudioInspector,
				TWeakPtr<FPinValueInspectorTooltip>& OutInspectorTooltip,
				TFunctionRef<void(FVector2f&)> InGetTooltipLocation)
			{
				if (bIsHoveringPin)
				{
					const bool bCanInspectPin = CanInspectPin(&InPin);
					if (bCanInspectPin)
					{
						if (!OutPinAudioInspector.IsValid())
						{
							OutInspectorTooltip = OpenPinAudioInspector(InPin, OutPinAudioInspector);
							TSharedPtr<FPinValueInspectorTooltip> NewTooltip = OutInspectorTooltip.Pin();
							if (NewTooltip.IsValid())
							{
								FVector2f TooltipLocation;
								InGetTooltipLocation(TooltipLocation);
								NewTooltip->MoveTooltip(TooltipLocation);
							}
						}

						return;
					}
				}

				if (OutPinAudioInspector.IsValid())
				{
					TSharedPtr<FPinValueInspectorTooltip> InspectorTooltip = OutInspectorTooltip.Pin();
					if (InspectorTooltip.IsValid())
					{
						if (InspectorTooltip->TooltipCanClose())
						{
							constexpr bool bForceDismiss = true;
							InspectorTooltip->TryDismissTooltip(bForceDismiss);
							OutInspectorTooltip.Reset();
							OutPinAudioInspector.Reset();
						}
					}
					else
					{
						OutPinAudioInspector.Reset();
					}
				}
			}

			void CacheAccessType()
			{
				AccessType = EMetasoundFrontendVertexAccessType::Unset;

				if (const UEdGraphPin* Pin = ParentPinType::GetPinObj())
				{
					if (const UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Pin->GetOwningNode()))
					{
						if (const UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(Node))
						{
							if (const UMetasoundEditorGraphVertex* Vertex = Cast<UMetasoundEditorGraphVertex>(MemberNode->GetMember()))
							{
								AccessType = Vertex->GetVertexAccessType();
							}
						}
						else if (const UMetasoundEditorGraphExternalNode* ExternalNode = Cast<UMetasoundEditorGraphExternalNode>(Node))
						{
							if (UMetaSoundBuilderBase* Builder = GetBuilder())
							{
								const FMetasoundFrontendVertexHandle VertexHandle = FGraphBuilder::GetPinVertexHandle(Builder->GetConstBuilder(), Pin);
								if (Pin->Direction == EGPD_Input)
								{
									AccessType = Builder->GetConstBuilder().GetNodeInputAccessType(VertexHandle.NodeID, VertexHandle.VertexID);
								}
								else if (Pin->Direction == EGPD_Output)
								{
									AccessType = Builder->GetConstBuilder().GetNodeOutputAccessType(VertexHandle.NodeID, VertexHandle.VertexID);
								}
							}
						}
					}
				}
			}

			void CacheNodeOffset(const FGeometry& AllottedGeometry)
			{
				const FVector2D UnscaledPosition = ParentPinType::OwnerNodePtr.Pin()->GetUnscaledPosition();
				ParentPinType::CachedNodeOffset = FVector2D(AllottedGeometry.AbsolutePosition) / AllottedGeometry.Scale - UnscaledPosition;
				ParentPinType::CachedNodeOffset.Y += AllottedGeometry.Size.Y * 0.5f;
			}

			EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Unset;

		public:
			SLATE_BEGIN_ARGS(TMetasoundGraphPin<ParentPinType>)
			{
			}
			SLATE_END_ARGS()

			virtual ~TMetasoundGraphPin() = default;

			const UMetasoundEditorGraphNode* GetOwningMetaSoundNode() const
			{
				if (const UEdGraphPin* Pin = ParentPinType::GetPinObj())
				{
					UObject* Node = Pin->GetOwningNode();
					return Cast<UMetasoundEditorGraphNode>(Node);
				}

				return nullptr;
			}

			UMetaSoundBuilderBase& GetBuilderChecked() const
			{
				UMetaSoundBuilderBase* Builder = GetBuilder();
				check(Builder);
				return *Builder;
			}

			UMetaSoundBuilderBase* GetBuilder() const
			{
				using namespace Metasound::Engine;

				if (UMetaSoundBuilderBase* Builder = BuilderPtr.Get())
				{
					return Builder;
				}

				const UMetasoundEditorGraphNode* Node = GetOwningMetaSoundNode();
				check(Node);
				UObject* Outermost = Node->GetOutermostObject();
				check(Outermost);
				BuilderPtr = &FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*Outermost);
				return BuilderPtr.Get();
			}

			const FMetasoundFrontendNode* GetFrontendNode() const
			{
				using namespace Metasound::Engine;

				if (const UMetasoundEditorGraphNode* Node = GetOwningMetaSoundNode())
				{
					if (const UMetaSoundBuilderBase* Builder = GetBuilder())
					{
						const FGuid NodeID = Node->GetNodeID();
						return Builder->GetConstBuilder().FindNode(NodeID);
					}
				}

				return nullptr;
			}

			const FMetasoundFrontendVertex* GetFrontendVertex() const
			{
				using namespace Metasound::Engine;

				if (UEdGraphPin* Pin = ParentPinType::GetPinObj())
				{
					if (UMetaSoundBuilderBase* Builder = GetBuilder())
					{
						return FGraphBuilder::GetPinVertex(Builder->GetConstBuilder(), Pin);
					}
				}

				return nullptr;
			}

			const FMetasoundFrontendNode& GetFrontendNodeChecked() const
			{
				const FMetasoundFrontendNode* FrontendNode = GetFrontendNode();
				check(FrontendNode);
				return *FrontendNode;
			}

			bool ShowDefaultValueWidget() const
			{
				UEdGraphPin* Pin = ParentPinType::GetPinObj();
				if (!Pin)
				{
					return true;
				}

				UMetasoundEditorGraphMemberNode* Node = Cast<UMetasoundEditorGraphMemberNode>(Pin->GetOwningNode());
				if (!Node)
				{
					return true;
				}

				UMetasoundEditorGraphMember* Member = Node->GetMember();
				if (!Member)
				{
					return true;
				}

				UMetasoundEditorGraphMemberDefaultFloat* DefaultFloat = Cast<UMetasoundEditorGraphMemberDefaultFloat>(Member->GetLiteral());
				if (!DefaultFloat)
				{
					return true;
				}

				return DefaultFloat->WidgetType == EMetasoundMemberDefaultWidget::None;
			}

			TSharedRef<SWidget> CreateResetToDefaultWidget()
			{
				return SNew(SButton)
					.ToolTipText(LOCTEXT("ResetToClassDefaultToolTip", "Reset to class default"))
					.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
					.ContentPadding(0.0f)
					.Visibility(TAttribute<EVisibility>::CreateSPLambda(ParentPinType::AsShared(), [this]()
					{
						using namespace Frontend;
						if (!ParentPinType::IsConnected())
						{
							UMetaSoundBuilderBase* Builder = GetBuilder();
							UEdGraphPin* GraphPin = ParentPinType::GetPinObj();
							if (!Builder || !Builder->GetBuilder().IsValid() || !GraphPin)
							{
								return EVisibility::Collapsed;
							}

							const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
							const FMetasoundFrontendNode* FrontendNode = nullptr;
							if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(DocBuilder, GraphPin, &FrontendNode))
							{
								check(FrontendNode);
								if (const FMetasoundFrontendVertexLiteral* Literal = DocBuilder.FindNodeInputDefault(FrontendNode->GetID(), Vertex->Name))
								{
									const bool bIsDefaultConstructed = Literal->Value.GetType() == EMetasoundFrontendLiteralType::None;
									const bool bIsTriggerDataType = Vertex->TypeName == GetMetasoundDataTypeName<FTrigger>();
									bool bIsRerouteNode = false;
									if (UMetasoundEditorGraphExternalNode* Node = Cast<UMetasoundEditorGraphExternalNode>(GraphPin->GetOwningNode()))
									{
										if (Node->GetBreadcrumb().ClassName == FRerouteNodeTemplate::ClassName)
										{
											bIsRerouteNode = true;
										}
									}
									
									if (!bIsDefaultConstructed && !bIsTriggerDataType && !bIsRerouteNode)
									{
										return EVisibility::Visible;
									}
								}
							}
						}

						return EVisibility::Collapsed;
					}))
					.OnClicked(FOnClicked::CreateSPLambda(ParentPinType::AsShared(), [this]()
					{
						using namespace Editor;
						using namespace Frontend;

						if (UEdGraphPin* Pin = ParentPinType::GetPinObj())
						{
							if (UMetasoundEditorGraphNode* Node = Cast<UMetasoundEditorGraphNode>(Pin->GetOwningNode()))
							{
								if (UMetasoundEditorGraph* MetaSoundGraph = CastChecked<UMetasoundEditorGraph>(Node->GetGraph()))
								{
									UObject& MetaSound = MetaSoundGraph->GetMetasoundChecked();
									const FScopedTransaction Transaction(LOCTEXT("MetaSoundEditorResetToClassDefault", "Reset to Class Default"));
									MetaSound.Modify();
									MetaSoundGraph->Modify();

									if (UMetasoundEditorGraphMemberNode* MemberNode = Cast<UMetasoundEditorGraphMemberNode>(Node))
									{
										FMetasoundFrontendDocumentModifyContext& ModifyContext = FGraphBuilder::GetOutermostMetaSoundChecked(MetaSound).GetModifyContext();
										UMetasoundEditorGraphMember* Member = MemberNode->GetMember();
										if (ensure(Member))
										{
											Member->ResetToClassDefault();
											ModifyContext.AddMemberIDsModified({ Member->GetMemberID() });
										}
										else
										{
											ModifyContext.SetDocumentModified();
										}
									}
									else
									{
										if (UMetaSoundBuilderBase* Builder = GetBuilder())
										{
											FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetBuilder();
											if (DocBuilder.IsValid())
											{
												if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(DocBuilder, Pin))
												{
													Builder->GetBuilder().RemoveNodeInputDefault(Node->GetNodeID(), Vertex->VertexID);
												}
											}
										}
									}
								}
							}
						}

						return FReply::Handled();
					}))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					];
			}

			virtual TSharedRef<SWidget> GetDefaultValueWidget() override
			{
				using namespace Frontend;
				TSharedRef<SWidget> DefaultWidget = ParentPinType::GetDefaultValueWidget();

				if (!ShowDefaultValueWidget())
				{
					return SNullWidget::NullWidget;
				}

				// For now, arrays do not support literals.
				// TODO: Support array literals by displaying
				// default literals (non-array too) in inspector window.
				const FMetasoundFrontendVertex* FrontendVertex = GetFrontendVertex();
				if (!FrontendVertex || ParentPinType::IsArray())
				{
					return DefaultWidget;
				}

				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						DefaultWidget
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						CreateResetToDefaultWidget()
					];
			}

			virtual const FSlateBrush* GetPinIcon() const override
			{
				const bool bIsConnected = ParentPinType::IsConnected();

				// Is constructor pin 
				if (AccessType == EMetasoundFrontendVertexAccessType::Value)
				{
					if (const ISlateStyle* MetasoundStyle = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
					{
						if (ParentPinType::IsArray())
						{
							return bIsConnected ? MetasoundStyle->GetBrush(TEXT("MetasoundEditor.Graph.ConstructorPinArray")) :
								MetasoundStyle->GetBrush(TEXT("MetasoundEditor.Graph.ConstructorPinArrayDisconnected"));
						}
						else
						{
							return bIsConnected ? MetasoundStyle->GetBrush(TEXT("MetasoundEditor.Graph.ConstructorPin")) :
								MetasoundStyle->GetBrush(TEXT("MetasoundEditor.Graph.ConstructorPinDisconnected"));
						}
					}
				}
				return SGraphPin::GetPinIcon();
			}

			virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
			{
				CacheNodeOffset(AllottedGeometry);

				if (UEdGraphPin* GraphPin = ParentPinType::GetPinObj())
				{
					const bool bIsHoveringPin = ParentPinType::IsHovered();

					// General Value Inspector update
					if (bIsHoveringPin || PinInspector.IsValid())
					{
						UpdatePinInspector(*GraphPin, bIsHoveringPin, PinInspector, ParentPinType::ValueInspectorTooltip,
						[this](FVector2f& OutTooltipLocation)
						{
							ParentPinType::GetInteractiveTooltipLocation(OutTooltipLocation);
						});
					}

					// Audio Pin Inspector update
					if (bIsHoveringPin || PinAudioInspector.IsValid())
					{
						if (UMetaSoundBuilderBase* Builder = GetBuilder())
						{
							if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(Builder->GetConstBuilder(), GraphPin))
							{
								if (Vertex->TypeName == GetMetasoundDataTypeName<FAudioBuffer>())
								{
									UpdatePinAudioInspector(*GraphPin, bIsHoveringPin, PinAudioInspector, ParentPinType::ValueInspectorTooltip,
									[this](FVector2f& OutTooltipLocation)
									{
										ParentPinType::GetInteractiveTooltipLocation(OutTooltipLocation);
									});
								}
							}
						}
					}
				}
			}
		};

		class SMetasoundGraphPin : public TMetasoundGraphPin<SGraphPin>
		{
		public:
			virtual ~SMetasoundGraphPin() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetasoundGraphPinBool : public TMetasoundGraphPin<SGraphPinBool>
		{
		public:
			virtual ~SMetasoundGraphPinBool() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPinBool::Construct(SGraphPinBool::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetasoundGraphPinFloat : public TMetasoundGraphPin<SGraphPinNum<float>>
		{
		public:
			virtual ~SMetasoundGraphPinFloat() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPinNum<float>::Construct(SGraphPinNum<float>::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetasoundGraphPinInteger : public TMetasoundGraphPin<SGraphPinInteger>
		{
		public:
			virtual ~SMetasoundGraphPinInteger() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPinInteger::Construct(SGraphPinInteger::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetasoundGraphPinObject : public TMetasoundGraphPin<SGraphPinObject>
		{
		public:
			virtual ~SMetasoundGraphPinObject() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPinObject::Construct(SGraphPinObject::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetasoundGraphPinString : public TMetasoundGraphPin<SGraphPinString>
		{
		public:
			virtual ~SMetasoundGraphPinString() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
			{
				SGraphPinString::Construct(SGraphPinString::FArguments(), InGraphPinObj);
				CacheAccessType();
			}
		};

		class SMetaSoundGraphPinKnot : public TMetasoundGraphPin<SGraphPinKnot>
		{
		public:
			virtual ~SMetaSoundGraphPinKnot() = default;

			void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

			virtual FSlateColor GetPinColor() const override;

			virtual const FSlateBrush* GetPinIcon() const override;

		protected:
			virtual bool CanInspectPin(const UEdGraphPin* InPin) override;

			void CacheHasRequiredConnections();

			bool bHasRequiredConnections = false;
		};
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
