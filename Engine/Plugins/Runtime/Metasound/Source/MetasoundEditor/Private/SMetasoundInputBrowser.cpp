// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetasoundInputBrowser.h"

#include "ContentBrowserModule.h"
#include "DocumentTemplates/MetasoundFrontendDocumentVertexTemplate.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IContentBrowserSingleton.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorModule.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "MetaSoundInputBrowser"

namespace Metasound::Editor
{

	const FName SMetasoundInputBrowser::TabId("MetaSoundInputBrowser");

	namespace InputBrowserPrivate
	{
		const FName Column_AssetName("AssetName");
		const FName TriggerTypeName("Trigger");
		const FName AudioTypeName("Audio");
		constexpr float ResetButtonWidth = 20.0f;

		FLinearColor GetColorForTypeName(FName TypeName)
		{
			TSharedRef<FMetaSoundGraphPanelPinFactory> PinFactory = FMetaSoundGraphPanelPinFactory::GetChecked();
			if (const FEdGraphPinType* PinType = PinFactory->FindPinType(TypeName))
			{
				return PinFactory->GetPinColor(*PinType);
			}
			return PinFactory->GetPinColor(FEdGraphPinType());
		}

		const FSlateBrush* GetIconForTypeName(FName TypeName)
		{
			const IMetasoundEditorModule& EditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
			constexpr bool bIsConstructorType = false;
			return EditorModule.GetIconBrush(TypeName, bIsConstructorType);
		}
		// Returns true if the given input has non-default page values that differ
		// from the default page value, meaning a reset would lose page-specific data.
		bool HasNonDefaultPageOverrides(const FMetaSoundFrontendDocumentBuilder& Builder, FName InputName)
		{
			using namespace Metasound::Frontend;

			const FMetasoundFrontendClassInput* GraphInput = Builder.FindGraphInput(InputName);
			if (!GraphInput)
			{
				return false;
			}

			const FMetasoundFrontendLiteral* DefaultPageLiteral = GraphInput->FindConstDefault(DefaultPageID);
			if (!DefaultPageLiteral)
			{
				return false;
			}

			bool bHasPageOverrides = false;
			GraphInput->IterateDefaults([&](const FGuid& PageID, const FMetasoundFrontendLiteral& Literal)
			{
				if (PageID != DefaultPageID && !Literal.IsEqual(*DefaultPageLiteral))
				{
					bHasPageOverrides = true;
				}
			});

			return bHasPageOverrides;
		}
	} // namespace InputBrowserPrivate

	// ---------------------------------------------------------------
	// FInputBrowserRow
	// ---------------------------------------------------------------

	IMetaSoundDocumentInterface* FInputBrowserRow::GetDocumentInterface() const
	{
		return Cast<IMetaSoundDocumentInterface>(Asset.Get());
	}

	void FInputBrowserRow::ReadCellData(
		FInputBrowserCellData& OutCell,
		const FMetasoundFrontendClassInput& Input,
		const FMetaSoundFrontendPresetTemplate* PresetTemplate)
	{
		using namespace Metasound::Frontend;

		OutCell.NodeID = Input.NodeID;
		OutCell.bIsPreset = (PresetTemplate != nullptr);
		OutCell.bInputExists = true;
		OutCell.bOverridesDefault = false;

		if (PresetTemplate)
		{
			const FMetaSoundFrontendPresetVertexMetadata* VertexMeta =
				PresetTemplate->FindConstVertexMetadata<FMetaSoundFrontendPresetVertexMetadata>(Input.NodeID);
			OutCell.bOverridesDefault = VertexMeta ? VertexMeta->bOverrideInheritedDefault : false;
		}

		if (const FMetasoundFrontendLiteral* DefaultLiteral = Input.FindConstDefault(DefaultPageID))
		{
			OutCell.Value = *DefaultLiteral;
		}
	}

	TArray<FInputBrowserColumnDef> FInputBrowserRow::BuildRow(TSharedPtr<FInputBrowserRow>& OutRow, UObject* InAsset)
	{
		using namespace Metasound::Frontend;

		IMetaSoundDocumentInterface* DocInterface = Cast<IMetaSoundDocumentInterface>(InAsset);
		if (!DocInterface)
		{
			return {};
		}

		TArray<FInputBrowserColumnDef> FoundColumns;

		OutRow = MakeShared<FInputBrowserRow>();
		OutRow->Asset = InAsset;
		OutRow->AssetName = InAsset->GetName();

		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		const FMetasoundFrontendClassInterface& Interface = Document.RootGraph.GetDefaultInterface();

		const FMetaSoundFrontendPresetTemplate* PresetTemplate = Document.Template.GetPtr<FMetaSoundFrontendPresetTemplate>();

		for (const FMetasoundFrontendClassInput& Input : Interface.Inputs)
		{
			if (Input.TypeName == InputBrowserPrivate::TriggerTypeName || Input.TypeName == InputBrowserPrivate::AudioTypeName)
			{
				continue;
			}

			FInputBrowserCellData CellData;
			ReadCellData(CellData, Input, PresetTemplate);
			OutRow->Cells.Add(Input.Name, MoveTemp(CellData));

			FInputBrowserColumnDef ColDef;
			ColDef.InputName = Input.Name;
			ColDef.TypeName = Input.TypeName;
			const FMetasoundFrontendLiteral* DefaultLiteral = Input.FindConstDefault(DefaultPageID);
			ColDef.LiteralType = DefaultLiteral ? DefaultLiteral->GetType() : EMetasoundFrontendLiteralType::None;
			FoundColumns.Add(MoveTemp(ColDef));
		}

		return FoundColumns;
	}

	// ---------------------------------------------------------------
	// SMetasoundInputBrowser
	// ---------------------------------------------------------------

	void SMetasoundInputBrowser::Construct(const FArguments& InArgs)
	{
		HeaderRow = SNew(SHeaderRow);

		ChildSlot
			[
				SNew(SVerticalBox)

					// Toolbar
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f)
					[
						SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("FilterLabel", "Filter:"))
							]

							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNew(SSearchBox)
									.HintText(LOCTEXT("SearchHint", "Filter inputs by name..."))
									.OnTextChanged(this, &SMetasoundInputBrowser::OnNameFilterChanged)
							]

					]

				// Splitter: Pin list | Table
				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SSplitter)
							.Orientation(Orient_Horizontal)

							// Left panel: Table
							+ SSplitter::Slot()
							.Value(0.75f)
							[
								SNew(SScrollBox)
									.Orientation(Orient_Horizontal)
									+ SScrollBox::Slot()
									[
										SAssignNew(ListView, SListView<TSharedPtr<FInputBrowserRow>>)
											.ListItemsSource(&Rows)
											.OnGenerateRow(this, &SMetasoundInputBrowser::OnGenerateRow)
											.HeaderRow(HeaderRow)
											.SelectionMode(ESelectionMode::Single)
											.OnContextMenuOpening(this, &SMetasoundInputBrowser::OnAssetContextMenuOpening)
									]
							]

						// Right panel: Pinned columns list
						+ SSplitter::Slot()
							.Value(0.25f)
							[
								SNew(SVerticalBox)

									+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(4.0f)
									[
										SNew(SHorizontalBox)

											+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.VAlign(VAlign_Center)
											[
												SNew(STextBlock)
													.Text(LOCTEXT("InputsHeader", "Inputs"))
													.Font(FAppStyle::GetFontStyle("BoldFont"))
											]

											+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(2.0f, 0.0f)
											[
												SNew(SButton)
													.ButtonStyle(FAppStyle::Get(), "SimpleButton")
													.ToolTipText(LOCTEXT("ShowAllTooltip", "Show All"))
													.OnClicked_Lambda([this]()
														{
															bHasExplicitPins = true;
															PinnedInputNames.Empty();
															for (const FInputBrowserColumnDef& ColDef : AllColumns)
															{
																PinnedInputNames.Add(ColDef.InputName);
															}
															ApplyFilters();
															RebuildColumns();
															if (ListView.IsValid()) { ListView->RequestListRefresh(); }
															if (PinListView.IsValid()) { PinListView->RequestListRefresh(); }
															return FReply::Handled();
														})
													[
														SNew(SImage)
															.Image(FAppStyle::GetBrush("Icons.Pinned"))
															.ColorAndOpacity(FSlateColor::UseForeground())
													]
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(2.0f, 0.0f)
											[
												SNew(SButton)
													.ButtonStyle(FAppStyle::Get(), "SimpleButton")
													.ToolTipText(LOCTEXT("HideAllTooltip", "Hide All"))
													.OnClicked_Lambda([this]()
														{
															bHasExplicitPins = true;
															PinnedInputNames.Empty();
															ApplyFilters();
															RebuildColumns();
															if (ListView.IsValid()) { ListView->RequestListRefresh(); }
															if (PinListView.IsValid()) { PinListView->RequestListRefresh(); }
															return FReply::Handled();
														})
													[
														SNew(SImage)
															.Image(FAppStyle::GetBrush("Icons.Unpinned"))
															.ColorAndOpacity(FSlateColor::UseForeground())
													]
											]
									]

								+ SVerticalBox::Slot()
									.FillHeight(1.0f)
									[
										SAssignNew(PinListView, SListView<TSharedPtr<FInputBrowserColumnDef>>)
											.ListItemsSource(&FilteredPinListItems)
											.OnGenerateRow(this, &SMetasoundInputBrowser::OnGeneratePinRow)
											.SelectionMode(ESelectionMode::None)
									]
							]
					]
			];

		PostUndoRedoHandle = FEditorDelegates::PostUndoRedo.AddSP(this, &SMetasoundInputBrowser::RefreshAllRows);
	}

	SMetasoundInputBrowser::~SMetasoundInputBrowser()
	{
		FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoHandle);
		UnsubscribeAllBuilders();
	}

	FReply SMetasoundInputBrowser::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
	{
		RefreshAllRows();
		ScanAndSubscribeBuilders();
		return SCompoundWidget::OnFocusReceived(MyGeometry, InFocusEvent);
	}

	void SMetasoundInputBrowser::SetAssets(const TArray<TWeakObjectPtr<UObject>>& InAssets)
	{
		// Unsubscribe and resubscribe to builders, because the delegate lambdas refer to row data
		//  which will no longer be valid after rebuilding.
		UnsubscribeAllBuilders();

		SourceAssets = InAssets;
		RebuildData();
	}

	void SMetasoundInputBrowser::RebuildData()
	{
		Rows.Empty();
		AllColumns.Empty();
		PinnedInputNames.Empty();
		bHasExplicitPins = false;

		for (const TWeakObjectPtr<UObject>& WeakAsset : SourceAssets)
		{
			UObject* Asset = WeakAsset.Get();
			if (!Asset)
			{
				continue;
			}

			TSharedPtr<FInputBrowserRow> Row;
			TArray<FInputBrowserColumnDef> AssetColumns = FInputBrowserRow::BuildRow(Row, Asset);

			if (Row.IsValid())
			{
				Rows.Add(Row);

				// Merge into AllColumns (union)
				for (const FInputBrowserColumnDef& ColDef : AssetColumns)
				{
					AllColumns.AddUnique(ColDef);
				}
			}
		}

		// Sort columns alphabetically
		AllColumns.Sort([](const FInputBrowserColumnDef& A, const FInputBrowserColumnDef& B)
			{
				return A.InputName.LexicalLess(B.InputName);
			});

		ApplyFilters();
		RebuildColumns();

		if (ListView.IsValid())
		{
			ListView->RebuildList();
		}
		if (PinListView.IsValid())
		{
			PinListView->RequestListRefresh();
		}

		ScanAndSubscribeBuilders();
	}

	void SMetasoundInputBrowser::ApplyFilters()
	{
		FilteredColumns.Empty();
		FilteredPinListItems.Empty();

		for (const FInputBrowserColumnDef& ColDef : AllColumns)
		{
			// Name filter
			if (!NameFilter.IsEmpty())
			{
				if (!ColDef.InputName.ToString().Contains(NameFilter))
				{
					continue;
				}
			}

			// Passes name filter - add to pin list (always)
			FilteredPinListItems.Add(MakeShared<FInputBrowserColumnDef>(ColDef));

			// Pin filter - only add to table columns if pinned (or no explicit pins yet)
			if (!bHasExplicitPins || PinnedInputNames.Contains(ColDef.InputName))
			{
				FilteredColumns.Add(ColDef);
			}
		}

		// Update per-row visibility and sort rows without visible inputs to the bottom
		for (TSharedPtr<FInputBrowserRow>& Row : Rows)
		{
			bool bHasAny = false;
			for (const FInputBrowserColumnDef& ColDef : FilteredColumns)
			{
				const FInputBrowserCellData* Cell = Row->Cells.Find(ColDef.InputName);
				if (Cell && Cell->bInputExists)
				{
					bHasAny = true;
					break;
				}
			}
			Row->bHasVisibleInputs = bHasAny;
		}

		Rows.StableSort([](const TSharedPtr<FInputBrowserRow>& A, const TSharedPtr<FInputBrowserRow>& B)
			{
				if (A->bHasVisibleInputs != B->bHasVisibleInputs)
				{
					return A->bHasVisibleInputs; // rows with visible inputs first
				}
				return false; // preserve original order otherwise
			});
	}

	void SMetasoundInputBrowser::RebuildColumns()
	{
		if (!HeaderRow.IsValid())
		{
			return;
		}

		// Build all column args up front, then apply in one batch
		TArray<SHeaderRow::FColumn::FArguments> ColumnArgs;

		// Asset name column (always first)
		ColumnArgs.Add(
			SHeaderRow::Column(InputBrowserPrivate::Column_AssetName)
			.DefaultLabel(LOCTEXT("AssetNameColumn", "Asset"))
			.ManualWidth(200.0f)
			.ShouldGenerateWidget(true)
		);

		// One column per filtered input
		for (const FInputBrowserColumnDef& ColDef : FilteredColumns)
		{
			FText TypeSuffix = FText::FromName(ColDef.TypeName);
			FText Label = FText::Format(LOCTEXT("InputColumnLabelFmt", "{0}"), FText::FromName(ColDef.InputName));
			FText Tooltip = FText::Format(LOCTEXT("InputColumnTooltipFmt", "{0} ({1})"), FText::FromName(ColDef.InputName), TypeSuffix);

			ColumnArgs.Add(
				SHeaderRow::Column(ColDef.InputName)
				.DefaultLabel(Label)
				.DefaultTooltip(Tooltip)
				.ManualWidth(140.0f)
				.ShouldGenerateWidget(true)
			);
		}

		HeaderRow->ClearColumns();
		HeaderRow->AddColumns(ColumnArgs);
	}

	TSharedRef<ITableRow> SMetasoundInputBrowser::OnGenerateRow(
		TSharedPtr<FInputBrowserRow> InRow,
		const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SMetasoundInputBrowserTableRow, OwnerTable)
			.RowData(InRow)
			.Browser(this);
	}

	TSharedRef<SWidget> SMetasoundInputBrowser::GenerateCellWidget(
		const TSharedRef<FInputBrowserRow>& InRow,
		const FInputBrowserColumnDef& InColumnDef)
	{
		FInputBrowserCellData* CellData = InRow->Cells.Find(InColumnDef.InputName);

		if (!CellData || !CellData->bInputExists)
		{

			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("NAText", "\u2014"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Justification(ETextJustify::Center)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SBox).MinDesiredWidth(InputBrowserPrivate::ResetButtonWidth)
				];
		}

		// Determine visual style for inherited preset values
		const bool bIsInherited = CellData->bIsPreset && !CellData->bOverridesDefault;
		FText InheritedTooltip = bIsInherited
			? LOCTEXT("InheritedTooltip", "Inherited from parent preset (not overridden)")
			: FText::GetEmpty();

		TWeakPtr<FInputBrowserRow> WeakRow = InRow;
		FName InputName = InColumnDef.InputName;

		// Build the value widget based on type
		TSharedRef<SWidget> ValueWidget = SNullWidget::NullWidget;

		switch (CellData->Value.GetType())
		{
		case EMetasoundFrontendLiteralType::Boolean:
		{
			bool bCurrentValue = false;
			CellData->Value.TryGet(bCurrentValue);

			ValueWidget = SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(bCurrentValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
						.OnCheckStateChanged_Lambda([this, WeakRow, InputName](ECheckBoxState NewState)
							{
								if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
								{
									FMetasoundFrontendLiteral NewLiteral;
									NewLiteral.Set(NewState == ECheckBoxState::Checked);
									CommitCellValue(Row.ToSharedRef(), InputName, NewLiteral);
								}
							})
						.ToolTipText(InheritedTooltip)
				];
			break;
		}

		case EMetasoundFrontendLiteralType::Float:
		{
			float CurrentValue = 0.0f;
			CellData->Value.TryGet(CurrentValue);

			ValueWidget = SNew(SSpinBox<float>)
				.Value(CurrentValue)
				.OnValueCommitted_Lambda([this, WeakRow, InputName](float NewValue, ETextCommit::Type)
					{
						if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
						{
							FMetasoundFrontendLiteral NewLiteral;
							NewLiteral.Set(NewValue);
							CommitCellValue(Row.ToSharedRef(), InputName, NewLiteral);
						}
					})
				.ToolTipText(InheritedTooltip);
			break;
		}

		case EMetasoundFrontendLiteralType::Integer:
		{
			int32 CurrentValue = 0;
			CellData->Value.TryGet(CurrentValue);

			ValueWidget = SNew(SSpinBox<int32>)
				.Value(CurrentValue)
				.OnValueCommitted_Lambda([this, WeakRow, InputName](int32 NewValue, ETextCommit::Type)
					{
						if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
						{
							FMetasoundFrontendLiteral NewLiteral;
							NewLiteral.Set(NewValue);
							CommitCellValue(Row.ToSharedRef(), InputName, NewLiteral);
						}
					})
				.ToolTipText(InheritedTooltip);
			break;
		}

		case EMetasoundFrontendLiteralType::String:
		{
			FString CurrentValue;
			CellData->Value.TryGet(CurrentValue);

			ValueWidget = SNew(SEditableTextBox)
				.Text(FText::FromString(CurrentValue))
				.OnTextCommitted_Lambda([this, WeakRow, InputName](const FText& NewText, ETextCommit::Type)
					{
						if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
						{
							FMetasoundFrontendLiteral NewLiteral;
							NewLiteral.Set(NewText.ToString());
							CommitCellValue(Row.ToSharedRef(), InputName, NewLiteral);
						}
					})
				.ToolTipText(InheritedTooltip);
			break;
		}

		default:
		{
			// For arrays, UObjects, and other complex types: display read-only summary
			FText FullText = FText::FromString(CellData->Value.ToString());
			FText DisplayText;

			if (CellData->Value.IsArray())
			{
				DisplayText = FText::Format(LOCTEXT("ArrayElementCountFmt", "({0} element array)"), CellData->Value.GetArrayNum());
			}
			else if (CellData->Value.GetType() == EMetasoundFrontendLiteralType::UObject)
			{
				UObject* Object = nullptr;
				CellData->Value.TryGet(Object);
				DisplayText = Object
					? FText::FromString(Object->GetName())
					: LOCTEXT("NoneObject", "(None)");
			}
			else
			{
				DisplayText = FullText;
			}

			ValueWidget = SNew(STextBlock)
				.Text(DisplayText)
				.ToolTipText(FullText);
			break;
		}
		}

		// Non-preset cells: value widget + right spacer for alignment
		if (!CellData->bIsPreset)
		{
			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					ValueWidget
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SBox).MinDesiredWidth(InputBrowserPrivate::ResetButtonWidth)
				];
		}

		// Preset cells: disable the value widget when inheriting, and show either
		// a pencil icon (to begin overriding) or a reset icon (to revert to inherited).
		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
					.IsEnabled_Lambda([WeakRow, InputName]()
						{
							if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
							{
								if (const FInputBrowserCellData* Cell = Row->Cells.Find(InputName))
								{
									return Cell->bOverridesDefault;
								}
							}
							return false;
						})
					[
						ValueWidget
					]
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				// Pencil icon: visible when inheriting, click to begin overriding
				SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("BeginOverrideTooltip", "Override inherited default"))
					.Visibility_Lambda([WeakRow, InputName]()
						{
							if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
							{
								if (const FInputBrowserCellData* Cell = Row->Cells.Find(InputName))
								{
									return Cell->bOverridesDefault ? EVisibility::Collapsed : EVisibility::Visible;
								}
							}
							return EVisibility::Collapsed;
						})
					.OnClicked_Lambda([this, WeakRow, InputName]()
						{
							if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
							{
								BeginOverride(Row.ToSharedRef(), InputName);
							}
							return FReply::Handled();
						})
					.ContentPadding(0)
					[
						SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Edit"))
							.ColorAndOpacity(FSlateColor::UseForeground())
					]
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				// Reset icon: visible when overriding, click to revert to inherited
				SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("ResetToInheritedTooltip", "Reset to inherited default from parent preset"))
					.Visibility_Lambda([WeakRow, InputName]()
						{
							if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
							{
								if (const FInputBrowserCellData* Cell = Row->Cells.Find(InputName))
								{
									return Cell->bOverridesDefault ? EVisibility::Visible : EVisibility::Collapsed;
								}
							}
							return EVisibility::Collapsed;
						})
					.OnClicked_Lambda([this, WeakRow, InputName]()
						{
							if (TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin())
							{
								ResetToDefault(Row.ToSharedRef(), InputName);
							}
							return FReply::Handled();
						})
					.ContentPadding(0)
					[
						SNew(SImage)
							.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
							.ColorAndOpacity(FSlateColor::UseForeground())
					]
			];
	}

	// Apply a new value to the asset
	void SMetasoundInputBrowser::CommitCellValue(
		const TSharedRef<FInputBrowserRow>& InRow,
		FName InputName,
		const FMetasoundFrontendLiteral& NewValue)
	{
		using namespace Metasound::Frontend;

		IMetaSoundDocumentInterface* DocInterface = InRow->GetDocumentInterface();
		if (!DocInterface)
		{
			return;
		}

		UObject* Asset = InRow->Asset.Get();

		FScopedTransaction Transaction(LOCTEXT("EditMetaSoundInput", "Edit MetaSound Input Value"));
		Asset->Modify();

		IDocumentBuilderRegistry& Registry = IDocumentBuilderRegistry::GetChecked();
		FMetaSoundFrontendDocumentBuilder& Builder = Registry.FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface>(Asset));
		SubscribeToBuilder(Builder, InRow);
		Builder.SetGraphInputDefault(InputName, NewValue, &DefaultPageID);

		RefreshRow(InRow);

		if (ListView.IsValid())
		{
			ListView->RebuildList();
		}
	}

	void SMetasoundInputBrowser::ResetToDefault(
		const TSharedRef<FInputBrowserRow>& InRow,
		FName InputName)
	{
		using namespace Metasound::Frontend;

		IMetaSoundDocumentInterface* DocInterface = InRow->GetDocumentInterface();
		if (!DocInterface)
		{
			return;
		}

		UObject* Asset = InRow->Asset.Get();

		IDocumentBuilderRegistry& Registry = IDocumentBuilderRegistry::GetChecked();
		FMetaSoundFrontendDocumentBuilder& Builder = Registry.FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface>(Asset));
		SubscribeToBuilder(Builder, InRow);

		if (InputBrowserPrivate::HasNonDefaultPageOverrides(Builder, InputName))
		{
			EAppReturnType::Type Result = FMessageDialog::Open(
				EAppMsgType::YesNo,
				LOCTEXT("ResetPagedDefaultsWarning",
					"This input has overridden values on non-default pages that will be lost.\n\n"
					"Are you sure you want to reset all pages to inherited default values?"),
				LOCTEXT("ResetPagedDefaultsTitle", "Reset Input Default"));

			if (Result != EAppReturnType::Yes)
			{
				return;
			}
		}

		FScopedTransaction Transaction(LOCTEXT("ResetMetaSoundInput", "Reset MetaSound Input to Inherited Default"));
		Asset->Modify();

		Builder.ResetGraphInputDefault(InputName);

		// ResetGraphInputDefault zeros the literal. Re-run ConfigureDocument so
		// the inherited value gets repopulated from the parent.
		Builder.ConfigureDocument();

		RefreshRow(InRow);

		if (ListView.IsValid())
		{
			ListView->RebuildList();
		}
	}

	void SMetasoundInputBrowser::BeginOverride(
		const TSharedRef<FInputBrowserRow>& InRow,
		FName InputName)
	{
		FInputBrowserCellData* CellData = InRow->Cells.Find(InputName);
		if (!CellData || !CellData->bIsPreset)
		{
			return;
		}

		// Read the current (inherited) value from the cell, then write it back
		// as an explicit override via SetGraphInputDefault, which will set
		// bOverrideInheritedDefault to true.
		CommitCellValue(InRow, InputName, CellData->Value);
	}

	TSharedPtr<SWidget> SMetasoundInputBrowser::OnAssetContextMenuOpening()
	{
		if (!ListView.IsValid())
		{
			return nullptr;
		}

		TArray<TSharedPtr<FInputBrowserRow>> SelectedItems = ListView->GetSelectedItems();
		if (SelectedItems.IsEmpty())
		{
			return nullptr;
		}

		TSharedPtr<FInputBrowserRow> SelectedRow = SelectedItems[0];
		UObject* Asset = SelectedRow->Asset.Get();
		if (!Asset)
		{
			return nullptr;
		}

		FMenuBuilder MenuBuilder(true, nullptr);

		FAssetData AssetData(Asset);

		MenuBuilder.BeginSection("AssetActions", LOCTEXT("AssetActionsSection", "Asset Actions"));

		TWeakObjectPtr<UObject> WeakAsset = Asset;
		MenuBuilder.AddMenuEntry(
			LOCTEXT("EditAsset", "Edit Asset"),
			LOCTEXT("EditAssetTooltip", "Open this asset in its editor"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateLambda([WeakAsset]()
				{
					if (UObject* AssetPtr = WeakAsset.Get())
					{
						if (GEditor)
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetPtr);
						}
					}
				}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("BrowseToAsset", "Browse to Asset"),
			LOCTEXT("BrowseToAssetTooltip", "Show this asset in the Content Browser"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
			FUIAction(FExecuteAction::CreateLambda([AssetData]()
				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().SyncBrowserToAssets(TArray<FAssetData>{ AssetData });
				}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyAssetReference", "Copy Reference"),
			LOCTEXT("CopyAssetReferenceTooltip", "Copy the asset reference path to the clipboard"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(FExecuteAction::CreateLambda([AssetData]()
				{
					FPlatformApplicationMisc::ClipboardCopy(*AssetData.GetExportTextName());
				}))
		);

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void SMetasoundInputBrowser::OnNameFilterChanged(const FText& InText)
	{
		NameFilter = InText.ToString();
		ApplyFilters();
		RebuildColumns();

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
		if (PinListView.IsValid())
		{
			PinListView->RequestListRefresh();
		}
	}

	// ---------------------------------------------------------------
	// Auto-refresh
	// ---------------------------------------------------------------

	void SMetasoundInputBrowser::RefreshAllRows()
	{
		for (const TSharedPtr<FInputBrowserRow>& Row : Rows)
		{
			if (Row.IsValid())
			{
				RefreshRow(Row.ToSharedRef());
			}
		}

		if (ListView.IsValid())
		{
			ListView->RebuildList();
		}
	}

	void SMetasoundInputBrowser::SubscribeToBuilder(FMetaSoundFrontendDocumentBuilder& Builder, const TSharedRef<FInputBrowserRow>& InRow)
	{
		FObjectKey AssetKey(InRow->Asset.Get());
		if (SubscribedBuilders.Contains(AssetKey))
		{
			return;
		}

		TWeakPtr<FInputBrowserRow> WeakRow = InRow;
		TWeakPtr<SMetasoundInputBrowser> WeakThis = SharedThis(this);

		FDelegateHandle Handle = Builder.GetDocumentDelegates().InterfaceDelegates.OnInputDefaultChanged.AddLambda(
			[WeakThis, WeakRow](const Metasound::Frontend::FDocumentArrayPagedInputArgs&)
			{
				TSharedPtr<SMetasoundInputBrowser> This = WeakThis.Pin();
				TSharedPtr<FInputBrowserRow> Row = WeakRow.Pin();
				if (This && Row.IsValid())
				{
					This->RefreshRow(Row.ToSharedRef());
					if (This->ListView.IsValid())
					{
						This->ListView->RebuildList();
					}
				}
			});

		SubscribedBuilders.Add(AssetKey, Handle);
	}

	void SMetasoundInputBrowser::ScanAndSubscribeBuilders()
	{
		using namespace Metasound::Frontend;

		IDocumentBuilderRegistry& Registry = IDocumentBuilderRegistry::GetChecked();

		for (const TSharedPtr<FInputBrowserRow>& Row : Rows)
		{
			if (!Row.IsValid())
			{
				continue;
			}

			UObject* Asset = Row->Asset.Get();
			if (!Asset)
			{
				continue;
			}

			FMetaSoundFrontendDocumentBuilder* Builder = Registry.FindBuilder(TScriptInterface<IMetaSoundDocumentInterface>(Asset));
			if (Builder)
			{
				SubscribeToBuilder(*Builder, Row.ToSharedRef());
			}
		}
	}

	void SMetasoundInputBrowser::UnsubscribeAllBuilders()
	{
		using namespace Metasound::Frontend;

		IDocumentBuilderRegistry& Registry = IDocumentBuilderRegistry::GetChecked();

		for (TPair<FObjectKey, FDelegateHandle>& Pair : SubscribedBuilders)
		{
			UObject* Asset = Pair.Key.ResolveObjectPtr();
			if (!Asset)
			{
				continue;
			}

			FMetaSoundFrontendDocumentBuilder* Builder = Registry.FindBuilder(TScriptInterface<IMetaSoundDocumentInterface>(Asset));
			if (Builder)
			{
				Builder->GetDocumentDelegates().InterfaceDelegates.OnInputDefaultChanged.Remove(Pair.Value);
			}
		}
		SubscribedBuilders.Empty();
	}

	void SMetasoundInputBrowser::RefreshRow(const TSharedRef<FInputBrowserRow>& InRow)
	{
		using namespace Metasound::Frontend;

		IMetaSoundDocumentInterface* DocInterface = InRow->GetDocumentInterface();
		if (!DocInterface)
		{
			return;
		}

		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		const FMetasoundFrontendClassInterface& Interface = Document.RootGraph.GetDefaultInterface();

		const FMetaSoundFrontendPresetTemplate* PresetTemplate = Document.Template.GetPtr<FMetaSoundFrontendPresetTemplate>();

		// Update cell data in place (or create new entries)
		for (const FMetasoundFrontendClassInput& Input : Interface.Inputs)
		{
			if (Input.TypeName == InputBrowserPrivate::TriggerTypeName || Input.TypeName == InputBrowserPrivate::AudioTypeName)
			{
				continue;
			}

			FInputBrowserRow::ReadCellData(InRow->Cells.FindOrAdd(Input.Name), Input, PresetTemplate);
		}
	}

	// ---------------------------------------------------------------
	// Pin list
	// ---------------------------------------------------------------

	void SMetasoundInputBrowser::TogglePin(FName InputName)
	{
		if (!bHasExplicitPins)
		{
			// First pin toggle: initialize all current columns as pinned
			bHasExplicitPins = true;
			for (const FInputBrowserColumnDef& ColDef : AllColumns)
			{
				PinnedInputNames.Add(ColDef.InputName);
			}
		}

		if (PinnedInputNames.Contains(InputName))
		{
			PinnedInputNames.Remove(InputName);
		}
		else
		{
			PinnedInputNames.Add(InputName);
		}

		ApplyFilters();
		RebuildColumns();

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
		if (PinListView.IsValid())
		{
			PinListView->RequestListRefresh();
		}
	}

	bool SMetasoundInputBrowser::IsPinned(FName InputName) const
	{
		return !bHasExplicitPins || PinnedInputNames.Contains(InputName);
	}

	TSharedRef<ITableRow> SMetasoundInputBrowser::OnGeneratePinRow(
		TSharedPtr<FInputBrowserColumnDef> InItem,
		const TSharedRef<STableViewBase>& OwnerTable)
	{
		const FName InputName = InItem->InputName;
		const FName TypeName = InItem->TypeName;
		const FLinearColor TypeColor = InputBrowserPrivate::GetColorForTypeName(TypeName);

		return SNew(STableRow<TSharedPtr<FInputBrowserColumnDef>>, OwnerTable)
			[
				SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					[
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked_Lambda([this, InputName]()
								{
									TogglePin(InputName);
									return FReply::Handled();
								})
							.ContentPadding(0)
							[
								SNew(SImage)
									.Image_Lambda([this, InputName]()
										{
											return IsPinned(InputName)
												? FAppStyle::GetBrush("Icons.Pinned")
												: FAppStyle::GetBrush("Icons.Unpinned");
										})
									.ColorAndOpacity(FSlateColor::UseForeground())
							]
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SImage)
							.Image(InputBrowserPrivate::GetIconForTypeName(TypeName))
							.ColorAndOpacity(TypeColor)
							.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromName(InputName))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromName(TypeName))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
			];
	}

	// ---------------------------------------------------------------
	// SMetasoundInputBrowserTableRow
	// ---------------------------------------------------------------

	void SMetasoundInputBrowserTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
	{
		RowData = InArgs._RowData;
		Browser = InArgs._Browser;

		SMultiColumnTableRow<TSharedPtr<FInputBrowserRow>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "TableView.AlternatingRow"),
			OwnerTable);
	}

	TSharedRef<SWidget> SMetasoundInputBrowserTableRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (!RowData.IsValid() || !Browser)
		{
			return SNullWidget::NullWidget;
		}

		TSharedRef<SWidget> CellContent = SNullWidget::NullWidget;

		// Asset name column
		if (ColumnName == InputBrowserPrivate::Column_AssetName)
		{
			CellContent = SNew(STextBlock)
				.Text(FText::FromString(RowData->AssetName))
				.Margin(FMargin(4.0f, 0.0f))
				.ColorAndOpacity(RowData->bHasVisibleInputs ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground());
		}
		else
		{
			// Find matching column def
			for (const FInputBrowserColumnDef& ColDef : Browser->FilteredColumns)
			{
				if (ColDef.InputName == ColumnName)
				{
					CellContent = Browser->GenerateCellWidget(RowData.ToSharedRef(), ColDef);
					break;
				}
			}
		}

		return SNew(SBox)
			.MinDesiredHeight(20.0f)
			.VAlign(VAlign_Center)
			[
				CellContent
			];
	}

} // namespace Metasound::Editor

#undef LOCTEXT_NAMESPACE
