// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendLiteral.h"
#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FMetasoundFrontendClassInput;
struct FMetaSoundFrontendDocumentBuilder;
struct FMetaSoundFrontendPresetTemplate;
class IMetaSoundDocumentInterface;
class SHeaderRow;

namespace Metasound::Editor
{

	/** Describes a single column in the MetaSound Input Browser (one per unique input name across all loaded assets). */
	struct FInputBrowserColumnDef
	{
		FName InputName;
		FName TypeName;
		EMetasoundFrontendLiteralType LiteralType = EMetasoundFrontendLiteralType::None;

		bool operator==(const FInputBrowserColumnDef& Other) const
		{
			return InputName == Other.InputName;
		}
	};

	/** Per-cell data for a single (asset, input) pair. */
	struct FInputBrowserCellData
	{
		FMetasoundFrontendLiteral Value;
		FGuid NodeID;
		bool bIsPreset = false;
		bool bOverridesDefault = false;
		bool bInputExists = false;
	};

	/** One row in the Input Browser, representing a single MetaSound asset. */
	struct FInputBrowserRow
	{
		TWeakObjectPtr<UObject> Asset;
		FString AssetName;
		TMap<FName, FInputBrowserCellData> Cells;
		bool bHasVisibleInputs = true;

		/** Returns the document interface for this row's asset, or null if invalid. */
		IMetaSoundDocumentInterface* GetDocumentInterface() const;

		/** Populate a cell's data from a class input and optional preset template. */
		static void ReadCellData(
			FInputBrowserCellData& OutCell,
			const FMetasoundFrontendClassInput& Input,
			const FMetaSoundFrontendPresetTemplate* PresetTemplate);

		/** Populate this row's cell data from the given MetaSound asset. Returns the set of column defs found. */
		static TArray<FInputBrowserColumnDef> BuildRow(TSharedPtr<FInputBrowserRow>& OutRow, UObject* InAsset);
	};

	/**
	 * A bulk-editing tool for MetaSound input values, similar to the Property Matrix.
	 * Displays a table with rows = assets and columns = inputs.
	 */
	class SMetasoundInputBrowser : public SCompoundWidget
	{
	public:
		static const FName TabId;

		SLATE_BEGIN_ARGS(SMetasoundInputBrowser) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
		virtual ~SMetasoundInputBrowser();

		virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

		/** Populate the browser with the given MetaSound assets. */
		void SetAssets(const TArray<TWeakObjectPtr<UObject>>& InAssets);

	private:
		/** Rebuild rows and column union from the current asset set. */
		void RebuildData();

		/** Rebuild the SHeaderRow columns from FilteredColumns. */
		void RebuildColumns();

		/** Apply name/type filters to AllColumns to produce FilteredColumns. */
		void ApplyFilters();

		friend class SMetasoundInputBrowserTableRow;

		/** SListView callbacks. */
		TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FInputBrowserRow> InRow, const TSharedRef<STableViewBase>& OwnerTable);

		/** Generate a cell widget for a given row and column. */
		TSharedRef<SWidget> GenerateCellWidget(const TSharedRef<FInputBrowserRow>& InRow, const FInputBrowserColumnDef& InColumnDef);

		/** Commit an edited value back to the asset. */
		void CommitCellValue(const TSharedRef<FInputBrowserRow>& InRow, FName InputName, const FMetasoundFrontendLiteral& NewValue);

		/** Reset a preset input to its inherited default value. */
		void ResetToDefault(const TSharedRef<FInputBrowserRow>& InRow, FName InputName);

		/** Begin overriding an inherited preset input (copies the inherited value as an explicit override). */
		void BeginOverride(const TSharedRef<FInputBrowserRow>& InRow, FName InputName);

		/** Filter callback. */
		void OnNameFilterChanged(const FText& InText);

		/** Context menu for right-clicking on an asset row. */
		TSharedPtr<SWidget> OnAssetContextMenuOpening();

		/** Refresh a single row's cell data from its asset. */
		void RefreshRow(const TSharedRef<FInputBrowserRow>& InRow);

		/** Refresh all rows from their assets. Called when the widget regains focus. */
		void RefreshAllRows();

		/** Subscribe to OnInputDefaultChanged on a builder if not already subscribed. */
		void SubscribeToBuilder(FMetaSoundFrontendDocumentBuilder& Builder, const TSharedRef<FInputBrowserRow>& InRow);

		/** Scan all rows for existing builders and subscribe to their delegates. */
		void ScanAndSubscribeBuilders();

		/** Unsubscribe from all builder delegates and clear the subscriptions map. */
		void UnsubscribeAllBuilders();

		/** Pin column callbacks. */
		void TogglePin(FName InputName);
		bool IsPinned(FName InputName) const;
		TSharedRef<ITableRow> OnGeneratePinRow(TSharedPtr<FInputBrowserColumnDef> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		// Data
		TArray<TWeakObjectPtr<UObject>> SourceAssets;
		TArray<TSharedPtr<FInputBrowserRow>> Rows;
		TArray<FInputBrowserColumnDef> AllColumns;
		TArray<FInputBrowserColumnDef> FilteredColumns;

		// Filter state
		FString NameFilter;

		// Pin state
		TSet<FName> PinnedInputNames;
		bool bHasExplicitPins = false; // When false, all columns are shown
		TArray<TSharedPtr<FInputBrowserColumnDef>> FilteredPinListItems;

		// Assets whose builders we've subscribed to, mapped to delegate handles for cleanup.
		// Keyed on FObjectKey (not builder pointer) so we don't hold raw pointers that could dangle.
		TMap<FObjectKey, FDelegateHandle> SubscribedBuilders;
		FDelegateHandle PostUndoRedoHandle;

		// Widgets
		TSharedPtr<SHeaderRow> HeaderRow;
		TSharedPtr<SListView<TSharedPtr<FInputBrowserRow>>> ListView;
		TSharedPtr<SListView<TSharedPtr<FInputBrowserColumnDef>>> PinListView;
	};

	/** Multi-column table row for the Input Browser. */
	class SMetasoundInputBrowserTableRow : public SMultiColumnTableRow<TSharedPtr<FInputBrowserRow>>
	{
	public:
		SLATE_BEGIN_ARGS(SMetasoundInputBrowserTableRow) {}
			SLATE_ARGUMENT(TSharedPtr<FInputBrowserRow>, RowData)
			SLATE_ARGUMENT(SMetasoundInputBrowser*, Browser)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	private:
		TSharedPtr<FInputBrowserRow> RowData;
		SMetasoundInputBrowser* Browser = nullptr;
	};

} // namespace Metasound::Editor
