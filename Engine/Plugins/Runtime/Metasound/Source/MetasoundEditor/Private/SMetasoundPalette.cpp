// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetasoundPalette.h"
#include "MetasoundEditorGraphSchema.h"

void SMetasoundPalette::Construct(const FArguments& InArgs)
{
	SGraphPalette::Construct(SGraphPalette::FArguments().AutoExpandActionMenu(true));
}

void SMetasoundPalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	if (const UMetasoundEditorGraphSchema* Schema = GetDefault<UMetasoundEditorGraphSchema>())
	{
		FGraphActionMenuBuilder ActionMenuBuilder;
		Schema->GetPaletteActions(ActionMenuBuilder);
		OutAllActions.Append(ActionMenuBuilder);
	}
}
