// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphPalette.h"

//////////////////////////////////////////////////////////////////////////

class SMetasoundPalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS( SMetasoundPalette ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	/** Callback used to populate all actions list in SGraphActionMenu */
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
};
