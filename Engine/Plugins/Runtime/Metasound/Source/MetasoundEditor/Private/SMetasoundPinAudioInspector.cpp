// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundPinAudioInspector.h"

#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"

namespace Metasound
{
	namespace Editor
	{
		void SMetasoundPinAudioInspector::Construct(const FArguments& InArgs)
		{
			SPinValueInspector::FArguments InspectorArgs;
			SPinValueInspector::Construct(InspectorArgs);

			ChildSlot
			[
				SNew(SBox)
				.MaxDesiredWidth(250)
				.MaxDesiredHeight(200)
				[
					InArgs._VisualizationWidget
				]
			];
		}
	} // namespace Editor
} // namespace Metasound

#undef LOCTEXT_NAMESPACE
