// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SPinValueInspector.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

#define UE_API METASOUNDEDITOR_API

namespace Metasound
{
	namespace Editor
	{
		class SMetasoundPinAudioInspector : public SPinValueInspector
		{

		public:
			SLATE_BEGIN_ARGS(SMetasoundPinAudioInspector)
			: _VisualizationWidget(SNullWidget::NullWidget)
			{}

			SLATE_ARGUMENT(TSharedRef<SWidget>, VisualizationWidget)
			SLATE_END_ARGS()

			UE_API void Construct(const FArguments& InArgs);
		};
	} // namespace Editor
} // namespace Metasound

#undef UE_API
