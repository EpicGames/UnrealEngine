// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundPinValueInspector.h"

#include "EditorStyleSet.h"
#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphInputNode.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorSettings.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "MetasoundFrontendDocument.h"
#include "SPinValueInspector.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MetasoundEditor"


namespace Metasound
{
	namespace Editor
	{
		namespace PinValueInspectorPrivate
		{
			static const FText EnableValueColorizationText = LOCTEXT("PinValueInspector_EnableConnectionColorization", "Enable Colorized Connection");
			static const FText DisableValueColorizationText = LOCTEXT("PinValueInspector_DisableConnectionColorization", "Disable Colorized Connection");

			static const FText MinNameText = LOCTEXT("PinValueInspector_MinValue", "Min");
			static const FText MaxNameText = LOCTEXT("PinValueInspector_MaxValue", "Max");

			static const FText ColorRowNameFormat = LOCTEXT("PinValueInspector_ColorRowNameFormat", "{0}:");

			bool IsMetasoundPrimitiveValueColorizationSupported(FName InDataType)
			{
				// Strings are the only non-numeric literal primitive which is not supported (for now).
				return InDataType != GetMetasoundDataTypeName<FString>();
			}

			TArray<FMetasoundFrontendLiteral> InitializeLiterals(FName InDataType)
			{
				TArray<FMetasoundFrontendLiteral> Literals;

				// For now, just hardcoded to two values.
				// TODO: Support multiple literals, which could be
				// really useful for strings, enums, and non-bipolar
				// or non-binary numeric ranges of interest.
				Literals.SetNum(2);

				if (InDataType == GetMetasoundDataTypeName<float>())
				{
					Literals[0].Set(0.0f);
					Literals[1].Set(1.0f);
				}
				else if (InDataType == GetMetasoundDataTypeName<int32>())
				{
					Literals[0].Set(0);
					Literals[1].Set(1);
				}
				else if (InDataType == GetMetasoundDataTypeName<bool>())
				{
					Literals[0].Set(false);
					Literals[1].Set(true);
				}
				// TODO: Potentially support strings
// 				else if (InDataType == GetMetasoundDataTypeName<FString>())
// 				{
// 					Literals[0].Set(FString());
// 					Literals[1].Set(FString());
// 				}

				return Literals;
			}

			UEdGraphPin* ResolvePinObjectAsOutput(UEdGraphPin* InPin)
			{
				if (InPin && !InPin->LinkedTo.IsEmpty())
				{
					// Swap to show connected output if input (Only ever one)
					if (InPin->Direction == EGPD_Input)
					{
						InPin = InPin->LinkedTo.Last();
						check(InPin->Direction == EGPD_Output);
					}
				}

				return InPin;
			}

			void SetLiteralFromText(const FName InDataType, const FText& InText, FMetasoundFrontendLiteral& OutLiteral)
			{
				const FString TextString = InText.ToString();
				if (InDataType == GetMetasoundDataTypeName<float>())
				{
					const float Value = FCString::Atof(*TextString);
					OutLiteral.Set(Value);
				}
				else if (InDataType == GetMetasoundDataTypeName<int32>())
				{
					const int32 Value = FCString::Atoi(*TextString);
					OutLiteral.Set(Value);
				}
				else if (InDataType == GetMetasoundDataTypeName<bool>())
				{
					const int32 Value = FCString::ToBool(*TextString);
					OutLiteral.Set(Value);
				}
// 				else if (InDataType == GetMetasoundDataTypeName<FString>())
// 				{
// 					OutLiteral.Set(TextString);
// 				}
			}
		} // namespace PinValueInspectorPrivate

		FMetasoundNumericDebugLineItem::FMetasoundNumericDebugLineItem(UEdGraphPin* InGraphPinObj, FGetValueStringFunction&& InGetValueStringFunction)
			: FDebugLineItem(DLT_Message)
			, GraphPinObj(PinValueInspectorPrivate::ResolvePinObjectAsOutput(InGraphPinObj))
			, GetValueStringFunction(MoveTemp(InGetValueStringFunction))
		{
			using namespace Frontend;

			FConstOutputHandle ReroutedOutputHandle = GetConstReroutedOutputHandle();
			ColorLiterals = PinValueInspectorPrivate::InitializeLiterals(ReroutedOutputHandle->GetDataType());
			Update();
			DisplayName = FGraphBuilder::GetDisplayName(*ReroutedOutputHandle);
		}

		FColorPickerArgs FMetasoundNumericDebugLineItem::InitPickerArgs()
		{
			FColorPickerArgs PickerArgs;
			PickerArgs.bIsModal = true;
			PickerArgs.bUseAlpha = true;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
			PickerArgs.ParentWidget = ValueWidget;

			return PickerArgs;
		}

		const UMetasoundEditorGraphNode* FMetasoundNumericDebugLineItem::GetReroutedNode() const
		{
			if (UEdGraphPin* ReroutedOutputPin = FGraphBuilder::FindReroutedOutputPin(GraphPinObj))
			{
				return Cast<UMetasoundEditorGraphNode>(ReroutedOutputPin->GetOwningNode());
			}

			return nullptr;
		}

		const UMetasoundEditorGraphNode& FMetasoundNumericDebugLineItem::GetReroutedNodeChecked() const
		{
			const UMetasoundEditorGraphNode* Node = GetReroutedNode();
			check(Node);
			return *Node;
		}

		UObject* FMetasoundNumericDebugLineItem::GetOutermostObject()
		{
			if (GraphPinObj)
			{
				if (UEdGraphNode* OwningNode = GraphPinObj->GetOwningNode())
				{
					return OwningNode->GetOutermostObject();
				}
			}

			return nullptr;
		}

		Frontend::FConstOutputHandle FMetasoundNumericDebugLineItem::GetConstReroutedOutputHandle() const
		{
			return FGraphBuilder::FindReroutedConstOutputHandleFromPin(GraphPinObj);
		}

		bool FMetasoundNumericDebugLineItem::GetNodeIDAndOutputName(FGuid& OutNodeID, FName& OutName) const
		{
			if (UEdGraphPin* ReroutedOutputPin = FGraphBuilder::FindReroutedOutputPin(GraphPinObj))
			{
				const UMetasoundEditorGraphNode& OutputsNode = GetReroutedNodeChecked();
				const FMetaSoundFrontendDocumentBuilder& Builder = OutputsNode.GetBuilderChecked().GetConstBuilder();
				if (const UMetasoundEditorGraphInputNode* InputNode = Cast<UMetasoundEditorGraphInputNode>(&OutputsNode))
				{
					if (const UMetasoundEditorGraphMember* Member = InputNode->GetMember())
					{
						OutNodeID = Member->GetMemberID();
						if (const FMetasoundFrontendNode* RedirectedInputNode = Builder.FindNode(OutNodeID))
						{
							OutName = RedirectedInputNode->Interface.Outputs.Last().Name;
							return true;
						}
					}
				}
				else
				{
					const FMetasoundFrontendVertexHandle VertexHandle = FGraphBuilder::GetPinVertexHandle(Builder, ReroutedOutputPin);
					if (VertexHandle.IsSet())
					{
						OutNodeID = VertexHandle.NodeID;
						if (const FMetasoundFrontendVertex* Vertex = FGraphBuilder::GetPinVertex(Builder, ReroutedOutputPin))
						{
							OutName = Vertex->Name;
							return true;
						}
					}
				}
			}

			OutNodeID = { };
			OutName = { };
			return false;
		}

		FLinearColor FMetasoundNumericDebugLineItem::GetEdgeStyleColorAtIndex(int32 InIndex) const
		{
			if (const FMetasoundFrontendEdgeStyle* EdgeStyle = FindConstEdgeStyle())
			{
				if (ensure(EdgeStyle->LiteralColorPairs.Num() > InIndex))
				{
					return EdgeStyle->LiteralColorPairs[InIndex].Color;
				}
			}
			return FLinearColor::Transparent;
		}

		void FMetasoundNumericDebugLineItem::DisableValueColorization()
		{
			using namespace Frontend;

			FScopedTransaction Transaction(PinValueInspectorPrivate::DisableValueColorizationText);

			UObject* ParentObject = GetOutermostObject();
			check(ParentObject);
			ParentObject->Modify();

			FGuid NodeID;
			FName OutputName;
			if (GetNodeIDAndOutputName(NodeID, OutputName))
			{
				if (const UMetasoundEditorGraphNode* Node = GetReroutedNode(); ensure(Node))
				{
					FMetaSoundFrontendDocumentBuilder& Builder = Node->GetBuilderChecked().GetBuilder();
					Builder.RemoveEdgeStyle(NodeID, OutputName);
					bIsValueColorizationEnabled = false;
				}
			}
		}

		void FMetasoundNumericDebugLineItem::EnableValueColorization()
		{
			using namespace Frontend;

			FScopedTransaction Transaction(PinValueInspectorPrivate::EnableValueColorizationText);

			UObject* ParentObject = GetOutermostObject();
			check(ParentObject);
			ParentObject->Modify();

			TArray<FMetasoundFrontendEdgeStyleLiteralColorPair> DefaultPairs
			{
				{ ColorLiterals[0], FLinearColor::Transparent },
				{ ColorLiterals[1], FLinearColor::White }
			};

			if (FMetasoundFrontendEdgeStyle* EdgeStyle = FindOrAddEdgeStyle())
			{
				EdgeStyle->LiteralColorPairs = MoveTemp(DefaultPairs);
			}

			bIsValueColorizationEnabled = true;
		}

		void FMetasoundNumericDebugLineItem::OnColorCommitted(FLinearColor InColor, int32 InIndex)
		{
			using namespace Frontend;

			static const FText SetValueFormat = LOCTEXT("PinValueInspector_SetMinColorFormat", "Set '{0}[{1}]' Color to '{2}'");

			const FConstOutputHandle OutputHandle = GetConstReroutedOutputHandle();
			const FName Name = OutputHandle->GetName();
			FScopedTransaction Transaction(FText::Format(SetValueFormat, FText::FromName(Name), FText::AsNumber(InIndex), FText::FromString(InColor.ToString())));

			UObject* ParentObject = GetOutermostObject();
			check(ParentObject);
			ParentObject->Modify();

			if (FMetasoundFrontendEdgeStyle* EdgeStyle = FindOrAddEdgeStyle())
			{
				if (ensure(InIndex < EdgeStyle->LiteralColorPairs.Num()))
				{
					EdgeStyle->LiteralColorPairs[InIndex].Color = InColor;
				}
			}
		}

		void FMetasoundNumericDebugLineItem::OnValueCommitted(const FText& InValueText, FMetasoundFrontendLiteral& OutNewLiteral, int32 InIndex)
		{
			using namespace Frontend;

			FConstOutputHandle OutputHandle = GetConstReroutedOutputHandle();

			static const FText SetValueFormat = LOCTEXT("PinValueInspector_SetColorLiteralValueFormat", "Set '{0}[{1}]' EdgeStyle Color Value to '{2}'");
			const FName Name = OutputHandle->GetName();
			FScopedTransaction Transaction(FText::Format(SetValueFormat, FText::FromName(Name), FText::AsNumber(InIndex), InValueText));

			UObject* ParentObject = GetOutermostObject();
			check(ParentObject);
			ParentObject->Modify();

			if (FMetasoundFrontendEdgeStyle* EdgeStyle = FindOrAddEdgeStyle())
			{
				if (ensure(EdgeStyle->LiteralColorPairs.Num() > InIndex))
				{
					const FName DataType = OutputHandle->GetDataType();
					PinValueInspectorPrivate::SetLiteralFromText(DataType, InValueText, OutNewLiteral);
					EdgeStyle->LiteralColorPairs[InIndex].Value = OutNewLiteral;
				}
			}
		}

		Frontend::FConstGraphHandle FMetasoundNumericDebugLineItem::GetConstGraphHandle() const
		{
			Frontend::FConstOutputHandle OutputHandle = FGraphBuilder::GetConstOutputHandleFromPin(GraphPinObj);
			return OutputHandle->GetOwningNode()->GetOwningGraph();
		}

		const FGraphConnectionManager* FMetasoundNumericDebugLineItem::GetConnectionManager() const
		{
			using namespace Frontend;

			const UMetasoundEditorGraphNode& Node = GetReroutedNodeChecked();
			TSharedPtr<FEditor> Editor = FGraphBuilder::GetEditorForNode(Node);
			if (Editor.IsValid())
			{
				return &Editor->GetConnectionManager();
			}

			return nullptr;
		}

		const FMetasoundFrontendEdgeStyle* FMetasoundNumericDebugLineItem::FindConstEdgeStyle() const
		{
			using namespace Frontend;

			FGuid NodeID;
			FName OutputName;
			if (GetNodeIDAndOutputName(NodeID, OutputName))
			{
				if (const UMetasoundEditorGraphNode* Node = GetReroutedNode())
				{
					const FMetaSoundFrontendDocumentBuilder& Builder = Node->GetBuilderChecked().GetConstBuilder();
					return Builder.FindConstEdgeStyle(NodeID, OutputName);
				}
			}

			return nullptr;
		}

		FMetasoundFrontendEdgeStyle* FMetasoundNumericDebugLineItem::FindEdgeStyle()
		{
			using namespace Frontend;

			FGuid NodeID;
			FName OutputName;
			if (GetNodeIDAndOutputName(NodeID, OutputName))
			{
				if (const UMetasoundEditorGraphNode* Node = GetReroutedNode())
				{
					FMetaSoundFrontendDocumentBuilder& Builder = Node->GetBuilderChecked().GetBuilder();
					return Builder.FindEdgeStyle(NodeID, OutputName);
				}
			}

			return nullptr;
		}

		FMetasoundFrontendEdgeStyle* FMetasoundNumericDebugLineItem::FindOrAddEdgeStyle()
		{
			using namespace Frontend;

			FGuid NodeID;
			FName OutputName;
			if (GetNodeIDAndOutputName(NodeID, OutputName))
			{
				if (const UMetasoundEditorGraphNode* Node = GetReroutedNode())
				{
					FMetaSoundFrontendDocumentBuilder& Builder = Node->GetBuilderChecked().GetBuilder();
					return &Builder.FindOrAddEdgeStyle(NodeID, OutputName);
				}
			}

			return nullptr;
		}

		void FMetasoundNumericDebugLineItem::Update()
		{
			using namespace Frontend;

			bIsValueColorizationEnabled = false;

			FGuid NodeID;
			FName OutputName;
			if (GetNodeIDAndOutputName(NodeID, OutputName))
			{
				if (const FMetasoundFrontendEdgeStyle* EdgeStyle = FindConstEdgeStyle())
				{
					if (ensure(EdgeStyle->LiteralColorPairs.Num() == 2))
					{
						const FMetasoundFrontendLiteral& CurrentMin = EdgeStyle->LiteralColorPairs[0].Value;
						ColorLiterals[0] = CurrentMin;
						const FMetasoundFrontendLiteral& CurrentMax = EdgeStyle->LiteralColorPairs[1].Value;
						ColorLiterals[1] = CurrentMax;
						bIsValueColorizationEnabled = true;
					}
				}

				if (const FGraphConnectionManager* ConnectionManager = GetConnectionManager())
				{
					Message = FString::Format(TEXT("Value: {0}"), { GetValueStringFunction(*ConnectionManager, NodeID, OutputName) });
				}
			}
		}

		FReply FMetasoundNumericDebugLineItem::OnColorBoxClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FLinearColor& InInitColor, int32 InIndex)
		{
			using namespace Frontend;

			if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
			{
				return FReply::Unhandled();
			}

			const FMetasoundFrontendEdgeStyle* EdgeStyle = FindEdgeStyle();
			if (!EdgeStyle)
			{
				return FReply::Unhandled();
			}

			FColorPickerArgs PickerArgs = InitPickerArgs();
			PickerArgs.InitialColor = InInitColor;
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FMetasoundNumericDebugLineItem::OnColorCommitted, InIndex);

			OpenColorPicker(PickerArgs);

			return FReply::Handled();
		}

		FText FMetasoundNumericDebugLineItem::GetDescription() const
		{
			return FText::FromString(Message);
		}

		TSharedRef<SWidget> FMetasoundNumericDebugLineItem::GenerateNameWidget(TSharedPtr<FString> InSearchString)
		{
			if (!GraphPinObj)
			{
				return SNullWidget::NullWidget;
			}

			return SNew(STextBlock)
				.Text(DisplayName)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
		}

		TSharedRef<SWidget> FMetasoundNumericDebugLineItem::GenerateValueWidget(TSharedPtr<FString> InSearchString)
		{
			using namespace Frontend;
			const FSlateFontInfo FontInfo = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));

			TSharedPtr<SVerticalBox> VerticalBox;
			SAssignNew(VerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(1.0f, 1.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::FromString(Message); })
					.Font(FontInfo)
				];

			const FName DataType = GetConstReroutedOutputHandle()->GetDataType();
			const bool bConnectionAnimationsEnabled = GetDefault<UMetasoundEditorSettings>()->AnalyzerAnimationSettings.bAnimateConnections;
			const bool bIsTypeSupported = PinValueInspectorPrivate::IsMetasoundPrimitiveValueColorizationSupported(DataType);
			if (bConnectionAnimationsEnabled && bIsTypeSupported)
			{
				AddValueColorizationWidgets(VerticalBox);
			}

			ValueWidget = VerticalBox;
			return ValueWidget->AsShared();
		}

		void FMetasoundNumericDebugLineItem::AddValueColorizationWidgets(TSharedPtr<SVerticalBox> VerticalBox)
		{
			using namespace Frontend;
			const FSlateFontInfo FontInfo = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));

			VerticalBox->AddSlot()
				.AutoHeight()
				.Padding(1.0f, 1.0f, 0.0f, 1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.OnClicked_Lambda([this]()
					{
						bIsValueColorizationEnabled ? DisableValueColorization() : EnableValueColorization();
						return FReply::Handled();
					})
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return bIsValueColorizationEnabled ? PinValueInspectorPrivate::DisableValueColorizationText : PinValueInspectorPrivate::EnableValueColorizationText;
					})
					.ToolTipText(LOCTEXT("EnableConnectionVisualizationToolTip", "Enable or disable if connection has fill colorization interpolating between provided min and max values."))
					.Font(FontInfo)
				]
			];

			auto CreateColorLabel = [this, &FontInfo](const FText& InLabel) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(InLabel)
					.Font(FontInfo)
					.MinDesiredWidth(40)
					.Visibility_Lambda([this]() { return bIsValueColorizationEnabled ? EVisibility::Visible : EVisibility::Collapsed; });
			};

			auto CreateColorValueWidget = [this, &FontInfo](int32 InIndex) -> TSharedRef<SWidget>
			{
				TSharedRef<SWidget> ColorValueWidget = SNew(SEditableTextBox)
					.Text_Lambda([this, InIndex]()
					{
						return FText::FromString(ColorLiterals[InIndex].ToString());
					})
					.OnTextCommitted_Lambda([this, InIndex](const FText& InValueText, ETextCommit::Type InTextCommit)
					{
						OnValueCommitted(InValueText, ColorLiterals[InIndex], InIndex);
					})
					.SelectAllTextWhenFocused(true)
					.MinDesiredWidth(30)
					.Font(FontInfo)
					.Visibility_Lambda([this]() { return bIsValueColorizationEnabled ? EVisibility::Visible : EVisibility::Collapsed; });

				// Boolean values only have two states that default to true & false, so no reason to allow user to edit.
				if (ColorLiterals[InIndex].GetType() == EMetasoundFrontendLiteralType::Boolean)
				{
					ColorValueWidget->SetEnabled(false);
				}

				return ColorValueWidget;
			};

			auto CreateColorPickerBlock = [this](int32 InIndex) -> TSharedRef<SWidget>
			{
				return SNew(SColorBlock)
					.Color_Lambda([this, InIndex] () { return GetEdgeStyleColorAtIndex(InIndex); })
					.ShowBackgroundForAlpha(true)
					.OnMouseButtonDown_Lambda([this, InIndex](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
					{
						const FLinearColor InitColor = GetEdgeStyleColorAtIndex(InIndex);
						return OnColorBoxClicked(MyGeometry, MouseEvent, InitColor, InIndex);
					})
					.Visibility_Lambda([this]() { return bIsValueColorizationEnabled ? EVisibility::Visible : EVisibility::Collapsed; });
			};

			VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(1.0f, 1.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorLabel(FText::Format(PinValueInspectorPrivate::ColorRowNameFormat, PinValueInspectorPrivate::MinNameText))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorPickerBlock(0)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f, 0.0f, 1.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorValueWidget(0)
				]
			];

			VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(1.0f, 1.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorLabel(FText::Format(PinValueInspectorPrivate::ColorRowNameFormat, PinValueInspectorPrivate::MaxNameText))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorPickerBlock(1)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f, 0.0f, 1.0f)
				.HAlign(HAlign_Center)
				[
					CreateColorValueWidget(1)
				]
			];
		}

		FDebugLineItem* FMetasoundNumericDebugLineItem::Duplicate() const
		{
			FGetValueStringFunction DuplicateGetValueStringFunction = GetValueStringFunction;
			return new FMetasoundNumericDebugLineItem(GraphPinObj, MoveTemp(DuplicateGetValueStringFunction));
		}

		bool FMetasoundNumericDebugLineItem::Compare(const FDebugLineItem* BaseOther) const
		{
			const FMetasoundNumericDebugLineItem* Other = static_cast<const FMetasoundNumericDebugLineItem*>(BaseOther);
			return Message == Other->Message;
		}

		uint32 FMetasoundNumericDebugLineItem::GetHash() const
		{
			return GetTypeHash(Message);
		}

		void SMetasoundPinValueInspector::Construct(const FArguments& InArgs)
		{
			SPinValueInspector::FArguments InspectorArgs;
			SPinValueInspector::Construct(InspectorArgs);
		}

		void SMetasoundPinValueInspector::UpdateMessage()
		{
			if (LineItem.IsValid())
			{
				LineItem->Update();
			}
		}

		void SMetasoundPinValueInspector::PopulateTreeView()
		{
			// Locate the class property associated with the source pin and set it to the root node.
			UEdGraphPin* GraphPinObj = PinValueInspectorPrivate::ResolvePinObjectAsOutput(GetPinRef().Get());

			// Don't populate if no pin found or pin is not connected (analyzers are not enabled for disconnected node outputs)
			if (!GraphPinObj || GraphPinObj->LinkedTo.IsEmpty())
			{
				LineItem.Reset();
				return;
			}

			TSharedPtr<FMetasoundNumericDebugLineItem> NewLine;

			static const FString UnsetValueString = TEXT("Unset");
			if (GraphPinObj->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryFloat)
			{
				NewLine = MakeShared<FMetasoundNumericDebugLineItem>(GraphPinObj, [&](const FGraphConnectionManager& ConnectionManager, const FGuid& NodeID, const FName OutputName)
				{
					float Value = 0.0f;
					if (ConnectionManager.GetValue(NodeID, OutputName, Value))
					{
						return FString::Printf(TEXT("%.6f"), Value);
					}
					return UnsetValueString;
				});
			}
			else if (GraphPinObj->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryInt32)
			{
				NewLine = MakeShared<FMetasoundNumericDebugLineItem>(GraphPinObj, [&](const FGraphConnectionManager& ConnectionManager, const FGuid& NodeID, const FName OutputName)
				{
					int32 Value = 0;
					if (ConnectionManager.GetValue(NodeID, OutputName, Value))
					{
						return FString::Printf(TEXT("%d"), Value);
					}
					return UnsetValueString;
				});
			}
			else if (GraphPinObj->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryString)
			{
				NewLine = MakeShared<FMetasoundNumericDebugLineItem>(GraphPinObj, [&](const FGraphConnectionManager& ConnectionManager, const FGuid& NodeID, const FName OutputName)
				{
					FString Value = UnsetValueString;
					ConnectionManager.GetValue(NodeID, OutputName, Value);
					return Value;
				});
			}
			else if (GraphPinObj->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryBoolean)
			{
				NewLine = MakeShared<FMetasoundNumericDebugLineItem>(GraphPinObj, [&](const FGraphConnectionManager& ConnectionManager, const FGuid& NodeID, const FName OutputName)
				{
					bool Value = false;
					if (ConnectionManager.GetValue(NodeID, OutputName, Value))
					{
						return FString::Printf(TEXT("%s"), Value ? TEXT("true") : TEXT("false"));
					}
					return UnsetValueString;
				});
			}

			if (NewLine.IsValid())
			{
				LineItem = NewLine;
				AddTreeItemUnique(NewLine);
			}
		}
	} // namespace Editor
} // namespace Metasound
#undef LOCTEXT_NAMESPACE
