// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Debugging/SKismetDebugTreeView.h"
#include "Engine/Engine.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphConnectionManager.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "SGraphPin.h"
#include "SPinValueInspector.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Colors/SColorPicker.h"

#define UE_API METASOUNDEDITOR_API


// Forward Declarations
class UMetasoundEditorGraphNode;


namespace Metasound
{
	namespace Editor
	{
		struct FMetasoundNumericDebugLineItem : public FDebugLineItem
		{
			using FGetValueStringFunction = TFunction<FString(const FGraphConnectionManager&, const FGuid& /* InNodeID */, const FName /* InOutputName */)>;

		private:
			/** The GraphPin that this debug line is reporting. */
			UEdGraphPin* GraphPinObj = nullptr;

			/** Function used to get the string value from the connection manager. */
			FGetValueStringFunction GetValueStringFunction;

			FColorPickerArgs InitPickerArgs();

		protected:
			TArray<FMetasoundFrontendLiteral> ColorLiterals;

			FString Message;
			FText DisplayName;

			TSharedPtr<SWidget> ValueWidget;

			bool bIsValueColorizationEnabled = false;

		public:
			FMetasoundNumericDebugLineItem(UEdGraphPin* InGraphPinObj, FGetValueStringFunction&& InGetValueStringFunction);
			virtual ~FMetasoundNumericDebugLineItem() = default;

			const UMetasoundEditorGraphNode* GetReroutedNode() const;

			const UMetasoundEditorGraphNode& GetReroutedNodeChecked() const;

			UObject* GetOutermostObject();

			Frontend::FConstOutputHandle GetConstReroutedOutputHandle() const;

			Frontend::FConstGraphHandle GetConstGraphHandle() const;

			const FGraphConnectionManager* GetConnectionManager() const;

			const FMetasoundFrontendEdgeStyle* FindConstEdgeStyle() const;

			void Update();

			virtual FText GetDescription() const override;

			virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override;

			virtual TSharedRef<SWidget> GenerateValueWidget(TSharedPtr<FString> InSearchString) override;

		protected:
			virtual void AddValueColorizationWidgets(TSharedPtr<SVerticalBox> VerticalBox);

			virtual FDebugLineItem* Duplicate() const override;

			virtual bool Compare(const FDebugLineItem* BaseOther) const override;

			virtual uint32 GetHash() const override;

		private:
			FLinearColor GetEdgeStyleColorAtIndex(int32 InIndex) const;
			FMetasoundFrontendEdgeStyle* FindOrAddEdgeStyle();
			FMetasoundFrontendEdgeStyle* FindEdgeStyle();
			bool GetNodeIDAndOutputName(FGuid& OutNodeID, FName& OutName) const;

			void DisableValueColorization();
			void EnableValueColorization();

			void OnColorCommitted(FLinearColor InColor, int32 InIndex);
			FReply OnColorBoxClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FLinearColor& InInitColor, int32 InIndex);
			void OnValueCommitted(const FText& InValueText, FMetasoundFrontendLiteral& OutNewLiteral, int32 InIndex);
		};

		class SMetasoundPinValueInspector : public SPinValueInspector
		{
			TSharedPtr<FMetasoundNumericDebugLineItem> LineItem;

		public:
			SLATE_BEGIN_ARGS(SMetasoundPinValueInspector)
			{
			}
			SLATE_END_ARGS()

			UE_API void Construct(const FArguments& InArgs);

			UE_API void UpdateMessage();

			UE_API virtual void PopulateTreeView() override;
		};
	} // namespace Editor
} // namespace Metasound

#undef UE_API
