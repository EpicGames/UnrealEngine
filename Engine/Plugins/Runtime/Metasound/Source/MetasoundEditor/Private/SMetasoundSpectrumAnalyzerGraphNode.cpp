// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundSpectrumAnalyzerGraphNode.h"

#include "AudioSpectrumAnalyzer.h"
#include "Editor.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorModule.h"
#include "MetaSoundGraphPanelPinFactory.h"
#include "Widgets/Layout/SBox.h"

namespace Metasound::Editor
{
	SMetaSoundSpectrumAnalyzerGraphNode::~SMetaSoundSpectrumAnalyzerGraphNode()
	{
		if (AnalyzerInstanceID.IsValid())
		{
			if (TSharedPtr<FEditor> Editor = FGraphBuilder::GetEditorForNode(GetMetaSoundNode()))
			{
				Editor->GetConnectionManager().RemoveAudioBusWriter(AnalyzerInstanceID);
			}

			AnalyzerInstanceID.Invalidate();
		}
	}

	void SMetaSoundSpectrumAnalyzerGraphNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SMetaSoundGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		const UMetasoundEditorGraphNode& MetaSoundNode = GetMetaSoundNode();
		if (SpectrumAnalyzer.IsValid() && ensure(MetaSoundNode.Pins.Num() == 1))
		{
			const UEdGraphPin* Pin = MetaSoundNode.Pins.Last();
			if (!Pin->LinkedTo.IsEmpty() && ensure(Pin->PinType.PinCategory == FMetaSoundGraphPanelPinFactory::PinCategoryAudio && Pin->Direction == EGPD_Input))
			{
				// Find connected output for the input (only ever one):
				UEdGraphPin* SourcePin = Pin->LinkedTo.Last();
				ensure(SourcePin->Direction == EGPD_Output);

				TSharedPtr<FEditor> Editor = FGraphBuilder::GetEditorForNode(MetaSoundNode);
				if (Editor.IsValid() && ensure(GEditor))
				{
					// Analyze audio with AudioBusWriter, if not already doing so:
					FGraphConnectionManager& ConnectionManager = Editor->GetConnectionManager();
					if (!AnalyzerInstanceID.IsValid() || !ConnectionManager.HasAudioBusWriter(AnalyzerInstanceID))
					{
						const Frontend::FConstOutputHandle OutputHandle = FGraphBuilder::FindReroutedConstOutputHandleFromPin(SourcePin);
						const FGuid NodeID = OutputHandle->GetOwningNodeID();
						const FName OutputName = OutputHandle->GetName();
						const Audio::FDeviceId DeviceId = GEditor->GetMainAudioDeviceID();
						AnalyzerInstanceID = ConnectionManager.AddAudioBusWriter(NodeID, OutputName, DeviceId, SpectrumAnalyzer->GetAudioBus());
					}
				}
			}
		}
	}

	void SMetaSoundSpectrumAnalyzerGraphNode::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
	{
		using namespace AudioWidgets;

		if (ensure(GEditor))
		{
			if (!SpectrumAnalyzer.IsValid())
			{
				FAudioSpectrumAnalyzerParams Params;
				Params.NumChannels = 1;
				Params.AudioDeviceId = GEditor->GetMainAudioDeviceID();
				if (const ISlateStyle* StyleSet = FSlateStyleRegistry::FindSlateStyle("MetaSoundStyle"))
				{
					Params.PlotStyle = &StyleSet->GetWidgetStyle<FAudioSpectrumPlotStyle>("AudioSpectrumPlot.Style");
				}
				SpectrumAnalyzer = MakeShared<FAudioSpectrumAnalyzer>(Params);
			}

			MainBox->AddSlot()
				.AutoHeight()
				.Padding(1.0f, 0.0f)
				[
					SNew(SBox)
						.MinDesiredWidth(250.0f)
						.MinDesiredHeight(250.0f)
						[
							SpectrumAnalyzer->GetWidget()
						]
				];
		}
	}
}
