// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMetasoundGraphNode.h"

// Forward Declarations
namespace AudioWidgets
{
	class FAudioSpectrumAnalyzer;
}

namespace Metasound::Editor
{
	class SMetaSoundSpectrumAnalyzerGraphNode : public SMetaSoundGraphNode
	{
	public:
		virtual ~SMetaSoundSpectrumAnalyzerGraphNode();

		// Begin SWidget overrides.
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		// End SWidget overrides.

	protected:
		virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;

	private:
		TSharedPtr<AudioWidgets::FAudioSpectrumAnalyzer> SpectrumAnalyzer;
		FGuid AnalyzerInstanceID;
	};
}
