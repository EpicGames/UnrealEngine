// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMetasoundStats.h"

#include "Components/AudioComponent.h"
#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Internationalization/Text.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorModule.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundGenerator.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "MetaSoundEditor"

namespace Metasound::Editor
{
	namespace StatsPrivate
	{
		const FLinearColor BaseTextColor(1, 1, 1, 0.30f);
	}

	void SPageStats::Construct(const FArguments& InArgs)
	{
		SVerticalBox::Construct(SVerticalBox::FArguments());

		DisplayedPageID = Metasound::Frontend::DefaultPageID;
		DisplayedPageName = Metasound::Frontend::DefaultPageName;

		AddSlot().HAlign(HAlign_Left)
		[
			SAssignNew(AuditionPageTextWidget, STextBlock)
				.Visibility(EVisibility::Collapsed)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.ColorAndOpacity(StatsPrivate::BaseTextColor)
		];

		AddSlot().HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.0f)
				.HAlign(HAlign_Center)
				[
					SAssignNew(GraphPageTextWidget, STextBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
					.ColorAndOpacity(StatsPrivate::BaseTextColor)
				]

				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(ExecImageWidget, SImage)
						.Image(Style::CreateSlateIcon("MetasoundEditor.Page.Executing").GetIcon())
						.DesiredSizeOverride(FVector2D(24.f, 24.f))
						.ColorAndOpacity(Style::GetPageExecutingColor())
						.Visibility(EVisibility::Collapsed)
				]
		];
	}

	void SPageStats::SetExecVisibility(TAttribute<EVisibility> InVisibility)
	{
		ExecImageWidget->SetVisibility(MoveTemp(InVisibility));
	}

	void SPageStats::Update(const FMetaSoundPageSettings* AuditionPageSettings, const FMetaSoundPageSettings* GraphPageSettings, const FSlateColor* ActiveColor)
	{
		using namespace Engine;

		const FText PageStatsFormat = LOCTEXT("PageStatsFormat", "{0}: {1}");

		{
			FText PageInfo;
			EVisibility Visibility = EVisibility::Collapsed;
			if (AuditionPageTextWidget.IsValid() && AuditionPageSettings)
			{
				const FText Header = LOCTEXT("AuditionPageHeader", "Auditioning Page");
				PageInfo = FText::Format(PageStatsFormat, Header, FText::FromString(AuditionPageSettings->Name.ToString()));
				Visibility = EVisibility::Visible;
			}
			AuditionPageTextWidget->SetText(PageInfo);
		}

		{
			FText PageInfo;
			EVisibility Visibility = EVisibility::Collapsed;
			if (GraphPageTextWidget.IsValid() && GraphPageSettings)
			{
				const FText Header = LOCTEXT("GraphPageTargetHeader", "Graph Page");
				PageInfo = FText::Format(PageStatsFormat, Header, FText::FromString(GraphPageSettings->Name.ToString()));
				Visibility = EVisibility::Visible;
			}
			GraphPageTextWidget->SetText(PageInfo);
			GraphPageTextWidget->SetColorAndOpacity(ActiveColor ? *ActiveColor : StatsPrivate::BaseTextColor);
		}

		DisplayedPageID = GraphPageSettings ? GraphPageSettings->UniqueId : Metasound::Frontend::DefaultPageID;
		DisplayedPageName = GraphPageSettings ? GraphPageSettings->Name : Metasound::Frontend::DefaultPageName;
	}

	const FGuid& SPageStats::GetDisplayedPageID() const
	{
		return DisplayedPageID;
	}

	FName SPageStats::GetDisplayedPageName() const
	{
		return DisplayedPageName;
	}

	void SRenderStats::Construct(const FArguments& InArgs)
	{
		SVerticalBox::Construct(SVerticalBox::FArguments());

		auto AddTextWidgetSlot = [this](TSharedPtr<STextBlock>& OutBlock)
		{
			AddSlot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SAssignNew(OutBlock, STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.TextStyle(FAppStyle::Get(), "GraphPreview.CornerText")
				.ColorAndOpacity(FLinearColor(1, 1, 1, 0.30f))
			];
		};

		AddTextWidgetSlot(PlayTimeWidget);
		AddTextWidgetSlot(RenderStatsCostWidget);
		AddTextWidgetSlot(RenderStatsCPUWidget);
		AddTextWidgetSlot(AuditionPageWidget);
		AddTextWidgetSlot(AuditionPlatformWidget);
	}

	void SRenderStats::Update(bool bIsPlaying, double InDeltaTime, const UMetaSoundSource* InSource)
	{
		using namespace Metasound;

		// Reset maximum values when play restarts
		const bool bPlayStateChanged = bIsPlaying != bPreviousIsPlaying;
		if (bIsPlaying)
		{
			if (!bPreviousIsPlaying)
			{
				MaxRelativeRenderCost = 0.f;
				MaxCPUCoreUtilization = 0;
			}
			PlayTime += InDeltaTime;
		}
		else
		{
			PlayTime = 0.0;
		}

		bPreviousIsPlaying = bIsPlaying;

		if (!RenderStatsCPUWidget.IsValid()
			|| !RenderStatsCostWidget.IsValid()
			|| !PlayTimeWidget.IsValid()
			|| !AuditionPageWidget.IsValid()
			|| !AuditionPlatformWidget.IsValid())
			{
			return;
		}

		double CPUCoreUtilization = 0;
		float RelativeRenderCost = 0.f;

		if (bIsPlaying && InSource)
		{
			if (const UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent())
			{
				TSharedPtr<FMetasoundGenerator> Generator = InSource->GetGeneratorForAudioComponent(PreviewComponent->GetAudioComponentID()).Pin();
				if (Generator.IsValid())
				{
					// Update render stats
					CPUCoreUtilization = Generator->GetCPUCoreUtilization();
					MaxCPUCoreUtilization = FMath::Max(MaxCPUCoreUtilization, CPUCoreUtilization);

					RelativeRenderCost = Generator->GetRelativeRenderCost();
					MaxRelativeRenderCost = FMath::Max(MaxRelativeRenderCost, RelativeRenderCost);
				}
			}
		}

		// Display updated render stats.
		FString PlayTimeString = FTimespan::FromSeconds(PlayTime).ToString();
		PlayTimeString.ReplaceInline(TEXT("+"), TEXT(""));
		PlayTimeWidget->SetText(FText::FromString(MoveTemp(PlayTimeString)));

		FString RenderCostString = FString::Printf(TEXT("Relative Render Cost: %3.2f (%3.2f Max)"), RelativeRenderCost, MaxRelativeRenderCost);
		RenderStatsCostWidget->SetText(FText::FromString(MoveTemp(RenderCostString)));

		FString CPUCoreUtilizationString = FString::Printf(TEXT("CPU Core: %3.2f%% (%3.2f%% Max)"), 100. * CPUCoreUtilization, 100. * MaxCPUCoreUtilization);
		RenderStatsCPUWidget->SetText(FText::FromString(MoveTemp(CPUCoreUtilizationString)));

		if (bPlayStateChanged)
		{
			FText AuditionPage;
			FText AuditionPlatform;

			if (bIsPlaying)
			{
				if (const UMetasoundEditorSettings* EditorSettings = GetDefault<UMetasoundEditorSettings>())
				{
					if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
					{
						if (!Settings->GetProjectPageSettings().IsEmpty())
						{
							const FText AuditionPageHeader = LOCTEXT("AuditionPageActive_DebugFormat", "Auditioning Page: {0}");
							AuditionPage = FText::Format(AuditionPageHeader, FText::FromName(EditorSettings->GetAuditionPage()));

							if (EditorSettings->GetAuditionPlatform() != "Editor")
							{
								const FText AuditionPlatformHeader = LOCTEXT("AuditionPlatformActive_DebugFormat", "Auditioning Platform: {0}");
								AuditionPlatform = FText::Format(AuditionPlatformHeader, FText::FromName(EditorSettings->GetAuditionPlatform()));
							}
						}
					}
				}
			}
			AuditionPageWidget->SetText(AuditionPage);
			AuditionPlatformWidget->SetText(AuditionPlatform);
		}
	}
} // namespace Metasound::Editor
#undef LOCTEXT_NAMESPACE
