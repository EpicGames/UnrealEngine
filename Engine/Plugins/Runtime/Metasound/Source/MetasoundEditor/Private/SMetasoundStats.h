// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"


// Forward Declarations
class IMetaSoundDocumentInterface;
class STextBlock;
class UMetaSoundBuilderBase;
class UMetaSoundSource;

struct FMetaSoundPageSettings;
struct FSlateColor;


namespace Metasound::Editor
{
	/** Widget for displaying page stats of a previewing MetaSound. */
	class SPageStats : public SVerticalBox
	{
	public:
		SLATE_BEGIN_ARGS(SPageStats) { };
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
		void SetExecVisibility(TAttribute<EVisibility> InVisibility);
		void Update(const FMetaSoundPageSettings* AuditionPageSettings, const FMetaSoundPageSettings* GraphPageSettings, const FSlateColor* ActiveColor);

		const FGuid& GetDisplayedPageID() const;
		FName GetDisplayedPageName() const;

	private:
		TSharedPtr<SImage> ExecImageWidget;
		TSharedPtr<STextBlock> GraphPageTextWidget;
		TSharedPtr<STextBlock> AuditionPageTextWidget;

		FGuid DisplayedPageID;
		FName DisplayedPageName;
	};


	/** Widget for displaying render stats of a previewing MetaSound. */
	class SRenderStats : public SVerticalBox
	{
	public:
		SLATE_BEGIN_ARGS( SRenderStats) { };
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
		void Update(bool bIsPlaying, double InDeltaTime, const UMetaSoundSource* InSource);

	private:
		TSharedPtr<STextBlock> AuditionPageWidget;
		TSharedPtr<STextBlock> AuditionPlatformWidget;
		TSharedPtr<STextBlock> PlayTimeWidget;
		TSharedPtr<STextBlock> RenderStatsCostWidget;
		TSharedPtr<STextBlock> RenderStatsCPUWidget;

		bool bPreviousIsPlaying = false;
		double MaxCPUCoreUtilization = 0.0;
		double PlayTime = 0.0;
		float MaxRelativeRenderCost = 0.f;
	};
} // namespace Metasound::Editor
