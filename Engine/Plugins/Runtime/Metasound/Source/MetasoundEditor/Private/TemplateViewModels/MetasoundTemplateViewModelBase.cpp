// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateViewModels/MetasoundTemplateViewModelBase.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "View/MVVMView.h"


void UMetaSoundTemplateViewModelBase::ConstructWidgetViewModel(
	UUserWidget* UserWidget,
	UMetaSoundBuilderBase& InBuilder,
	TConstStructView<FMetaSoundFrontendDocumentTemplate> Config,
	TSubclassOf<UMetaSoundTemplateViewModelBase> VMClass)
{
	using namespace Metasound::Engine;

	if (!UserWidget)
	{
		return;
	}

	UMVVMView* View = UserWidget->GetExtension<UMVVMView>();
	const UScriptStruct* ConfigStruct = Config.GetScriptStruct();
	
	if (View && ConfigStruct && VMClass.Get() != nullptr)
	{
		if (UMetaSoundTemplateViewModelBase* ViewModel = NewObject<UMetaSoundTemplateViewModelBase>(GetTransientPackage(), VMClass))
		{
			if (ViewModel->IsSupportedTemplate(*ConfigStruct))
			{
				ViewModel->Initialize(&InBuilder);
				View->SetViewModelByClass(ViewModel);
			}
		}
	}
}

bool UMetaSoundTemplateViewModelBase::IsSupportedTemplate(const UScriptStruct& InStruct) const
{
	return true;
}

void UMetaSoundTemplateViewModelBase::Initialize(UMetaSoundBuilderBase* InBuilder)
{
	using namespace Metasound::Frontend;
	Builder = InBuilder;

	BuilderListener = MakeShared<FBuilderListener>(this);
	Builder->AddTransactionListener(BuilderListener->AsShared());

	Reload();
}

void UMetaSoundTemplateViewModelBase::Reload()
{
}

void UMetaSoundTemplateViewModelBase::FBuilderListener::OnBuilderReloaded(Metasound::Frontend::FDocumentModifyDelegates& OutDelegates)
{
	if (TStrongObjectPtr<UMetaSoundTemplateViewModelBase> PinnedParent = Parent.Pin())
	{
		PinnedParent->Reload();
	}

	OutDelegates.OnDocumentTemplateChanged.Remove(TemplateChangeDelegateHandle);
	TemplateChangeDelegateHandle = OutDelegates.OnDocumentTemplateChanged.AddSP(this, &UMetaSoundTemplateViewModelBase::FBuilderListener::OnTemplateChanged);
}

void UMetaSoundTemplateViewModelBase::FBuilderListener::OnTemplateChanged(const Metasound::Frontend::FDocumentTemplateChangedArgs& InArgs)
{
	using namespace Metasound::Frontend;

	// Property change covered by template's direct property changed delegate
	// Covers if the whole struct is replaced, in which case just re-initialize
	// or if editor transaction stack has been rolled forward or back.
	if (InArgs.Type != Metasound::Frontend::EDocumentTemplateChangeType::Property)
	{
		if (TStrongObjectPtr<UMetaSoundTemplateViewModelBase> PinnedParent = Parent.Pin())
		{
			if (UMetaSoundBuilderBase* ThisBuilder = PinnedParent->Builder.Get())
			{
				FMetaSoundFrontendDocumentBuilder& DocBuilder = ThisBuilder->GetBuilder();
				TConstStructView<FMetaSoundTemplate> DocTemplate = DocBuilder.GetConstDocumentTemplate();
				if (const UScriptStruct* ScriptStruct = DocTemplate.GetScriptStruct();
					ScriptStruct && PinnedParent->IsSupportedTemplate(*ScriptStruct))
				{
					PinnedParent->Reload();
				}
				else
				{
					// Builder is now associated with MetaSound that no longer supports this configuration, so stop listening.
					// This UObject (and owning object, likely an associated UUserWidget) will be GC'ed at an indeterminate
					// later point.
					DocBuilder.GetDocumentDelegates().OnDocumentTemplateChanged.Remove(TemplateChangeDelegateHandle);
				}
			}
		}
	}
}
