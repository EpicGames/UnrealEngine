// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateViewModels/RandomizerConfigurationViewModel.h"

#include "DocumentTemplates/MetasoundRandomizerTemplate.h"


bool UMetaSoundRandomizerViewModel::IsSupportedTemplate(const UScriptStruct& InStruct) const
{
	return InStruct.IsChildOf(FMetaSoundRandomizerTemplate::StaticStruct());
}

bool UMetaSoundRandomizerViewModel::GetIsOneShot() const
{
	return bIsOneShot;
}

const TArray<TObjectPtr<USoundWave>>& UMetaSoundRandomizerViewModel::GetSounds() const
{
	return Sounds;
}

void UMetaSoundRandomizerViewModel::Reload()
{
	Super::Reload();

	if (const FMetaSoundRandomizerTemplate* Template = GetConstTemplate<FMetaSoundRandomizerTemplate>())
	{
		UE_MVVM_SET_PROPERTY_VALUE(bIsOneShot, Template->bIsOneShot);
		UE_MVVM_SET_PROPERTY_VALUE(Pitch, Template->Pitch);
		UE_MVVM_SET_PROPERTY_VALUE(Sounds, Template->Sounds);

		Template->GetPropertyChangedDelegate().Remove(PropertyChangedHandle);
		PropertyChangedHandle = Template->GetPropertyChangedDelegate().AddUObject(this, &UMetaSoundRandomizerViewModel::OnPropertyChanged);
	}
}

void UMetaSoundRandomizerViewModel::OnPropertyChanged(const FPropertyChangedEvent& InEvent)
{
	if (const FMetaSoundRandomizerTemplate* Template = GetConstTemplate<FMetaSoundRandomizerTemplate>())
	{
		if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundRandomizerTemplate, bIsOneShot))
		{
			UE_MVVM_SET_PROPERTY_VALUE(bIsOneShot, Template->bIsOneShot);
		}
		else if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundRandomizerTemplate, Pitch))
		{
			UE_MVVM_SET_PROPERTY_VALUE(Pitch, Template->Pitch);
		}
		else if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FVector2f, X))
		{
			UE_MVVM_SET_PROPERTY_VALUE(Pitch, Template->Pitch);
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMin);
		}
		else if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FVector2f, Y))
		{
			UE_MVVM_SET_PROPERTY_VALUE(Pitch, Template->Pitch);
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMax);
		}
		else if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundRandomizerTemplate, Sounds))
		{
			UE_MVVM_SET_PROPERTY_VALUE(Sounds, Template->Sounds);
		}
	}
}

FVector2f UMetaSoundRandomizerViewModel::GetPitch() const
{
	return Pitch;
}

float UMetaSoundRandomizerViewModel::GetPitchMin() const
{
	return Pitch.X;
}

float UMetaSoundRandomizerViewModel::GetPitchMax() const
{
	return Pitch.Y;
}

void UMetaSoundRandomizerViewModel::SetIsOneShot(bool bInIsOneShot)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(bIsOneShot, bInIsOneShot))
	{
		Builder->SetTemplateProperties<FMetaSoundRandomizerTemplate>([this](FMetaSoundRandomizerTemplate& RandomTemplate)
		{
			RandomTemplate.bIsOneShot = bIsOneShot;
		});
	}
}

void UMetaSoundRandomizerViewModel::SetPitch(const FVector2f& InPitch)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Pitch, InPitch))
	{
		Builder->SetTemplateProperties<FMetaSoundRandomizerTemplate>([this](FMetaSoundRandomizerTemplate& RandomTemplate)
		{
			RandomTemplate.Pitch = Pitch;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMin);
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMax);
		});
	}
}

void UMetaSoundRandomizerViewModel::SetPitchMax(float PitchMax)
{
	FVector2f NewPitch = Pitch;
	NewPitch.Y = PitchMax;
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Pitch, NewPitch))
	{
		Builder->SetTemplateProperties<FMetaSoundRandomizerTemplate>([this](FMetaSoundRandomizerTemplate& RandomTemplate)
		{
			RandomTemplate.Pitch = Pitch;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMax);
		});
	}
}

void UMetaSoundRandomizerViewModel::SetPitchMin(float PitchMin)
{
	FVector2f NewPitch = Pitch;
	NewPitch.X = PitchMin;
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Pitch, NewPitch))
	{
		Builder->SetTemplateProperties<FMetaSoundRandomizerTemplate>([this](FMetaSoundRandomizerTemplate& RandomTemplate)
		{
			RandomTemplate.Pitch = Pitch;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::GetPitchMin);
		});
	}
}

void UMetaSoundRandomizerViewModel::SetNumSounds(int32 NumSounds)
{
	if (NumSounds < 0)
	{
		return;
	}

	TArray<TObjectPtr<USoundWave>> NewSounds = Sounds;
	NewSounds.SetNumZeroed(NumSounds);
	SetSounds(MoveTemp(NewSounds));
}

void UMetaSoundRandomizerViewModel::SetSounds(TArray<TObjectPtr<USoundWave>> InSounds)
{
	if (Builder)
	{
		FMetaSoundRandomizerTemplate::RemoveInvalidSounds(Builder->GetConstBuilder(), true /* bRemoveNullEntries */, InSounds);
		if (UE_MVVM_SET_PROPERTY_VALUE(Sounds, InSounds))
		{
			Builder->SetTemplateProperties<FMetaSoundRandomizerTemplate>([this](FMetaSoundRandomizerTemplate& RandomTemplate)
			{
				RandomTemplate.Sounds = Sounds;
			});
		}
	}
}
