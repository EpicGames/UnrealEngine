// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Input/Reply.h"
#include "MetasoundEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

// FindInMetaSound is largely taken from FindInMaterial
namespace Metasound::Editor
{
	/** Item that matched the search results */
	class FFindInMetasoundResult
	{
	public:
		/** Create a root (or only text) result */
		FFindInMetasoundResult(const FString& InResultName);

		/** Create a listing for a search result*/
		FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UClass* InClass, int InDuplicationIndex);

		/** Create a listing for a pin result */
		FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UEdGraphPin* InPin);

		/** Create a listing for a node result */
		FFindInMetasoundResult(const FString& InResultName, TSharedPtr<FFindInMetasoundResult>& InParent, UEdGraphNode* InNode);

		/** Called when user clicks on the search item */
		FReply OnClick(TWeakPtr<class FEditor> MetasoundEditor);

		/* Get Category for this search result */
		FText GetCategory() const;

		/** Create an icon to represent the result */
		TSharedRef<SWidget> CreateIcon() const;

		/** Gets the comment on this node if any */
		FString GetCommentText() const;

		/** Gets the value of the pin or node if any */
		FText GetValueText();
			
		/** Gets the value tooltip (longer version of value text) of the pin or node if any */
		FText GetValueTooltipText();

		/** Any children listed under this category */
		TArray<TSharedPtr<FFindInMetasoundResult>> Children;

		/** Search result Parent */
		TWeakPtr<FFindInMetasoundResult> Parent;

		/*The meta string that was stored in the asset registry for this item */
		FString Value;

		/*The graph may have multiple instances of whatever we are looking for, this tells us which instance # we refer to*/
		int	DuplicationIndex;

		/*The class this item refers to */
		UClass* Class;

		/** The pin that this search result refers to */
		FEdGraphPinReference Pin;

		/** The graph node that this search result refers to (if not by asset registry or UK2Node) */
		TWeakObjectPtr<UEdGraphNode> GraphNode;

		/** Display text for comment information */
		FString CommentText;

		/** Cache value text */
		FText ValueText;

		/** Helper function to get MetaSound graph member from an ed graph node */
		static const UMetasoundEditorGraphMember* GetMetaSoundGraphMember(const UEdGraphNode* EdGraphNode);
	};


	/** Widget for searching for items that are part of a UEdGraph */
	class SFindInMetasound : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SFindInMetasound) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<class FEditor> InMetaSoundEditor);

		/** Focuses this widget's search box */
		void FocusForUse();
		/** Focuses this widget's search box, and Initiates a search with given SearchTerms */
		void FocusForUse(const FString& NewSearchTerms);

	protected:
		typedef TSharedPtr<FFindInMetasoundResult> FSearchResult;
		typedef STreeView<FSearchResult> STreeViewType;

		/** Called when user changes the text they are searching for */
		void OnSearchTextChanged(const FText& Text);

		/** Called when user changes commits text to the search box */
		void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

		/** Get the children of a row */
		void OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren);

		/** Called when user clicks on a new result */
		void OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type SelectInfo);

		/** Called when user double clicks on a new result */
		void OnTreeSelectionDoubleClick(FSearchResult Item);

		/** Called when a new row is being generated */
		TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);

		/** Begins the search based on the SearchValue */
		void InitiateSearch();

		/** Find any results that contain all of the tokens */
		void MatchTokens(const TArray<FString>& Tokens);

		/** Find any results that contain all of the tokens in provided graph and subgraphs */
		void MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens);

		/** Determines if a string matches the search tokens */
		static bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

		/** Pointer back to the MetaSound editor that owns us */
		TWeakPtr<class FEditor> MetaSoundEditorPtr;

		/** The tree view displays the results */
		TSharedPtr<STreeViewType> TreeView;

		/** The search text box */
		TSharedPtr<class SSearchBox> SearchTextField;

		/** This buffer stores the currently displayed results */
		TArray<FSearchResult> ItemsFound;

		/** we need to keep a handle on the root result, because it won't show up in the tree */
		FSearchResult RootSearchResult;

		/** The string to highlight in the results */
		FText HighlightText;

		/** The string to search for */
		FString SearchValue;

		/** Counters for number of results of each type */
		uint32 FoundNodeCount = 0;
		uint32 FoundPinCount = 0;
	};
} // namespace Metasound