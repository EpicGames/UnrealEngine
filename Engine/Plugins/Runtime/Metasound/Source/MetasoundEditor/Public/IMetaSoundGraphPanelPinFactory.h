// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraphUtilities.h"
#include "Math/Color.h"
#include "Misc/CoreDelegates.h"
#include "Styling/SlateBrush.h"

#define UE_API METASOUNDEDITOR_API


namespace Metasound::Editor
{
	// Delegate for creating a custom MetaSound pin widget
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SGraphPin>, FOnCreateMetaSoundPinWidget, UEdGraphPin*);

	/**
	 * Params for registering an external pin type.
	 *
	 * Used with IMetaSoundGraphPanelPinFactory::RegisterPin to provide optional
	 * styling data (category, color, and icons) for a registered data type.
	 */
	struct FGraphPinParams
	{
		FName PinCategory = { };
		FName PinSubCategory = { };
		const FLinearColor* PinColor = nullptr;
		const FSlateBrush* PinConnectedIcon = nullptr;
		const FSlateBrush* PinDisconnectedIcon = nullptr;
	};

	/**
	 * Factory for creating MetaSound Editor Graph pin icons and widgets.
	 *
	 * Provides a single registration API for external plugins to register custom pin types
	 * with optional styling (category, color, icons). Internal core MetaSound pin types
	 * (Boolean, Float, Trigger, etc.) are registered automatically at startup.
	 *
	 * Lifecycle:
	 * - Register pins in your module's StartupModule() after MetaSoundEditor is loaded.
	 * - Unregister pins in your module's ShutdownModule(). Guard with
	 *   FModuleManager::Get().IsModuleLoaded(IMetasoundEditorModule::ModuleName)
	 *   in case MetaSoundEditor shuts down before your module.
	 * - All Register/Unregister calls must be made on the game thread.
	 */
	class IMetaSoundGraphPanelPinFactory : public FGraphPanelPinFactory
	{
	public:
		// Register a pin widget factory for a specific pin category. Categories are broader
		// classifications like Boolean, Float, Int32, etc. Category delegates are used as a
		// fallback when no data-type-specific delegate is found.
		UE_API virtual void RegisterCategoryPin(FName InPinCategory, FOnCreateMetaSoundPinWidget InCreateDelegate) = 0;

		// Unregister a pin widget factory for a pin category.
		UE_API virtual void UnregisterCategoryPin(FName InPinCategory) = 0;

		// Register a pin widget factory for a specific data type. Data type registrations
		// take priority over category registrations in the widget creation lookup.
		UE_API virtual void RegisterDataTypePin(FName InDataTypeName, FOnCreateMetaSoundPinWidget InCreateDelegate) = 0;

		// Unregister a pin widget factory for a data type.
		UE_API virtual void UnregisterDataTypePin(FName InDataTypeName) = 0;

		// Register a pin type for a MetaSound data type with optional styling parameters.
		UE_API virtual void RegisterPin(FName InDataTypeName, const FGraphPinParams& Params = FGraphPinParams { }) = 0;

		// Unregister a previously registered pin type.
		UE_API virtual void UnregisterPin(FName InDataTypeName) = 0;

		// Find the pin type configuration for a registered data type. Returns nullptr if not registered.
		UE_API virtual const FEdGraphPinType* FindPinType(FName InDataTypeName) const = 0;

		// Get custom pin icons for a registered data type via pin reference.
		UE_API virtual bool GetCustomPinIcons(UEdGraphPin* InPin, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const = 0;

		// Get custom pin icons for a registered data type by name.
		UE_API virtual bool GetCustomPinIcons(FName InDataType, const FSlateBrush*& OutConnectedIcon, const FSlateBrush*& OutDisconnectedIcon) const = 0;

		// Get pin color for a pin type. Checks built-in categories (from UMetasoundEditorSettings),
		// then registered custom colors, then falls back to DefaultPinTypeColor.
		UE_API virtual FLinearColor GetPinColor(const FEdGraphPinType& InPinType) const = 0;
	};
} // namespace Metasound::Editor

#undef UE_API
