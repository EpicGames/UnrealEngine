// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Layout/SlateRect.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"


class IMetasoundEditor : public FAssetEditorToolkit
{
public:
	// Returns the editor's assigned MetaSound UObject.
	virtual UObject* GetMetasoundObject() const = 0;

	// Sets the selected object(s).
	virtual void SetSelection(const TArray<UObject*>& SelectedObjects, bool bInvokeTabOnSelectionSet = true) = 0;

	// Returns the bounds for the selected node(s).
	virtual bool GetBoundsForSelectedNodes(FSlateRect& OutRect, float Padding) = 0;

	// Plays the associated preview UAudioComponent & update editor state to reflect.
	virtual void Play() = 0;

	// Stop the associated preview UAudioComponent & update editor state to reflect.
	virtual void Stop() = 0;

	/** Whether or not the given editor is current associated with a UAudioComponent that is actively playing. */
	virtual bool IsPlaying() const = 0;
};

namespace Metasound::Editor
{
	// For tracking objects being dropped onto the editor graph
	struct FDroppedObjectsInfo
	{
		TArray<UObject*> DroppedObjects;
		FModifierKeysState ModifierKeys;
		FName TypeName;
	};
}
