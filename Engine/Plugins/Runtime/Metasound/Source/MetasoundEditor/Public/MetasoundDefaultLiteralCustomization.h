// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistries.h"
#include "Modules/ModuleInterface.h"
#include "PropertyHandle.h"
#include "Templates/Function.h"

#define UE_API METASOUNDEDITOR_API

// Forward Declarations
class IDetailPropertyRow;
class IPropertyHandle;
class SSearchableComboBox;
class UEdGraphPin;
class UMetasoundEditorGraph;
class UMetasoundEditorGraphMemberDefaultLiteral;
class UMetasoundEditorGraphNode;


namespace Metasound::Editor
{
	namespace LiteralCustomization
	{
		FGuid GetGuidPropertyValue(TSharedPtr<IPropertyHandle> GuidProperty);
	} // namespace LiteralCustomization

	class FMetasoundDefaultLiteralCustomizationBase
	{
	protected:
		IDetailCategoryBuilder* DefaultCategoryBuilder = nullptr;

	public:
		using FOnDefaultPageRowAdded = TFunction<void(IDetailPropertyRow& /* Value Row */, TSharedRef<IPropertyHandle> /* Page Property Handle */)>;

		UE_API FMetasoundDefaultLiteralCustomizationBase(IDetailCategoryBuilder& InDefaultCategoryBuilder);
		UE_API virtual ~FMetasoundDefaultLiteralCustomizationBase();

		UE_API virtual void CustomizeDefaults(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout);

		UE_DEPRECATED(5.5, "Use CustomizeDefaults instead and provide returned customized handles")
		UE_API virtual TArray<IDetailPropertyRow*> CustomizeLiteral(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout);

		UE_API virtual TAttribute<EVisibility> GetDefaultVisibility() const;
		UE_API virtual TAttribute<bool> GetEnabled() const;

		UE_API virtual void SetDefaultVisibility(TAttribute<EVisibility> VisibilityAttribute);
		UE_API virtual void SetEnabled(TAttribute<bool> EnableAttribute);
		UE_API virtual void SetResetOverride(const TOptional<FResetToDefaultOverride>& InResetOverride);

	protected:
		UE_API void CustomizePageDefaultRows(UMetasoundEditorGraphMemberDefaultLiteral& InLiteral, IDetailLayoutBuilder& InDetailLayout);
		UE_API virtual void BuildDefaultValueWidget(IDetailPropertyRow& ValueRow, TSharedPtr<IPropertyHandle> ValueProperty);

		TArray<TSharedPtr<IPropertyHandle>> DefaultProperties;

	private:
		UE_API TSharedRef<SWidget> BuildPageDefaultNameWidget(UMetasoundEditorGraphMemberDefaultLiteral& Literal, TSharedRef<IPropertyHandle> ElementProperty);
		UE_API void BuildPageDefaultComboBox(UMetasoundEditorGraphMemberDefaultLiteral& Literal, FText RowName);
		UE_API void UpdatePagePickerNames(TWeakObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> LiteralPtr);

		TArray<TSharedPtr<FString>> AddablePageStringNames;
		TSet<FName> ImplementedPageNames;
		TSharedPtr<SSearchableComboBox> PageDefaultComboBox;
		FDelegateHandle OnPageSettingsUpdatedHandle;

		TAttribute<bool> Enabled;
		TAttribute<EVisibility> Visibility;
		TOptional<FResetToDefaultOverride> ResetOverride;
	};

	class IMemberDefaultLiteralCustomizationFactory
	{
	public:
		virtual ~IMemberDefaultLiteralCustomizationFactory() = default;

		virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateLiteralCustomization(IDetailCategoryBuilder& DefaultCategoryBuilder) const = 0;
	};
} // namespace Metasound::Editor

#undef UE_API
