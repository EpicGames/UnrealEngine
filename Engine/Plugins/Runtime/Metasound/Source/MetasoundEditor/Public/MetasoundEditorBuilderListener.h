// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Delegates/DelegateCombinations.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"

#include "MetasoundEditorBuilderListener.generated.h"

// Forward Declarations
struct FMetaSoundNodeHandle;

USTRUCT(BlueprintType)
struct FGraphInputDefaultChangedArgs
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "Audio|MetaSound|BuilderListener")
	FName InputName;

	UPROPERTY(BlueprintReadWrite, Category = "Audio|MetaSound|BuilderListener")
	FName PageName;

	UPROPERTY(BlueprintReadWrite, Category = "Audio|MetaSound|BuilderListener")
	FMetasoundFrontendLiteral Literal;

	UPROPERTY(BlueprintReadWrite, Category = "Audio|MetaSound|BuilderListener")
	EMetaSoundLiteralChangeType ChangeType = EMetaSoundLiteralChangeType::Invalid;
};

// BP Delegates for builder changes
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderDocumentMetadataTextChanged, FText, NewText);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderDocumentMetadataTextArrayChanged, TArray<FText>, NewTextArray);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderDocumentMetadataStringChanged, FString, NewString);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderDocumentMetadataBoolChanged, bool, NewBool);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderPageMutate, FName, PageName);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundBuilderGraphInterfaceMutate, FName, VertexName, FName, DataType);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnMetaSoundBuilderGraphLiteralMutate, FName, VertexName, FMetasoundFrontendLiteral, LiteralValue, FName, PageName);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMetaSoundBuilderGraphInputDefaultChanged, const FGraphInputDefaultChangedArgs&, GraphInputDefaultChangedArgs);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundBuilderGraphVertexRename, FName, OldName, FName, NewName);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundBuilderGraphVertexBoolChanged, FName, VertexName, bool, bNewBool);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundBuilderGraphVertexIntChanged, FName, VertexName, int32, NewInt);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundBuilderGraphVertexTextChanged, FName, VertexName, FText, NewText);

UCLASS(MinimalAPI, BlueprintType)
class UMetaSoundEditorBuilderListener : public UObject
{
public:
	GENERATED_BODY()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~UMetaSoundEditorBuilderListener() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	void Init(const TWeakObjectPtr<UMetaSoundBuilderBase> InBuilder);

	void OnDocumentDisplayNameChanged();
	void OnDocumentDescriptionChanged();
	void OnDocumentAuthorChanged();
	void OnDocumentKeywordsChanged();
	void OnDocumentCategoryHierarchyChanged();
	void OnDocumentIsDeprecatedChanged();
	
	void OnPageAdded(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs);
	void OnRemovingPage(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs);
	void OnBuildPageChanged(const Metasound::Frontend::FDocumentMutatePageArgs& PageArgs);

	void OnGraphInputAdded(int32 Index);
	void OnRemovingGraphInput(int32 Index);
	void OnGraphInputNameChanged(FName OldName, FName NewName);
	void OnGraphInputDisplayNameChanged(int32 Index);
	void OnGraphInputDataTypeChanged(int32 Index);
	void OnGraphInputDescriptionChanged(int32 Index);
	void OnGraphInputSortOrderIndexChanged(int32 Index);
	void OnGraphInputIsConstructorPinChanged(int32 Index);
	void OnGraphInputIsAdvancedDisplayChanged(int32 Index);
	UE_DEPRECATED(5.8, "Use the input default changed delegate that includes additional args")
	void OnGraphInputDefaultChanged(int32 Index);
	void OnGraphInputDefaultChanged(const Metasound::Frontend::FDocumentArrayPagedInputArgs& Args);
	void OnGraphInputInheritsDefaultChanged(int32 Index);

	void OnGraphOutputAdded(int32 Index);
	void OnRemovingGraphOutput(int32 Index);
	void OnGraphOutputNameChanged(FName OldName, FName NewName);
	void OnGraphOutputDisplayNameChanged(int32 Index);
	void OnGraphOutputDataTypeChanged(int32 Index);
	void OnGraphOutputDescriptionChanged(int32 Index);
	void OnGraphOutputSortOrderIndexChanged(int32 Index);
	void OnGraphOutputIsConstructorPinChanged(int32 Index);
	void OnGraphOutputIsAdvancedDisplayChanged(int32 Index);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|BuilderListener")
	METASOUNDEDITOR_API void RemoveAllDelegates();

	// Metadata
	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Display Name Changed")
	FOnMetaSoundBuilderDocumentMetadataTextChanged OnDocumentDisplayNameChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Description Changed")
	FOnMetaSoundBuilderDocumentMetadataTextChanged OnDocumentDescriptionChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Author Changed")
	FOnMetaSoundBuilderDocumentMetadataStringChanged OnDocumentAuthorChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Keywords Changed")
	FOnMetaSoundBuilderDocumentMetadataTextArrayChanged OnDocumentKeywordsChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Category Hierarchy Changed")
	FOnMetaSoundBuilderDocumentMetadataTextArrayChanged OnDocumentCategoryHierarchyChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Document Is Deprecated Changed")
	FOnMetaSoundBuilderDocumentMetadataBoolChanged OnDocumentIsDeprecatedChangedDelegate;

	// Pages
	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Page Added")
	FOnMetaSoundBuilderPageMutate OnPageAddedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Page Removed")
	FOnMetaSoundBuilderPageMutate OnRemovingPageDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener", DisplayName = "On Build Page Changed")
	FOnMetaSoundBuilderPageMutate OnBuildPageChangedDelegate;

	// Graph Interface
	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Added")
	FOnMetaSoundBuilderGraphInterfaceMutate OnGraphInputAddedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Removed")
	FOnMetaSoundBuilderGraphInterfaceMutate OnRemovingGraphInputDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Name Changed")
	FOnMetaSoundBuilderGraphVertexRename OnGraphInputNameChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Display Name Changed")
	FOnMetaSoundBuilderGraphVertexTextChanged OnGraphInputDisplayNameChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Data Type Changed")
	FOnMetaSoundBuilderGraphInterfaceMutate OnGraphInputDataTypeChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Description Changed")
	FOnMetaSoundBuilderGraphVertexTextChanged OnGraphInputDescriptionChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Sort Order Index Changed")
	FOnMetaSoundBuilderGraphVertexIntChanged OnGraphInputSortOrderIndexChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Is Constructor Pin Changed")
	FOnMetaSoundBuilderGraphVertexBoolChanged OnGraphInputIsConstructorPinChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Is Advanced Display Changed")
	FOnMetaSoundBuilderGraphVertexBoolChanged OnGraphInputIsAdvancedDisplayChangedDelegate;

	UE_DEPRECATED(5.8, "Use the GraphInputDefaultChanged delegate that includes additional args")
	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "[Deprecated] On Graph Input Default Changed")
	FOnMetaSoundBuilderGraphLiteralMutate OnGraphInputDefaultChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Default Changed")
	FOnMetaSoundBuilderGraphInputDefaultChanged OnGraphInputDefaultChangedWithArgsDelegate;
	
	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|InputDelegates", DisplayName = "On Graph Input Inherits Default Changed")
	FOnMetaSoundBuilderGraphVertexBoolChanged OnGraphInputInheritsDefaultChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Added")
	FOnMetaSoundBuilderGraphInterfaceMutate OnGraphOutputAddedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Removed")
	FOnMetaSoundBuilderGraphInterfaceMutate OnRemovingGraphOutputDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Name Changed")
	FOnMetaSoundBuilderGraphVertexRename OnGraphOutputNameChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Display Name Changed")
	FOnMetaSoundBuilderGraphVertexTextChanged OnGraphOutputDisplayNameChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Data Type Changed")
	FOnMetaSoundBuilderGraphInterfaceMutate OnGraphOutputDataTypeChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Description Changed")
	FOnMetaSoundBuilderGraphVertexTextChanged OnGraphOutputDescriptionChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Sort Order Index Changed")
	FOnMetaSoundBuilderGraphVertexIntChanged OnGraphOutputSortOrderIndexChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Is Constructor Pin Changed")
	FOnMetaSoundBuilderGraphVertexBoolChanged OnGraphOutputIsConstructorPinChangedDelegate;

	UPROPERTY(BlueprintAssignable, BlueprintCallable, Category = "Audio|MetaSound|BuilderListener|OutputDelegates", DisplayName = "On Graph Output Is Advanced Display Changed")
	FOnMetaSoundBuilderGraphVertexBoolChanged OnGraphOutputIsAdvancedDisplayChangedDelegate;

private:
	// Handles for document delegates
	FDelegateHandle OnDocumentDisplayNameChangedHandle;
	FDelegateHandle OnDocumentDescriptionChangedHandle;
	FDelegateHandle OnDocumentAuthorChangedHandle;
	FDelegateHandle OnDocumentKeywordsChangedHandle;
	FDelegateHandle OnDocumentCategoryHierarchyChangedHandle;
	FDelegateHandle OnDocumentIsDeprecatedChangedHandle;
	
	FDelegateHandle OnPageAddedHandle;
	FDelegateHandle OnRemovingPageHandle;
	FDelegateHandle OnBuildPageChangedHandle;

	FDelegateHandle OnInputAddedHandle;
	FDelegateHandle OnRemovingInputHandle;
	FDelegateHandle OnInputNameChangedHandle;
	FDelegateHandle OnInputDisplayNameChangedHandle;
	FDelegateHandle OnInputDataTypeChangedHandle;
	FDelegateHandle OnInputDescriptionChangedHandle;
	FDelegateHandle OnInputSortOrderIndexChangedHandle;
	FDelegateHandle OnInputIsConstructorPinChangedHandle;
	FDelegateHandle OnInputIsAdvancedDisplayChangedHandle;
	FDelegateHandle OnInputDefaultChangedHandle;
	FDelegateHandle OnInputInheritsDefaultChangedHandle;

	FDelegateHandle OnOutputAddedHandle;
	FDelegateHandle OnRemovingOutputHandle;
	FDelegateHandle OnOutputNameChangedHandle;
	FDelegateHandle OnOutputDisplayNameChangedHandle;
	FDelegateHandle OnOutputDataTypeChangedHandle;
	FDelegateHandle OnOutputDescriptionChangedHandle;
	FDelegateHandle OnOutputSortOrderIndexChangedHandle;
	FDelegateHandle OnOutputIsConstructorPinChangedHandle;
	FDelegateHandle OnOutputIsAdvancedDisplayChangedHandle;

	TWeakObjectPtr<UMetaSoundBuilderBase> Builder;

	class FEditorBuilderListener : public Metasound::Frontend::IDocumentBuilderTransactionListener
	{
	public: 

		FEditorBuilderListener() = default;
		FEditorBuilderListener(TObjectPtr<UMetaSoundEditorBuilderListener> InParent)
			: Parent(InParent)
		{
		}

		virtual ~FEditorBuilderListener() = default;

		// IDocumentBuilderTransactionListener
		virtual void OnBuilderReloaded(Metasound::Frontend::FDocumentModifyDelegates& OutDelegates) override;

	private: 
		TObjectPtr<UMetaSoundEditorBuilderListener> Parent;
	};

	TSharedPtr<FEditorBuilderListener> BuilderListener;
};
