// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

namespace Metasound
{
	namespace Editor
	{
		class FEditorCommands : public TCommands<FEditorCommands>
		{
		public:
			/** Constructor */
			FEditorCommands()
				: TCommands<FEditorCommands>("MetaSoundEditor", NSLOCTEXT("Contexts", "MetaSoundEditor", "MetaSound Graph Editor"), NAME_None, "MetaSoundStyle")
			{
			}

			/** Plays the Metasound */
			TSharedPtr<FUICommandInfo> Play;

			/** Stops the currently playing Metasound */
			TSharedPtr<FUICommandInfo> Stop;

			/** Imports the Metasound from Json */
			TSharedPtr<FUICommandInfo> Import;

			/** Exports the Metasound */
			TSharedPtr<FUICommandInfo> Export;

			/** Plays stops the currently playing Metasound */
			TSharedPtr<FUICommandInfo> TogglePlayback;

			/** Toggles showing the graph for the given configured MetaSound */
			TSharedPtr<FUICommandInfo> ToggleShowGraph;

			/** Selects the Metasound in the content browser. If referencing Metasound nodes are selected, selects referenced assets instead. */
			TSharedPtr<FUICommandInfo> BrowserSync;

			/** Breaks the node input/output link */
			TSharedPtr<FUICommandInfo> BreakLink;

			/** Adds an input to the node */
			TSharedPtr<FUICommandInfo> AddInput;

			/** Shows the MetasoundSource's specific settings in the Details panel (if of respective type) */
			TSharedPtr<FUICommandInfo> EditSourceSettings;

			/** Shows the Metasound's top-level general object settings in the Details panel */
			TSharedPtr<FUICommandInfo> EditMetasoundSettings;

			// Run updates on selected node(s) class(es) if required by any subset of selected nodes
			TSharedPtr<FUICommandInfo> UpdateNodeClass;

			// Delete selected items. 
			TSharedPtr<FUICommandInfo> Delete;

			// Show the Find in MetaSound window. 
			TSharedPtr<FUICommandInfo> FindInMetaSound;

			/** Promotes pin to graph input */
			TSharedPtr<FUICommandInfo> PromoteToInput;

			/** Promotes pin to graph output */
			TSharedPtr<FUICommandInfo> PromoteToOutput;

			/** Promotes pin to graph variable */
			TSharedPtr<FUICommandInfo> PromoteToVariable;

			/** Promotes pin to deferred graph variable */
			TSharedPtr<FUICommandInfo> PromoteToDeferredVariable;

			/** Promotes all unconnected node inputs to graph inputs. */
			TSharedPtr<FUICommandInfo> PromoteAllToInput;

			/** Promotes all unconnected node inputs to shared graph inputs. */
			TSharedPtr<FUICommandInfo> PromoteAllToCommonInputs;

			/** Copies the selected nodes, shared connections, and required interfaces to the clipboard as configurator c++ */
			TSharedPtr<FUICommandInfo> CopySelectionAsConfiguratorCodeToClipboard;

			/** Initialize commands */
			virtual void RegisterCommands() override;
		};
	} // namespace Editor
} // namespace Metasound