// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameterControllerInterface.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundEditor.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundSource.h"
#include "MetasoundUObjectRegistry.h"
#include "Misc/AssertionMacros.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundEditorGraph.generated.h"

#define UE_API METASOUNDEDITOR_API


// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;
struct FPropertyChangedChainEvent;
struct FPropertyChangedEvent;

class FMetasoundAssetBase;
class ITargetPlatform;
class UMetasoundEditorGraphInputNode;
class UMetasoundEditorGraphNode;


namespace Metasound
{
	namespace Editor
	{
		struct FGraphValidationResults;

		struct FCreateNodeVertexParams
		{
			FName DataType;
			EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Reference;
		};
	} // namespace Editor
} // namespace Metasound

USTRUCT()
struct FMetasoundEditorGraphMemberBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	FName MemberName;

	UPROPERTY()
	FText Description;
};

USTRUCT()
struct FMetasoundEditorGraphVertexBreadcrumb : public FMetasoundEditorGraphMemberBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Unset;

	UPROPERTY()
	TMap<FGuid, FMetasoundFrontendLiteral> DefaultLiterals;

	UPROPERTY()
	bool bIsAdvancedDisplay = false;

	UPROPERTY()
	int32 SortOrderIndex = 0;
};

USTRUCT()
struct FMetasoundEditorGraphVariableBreadcrumb : public FMetasoundEditorGraphMemberBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	FMetasoundFrontendLiteral DefaultLiteral;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetasoundMemberNameChanged, FGuid /* NodeID */);
DECLARE_MULTICAST_DELEGATE(FOnMetasoundMemberRenameRequested);

UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultLiteral : public UMetaSoundFrontendMemberMetadata
{
	GENERATED_BODY()

public:
	// Returns the default literal casted as the given template type. Asserts if
	// literal is not provided type and enforces type is supported by literal.
	template <typename T>
	T GetDefaultAs(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) const
	{
		T Value;
		FMetasoundFrontendLiteral Val;
		bool bSuccess = TryFindDefault(Val, &InPageID);
		checkf(bSuccess, TEXT("Value assigned to PageID not found"));
		bSuccess = Val.TryGet(Value);
		checkf(bSuccess, TEXT("Literal type not supported"));
		return Value;
	}

	UE_API virtual void ForceRefresh();
	UE_API virtual FName GetDataType() const;
	UE_API virtual EMetasoundFrontendLiteralType GetLiteralType() const;
	UE_API virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID);
	virtual void Initialize() { };
	UE_API virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const;
	UE_API virtual bool RemoveDefault(const FGuid& InPageID);
	UE_API virtual void ResetDefaults();
	UE_API virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID = Metasound::Frontend::DefaultPageID);
	UE_API virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const;

	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const
	{
	}

	// Called when literal is initialized for the first time to allow for setting
	// initial editor-only fields based on context within editor/document model.

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	// Synchronizes local transient editor-only member data with changes made to the
	// associated Frontend Document member. Returns whether or not object was modified.
	virtual bool Synchronize() { return false; };

	UE_API UMetasoundEditorGraphMember* FindMember() const;

	static FName GetDefaultsPropertyName()
	{
		return "Defaults";
	}

protected:
	virtual void ResolvePageDefaults() { }
	virtual void SortPageDefaults() { }
	UE_API bool TryGetPreviewPageID(FGuid& OutPreviewPageID) const;
};

/** UMetasoundEditorGraphMember is a base class for non-node graph level members 
 * such as inputs, outputs and variables. */
UCLASS(MinimalAPI, Abstract)
class UMetasoundEditorGraphMember : public UObject
{
	GENERATED_BODY()

public:
	/** Delegate called when a rename is requested on a renameable member node. */
	FOnMetasoundMemberRenameRequested OnRenameRequested;

	/** Return the section of where this member belongs. */
	virtual Metasound::Editor::ENodeSection GetSectionID() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetSectionID, return Metasound::Editor::ENodeSection::None; );

	/** Return the nodes associated with this member */
	virtual TArray<UMetasoundEditorGraphMemberNode*> GetNodes() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetNodes, return TArray<UMetasoundEditorGraphMemberNode*>(); );

	/** Sets the datatype on the member. */
	virtual void SetDataType(FName InNewType, bool bPostTransaction = true) PURE_VIRTUAL(UMetasoundEditorGraphMember::SetDataType, );
	
	/** If the Member Name can be changed to InNewName, returns true,
	 * otherwise returns false with an error. */
	virtual bool CanRename(const FText& InNewName, FText& OutError) const PURE_VIRTUAL(UMetasoundEditorGraphMember::CanRename, return false; );

	/** Set the display name */
	virtual void SetDisplayName(const FText& InNewName, bool bPostTransaction) PURE_VIRTUAL(UMetasoundEditorGraphMember::SetDisplayName, );

	/** Get the member display name */
	virtual FText GetDisplayName() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetDisplayName, return FText::GetEmpty(); );

	/** Set the member name */
	virtual void SetMemberName(const FName& InNewName, bool bPostTransaction) PURE_VIRTUAL(UMetasoundEditorGraphMember::SetMemberName, );

	/** Gets the members name */
	virtual FName GetMemberName() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetMemberName, return FName(); );

	/** Get ID for this member */
	virtual FGuid GetMemberID() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetMemberID, return FGuid(); );

	/** Set the member description */
	virtual void SetDescription(const FText& InDescription, bool bPostTransaction) PURE_VIRTUAL(UMetasoundEditorGraphMember::SetDescription, );

	/** Get the member description */
	virtual FText GetDescription() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetDescription, return FText::GetEmpty(); );

	/** Returns the label of the derived member type (e.g. Input/Output/Variable) */
	virtual const FText& GetGraphMemberLabel() const PURE_VIRTUAL(UMetasoundEditorGraphMember::GetGraphMemberLabel, return FText::GetEmpty(); );

	/** Resets the member to the class default. */
	virtual void ResetToClassDefault() PURE_VIRTUAL(UMetasoundEditorGraphMember::ResetToClassDefault, );

	/** Update the frontend with the given member's default UObject value.
	 * @param bPostTransaction - Post as editor transaction if true
	 */
	virtual void UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID = nullptr, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set) PURE_VIRTUAL(UMetasoundEditorGraphMember::UpdateFrontendDefaultLiteral, );

	// Synchronizes cached data with the frontend representation on the represented document.
	UE_API virtual bool Synchronize();

	UE_API FMetaSoundFrontendDocumentBuilder& GetFrontendBuilderChecked() const;

	/** Returns the parent MetaSound Graph. If the Outer object of the member is non
	 * a UMetasoundEditorGraph, returns a nullptr. */
	UE_API UMetasoundEditorGraph* GetOwningGraph();

	/** Returns the parent MetaSound Graph. If the Outer object of the member is non
	 * a UMetasoundEditorGraph, returns a nullptr. */
	UE_API const UMetasoundEditorGraph* GetOwningGraph() const;

	/* Whether this member can be renamed. */
	virtual bool CanRename() const PURE_VIRTUAL(UMetasoundEditorGraphMember::CanRename, return false;);

	/** Whether the displayed default supports paged values or is a single default value (i.e. characterized per page). */
	virtual bool IsDefaultPaged() const { return false; }

	/** Cache breadcrumb data before copying member to the clipboard */
	virtual void CacheBreadcrumb() { };

#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	/** Returns the current data type */
	UE_API FName GetDataType() const;

	static FName GetLiteralPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UMetasoundEditorGraphMember, Literal);
	}

	/** Returns literal associated with the given member */
	UMetasoundEditorGraphMemberDefaultLiteral* GetLiteral() const { return Literal; }

	/** Creates new literal if there is none or conforms literal object type to member's DataType if not matching.
	  * If force rebind is true, will always attempt to reinitialize with updated data from frontend literal. */
	UE_API void InitializeLiteral(bool bForceRebind=false);

protected:
	/** Default literal value of member */
	UPROPERTY()
	TObjectPtr<UMetasoundEditorGraphMemberDefaultLiteral> Literal;

	/** Metasound Data Type. */
	UPROPERTY()
	FName TypeName;

	friend class UMetasoundEditorGraph;
	friend class UMetaSoundEditorSubsystem;
};

/** Base class for an input or output of the graph. */
UCLASS(MinimalAPI, Abstract)
class UMetasoundEditorGraphVertex : public UMetasoundEditorGraphMember
{
	GENERATED_BODY()

protected:
	UE_API const FMetasoundFrontendNode* GetFrontendNode() const;

	/** If the vertex's name contains an interface namespace, return true and set OutInterface to that interface if provided. */
	UE_API bool NameContainsInterfaceNamespace(FMetasoundFrontendInterface* OutInterface) const;

	/** Implements frontend call to rename member */
	virtual bool RenameFrontendMemberInternal(FMetaSoundFrontendDocumentBuilder& Builder, FName OldName, FName InNewName) const PURE_VIRTUAL(UMetasoundEditorGraphVertex::RenameFrontendMemberInternal, return false;)

public:
	/** Initializes all properties with the given parameters required to identify the frontend member from this editor graph member. */
	UE_API void InitMember(FName InDataType, const FMetasoundFrontendLiteral& InDefaultLiteral, FGuid InNodeID, FMetasoundFrontendClassName&& InClassName);

	/** ID of Metasound Frontend node. */
	UPROPERTY()
	FGuid NodeID;

	/* Class name of Metasound Frontend node. */
	UPROPERTY()
	FMetasoundFrontendClassName ClassName;

	/* ~Begin UMetasoundEditorGraphMember interface */
	UE_API virtual FGuid GetMemberID() const override;
	UE_API virtual FName GetMemberName() const override;
	UE_API virtual FText GetDisplayName() const override;

	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	UE_API virtual TArray<UMetasoundEditorGraphMemberNode*> GetNodes() const override;
	UE_API virtual void SetDescription(const FText& InDescription, bool bPostTransaction) override;
	UE_API virtual void SetMemberName(const FName& InNewName, bool bPostTransaction) override;
	UE_API virtual bool Synchronize() override;
	UE_API virtual void CacheBreadcrumb() override;
	/* ~End UMetasoundEditorGraphMember interface */

	/** Returns Breadcrumb data of this Vertex*/
	const FMetasoundEditorGraphVertexBreadcrumb& GetBreadcrumb() const { return Breadcrumb; };

	/** Version of interface membership, or invalid version if not an interface member. */
	UE_API virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const;

	/** Returns true if member is part of an interface. If supplied interface pointer, sets pointer's data to the interface vertex is member of. */
	UE_API virtual bool IsInterfaceMember(FMetasoundFrontendInterface* OutInterface = nullptr) const;

	/** Returns the Metasound class type of the associated node */
	virtual EMetasoundFrontendClassType GetClassType() const PURE_VIRTUAL(UMetasoundEditorGraphVertex::GetClassType, return EMetasoundFrontendClassType::Invalid; )

	virtual const FMetasoundFrontendClassVertex* GetFrontendClassVertex() const PURE_VIRTUAL(UMetasoundEditorGraphVertex::GetFrontendClassVertex, return nullptr; );

	/** Returns the SortOrderIndex assigned to this member. */
	virtual int32 GetSortOrderIndex() const PURE_VIRTUAL(UMetasoundEditorGraphVertex::GetSortOrderIndex, return 0; )

	/** Sets the DataType assigned to this member. */
	virtual void SetDataType(FName DataType, bool bPostTransaction = true) PURE_VIRTUAL(UMetasoundEditorGraphVertex::SetDataType, )

	/** Sets the SortOrderIndex assigned to this member. */
	virtual void SetSortOrderIndex(int32 InSortOrderIndex) PURE_VIRTUAL(UMetasoundEditorGraphVertex::SetSortOrderIndex, )
	
	/** Sets the VertexAccessType assigned to this member. */
	virtual void SetVertexAccessType(EMetasoundFrontendVertexAccessType InNewAccessType, bool bPostTransaction = true) PURE_VIRTUAL(UMetasoundEditorGraphVertex::SetVertexAccessType, )

#if WITH_EDITORONLY_DATA
	/**Sets if the Vertex is shown as Advanced Display*/
	UE_API bool SetIsAdvancedDisplay(const bool IsAdvancedDisplay);
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.6, "Node Handles are actively being deprecated, use the MetaSound Frontend Document Builder API")
	UE_API Metasound::Frontend::FNodeHandle GetNodeHandle();

	/** Returns the node handle associated with the vertex. */
	UE_API Metasound::Frontend::FConstNodeHandle GetConstNodeHandle() const;

	virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const PURE_VIRTUAL(UMetasoundEditorGraphVertex::GetVertexAccessType, return EMetasoundFrontendVertexAccessType::Reference; );

	UE_API virtual bool CanRename() const override;

	friend class Metasound::Editor::FEditor;

protected:
	UE_API void SetMemberNameInternal(const FName& InNewName, bool bPropagateToPinNames, bool bPostTransaction);

	UPROPERTY()
	FMetasoundEditorGraphVertexBreadcrumb Breadcrumb;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphInput : public UMetasoundEditorGraphVertex
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetDescription() const override;
	UE_API virtual const FMetasoundFrontendClassVertex* GetFrontendClassVertex() const override;
	UE_API virtual int32 GetSortOrderIndex() const override;
	UE_API virtual void SetSortOrderIndex(int32 InSortOrderIndex) override;
	UE_API virtual TArray<UMetasoundEditorGraphMemberNode*> GetNodes() const override;

	UE_API virtual const FText& GetGraphMemberLabel() const override;
	UE_API virtual bool IsDefaultPaged() const override;
	UE_API virtual bool IsInterfaceMember(FMetasoundFrontendInterface* OutInterface = nullptr) const override;
	UE_API virtual void ResetToClassDefault() override;
	UE_API virtual void SetDataType(FName InNewType, bool bPostTransaction) override;
	UE_API virtual void SetDescription(const FText& InDescription, bool bPostTransaction) override;
	UE_API virtual void SetDisplayName(const FText& InNewName, bool bPostTransaction) override;
	UE_API virtual void SetMemberName(const FName& InNewName, bool bPostTransaction) override;
	UE_API virtual void SetVertexAccessType(EMetasoundFrontendVertexAccessType InNewAccessType, bool bPostTransaction) override;
	UE_API virtual bool Synchronize() override;
	UE_API virtual void CacheBreadcrumb() override;
	UE_API virtual void UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID = nullptr, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set) override;
	UE_API virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

protected:
	UE_API virtual bool RenameFrontendMemberInternal(FMetaSoundFrontendDocumentBuilder& Builder, FName OldName, FName InNewName) const override;

	virtual EMetasoundFrontendClassType GetClassType() const override { return EMetasoundFrontendClassType::Input; }
	UE_API virtual Metasound::Editor::ENodeSection GetSectionID() const override;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphOutput : public UMetasoundEditorGraphVertex
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetDescription() const override;
	UE_API virtual const FMetasoundFrontendClassVertex* GetFrontendClassVertex() const override;
	UE_API virtual int32 GetSortOrderIndex() const override;
	UE_API virtual bool IsInterfaceMember(FMetasoundFrontendInterface* OutInterface = nullptr) const override;
	UE_API virtual void SetSortOrderIndex(int32 InSortOrderIndex) override;
	UE_API virtual const FText& GetGraphMemberLabel() const override;
	UE_API virtual void ResetToClassDefault() override;
	UE_API virtual void SetDataType(FName InNewType, bool bPostTransaction) override;
	UE_API virtual void SetDescription(const FText& InDescription, bool bPostTransaction) override;
	UE_API virtual void SetDisplayName(const FText& InNewName, bool bPostTransaction) override;
	UE_API virtual void SetVertexAccessType(EMetasoundFrontendVertexAccessType InNewAccessType, bool bPostTransaction) override;
	UE_API virtual bool Synchronize() override;
	UE_API virtual void UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID = nullptr, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set) override;
	UE_API virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

protected:
	UE_API virtual bool RenameFrontendMemberInternal(FMetaSoundFrontendDocumentBuilder& Builder, FName OldName, FName InNewName) const override;

	virtual EMetasoundFrontendClassType GetClassType() const override { return EMetasoundFrontendClassType::Output; }
	UE_API virtual Metasound::Editor::ENodeSection GetSectionID() const override;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphVariable : public UMetasoundEditorGraphMember
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid VariableID;

public:
	UE_API void InitMember(FName InDataType, const FMetasoundFrontendLiteral& InDefaultLiteral, FGuid InVariableID);

	/* ~Begin UMetasoundEditorGraphMember interface */
	UE_API virtual Metasound::Editor::ENodeSection GetSectionID() const override;
	UE_API virtual void SetDataType(FName InNewType, bool bPostTransaction = true) override;

	UE_API virtual FText GetDescription() const override;
	UE_API virtual void SetDescription(const FText& InDescription, bool bPostTransaction) override;

	UE_API virtual FGuid GetMemberID() const override;
	UE_API virtual bool CanRename(const FText& InNewName, FText& OutError) const override;
	UE_API virtual void SetMemberName(const FName& InNewName, bool bPostTransaction) override;
	UE_API virtual FName GetMemberName() const override;

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void SetDisplayName(const FText& InNewName, bool bPostTransaction) override;

	UE_API virtual void ResetToClassDefault() override;
	UE_API virtual bool Synchronize() override;
	UE_API virtual void CacheBreadcrumb() override;
	UE_API virtual void UpdateFrontendDefaultLiteral(bool bPostTransaction, const FGuid* InPageID = nullptr, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set) override;

	UE_API virtual TArray<UMetasoundEditorGraphMemberNode*> GetNodes() const override;

	UE_API virtual const FText& GetGraphMemberLabel() const override;
	/* ~EndUMetasoundEditorGraphMember interface */

	UE_API const FGuid& GetVariableID() const;
	UE_API const FMetasoundFrontendVariable* GetFrontendVariable() const;

	/**Returns Breadcrumb data of this Variable*/
	const FMetasoundEditorGraphVariableBreadcrumb& GetBreadcrumb() const { return Breadcrumb; };

	UE_DEPRECATED(5.6, "Use MetasoundFrontendDocumentBuilder to edit variables")
	UE_API Metasound::Frontend::FVariableHandle GetVariableHandle();

	UE_API Metasound::Frontend::FConstVariableHandle GetConstVariableHandle() const;

	UE_API virtual bool CanRename() const override;

protected:

	UPROPERTY()
	FMetasoundEditorGraphVariableBreadcrumb Breadcrumb;

private:
	struct FVariableEditorNodes
	{
		UMetasoundEditorGraphMemberNode* MutatorNode = nullptr;
		TArray<UMetasoundEditorGraphMemberNode*> AccessorNodes;
		TArray<UMetasoundEditorGraphMemberNode*> DeferredAccessorNodes;
	};

	struct FVariableNodeLocations
	{

		TOptional<FVector2D> MutatorLocation;
		TArray<FVector2D> AccessorLocations;
		TArray<FVector2D> DeferredAccessorLocations;
	};

	UE_API FVariableEditorNodes GetVariableNodes() const;
	UE_API FVariableNodeLocations GetVariableNodeLocations() const;
	UE_API void AddVariableNodes(UObject& InMetaSound, const FVariableNodeLocations& InNodeLocs);
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraph : public UMetasoundEditorGraphBase
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "Input node creation is no longer supported publically via the MetasoundEditorGraph.")
	UE_API UMetasoundEditorGraphInputNode* CreateInputNode(Metasound::Frontend::FNodeHandle InNodeHandle, bool bInSelectNewNode);

	UE_DEPRECATED(5.6, "Document Handles are actively being deprecated, use the MetaSound Frontend Document Builder API")
	UE_API Metasound::Frontend::FDocumentHandle GetDocumentHandle();

	UE_API Metasound::Frontend::FConstDocumentHandle GetDocumentHandle() const;

	UE_DEPRECATED(5.6, "Document Handles are actively being deprecated, use the MetaSound Frontend Document Builder API")
	UE_API Metasound::Frontend::FGraphHandle GetGraphHandle();

	UE_API Metasound::Frontend::FConstGraphHandle GetGraphHandle() const;

	UE_API virtual void PreSave(FObjectPreSaveContext InSaveContext) override;

	UE_API UMetaSoundBuilderBase& GetBuilderChecked() const;
	UE_API UObject* GetMetasound() const;
	UE_API UObject& GetMetasoundChecked() const;

	UE_API void IterateInputs(TFunctionRef<void(UMetasoundEditorGraphInput&)> InFunction) const;
	UE_API void IterateOutputs(TFunctionRef<void(UMetasoundEditorGraphOutput&)> InFunction) const;
	UE_API void IterateVariables(TFunctionRef<void(UMetasoundEditorGraphVariable&)> InFunction) const;
	UE_API void IterateMembers(TFunctionRef<void(UMetasoundEditorGraphMember&)> InFunction) const;

	UE_API bool ContainsInput(const UMetasoundEditorGraphInput& InInput) const;
	UE_API bool ContainsOutput(const UMetasoundEditorGraphOutput& InOutput) const;
	UE_API bool ContainsVariable(const UMetasoundEditorGraphVariable& InVariable) const;

	UE_API void SetPreviewID(uint32 InPreviewID);
	UE_API bool IsPreviewing() const;
	UE_API bool IsEditable() const;

	// UMetasoundEditorGraphBase Implementation
	UE_API virtual void RegisterGraphWithFrontend(Metasound::Frontend::FMetaSoundAssetRegistrationOptions* RegOptions = nullptr) override;
	UE_API virtual void MigrateEditorDocumentData(FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

private:
	UE_API void ValidateInternal(Metasound::Editor::FGraphValidationResults& OutResults);

	// Preview ID is the Unique ID provided by the UObject that implements
	// a sound's ParameterInterface when a sound begins playing.
	uint32 PreviewID = INDEX_NONE;

	// Used as a means of forcing the graph to rebuild nodes on next tick.
	// TODO: Will no longer require this once all editor metadata is migrated
	// to the frontend & the system can adequately rely on the changeIDs as a
	// mechanism for selectively updating nodes.
	bool bForceRefreshNodes = false;

	UPROPERTY()
	TArray<TObjectPtr<UMetasoundEditorGraphInput>> Inputs;

	UPROPERTY()
	TArray<TObjectPtr<UMetasoundEditorGraphOutput>> Outputs;

	UPROPERTY()
	TArray<TObjectPtr<UMetasoundEditorGraphVariable>> Variables;

public:
	UE_API UMetasoundEditorGraphInput* FindInput(FGuid InNodeID) const;
	UE_API UMetasoundEditorGraphInput* FindInput(FName InName) const;
	UE_API UMetasoundEditorGraphInput* FindOrAddInput(const FGuid& InNodeID);

	UE_DEPRECATED(5.6, "ConstNodeHandle is being deprecated, use overload which is provided NodeID instead")
	UE_API UMetasoundEditorGraphInput* FindOrAddInput(Metasound::Frontend::FConstNodeHandle InNodeHandle);

	UE_API UMetasoundEditorGraphOutput* FindOutput(FGuid InNodeID) const;
	UE_API UMetasoundEditorGraphOutput* FindOutput(FName InName) const;
	UE_API UMetasoundEditorGraphOutput* FindOrAddOutput(const FGuid& InNodeID);

	UE_DEPRECATED(5.6, "ConstNodeHandle is being deprecated, use overload which is provided NodeID instead")
	UE_API UMetasoundEditorGraphOutput* FindOrAddOutput(Metasound::Frontend::FConstNodeHandle InNodeHandle);

	UE_API UMetasoundEditorGraphVariable* FindVariable(const FGuid& InVariableID) const;
	UE_API UMetasoundEditorGraphVariable* FindOrAddVariable(FName VariableName);

	UE_DEPRECATED(5.6, "ConstVariableHandle is being deprecated, use overload which is provided VariableName instead")
	UE_API UMetasoundEditorGraphVariable* FindOrAddVariable(const Metasound::Frontend::FConstVariableHandle& InVariableHandle);

	UE_API UMetasoundEditorGraphMember* FindMember(FGuid InMemberID) const;
	UE_API UMetasoundEditorGraphMember* FindAdjacentMember(const UMetasoundEditorGraphMember& InMember);

private:
	/**
	* Sorts the incoming members by their MemberName and returns next element after sorting. Will not affect the order of the original list.
	* @param InMembers - Members to sort
	* @param InPredicate - Member to search for
	* @return the adjacent member from the sorted members
	*/
	template<typename T>
	TObjectPtr<T> FindAdjacentMemberFromSorted(const TArray<TObjectPtr<T>>& InMembers, TFunction<bool(const TObjectPtr<UMetasoundEditorGraphMember>&)> InPredicate)
	{
		auto SortMembers = [](const UMetasoundEditorGraphMember& InFirstMember, const UMetasoundEditorGraphMember& InSecondMember)
			{
				return InFirstMember.GetMemberName().ToString() < InSecondMember.GetMemberName().ToString();
			};

		TArray<TObjectPtr<T>> MembersCopy = InMembers;
		MembersCopy.Sort(SortMembers);

		int32 IndexInArray = MembersCopy.IndexOfByPredicate(InPredicate);

		if (IndexInArray < (InMembers.Num() - 1))
		{
			return MembersCopy[IndexInArray + 1];
		}
		else if (IndexInArray > 0)
		{
			return MembersCopy[IndexInArray - 1];
		}

		return nullptr;
	}

public:
	UE_DEPRECATED(5.5, "Use the associated Frontend builder and synchronize the graph to remove a given editor member.")
	bool RemoveMember(UMetasoundEditorGraphMember& InGraphMember) { return false; }

	UE_DEPRECATED(5.5, "Use the associated Frontend builder and synchronize the graph to remove all editor member nodes.")
	bool RemoveMemberNodes(UMetasoundEditorGraphMember& InGraphMember) { return false; }

	UE_DEPRECATED(5.5, "Use the associated Frontend builder and synchronize the graph to remove all editor member nodes")
	bool RemoveFrontendMember(UMetasoundEditorGraphMember& InGraphMember) { return false; }

	friend class UMetaSoundFactory;
	friend class UMetaSoundSourceFactory;
	friend class Metasound::Editor::FGraphBuilder;
};

#undef UE_API
