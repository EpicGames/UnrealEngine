// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraphNode_Comment.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "Misc/Guid.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MetasoundEditorGraphCommentNode.generated.h"

struct FPropertyChangedEvent;
class FMetasoundAssetBase;
class UMetaSoundBuilderBase;


// Forward Declarations
namespace Metasound::Editor
{
	class FGraphBuilder;
}


UCLASS(MinimalAPI)
class UMetasoundEditorGraphCommentNode : public UEdGraphNode_Comment
{
	GENERATED_BODY()

public:
	// Convenience function to convert underlying ed-graph properties to a frontend graph comment
	static void ConvertToFrontendComment(const UEdGraphNode_Comment& InEdNode, FMetaSoundFrontendGraphComment& OutComment);

	// Convenience function to convert underlying ed-graph properties from a frontend graph comment
	static void ConvertFromFrontendComment(const FMetaSoundFrontendGraphComment& InComment, UEdGraphNode_Comment& OutEdNode);

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	METASOUNDEDITOR_API virtual bool CanUserDeleteNode() const override;
	METASOUNDEDITOR_API virtual void OnRenameNode(const FString& NewName) override;
	METASOUNDEDITOR_API virtual void ResizeNode(const FVector2f& NewSize) override;
	//~ End UEdGraphNode Interface

	/** Set the Bounds for the comment node */
	METASOUNDEDITOR_API void SetBounds(const class FSlateRect& Rect);

	UMetaSoundBuilderBase& GetBuilderChecked() const;

	// Get/Set frontend comment ID
	FGuid GetCommentID() const;
	void SetCommentID(const FGuid& InGuid);

	// Get this node's MetaSound
	UObject& GetMetasoundChecked() const;

	bool RemoveFromDocument() const;

	// Update the frontend location with this editor node location
	void UpdateFrontendNodeLocation();

private:
	FMetasoundAssetBase& GetAssetChecked();

	UPROPERTY()
	FGuid CommentID;

	friend class Metasound::Editor::FGraphBuilder;
};
