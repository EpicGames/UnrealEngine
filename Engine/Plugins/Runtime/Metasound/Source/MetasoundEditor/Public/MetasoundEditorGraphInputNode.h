// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameterControllerInterface.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphBuilder.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundVertex.h"
#include "Misc/Guid.h"
#include "Textures/SlateIcon.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundEditorGraphInputNode.generated.h"

#define UE_API METASOUNDEDITOR_API

// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;


UCLASS(MinimalAPI)
class UMetasoundEditorGraphInputNode : public UMetasoundEditorGraphMemberNode
{
	GENERATED_BODY()

public:
	virtual ~UMetasoundEditorGraphInputNode() = default;

	UPROPERTY()
	TObjectPtr<UMetasoundEditorGraphInput> Input;

	UPROPERTY()
	FGuid NodeID;

public:
	UE_API const FMetasoundEditorGraphVertexNodeBreadcrumb& GetBreadcrumb() const;

	UE_API virtual void CacheTitle() override;

	UE_API virtual void CacheBreadcrumb() override;
	UE_API virtual UMetasoundEditorGraphMember* GetMember() const override;
	UE_API virtual void ReconstructNode() override;

	UE_API virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const;
	UE_API virtual FMetasoundFrontendClassName GetClassName() const;

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FGuid GetNodeID() const override;
	UE_API virtual FLinearColor GetNodeTitleTextColor() const override;
	UE_API virtual const FSlateBrush* GetNodeTitleIconBrush() const override;
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const override;
	UE_API virtual void Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const override;
	UE_API virtual bool EnableInteractWidgets() const override;
	UE_API virtual FText GetTooltipText() const override;

protected:
	// Breadcrumb used if associated FrontendNode cannot be found or has been unlinked
	UPROPERTY()
	FMetasoundEditorGraphVertexNodeBreadcrumb Breadcrumb;

	friend class Metasound::Editor::FGraphBuilder;
};

#undef UE_API
