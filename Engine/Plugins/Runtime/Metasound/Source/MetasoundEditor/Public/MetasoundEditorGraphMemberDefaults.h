// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Count.h"
#include "Algo/Transform.h"
#include "AudioParameterControllerInterface.h"
#include "AudioWidgetsEnums.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "GraphEditorSettings.h"
#include "MetasoundDataReference.h"
#include "MetasoundEditorGraph.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorSettings.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSettings.h"
#include "MetasoundVertex.h"
#include "Misc/Guid.h"
#include "SAudioRadialSlider.h"
#include "Sound/SoundWave.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include "MetasoundEditorGraphMemberDefaults.generated.h"


// Editor-only page default for more desirable customization behavior in representing
// Frontend Literal value. Should never be serialized as generation is non-deterministic.
USTRUCT()
struct FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefault() = default;
	FMetasoundEditorMemberPageDefault(const FGuid& InPageID)
		: PageID(InPageID)
	{
	}

	// Unique ID used to determine if this default is a newly created and uninitialized member
	// of a given collection.  ID is transient per application cycle, so non-deterministic
	// between editor sessions.
	static const FGuid& GetNewEntryID();

	// Selectable PageName
	UPROPERTY(VisibleAnywhere, Category = DefaultValue)
	FName PageName;

	// Used for hash and mirrors document-stored value. Defaults to random value to allow for assignment in
	// post-edit change. Allows for name collisions if user is amidst renaming or rebasing values
	UPROPERTY(Transient, VisibleAnywhere, Category = DefaultValue)
	FGuid PageID = GetNewEntryID();
};


// For bool input widget
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetasoundBoolStateChangedEvent, bool, const FGuid& /* PageID */);

// Broken out to be able to customize and swap enum behavior for boolean literal behavior (ex. for triggers)
USTRUCT()
struct FMetasoundEditorGraphMemberDefaultBoolRef
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	bool Value = false;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultBool : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultBool() = default;
	FMetasoundEditorMemberPageDefaultBool(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	FMetasoundEditorGraphMemberDefaultBoolRef Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultBool : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultBool> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultBool() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	FMetasoundEditorGraphMemberDefaultBoolRef Default;

	UPROPERTY(EditAnywhere, Category = Widget, meta = (DisplayName = "Widget"))
	EMetasoundBoolMemberDefaultWidget WidgetType = EMetasoundBoolMemberDefaultWidget::None;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

	UE_DEPRECATED(5.5, "Use SetFromLiteral instead and broadcast state change delegate explicitly where desired")
	void SetDefault(const bool InDefault) { }

	FOnMetasoundBoolStateChangedEvent OnDefaultStateChanged;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultBoolArray : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultBoolArray() = default;
	FMetasoundEditorMemberPageDefaultBoolArray(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TArray<FMetasoundEditorGraphMemberDefaultBoolRef> Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultBoolArray : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultBoolArray> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultBoolArray() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	TArray<FMetasoundEditorGraphMemberDefaultBoolRef> Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


// Broken out to be able to customize and swap enum behavior for basic integer literal behavior
USTRUCT()
struct FMetasoundEditorGraphMemberDefaultIntRef
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	int32 Value = 0;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultInt : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultInt() = default;
	FMetasoundEditorMemberPageDefaultInt(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue)
	FMetasoundEditorGraphMemberDefaultIntRef Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultInt : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultInt> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultInt() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	FMetasoundEditorGraphMemberDefaultIntRef Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultIntArray : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultIntArray() = default;
	FMetasoundEditorMemberPageDefaultIntArray(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TArray<FMetasoundEditorGraphMemberDefaultIntRef> Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultIntArray : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultIntArray> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultIntArray() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	TArray<FMetasoundEditorGraphMemberDefaultIntRef> Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


// For input widget
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetasoundInputValueChangedEvent, FGuid /* PageID */, float /* Value */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetasoundRangeChangedEvent, FVector2D);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetasoundInputClampDefaultChangedEvent, bool);


UENUM()
enum class UE_DEPRECATED(5.5, "EMetasoundMemberDefaultWidgetValueType is deprecated, use EAudioUnitsValueType instead") EMetasoundMemberDefaultWidgetValueType : uint8
{
	Linear,
	Frequency UMETA(DisplayName = "Frequency (Log)"),
	Volume
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultFloat : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultFloat() = default;
	FMetasoundEditorMemberPageDefaultFloat(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	float Value = 0.0f;
};


UCLASS(MinimalAPI, BlueprintType)
class UMetasoundEditorGraphMemberDefaultFloat : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	float Default = 0.f;

	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultFloat> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultFloat() = default;

	UPROPERTY(EditAnywhere, Category = DefaultValue)
	bool ClampDefault = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DefaultValue)
	FVector2D Range = FVector2D(0.0f, 1.0f);

	UPROPERTY(EditAnywhere, Category = Widget, meta=(DisplayName = "Widget"))
	EMetasoundMemberDefaultWidget WidgetType = EMetasoundMemberDefaultWidget::None;
	
	UPROPERTY(EditAnywhere, Category = Widget, meta=(DisplayName = "Orientation", EditCondition = "WidgetType == EMetasoundMemberDefaultWidget::Slider", EditConditionHides))
	TEnumAsByte<EOrientation> WidgetOrientation = EOrientation::Orient_Horizontal;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "WidgetValueType has been deprecated. Use WidgetUnitValueType instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "WidgetValueType has been deprecated. Use WidgetUnitValueType instead."))
	EMetasoundMemberDefaultWidgetValueType WidgetValueType_DEPRECATED = EMetasoundMemberDefaultWidgetValueType::Linear;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif//WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = Widget, meta = (DisplayName = "Value Type", EditCondition = "WidgetType != EMetasoundMemberDefaultWidget::None", EditConditionHides))
	EAudioUnitsValueType WidgetUnitValueType = EAudioUnitsValueType::Linear;

	/** If true, output linear value. Otherwise, output dB value. The volume widget itself will always display the value in dB. The Default Value and Range are linear. */
	UPROPERTY(EditAnywhere, Category = Widget, meta = (DisplayName = "Output Linear"))
	bool VolumeWidgetUseLinearOutput = true;

	/** Range in decibels. This will be converted to the linear range in the Default Value category. */
	UPROPERTY(EditAnywhere, Category = Widget, meta = (DisplayName = "Range in dB", EditCondition = "VolumeWidgetUseLinearOutput", EditConditionHides))
	FVector2D VolumeWidgetDecibelRange = FVector2D(SAudioVolumeRadialSlider::MinDbValue, 0.0f);

	FOnMetasoundInputValueChangedEvent OnDefaultValueChanged;
	FOnMetasoundRangeChangedEvent OnRangeChanged;
	FOnMetasoundInputClampDefaultChangedEvent OnClampChanged;

	virtual void ForceRefresh() override;
	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void Initialize() override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;

#if WITH_EDITORONLY_DATA
	virtual bool MigrateProperties() override;
#endif // WITH_EDITORONLY_DATA

	// Set Range to reasonable limit given current Default value
	void SetInitialRange();

	FVector2D GetRange() const;
	void SetRange(const FVector2D InRange);

protected:
	void ClampDefaults();
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultFloatArray : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultFloatArray() = default;
	FMetasoundEditorMemberPageDefaultFloatArray(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TArray<float> Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultFloatArray : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultFloatArray> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultFloatArray() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	TArray<float> Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultString : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultString() = default;
	FMetasoundEditorMemberPageDefaultString(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	FString Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultString : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

public:
	virtual ~UMetasoundEditorGraphMemberDefaultString() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	FString Default;

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultString> Defaults;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};


USTRUCT()
struct FMetasoundEditorMemberPageDefaultStringArray : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultStringArray() = default;
	FMetasoundEditorMemberPageDefaultStringArray(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TArray<FString> Value;
};


UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultStringArray : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultStringArray> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultStringArray() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	TArray<FString> Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};

// Broken out to be able to customize and swap AllowedClass based on provided object proxy
USTRUCT()
struct FMetasoundEditorGraphMemberDefaultObjectRef
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TObjectPtr<UObject> Object = nullptr;
};

USTRUCT()
struct FMetasoundEditorMemberPageDefaultObjectRef : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultObjectRef() = default;
	FMetasoundEditorMemberPageDefaultObjectRef(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue)
	FMetasoundEditorGraphMemberDefaultObjectRef Value;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultObject : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

public:
	virtual ~UMetasoundEditorGraphMemberDefaultObject() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	FMetasoundEditorGraphMemberDefaultObjectRef Default;

	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultObjectRef> Defaults;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;

protected:
	virtual void ResolvePageDefaults() override;
	virtual void SortPageDefaults() override;
};

USTRUCT()
struct FMetasoundEditorMemberPageDefaultObjectArray : public FMetasoundEditorMemberPageDefault
{
	GENERATED_BODY()

	FMetasoundEditorMemberPageDefaultObjectArray() = default;
	FMetasoundEditorMemberPageDefaultObjectArray(const FGuid& InPageID)
		: FMetasoundEditorMemberPageDefault(InPageID)
	{
	}

	UPROPERTY(EditAnywhere, Category = DefaultValue, meta = (DisplayName = "Default"))
	TArray<FMetasoundEditorGraphMemberDefaultObjectRef> Value;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphMemberDefaultObjectArray : public UMetasoundEditorGraphMemberDefaultLiteral
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Transient, Category = DefaultValue, meta = (NoResetToDefault, NoElementDuplicate, EditFixedOrder))
	TArray<FMetasoundEditorMemberPageDefaultObjectArray> Defaults;

public:
	virtual ~UMetasoundEditorGraphMemberDefaultObjectArray() = default;

	UPROPERTY(meta = (Deprecated = 5.5, DeprecationMessage = "Default is no longer serialized and is privately managed to support per-page default values"))
	TArray<FMetasoundEditorGraphMemberDefaultObjectRef> Default;

	virtual EMetasoundFrontendLiteralType GetLiteralType() const override;
	virtual void InitDefault(const FGuid& InPageID = Metasound::Frontend::DefaultPageID) override;
	virtual void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral)> Iter) const override;
	virtual bool RemoveDefault(const FGuid& InPageID) override;
	virtual void ResetDefaults() override;
	virtual void ResolvePageDefaults() override;
	virtual void SetFromLiteral(const FMetasoundFrontendLiteral& InLiteral, const FGuid& InPageID) override;
	virtual void SortPageDefaults() override;
	virtual bool Synchronize() override;
	virtual bool TryFindDefault(FMetasoundFrontendLiteral& OutLiteral, const FGuid* InPageID = nullptr) const override;
	virtual void UpdatePreviewInstance(const Metasound::FVertexName& InParameterName, TScriptInterface<IAudioParameterControllerInterface>& InParameterInterface) const override;
};