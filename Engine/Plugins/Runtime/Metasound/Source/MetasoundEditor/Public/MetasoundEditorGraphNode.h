// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "Misc/Guid.h"
#include "Sound/SoundWave.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include "MetasoundEditorGraphNode.generated.h"

// Forward Declarations
class UEdGraphPin;
class UMetaSoundBuilderBase;
class UMetaSoundPatch;
class UMetasoundEditorGraphOutput;
class UMetasoundEditorGraphMember;
class UMetasoundEditorGraphVariable;
class UMetasoundEditorGraphMemberDefaultFloat;

struct FMetaSoundFrontendDocumentBuilder;

namespace EMessageSeverity { enum Type : int32; }

namespace Metasound::Editor
{
	struct FDocumentClipboardUtils;
	struct FGraphNodeValidationResult;

	class FGraphBuilder;

	// Map of class names to sorted array of registered version numbers
	using FSortedClassVersionMap = TMap<FName, TArray<FMetasoundFrontendVersionNumber>>;
} // namespace Metasound::Editor


USTRUCT()
struct FMetasoundEditorGraphNodeBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	FMetasoundFrontendClassName ClassName;

	UPROPERTY()
	bool bIsClassNative = true;

	UPROPERTY()
	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NodeConfiguration;

	// For use with template nodes only
	UPROPERTY()
	TOptional<FNodeTemplateGenerateInterfaceParams> TemplateParams;

	UPROPERTY()
	FMetasoundFrontendVersionNumber Version;
};

USTRUCT()
struct FMetasoundEditorGraphMemberNodeBreadcrumb : public FMetasoundEditorGraphNodeBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	FName MemberName;

	UPROPERTY()
	FName DataType;

	UPROPERTY()
	TMap<FGuid, FMetasoundFrontendLiteral> DefaultLiterals;
	
	UPROPERTY()
	FMetasoundFrontendVertexMetadata VertexMetadata;

	UPROPERTY()
	TOptional<FSoftObjectPath> MemberMetadataPath;
};

USTRUCT()
struct FMetasoundEditorGraphVertexNodeBreadcrumb : public FMetasoundEditorGraphMemberNodeBreadcrumb
{
	GENERATED_BODY()

	UPROPERTY()
	EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Unset;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	/** Create a new input pin for this node */
	METASOUNDEDITOR_API void CreateInputPin();

	/** Estimate the width of this Node from the length of its title */
	METASOUNDEDITOR_API int32 EstimateNodeWidth() const;

	METASOUNDEDITOR_API void IteratePins(TUniqueFunction<void(UEdGraphPin& /* Pin */, int32 /* Index */)> Func, EEdGraphPinDirection InPinDirection = EGPD_MAX);

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual bool CanUserDeleteNode() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual FString GetDocumentationLink() const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FText GetTooltipText() const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void ReconstructNode() override;
	virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	virtual void OnUpdateCommentText(const FString& NewComment) override;
	// End of UEdGraphNode interface

	// UObject interface
	virtual void PreSave(FObjectPreSaveContext InSaveContext) override;

	virtual void PostLoad() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& InEvent) override;
	virtual void PostEditUndo() override;

	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	// End of UObject interface

	virtual bool ShouldShowTitleIcon() const { return true; }
	virtual const FSlateBrush* GetNodeTitleIconBrush() const;
	virtual FLinearColor GetNodeTitleIconColor() const { return FLinearColor::White; }
	virtual FText GetNodeTitleIconTooltipText() const { return FText(); }
	virtual FName GetCornerIcon() const { return FName(); }
	virtual bool CanAddInputPin() const { return false; }

	METASOUNDEDITOR_API UMetaSoundBuilderBase& GetBuilderChecked() const;

	// Returns document's cached Frontend Node Class (as defined by the
	// document's dependency array). If node or class is not found on
	// document, returns null.
	const FMetasoundFrontendClass* GetFrontendClass() const;

	// Returns document's cached Frontend Node.
	// If node is not found on document, returns null.
	const FMetasoundFrontendNode* GetFrontendNode() const;
	const FMetasoundFrontendNode& GetFrontendNodeChecked() const;

	virtual const FMetasoundEditorGraphNodeBreadcrumb& GetBreadcrumb() const;

	// Caches any "breadcrumb" data associated with a particular MetaSound Editor node. Called
	// when copying edgraph data to the clipboard or validating for fast access. Also generally
	// provides a mechanism for MetaSound nodes to cache Frontend data for use to look-up Frontend
	// data if re-associated should the associated document data/node becomes unlinked.
	virtual void CacheBreadcrumb() { }

	UObject* GetMetasound() const;
	UObject& GetMetasoundChecked() const;

	const FMetasoundFrontendClass* GetNodeFrontendClass() const;
	const UClass* GetNodeAssetClass() const;

	virtual bool RemoveFromDocument() const;

	UE_DEPRECATED(5.4, "Use UpdateFrontendNodeLocation and/or SyncLocationFromFrontendNode")
	void SetNodeLocation(const FVector2D& InLocation);

	// Finds the associated node with the given ID and sets this EdGraphNode's comment and comment visibility boolean.
	void SyncCommentFromFrontendNode();

	// Finds the associated node with the given ID and sets this EdGraphNode's location.
	// Returns whether or not node ID entry exists and if location was set.
	bool SyncLocationFromFrontendNode(bool bUpdateEditorNodeID = false);

	// Helper function that sets the associated frontend node's location. Does NOT set this EdGraphNode's location.
	void UpdateFrontendNodeLocation(const FVector2D& InLocation);

	Metasound::Frontend::FGraphHandle GetRootGraphHandle() const;
	Metasound::Frontend::FConstGraphHandle GetConstRootGraphHandle() const;

	UE_DEPRECATED(5.6, "Node Handles are actively being deprecated, use the MetaSound Frontend Document Builder API")
	Metasound::Frontend::FNodeHandle GetNodeHandle() const;

	Metasound::Frontend::FConstNodeHandle GetConstNodeHandle() const;
	Metasound::Frontend::FDataTypeRegistryInfo GetPinDataTypeInfo(const UEdGraphPin& InPin) const;

	TSet<FString> GetDisallowedPinClassNames(const UEdGraphPin& InPin) const;

	virtual FGuid GetNodeID() const { return FGuid(); }
	virtual FText GetDisplayName() const;
	virtual void CacheTitle();

	UE_DEPRECATED(5.7, "Use const version that is passed builder")
	virtual void Validate(Metasound::Editor::FGraphNodeValidationResult& OutResult) { }

	virtual void Validate(const FMetaSoundFrontendDocumentBuilder& Builder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const;

	// Mark node for refresh
	void SyncChangeIDs();

	FText GetCachedTitle() const { return CachedTitle; }

	// Returns whether or not the class interface, metadata, or style has been changed since the last node refresh
	bool ContainsClassChange() const;

	// Graph node visualization widgets can attempt to get the current value of the given named input pin. For connected input pins, a value may not be returned unless sound preview is active.
	METASOUNDEDITOR_API bool TryGetPinVisualizationValue(FName InPinName, bool& OutValue) const;
	METASOUNDEDITOR_API bool TryGetPinVisualizationValue(FName InPinName, int32& OutValue) const;
	METASOUNDEDITOR_API bool TryGetPinVisualizationValue(FName InPinName, float& OutValue) const;

	template<class T>
	inline TOptional<T> GetPinVisualizationValue(FName InPinName) const
	{
		if constexpr (std::is_enum_v<T>)
		{
			int32 Result;
			if (TryGetPinVisualizationValue(InPinName, Result))
			{
				return static_cast<T>(Result);
			}
			return NullOpt;
		}
		else
		{
			T Result;
			if (TryGetPinVisualizationValue(InPinName, Result))
			{
				return Result;
			}
			return NullOpt;
		}
	}
	
protected:
	FGuid InterfaceChangeID;
	FGuid MetadataChangeID;
	FGuid StyleChangeID;

	// Not be serialized to avoid text desync as the registry can provide
	// a new name if the external definition changes between application sessions.
	FText CachedTitle;

	static bool ShowNodeDebugData();

	UE_DEPRECATED(5.4, "Now set directly on implementing nodes")
	virtual void SetNodeID(FGuid InNodeID) { }
};

/** Node that represents a graph member */
UCLASS(Abstract, MinimalAPI)
class UMetasoundEditorGraphMemberNode : public UMetasoundEditorGraphNode
{
	GENERATED_BODY()

public:
	// Whether or not the member node supports interactivity on the graph node
	virtual UMetasoundEditorGraphMember* GetMember() const PURE_VIRTUAL(UMetasoundEditorGraphMemberNode::GetMember, return nullptr;)

	// Whether or not the member node supports interact widgets on the visual node (ex. float manipulation widgets)
	virtual bool EnableInteractWidgets() const
	{
		return true;
	}

	// Clamp float literal value based on the given default float literal. 
	// Returns whether the literal was clamped. 
	static bool ClampFloatLiteral(const UMetasoundEditorGraphMemberDefaultFloat* DefaultFloatLiteral, FMetasoundFrontendLiteral& LiteralValue);

protected:
	virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;

	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FLinearColor GetNodeTitleIconColor() const override;
};

/** Node that represents a graph output */
UCLASS(MinimalAPI)
class UMetasoundEditorGraphOutputNode : public UMetasoundEditorGraphMemberNode
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UMetasoundEditorGraphOutput> Output;

	const FMetasoundEditorGraphVertexNodeBreadcrumb& GetBreadcrumb() const;

	virtual void CacheBreadcrumb() override;	

	virtual FGuid GetNodeID() const override;
	virtual UMetasoundEditorGraphMember* GetMember() const override;


	// Disallow deleting outputs as they require being connected to some
	// part of the graph by the Frontend Graph Builder (which is enforced
	// even when the Editor Graph Node does not have a visible input by
	// way of a literal input.
	virtual bool CanUserDeleteNode() const override;

	virtual void PinDefaultValueChanged(UEdGraphPin* InPin) override;
	virtual void ReconstructNode() override;
	virtual bool RemoveFromDocument() const override;

	// Disables interact widgets (ex. sliders, knobs) when input is connected
	virtual bool EnableInteractWidgets() const override;

	virtual void Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const override;

protected:
	// Breadcrumb used if associated FrontendNode cannot be found or has been unlinked
	UPROPERTY()
	FMetasoundEditorGraphVertexNodeBreadcrumb Breadcrumb;

	virtual FLinearColor GetNodeTitleTextColor() const override;
	virtual const FSlateBrush* GetNodeTitleIconBrush() const override;
	virtual void SetNodeID(FGuid InNodeID) override;

	// Friended to enable mutation of Frontend NodeID & direct breadcrumb access
	friend struct Metasound::Editor::FDocumentClipboardUtils;
	friend class Metasound::Editor::FGraphBuilder;
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphExternalNode : public UMetasoundEditorGraphNode
{
	GENERATED_BODY()

protected:
	UPROPERTY()
	FMetasoundEditorGraphNodeBreadcrumb Breadcrumb;

	UPROPERTY(meta = (Deprecated = "5.4", DeprecationMessage = "Use Breadcrumb value when manipulating clipboard data or validating. Otherwise, look-up frontend node's associated class directly"))
	FMetasoundFrontendClassName ClassName;

	UPROPERTY()
	FGuid NodeID;

	UPROPERTY(meta = (Deprecated = "5.4", DeprecationMessage = "Use Breadcrumb value when manipulating clipboard data or validating, Otherwise, look-up frontend node's associated class directly"))
	bool bIsClassNative = true;

public:
	virtual const FMetasoundEditorGraphNodeBreadcrumb& GetBreadcrumb() const override;
	virtual FGuid GetNodeID() const override { return NodeID; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual const FSlateBrush* GetNodeTitleIconBrush() const override;
	virtual FText GetNodeTitleIconTooltipText() const override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;

	virtual void ReconstructNode() override;
	virtual void CacheBreadcrumb() override;
	virtual void CacheTitle() override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const override;

	FMetasoundFrontendVersionNumber FindHighestVersionInRegistry() const;
	bool CanAutoUpdate() const;

	// Validates node and returns whether or not the node is valid.
	virtual void Validate(const FMetaSoundFrontendDocumentBuilder& InBuilder, Metasound::Editor::FGraphNodeValidationResult& OutResult) const override;

	/**Set Unconnected Pins hidden/Unhidden*/
	void HideUnconnectedPins(const bool InHidePins);

protected:
	// Friended to enable mutation of Frontend NodeID & direct breadcrumb access
	friend struct Metasound::Editor::FDocumentClipboardUtils;
	friend class Metasound::Editor::FGraphBuilder;
};

/** Represents any of the several variable node types (Accessor, DeferredAccessor, Mutator). */
UCLASS(MinimalAPI)
class UMetasoundEditorGraphVariableNode : public UMetasoundEditorGraphMemberNode
{
	GENERATED_BODY()

protected:
	// Class type of the frontend node (Accessor, DeferredAccessor or Mutator)
	UPROPERTY()
	EMetasoundFrontendClassType ClassType;

	// Class name of the frontend node.
	UPROPERTY()
	FMetasoundFrontendClassName ClassName;

	// ID of the frontend node.
	UPROPERTY()
	FGuid NodeID;

public:
	// Associated graph variable.
	UPROPERTY()
	TObjectPtr<UMetasoundEditorGraphVariable> Variable;

	// Variables do not have titles to distinguish more visually from vertex types
	virtual void CacheTitle() override { }

	virtual const FMetasoundEditorGraphNodeBreadcrumb& GetBreadcrumb() const override;

	virtual void CacheBreadcrumb() override;
	virtual UMetasoundEditorGraphMember* GetMember() const override;
	virtual bool EnableInteractWidgets() const override;

	virtual FGuid GetNodeID() const override;
	virtual FName GetCornerIcon() const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& OutHoverText) const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;

	virtual EMetasoundFrontendClassType GetClassType() const;

protected:
	// Breadcrumb used if associated FrontendNode cannot be found or has been unlinked
	UPROPERTY()
	FMetasoundEditorGraphMemberNodeBreadcrumb Breadcrumb;

	virtual FLinearColor GetNodeTitleColor() const override;
	virtual const FSlateBrush* GetNodeTitleIconBrush() const override;
	virtual void SetNodeID(FGuid InNodeID) override;

	// Friended to enable mutation of Frontend NodeID & direct breadcrumb access
	friend struct Metasound::Editor::FDocumentClipboardUtils;
	friend class Metasound::Editor::FGraphBuilder;
};
