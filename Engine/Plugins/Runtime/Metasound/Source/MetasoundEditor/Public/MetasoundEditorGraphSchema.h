// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "ConnectionDrawingPolicy.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphUtilities.h"
#include "IMetasoundEditor.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendNodesCategories.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"

#include "MetasoundEditorGraphSchema.generated.h"

#define UE_API METASOUNDEDITOR_API


// Forward Declarations
class IAssetReferenceFilter;
class UEdGraph;
class UEdGraphNode;
class UMetaSoundPatch;
class UMetasoundEditorGraphNode;
class UMetasoundEditorGraph;


namespace Metasound::Editor
{
	// Ordered such that highest priority is highest value
	enum class EPrimaryContextGroup : uint8
	{
		Conversions = 0,
		Graphs,
		Functions,
		Outputs,
		Inputs,
		Variables,
		Favorites,

		// This MUST be the last one before Common, because the space between them is reserved for Recent actions ordered by most recent.
		Recents, 
		Common = 255
	};

	const FText& GetContextGroupDisplayName(EPrimaryContextGroup InContextGroup);

	class ISchemaQueryResult
	{
	public:
		virtual ~ISchemaQueryResult() = default;

		virtual bool CanConnectInputOfTypeAndAccess(FName TypeName, EMetasoundFrontendVertexAccessType AccessType) const = 0;
		virtual bool CanConnectOutputOfTypeAndAccess(FName TypeName, EMetasoundFrontendVertexAccessType AccessType) const = 0;

		virtual FMetasoundFrontendClassMetadata GenerateClassMetadata() const = 0;
		virtual EMetasoundFrontendClassAccessFlags GetAccessFlags() const = 0;
		virtual FMetaSoundAssetKey GetAssetKey() const = 0;
		virtual const TArray<FText>& GetCategoryHierarchy() const = 0;
		virtual FText GetDisplayName() const = 0;
		virtual const TArray<FText>& GetKeywords() const = 0;
		virtual EMetasoundFrontendClassType GetRegistryClassType() const = 0;
		virtual FText GetTooltip() const = 0;

		virtual bool IsNative() const = 0;
	};

	using FInterfaceNodeFilterFunction = TFunction<bool(Frontend::FConstNodeHandle)>;

	struct FActionVertexFilters
	{
		FName InputTypeName = { };
		EMetasoundFrontendVertexAccessType InputAccessType = EMetasoundFrontendVertexAccessType::Unset;

		bool HasInputFilters() const
		{
			return !InputTypeName.IsNone() && InputAccessType != EMetasoundFrontendVertexAccessType::Unset;
		}

		FName OutputTypeName = { };
		EMetasoundFrontendVertexAccessType OutputAccessType = EMetasoundFrontendVertexAccessType::Unset;

		bool HasOutputFilters() const
		{
			return !OutputTypeName.IsNone() && OutputAccessType != EMetasoundFrontendVertexAccessType::Unset;
		}
	};

	namespace SchemaUtils
	{
		UEdGraphNode* PromoteToInput(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode, TOptional<FMetasoundFrontendLiteral> Literal = {});
		UEdGraphNode* PromoteToOutput(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode);
		UEdGraphNode* PromoteToVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode);
		UEdGraphNode* PromoteToDeferredVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode);
		UEdGraphNode* PromoteToMutatorVariable(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& InLocation, bool bSelectNewNode);

		const FName MetasoundGraphSchemaActionType("FMetasoundGraphSchemaAction");
	} // namespace SchemaUtils
} // namespace Metasound::Editor


USTRUCT()
struct FMetasoundGraphSchemaAction : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId() { return Metasound::Editor::SchemaUtils::MetasoundGraphSchemaActionType; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FMetasoundGraphSchemaAction()
		: FEdGraphSchemaAction()
	{
	}

	FMetasoundGraphSchemaAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup, FText InKeywords = FText::GetEmpty())
		: FEdGraphSchemaAction(
			MoveTemp(InNodeCategory),
			MoveTemp(InMenuDesc),
			MoveTemp(InToolTip),
			static_cast<int32>(InGroup),
			MoveTemp(InKeywords))
	{
	}

	virtual ~FMetasoundGraphSchemaAction() = default;

	virtual const FSlateBrush* GetIconBrush() const
	{
		return FAppStyle::GetBrush("NoBrush");
	}

	virtual const FLinearColor& GetIconColor() const
	{
		static const FLinearColor DefaultColor = FLinearColor::Black;
		return DefaultColor;
	}

	// Used to store & match Recent and Favorite MetaSounds graph actions in the MetaSounds editor
	virtual FString GetUniqueStableFavoriteKey() const
	{
		// Unlikely: If the user changes the editor locale, favorites and recents could technically be invalidated if their display names are localized.
		return GetMenuDescription().ToString();
	}
};

/** This is used to combine functionality for nodes that can have multiple outputs and should never be directly instantiated. */
USTRUCT()
struct FMetasoundGraphSchemaAction_NodeWithMultipleOutputs : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NodeWithMultipleOutputs()
		: FMetasoundGraphSchemaAction()
	{}

	FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(FText InNodeCategory, FText InMenuDesc, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup, FText InKeywords = FText::GetEmpty())
		: FMetasoundGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGroup, InKeywords)
	{}

	virtual ~FMetasoundGraphSchemaAction_NodeWithMultipleOutputs() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) { return nullptr; }
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add an input node of an existing graph input to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewInput : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	UPROPERTY()
	FGuid NodeID;

	FMetasoundGraphSchemaAction_NewInput()
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs()
	{}

	UE_API FMetasoundGraphSchemaAction_NewInput(FText InNodeCategory, FText InDisplayName, FGuid InInputNodeID, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup);

	virtual ~FMetasoundGraphSchemaAction_NewInput() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;

	virtual FString GetUniqueStableFavoriteKey() const override
	{
		return NodeID.ToString();
	}
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a new graph member and member node to the graph, with the given default literal */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewGraphMember : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewGraphMember()
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs()
	{}

	UE_API FMetasoundGraphSchemaAction_NewGraphMember(FName InDataType, FMetasoundFrontendLiteral InLiteral, FText InNodeCategory, FText InMenuDesc, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(InNodeCategory, InMenuDesc, InToolTip, InGroup)
		, DataType(InDataType)
		, DefaultLiteral(InLiteral)
	{}

	virtual ~FMetasoundGraphSchemaAction_NewGraphMember() = default;

protected: 
	FName DataType;
	FMetasoundFrontendLiteral DefaultLiteral;
};

/** Action to add a new graph input and input node to the graph, with the given default literal */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewGraphInput : public FMetasoundGraphSchemaAction_NewGraphMember
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewGraphInput()
		: FMetasoundGraphSchemaAction_NewGraphMember()
	{}

	UE_API FMetasoundGraphSchemaAction_NewGraphInput(FName DataType, FMetasoundFrontendLiteral InLiteral);

	virtual ~FMetasoundGraphSchemaAction_NewGraphInput() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a new graph output and output node to the graph, with the given default literal */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewGraphOutput : public FMetasoundGraphSchemaAction_NewGraphMember
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewGraphOutput()
		: FMetasoundGraphSchemaAction_NewGraphMember()
	{}

	UE_API FMetasoundGraphSchemaAction_NewGraphOutput(FName DataType, FMetasoundFrontendLiteral InLiteral);

	virtual ~FMetasoundGraphSchemaAction_NewGraphOutput() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a new graph variable and getter node to the graph, with the given default literal */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewGraphVariable : public FMetasoundGraphSchemaAction_NewGraphMember
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewGraphVariable()
		: FMetasoundGraphSchemaAction_NewGraphMember()
	{}

	UE_API FMetasoundGraphSchemaAction_NewGraphVariable(FName DataType, FMetasoundFrontendLiteral InLiteral);

	virtual ~FMetasoundGraphSchemaAction_NewGraphVariable() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Promotes an input to a graph input, using its respective literal value as the default value */
USTRUCT()
struct FMetasoundGraphSchemaAction_PromoteToInput : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_PromoteToInput();

	virtual ~FMetasoundGraphSchemaAction_PromoteToInput() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Adds an output to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewOutput : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	UPROPERTY()
	FGuid NodeID;

	FMetasoundGraphSchemaAction_NewOutput()
		: FMetasoundGraphSchemaAction()
	{}

	UE_API FMetasoundGraphSchemaAction_NewOutput(FText InNodeCategory, FText InDisplayName, FGuid InOutputNodeID, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup);

	virtual ~FMetasoundGraphSchemaAction_NewOutput() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;

	virtual FString GetUniqueStableFavoriteKey() const override
	{
		return NodeID.ToString();
	}
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Promotes a node output to a graph output */
USTRUCT()
struct FMetasoundGraphSchemaAction_PromoteToOutput : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_PromoteToOutput();

	virtual ~FMetasoundGraphSchemaAction_PromoteToOutput() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Adds a variable node to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewVariableNode : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	UPROPERTY()
	FGuid VariableID;

	FMetasoundGraphSchemaAction_NewVariableNode()
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs()
	{}

	UE_API FMetasoundGraphSchemaAction_NewVariableNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip);

	virtual ~FMetasoundGraphSchemaAction_NewVariableNode() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;

	virtual FString GetUniqueStableFavoriteKey() const override
	{
		return VariableID.ToString();
	}
	//~ End FMetasoundGraphSchemaAction Interface
	
protected:
	UE_DEPRECATED(5.6, "Deprecated in favor of direct creation via Document Builder API")
	virtual Metasound::Frontend::FNodeHandle CreateFrontendVariableNode(const Metasound::Frontend::FGraphHandle& InFrontendGraph, const FGuid& InVariableID) const
	{
		return Metasound::Frontend::INodeController::GetInvalidHandle();
	}

	virtual const FMetasoundFrontendNode* CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
	{
		return nullptr;
	}
};

/** Adds a variable node to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewVariableAccessorNode : public FMetasoundGraphSchemaAction_NewVariableNode
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewVariableAccessorNode()
		: FMetasoundGraphSchemaAction_NewVariableNode()
	{}

	UE_API FMetasoundGraphSchemaAction_NewVariableAccessorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip);

	virtual ~FMetasoundGraphSchemaAction_NewVariableAccessorNode() = default;

protected:
	UE_API virtual const FMetasoundFrontendNode* CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const override;

};

/** Adds a variable node to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode : public FMetasoundGraphSchemaAction_NewVariableNode
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode()
		: FMetasoundGraphSchemaAction_NewVariableNode()
	{}

	virtual ~FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode() = default;

	UE_API FMetasoundGraphSchemaAction_NewVariableDeferredAccessorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip);
	
protected:
	UE_API virtual const FMetasoundFrontendNode* CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const override;
};

/** Adds a variable node to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewVariableMutatorNode : public FMetasoundGraphSchemaAction_NewVariableNode
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewVariableMutatorNode()
		: FMetasoundGraphSchemaAction_NewVariableNode()
	{}

	virtual ~FMetasoundGraphSchemaAction_NewVariableMutatorNode() = default;

	UE_API FMetasoundGraphSchemaAction_NewVariableMutatorNode(FText InNodeCategory, FText InDisplayName, FGuid InVariableID, FText InToolTip);

protected:
	UE_API virtual const FMetasoundFrontendNode* CreateFrontendVariableNode(FMetaSoundFrontendDocumentBuilder& DocBuilder) const override;
};

/** Promotes an input to a graph variable & respective getter node, using its respective literal value as the default value */
USTRUCT()
struct FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode();

	virtual ~FMetasoundGraphSchemaAction_PromoteToVariable_AccessorNode() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Promotes an output to a graph variable & respective setter node, using its respective literal value as the default value */
USTRUCT()
struct FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode();

	virtual ~FMetasoundGraphSchemaAction_PromoteToVariable_MutatorNode() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Promotes an input to a graph variable & respective deferred getter node, using its respective literal value as the default value */
USTRUCT()
struct FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode();

	virtual ~FMetasoundGraphSchemaAction_PromoteToVariable_DeferredAccessorNode() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};


/** Action to add a node to the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewNode : public FMetasoundGraphSchemaAction_NodeWithMultipleOutputs
{
	GENERATED_BODY();

	TSharedPtr<Metasound::Editor::ISchemaQueryResult> QueryResult;

	FMetasoundGraphSchemaAction_NewNode() 
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs()
	{}

	FMetasoundGraphSchemaAction_NewNode(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup, FText InKeywords = FText::GetEmpty(), TOptional<Metasound::Editor::FDroppedObjectsInfo> InDroppedObjectsInfo = {})
		: FMetasoundGraphSchemaAction_NodeWithMultipleOutputs(InNodeCategory, InMenuDesc, InToolTip, InGroup, InKeywords)
		, DroppedObjectsInfo(InDroppedObjectsInfo)
	{}

	virtual ~FMetasoundGraphSchemaAction_NewNode() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;

	UE_API virtual FString GetUniqueStableFavoriteKey() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
private: 
	TOptional<Metasound::Editor::FDroppedObjectsInfo> DroppedObjectsInfo;
};

/** Action to add nodes to the graph based on selected objects*/
USTRUCT()
struct FMetasoundGraphSchemaAction_NewFromSelected : public FMetasoundGraphSchemaAction_NewNode
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewFromSelected() 
		: FMetasoundGraphSchemaAction_NewNode()
	{}

	FMetasoundGraphSchemaAction_NewFromSelected(FText InNodeCategory, FText InMenuDesc, FText InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
		: FMetasoundGraphSchemaAction_NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGroup)
	{}

	virtual ~FMetasoundGraphSchemaAction_NewFromSelected() = default;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to create new Audio Analyzer node */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewAudioAnalyzer : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	UE_API FMetasoundGraphSchemaAction_NewAudioAnalyzer();

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to create new reroute node */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewReroute : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewReroute() = default;
	UE_API FMetasoundGraphSchemaAction_NewReroute(const FLinearColor* InIconColor, bool bInShouldTransact = true);

	virtual ~FMetasoundGraphSchemaAction_NewReroute() = default;

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

private:
	FLinearColor IconColor;
	bool bShouldTransact = true;
};

/** Action to create new comment */
USTRUCT()
struct FMetasoundGraphSchemaAction_NewComment : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_NewComment()
		: FMetasoundGraphSchemaAction()
	{}

	FMetasoundGraphSchemaAction_NewComment(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
		: FMetasoundGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGroup)
	{}

	//~ Begin FMetasoundGraphSchemaAction Interface
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;
	//~ End FMetasoundGraphSchemaAction Interface

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to paste clipboard contents into the graph */
USTRUCT()
struct FMetasoundGraphSchemaAction_Paste : public FMetasoundGraphSchemaAction
{
	GENERATED_BODY();

	FMetasoundGraphSchemaAction_Paste() 
		: FMetasoundGraphSchemaAction()
	{}

	FMetasoundGraphSchemaAction_Paste(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, Metasound::Editor::EPrimaryContextGroup InGroup)
		: FMetasoundGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGroup)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

UCLASS(MinimalAPI)
class UMetasoundEditorGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	/** Helper method to add items valid to the palette list */
	METASOUNDEDITOR_API void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder) const;

	//~ Begin EdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	virtual float GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual void TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bInMarkAsModified) const override;
	virtual void TrySetDefaultValue(UEdGraphPin& Pin, const FString& InNewDefaultValue, bool bInMarkAsModified) const override;
	virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* InNodeToDelete) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const override;
	virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	virtual void SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const;
	//~ End EdGraphSchema Interface

	void BreakNodeLinks(UEdGraphNode& TargetNode, bool bShouldActuallyTransact) const;

private:
	void QueryNodeClasses(TFunctionRef<void(TUniquePtr<Metasound::Editor::ISchemaQueryResult>&&)> OnClassFound) const;

	/** Adds actions for creating actions associated with graph DataTypes */
	void GetConversionActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters InFilters = Metasound::Editor::FActionVertexFilters(), bool bShowSelectedActions = true) const;
	void GetFunctionActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters InFilters = Metasound::Editor::FActionVertexFilters(), bool bShowSelectedActions = true, Metasound::Frontend::FConstGraphHandle InGraphHandle = Metasound::Frontend::IGraphController::GetInvalidHandle(), const UEdGraph* CurrentGraph = nullptr) const;
	void GetVariableActions(FGraphActionMenuBuilder& ActionMenuBuilder, Metasound::Editor::FActionVertexFilters InFilters = Metasound::Editor::FActionVertexFilters(), bool bShowSelectedActions = true, Metasound::Frontend::FConstGraphHandle InGraphHandle = Metasound::Frontend::IGraphController::GetInvalidHandle()) const;

	void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin, bool bShouldTransact) const;

	void GetDataTypeInputNodeActions(FGraphContextMenuBuilder& InMenuBuilder, Metasound::Frontend::FConstGraphHandle InGraphHandle, Metasound::Editor::FInterfaceNodeFilterFunction InFilter = Metasound::Editor::FInterfaceNodeFilterFunction(), bool bShowSelectedActions = true) const;
	void GetDataTypeOutputNodeActions(FGraphContextMenuBuilder& InMenuBuilder, Metasound::Frontend::FConstGraphHandle InGraphHandle, Metasound::Editor::FInterfaceNodeFilterFunction InFilter = Metasound::Editor::FInterfaceNodeFilterFunction(), bool bShowSelectedActions = true) const;

	/** Adds action for creating a comment */
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph = nullptr) const;

	// Process dropped assets
	void DroppedMetaSoundAssetsOnGraph(const TArray<FMetasoundAssetBase*>& MetaSoundAssets, const FVector2f& GraphPosition, UMetasoundEditorGraph* MetaSoundGraph) const;
	void DroppedNonMetaSoundAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UMetasoundEditorGraph* MetaSoundGraph) const;

	// Validate dropped assets, returning true if valid
	bool ValidateDroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString* OutTooltipText = nullptr) const;
	bool ValidateDroppedAssetsOnPin(const TArray<FAssetData>& Assets, const UEdGraphPin* Pin, FString* OutTooltipText = nullptr) const;

	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

#undef UE_API
