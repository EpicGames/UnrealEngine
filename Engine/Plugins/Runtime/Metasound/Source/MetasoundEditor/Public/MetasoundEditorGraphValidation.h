// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Logging/TokenizedMessage.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundEditorGraph.h"

#define UE_API METASOUNDEDITOR_API

class UMetasoundEditorGraphNode;


namespace Metasound
{
	namespace Editor
	{
		struct FGraphNodeValidationResult
		{
			UE_API FGraphNodeValidationResult(UMetasoundEditorGraphNode& InNode);

			// Whether or not validation operations on result have
			// dirtied the associated Node (Validation doesn't mark
			// node as dirtied to avoid resave being required at
			// asset level)
			UE_API bool GetHasDirtiedNode() const;

			UE_API UMetasoundEditorGraphNode& GetNodeChecked() const;

			// Whether associated node is in invalid state, i.e.
			// may fail to build or may result in undefined behavior.
			UE_API bool GetIsInvalid() const;

			UE_API void SetMessage(EMessageSeverity::Type InSeverity, const FString& InMessage);
			UE_API void SetPinOrphaned(UEdGraphPin& Pin, bool bIsOrphaned);
			UE_API void SetUpgradeMessage(const FText& InMessage);

		private:
			// Node associated with validation result
			UMetasoundEditorGraphNode* Node = nullptr;

			// Whether validation made changes to the node and is now in a dirty state
			bool bHasDirtiedNode = false;
		};

		struct FGraphValidationResults
		{
			TArray<FGraphNodeValidationResult> NodeResults;

			// Results corresponding with node validation
			UE_API const TArray<FGraphNodeValidationResult>& GetResults() const;

			// Returns highest message severity of validated nodes
			UE_API EMessageSeverity::Type GetHighestMessageSeverity() const;
		};
	} // namespace Editor
} // namespace Metasound

#undef UE_API
