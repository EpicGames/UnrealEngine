// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DetailCategoryBuilder.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "IDetailPropertyRow.h"
#include "IMetaSoundGraphPanelPinFactory.h"
#include "MetasoundDefaultLiteralCustomization.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistries.h"
#include "Modules/ModuleInterface.h"
#include "PropertyHandle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Templates/Function.h"

#define UE_API METASOUNDEDITOR_API

// Forward Declarations
struct FAudioMeterDefaultColorStyle;
struct FAudioOscilloscopePanelStyle;
struct FAudioSpectrumPlotStyle;
struct FAudioVectorscopePanelStyle;

class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;
class SSearchableComboBox;
class UEdGraphPin;
class UMetasoundEditorGraph;
class UMetasoundEditorGraphMemberDefaultLiteral;
class UMetasoundEditorGraphNode;

namespace Metasound::Engine
{
	enum class EAssetScanStatus : uint8;
	enum class ENodeClassRegistryPrimeStatus : uint8;
}

DECLARE_LOG_CATEGORY_EXTERN(LogMetasoundEditor, Log, All);


namespace Metasound::Editor
{
	namespace Style
	{
		METASOUNDEDITOR_API FSlateIcon CreateSlateIcon(FName InName);
		METASOUNDEDITOR_API const FSlateBrush& GetSlateBrushSafe(FName InName);

		const FSlateColor& GetPageExecutingColor();
	} // namespace Style

		class FMetaSoundNodeConfigurationCustomization;

	using EAssetScanStatus = Metasound::Engine::EAssetScanStatus;
	using EAssetPrimeStatus = Metasound::Engine::ENodeClassRegistryPrimeStatus;
		using FCreateNodeConfigurationDetails = TFunction<TSharedRef<FMetaSoundNodeConfigurationCustomization>(
		TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)>;

	struct FCreateGraphNodeVisualizationWidgetParams
	{
		UMetasoundEditorGraphNode* MetaSoundNode = nullptr;
	};

	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnCreateGraphNodeVisualizationWidget, const FCreateGraphNodeVisualizationWidgetParams&);


	class IMetasoundEditorModule : public IModuleInterface
	{
	public:
		/** Module name constant for use with FModuleManager. */
		static constexpr FLazyName ModuleName = "MetaSoundEditor";

		UE_DEPRECATED(5.3, "IsExplicitProxyClass is deprecated, use Metasound::Frontend::FDataTypeRegistryInfo::bIsExplicit")
		virtual bool IsExplicitProxyClass(const UClass& InClass) const = 0;

		UE_DEPRECATED(5.3, "RegisterExplicitProxyClass is deprecated, use Metasound::TIsExplicit<>")
		virtual void RegisterExplicitProxyClass(const UClass& InClass) = 0;

		UE_DEPRECATED(5.3, "IsMetaSoundAssetClass is deprecated, use IMetasoundUObjectRegistry::IsRegisteredClass")
		virtual bool IsMetaSoundAssetClass(const FTopLevelAssetPath& InClassName) const = 0;

		UE_DEPRECATED(5.5, "Prime no longer supported (nor recommended as AssetTag updates now enable querying MetaSound assets for interface information without loading full asset.")
		virtual void PrimeAssetRegistryAsync() { }

		UE_DEPRECATED(5.5, "Prime no longer supported (nor recommended as AssetTag updates now enable querying MetaSound assets for interface information without loading full asset.")
		UE_API virtual EAssetPrimeStatus GetAssetRegistryPrimeStatus() const;

		UE_DEPRECATED(5.5, "Use FMetaSoundAssetManager::IsInitialAssetScanComplete instead")
		UE_API virtual EAssetScanStatus GetAssetRegistryScanStatus() const;

		virtual TUniquePtr<FMetasoundDefaultLiteralCustomizationBase> CreateMemberDefaultLiteralCustomization(UClass& InClass, IDetailCategoryBuilder& DefaultCategoryBuilder) const = 0;

		UE_EXPERIMENTAL(5.6, "Node configuration custom details customizations are experimental.")
		virtual const FCreateNodeConfigurationDetails* FindCreateCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType) const = 0;

		virtual const TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> FindDefaultLiteralClass(EMetasoundFrontendLiteralType InLiteralType) const = 0;

		UE_DEPRECATED(5.8, "Use IMetaSoundGraphPanelPinFactory::FindPinType instead.")
		virtual const FEdGraphPinType* FindPinType(FName InDataTypeName) const = 0;

		virtual const FSlateBrush* GetIconBrush(FName InDataType, const bool bIsConstructorType) const = 0;

		UE_DEPRECATED(5.8, "Use IMetaSoundGraphPanelPinFactory::GetCustomPinIcons instead.")
		virtual bool GetCustomPinIcons(UEdGraphPin* InPin, const FSlateBrush*& PinConnectedIcon, const FSlateBrush*& PinDisconnectedIcon) const = 0;

		UE_DEPRECATED(5.8, "Use IMetaSoundGraphPanelPinFactory::GetCustomPinIcons instead.")
		virtual bool GetCustomPinIcons(FName InDataType, const FSlateBrush*& PinConnectedIcon, const FSlateBrush*& PinDisconnectedIcon) const = 0;

		// Returns the MetaSound GraphPanelPinFactory for registering custom pin types.
		UE_API virtual TSharedRef<IMetaSoundGraphPanelPinFactory> GetGraphPanelPinFactory() const = 0;

		UE_DEPRECATED(5.8, "Use IMetaSoundGraphPanelPinFactory::RegisterPin instead.")
		virtual void RegisterPinType(FName InDataTypeName, FName InPinCategory = { }, FName InPinSubCategory = { },
			const FSlateBrush* InPinConnectedIcon = nullptr, const FSlateBrush* InPinDisconnectedIcon = nullptr) = 0;

		UE_DEPRECATED(5.8, "Use IMetaSoundGraphPanelPinFactory::RegisterPin instead.")
		virtual void RegisterCustomPinType(FName InDataTypeName, const FGraphPinParams& Params) = 0;

		/** Register a custom details customization for a specific substruct of FMetaSoundFrontendNodeConfiguration
			InNodeConfigurationStructType - struct type name - use MyStruct::StaticStruct()->GetFName()
			InCreateDetailsFunc - function that creates the specific details customization */
		UE_EXPERIMENTAL(5.6, "Node configuration custom details customizations are experimental.")
		virtual void RegisterCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType, FCreateNodeConfigurationDetails InCreateDetailsFunc) = 0;

		// Unregister a custom details customization for a node configuration struct type, using the same struct type name as RegisterCustomNodeConfigurationDetailsCustomization
		UE_EXPERIMENTAL(5.6, "Node configuration custom details customizations are experimental.")
		virtual void UnregisterCustomNodeConfigurationDetailsCustomization(FName InNodeConfigurationStructType) = 0;

		// For the given node class, register a delegate that can be used for creating in-graph node visualizations.
		virtual void RegisterGraphNodeVisualization(FName InNodeClassName, FOnCreateGraphNodeVisualizationWidget OnCreateGraphNodeVisualizationWidget) = 0;

		// Queries if the MetaSound Editor is in "restricted mode" (i.e. can only make new presets and not make new assets or edit asset graphs)
		virtual bool IsRestrictedMode() const = 0;

		// Sets if the MetaSound editor is in "restricted mode" (i.e. can only make new presets and not make new assets or edit asset graphs)
		virtual void SetRestrictedMode(bool bInRestricted) = 0;
	};
} // namespace Metasound::Editor

#undef UE_API
