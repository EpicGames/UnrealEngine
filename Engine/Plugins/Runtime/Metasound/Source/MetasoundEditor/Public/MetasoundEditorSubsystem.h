// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "EditorSubsystem.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundDocumentInterface.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundEditorSubsystem.generated.h"

#define UE_API METASOUNDEDITOR_API


// Forward Declarations
class UMetaSoundBuilderBase;
class UMetaSoundEditorBuilderListener;
class UMetasoundEditorGraphMember;
class UMetasoundEditorGraphMemberDefaultLiteral;

struct FMetaSoundPageSettings;


/** The subsystem in charge of editor MetaSound functionality */
UCLASS(MinimalAPI, meta = (DisplayName = "MetaSound Editor Subsystem"))
class UMetaSoundEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	struct FInitAssetArgs
	{
		// (Optional) Template to apply to newly initialized asset
		TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Template;

		// Currently selected objects in the focused ContentBrowser.
		TArray<UObject*> SelectedObjects;
	};

	// Binds literal editor Metadata to the given member.  If the literal already exists, adds literal
	// reference to given member (asserts that existing literal is of similar subclass provided).  If 
	// it does not exist, or an optional template object is provided, metadata is generated then bound.
	// Returns true if new literal metadata was generated, false if not. Asserts if bind failed.
	UE_API bool BindMemberMetadata(
		FMetaSoundFrontendDocumentBuilder& Builder,
		UMetasoundEditorGraphMember& InMember,
		TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass,
		UMetasoundEditorGraphMemberDefaultLiteral* TemplateObject = nullptr);

	// Build the given builder to a MetaSound asset
	// @param Author - Sets the author on the given builder's document.
	// @param AssetName - Name of the asset to build.
	// @param PackagePath - Path of package to build asset to.
	// @param TemplateSoundWave - SoundWave settings such as attenuation, modulation, and sound class will be copied from the optional TemplateSoundWave.
	// For preset builders, TemplateSoundWave will override the template values from the referenced asset.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Editor", meta = (WorldContext = "Parent", ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "MetaSound Asset") TScriptInterface<IMetaSoundDocumentInterface> BuildToAsset(
		UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* InBuilder,
		const FString& Author,
		const FString& AssetName,
		const FString& PackagePath,
		EMetaSoundBuilderResult& OutResult,
		UPARAM(DisplayName = "Template SoundWave") const USoundWave* TemplateSoundWave = nullptr);

	UE_DEPRECATED(5.8, "Functionality is now private")
	UE_API UMetasoundEditorGraphMemberDefaultLiteral* CreateMemberMetadata(FMetaSoundFrontendDocumentBuilder& Builder, FName InMemberName, TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass) const;

	// Returns a builder for the given MetaSound asset. Returns null if provided a transient MetaSound. For finding builders for transient
	// MetaSounds, use the UMetaSoundBuilderSubsystem's API (FindPatchBuilder, FindSourceBuilder, FindBuilderByName etc.)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Editor", meta = (DisplayName = "Find Or Begin Building MetaSound Asset", ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface> MetaSound, EMetaSoundBuilderResult& OutResult) const;
	
	// Find graph input metadata (which includes editor only range information for floats) for a given input. If the metadata does not exist, create it. 
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Input Metadata") UMetaSoundFrontendMemberMetadata* FindOrCreateGraphInputMetadata(UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* InBuilder, FName InputName, EMetaSoundBuilderResult& OutResult);

	
	// Returns the corresponding literal class for a given type
	UE_API TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> GetLiteralClassForType(FName TypeName) const;

	// Get the build page 
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Editor")
	UE_API FName GetBuildPage(UMetaSoundBuilderBase* Builder) const;

	// Sets the visual location to InLocation of a given node InNode of a given builder's document.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Editor", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetNodeLocation(
		UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* InBuilder,
		UPARAM(DisplayName = "Node Handle") const FMetaSoundNodeHandle& InNode,
		UPARAM(DisplayName = "Location") const FVector2D& InLocation,
		EMetaSoundBuilderResult& OutResult);

	// Initialize the UObject asset with optional initialization arguments to be applied.
	// Clears the existing MetaSound document (for the case of duplicated assets).
	UE_API void InitAsset(UObject& InNewMetaSound, FInitAssetArgs InitAssetArgs = { });

	UE_DEPRECATED(5.8, "Use overload that is passed optional initialization arguments. Call SetSoundWaveSettingsFromTemplate directly if desired.")
	UE_API void InitAsset(UObject& InNewMetaSound, UObject* InReferencedMetaSound = nullptr, const bool bClearDocument = false);

	// Returns whether or not a page with the given name both exists and is set as
	// a valid, cooked target for the currently set audition platform in editor.
	UE_API bool IsPageAuditionPlatformCookTarget(FName InPageName) const;
	UE_API bool IsPageAuditionPlatformCookTarget(const FGuid& InPageID) const;

	// Add a builder listener for a builder which is used to add and remove custom editor builder delegates.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Editor", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Builder Listener") UMetaSoundEditorBuilderListener* AddBuilderDelegateListener(UMetaSoundBuilderBase* InBuilder, EMetaSoundBuilderResult& OutResult);
	
	// Wraps RegisterGraphWithFrontend logic in Frontend with any additional logic required to refresh editor & respective editor object state.
	// @param InMetaSound - MetaSound to register
	// @param bInForceSynchronize - Forces the synchronize flag for all open graphs being registered by this call (all referenced graphs and
	// referencing graphs open in editors)
	UE_API void RegisterGraphWithFrontend(UObject& InMetaSound, bool bInForceViewSynchronization = false) const;

	// Register toolbar extender that will be displayed in the MetaSound Asset Editor.
	UE_API void RegisterToolbarExtender(TSharedRef<FExtender> InExtender);

	// If the given page name is implemented on the provided builder, sets the focused (and build) page of
	// the provided builder to the associated page and sets the audition page to
	// the provided name. If the given builder has an asset editor open, optionally opens or brings
	// that editor's associated PageID into user focus.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "Audio|MetaSound|Builder|Editor", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetFocusedPage(UMetaSoundBuilderBase* Builder, FName PageName, bool bOpenEditor, EMetaSoundBuilderResult& OutResult) const;

	// If the given page name is implemented on the provided builder, sets the build page.
	// This page will be the page that graph operations (ex. node add)
	// will be applied to by default if a page name is not specified.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "Audio|MetaSound|Builder|Editor", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetBuildPage(UMetaSoundBuilderBase* Builder, FName PageName, EMetaSoundBuilderResult& OutResult) const;
	
	// If the given PageID is implemented on the provided builder, sets the focused page of
	// the provided builder to the associated page and sets the audition target page to
	// the provided ID. If the given builder has an asset editor open, optionally opens or brings
	// that editor's associated PageID into user focus. Returns whether or not the audition page
	// was set to the provided focus page.
	UE_API bool SetFocusedPage(UMetaSoundBuilderBase& Builder, const FGuid& InPageID, bool bOpenEditor, bool bPostTransaction = true) const;

	// Unregisters toolbar extender that is displayed in the MetaSound Asset Editor.
	UE_API bool UnregisterToolbarExtender(TSharedRef<FExtender> InExtender);

	// Get the default author for a MetaSound asset
	UE_API const FString GetDefaultAuthor();

	// Returns all currently toolbar extenders registered to be displayed within the MetaSound Asset Editor.
	UE_API const TArray<TSharedRef<FExtender>>& GetToolbarExtenders() const;

	// Copy over sound wave settings such as attenuation, modulation, and sound class from the template sound wave to the MetaSound
	UE_API void SetSoundWaveSettingsFromTemplate(USoundWave& NewMetasound, const USoundWave& TemplateSoundWave) const;

	static UE_API UMetaSoundEditorSubsystem& GetChecked();
	static UE_API const UMetaSoundEditorSubsystem& GetConstChecked();

private:
	UE_API UMetasoundEditorGraphMemberDefaultLiteral* CreateMemberMetadata(FMetaSoundFrontendDocumentBuilder& Builder, TSubclassOf<UMetasoundEditorGraphMemberDefaultLiteral> LiteralClass) const;

	UE_API bool SetFocusedPageInternal(FName PageName, const FGuid& InPageID, UMetaSoundBuilderBase& Builder, bool bOpenEditor, bool bPostTransaction, bool bSetAuditionPage = true) const;


	// Editor Toolbar Extenders
	TArray<TSharedRef<FExtender>> EditorToolbarExtenders;
};

#undef UE_API
