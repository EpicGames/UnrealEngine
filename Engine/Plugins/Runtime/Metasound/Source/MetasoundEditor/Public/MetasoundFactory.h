// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "Factories/Factory.h"
#include "MetasoundFrontendDocument.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectMacros.h"

#include "MetasoundFactory.generated.h"

// Forward Declarations
class UMetaSoundPatch;
class UMetaSoundSource;


UCLASS(abstract)
class UMetaSoundBaseFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Transient, meta = (Deprecated = 5.8, DeprecationMessage = "Use document template instead"))
	TObjectPtr<UObject> ReferencedMetaSoundObject;

	UPROPERTY(EditAnywhere, Transient, meta = (Category = Factory))
	TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Template;

	// If Document configuration is set, selection is provided to the configuration
	// as an initial argument to be set on the configuration if applicable.
	UPROPERTY(EditAnywhere, Transient, meta = (Category = Factory))
	TArray<TObjectPtr<UObject>> SelectedObjects;
};

UCLASS(hidecategories=Object, MinimalAPI)
class UMetaSoundFactory : public UMetaSoundBaseFactory
{
	GENERATED_UCLASS_BODY()
 
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext) override;
	//~ Begin UFactory Interface
 };

UCLASS(hidecategories = Object, MinimalAPI)
class UMetaSoundSourceFactory : public UMetaSoundBaseFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext) override;
	//~ Begin UFactory Interface
};
