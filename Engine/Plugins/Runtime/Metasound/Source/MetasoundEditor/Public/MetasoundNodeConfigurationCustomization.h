// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InstancedStructDetails.h"

#define UE_API METASOUNDEDITOR_API

// Forward Declarations
struct FPropertyChangedEvent;
class FInstancedStructDataDetails;
class IDetailPropertyRow;
class IPropertyHandle;
class UMetasoundEditorGraphNode;

namespace Metasound::Editor
{
	class FMetaSoundNodeConfigurationCustomization : public FInstancedStructDataDetails
	{
	public:
		FMetaSoundNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
			: FInstancedStructDataDetails(InStructProperty)
			, GraphNode(InNode)
		{
		}

		UE_API virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;

	protected:
		UE_API void OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

		TWeakObjectPtr<UMetasoundEditorGraphNode> GraphNode;
	};
	
	using FMetaSoundNodeConfigurationDataDetails UE_DEPRECATED(5.8, "FMetaSoundNodeConfigurationDataDetails has been renamed. Use FMetaSoundNodeConfigurationCustomization instead") = FMetaSoundNodeConfigurationCustomization;
} // namespace Metasound::Editor

#undef UE_API
