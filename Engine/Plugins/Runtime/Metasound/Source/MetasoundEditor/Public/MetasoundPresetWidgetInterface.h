// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio/AudioPanelWidgetInterface.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundTemplateWidgetInterface.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundPresetWidgetInterface.generated.h"


// IMetaSoundPresetWidgetInterface has been soft deprecated. Use IMetaSoundTemplateWidgetInterface instead.
UINTERFACE(Blueprintable, MinimalAPI)
class UMetaSoundPresetWidgetInterface : public UAudioPanelWidgetInterface
{
	GENERATED_BODY()
};

class IMetaSoundPresetWidgetInterface : public IAudioPanelWidgetInterface
{
	GENERATED_BODY()

public:
	// The MetaSounds whose presets are supported by this widget. If Support All Presets is true, this widget is supported by all presets except those in the Excluded array.
	// MetaSounds in the include/exclude arrays can be MetaSound presets or non presets. 
	// If a MetaSound is not a preset, then presets of that MetaSound will be supported/excluded by this widget.
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound Preset Widget")
	void GetSupportedMetaSounds(bool& bSupportAllPresets, TArray<TScriptInterface<IMetaSoundDocumentInterface>>& ExcludedMetaSounds, TArray<TScriptInterface<IMetaSoundDocumentInterface>>& IncludedMetaSounds) const;

	// Called when the preset widget is constructed, giving the builder of the associated MetaSound preset
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound Preset Widget", meta = (DisplayName = "On MetaSoundPreset Widget Constructed"))
	void OnConstructed(UMetaSoundBuilderBase* Builder);

	// Called when the MetaSound starts and stops auditioning. Provides a reference to the audio component when auditioning starts, and returns nullptr when auditioning stops.
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "MetaSound Preset Widget", meta = (DisplayName = "On MetaSound Audition State Changed"))
	void OnAuditionStateChanged(UAudioComponent* AudioComponent, bool bIsAuditioning);

	// (Optional) Called when initializing a tab owning the given widget to allow implementation to provide basic tab display options and information.
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound Preset Widget", meta = (DisplayName = "On Initializing Tab"))
	void OnInitializingTab(FMetaSoundTemplateWidgetTabInfo& TabInfo);
};
