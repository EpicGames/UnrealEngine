// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentInterface.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ScriptInterface.h"


#include "MetasoundTemplateWidgetInterface.generated.h"


// Forward Declarations
class UMetaSoundTemplateViewModelBase;


UINTERFACE(Blueprintable, MinimalAPI, DisplayName = "MetaSound Template Widget Interface")
class UMetaSoundTemplateWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

USTRUCT(BlueprintType, MinimalAPI, DisplayName = "MetaSound Template Widget Tab Info")
struct FMetaSoundTemplateWidgetTabInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = TabInfo)
	FName TabName;

	UPROPERTY(BlueprintReadWrite, Category = TabInfo)
	FText TabDisplayName;

	UPROPERTY(BlueprintReadWrite, Category = TabInfo)
	FName StyleName = "MetaSoundStyle";

	UPROPERTY(BlueprintReadWrite, Category = TabInfo)
	FName TabIcon = "MetasoundEditor.Metasound.Icon";

	UPROPERTY(BlueprintReadWrite, Category = TabInfo)
	FText ToolTip;
};

class IMetaSoundTemplateWidgetInterface
{
	GENERATED_BODY()

public:
	// Returns whether or not the widget supports the given template. Called when initializing
	// the MetaSound Editor to determine if an implemented template supports the given widget.
	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category = "MetaSound|Template", meta = (DisplayName = "Is Supported Template"))
	void IsSupportedTemplate(UPARAM(DisplayName = "Template Name") FName ConfigName, const UMetaSoundBuilderBase* Builder, UPARAM(DisplayName = "Is Supported") bool& bOutIsSupported);

	// Called when the MetaSound starts and stops auditioning. Provides a reference to the audio component when auditioning starts, and returns nullptr when auditioning stops.
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound|Template", meta = (DisplayName = "On MetaSound Audition State Changed"))
	void OnAuditionStateChanged(UAudioComponent* AudioComponent, bool bIsAuditioning);

	// Called when the builder is initialized (prior to PreConstruct).
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound|Template", meta = (DisplayName = "On MetaSound Builder Set"))
	void OnBuilderSet(UMetaSoundBuilderBase* Builder);

	// (Optional) Called when initializing a tab owning the given widget to allow implementation to provide basic tab display options and information.
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound|Template", meta = (DisplayName = "On Initializing Tab"))
	void OnInitializingTab(FMetaSoundTemplateWidgetTabInfo& TabInfo);

	// (Optional) If ViewModel class provided, automatically initializes and sets ViewModel of the given class for the given widget.
	UFUNCTION(BlueprintImplementableEvent, Category = "MetaSound|Template", meta = (DisplayName = "On Initializing ViewModel"))
	void OnInitializingViewModel(TSubclassOf<UMetaSoundTemplateViewModelBase>& Class);
};
