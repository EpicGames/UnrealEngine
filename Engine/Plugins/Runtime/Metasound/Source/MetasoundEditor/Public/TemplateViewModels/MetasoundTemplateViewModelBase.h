// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Delegates/IDelegateInstance.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MVVMViewModelBase.h"
#include "StructUtils/StructView.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "UObject/Class.h"

#include "MetasoundTemplateViewModelBase.generated.h"

// Forward Declarations
class UUserWidget;

#define UE_API METASOUNDEDITOR_API


UCLASS(MinimalAPI, Blueprintable, Abstract, DisplayName = "MetaSound Template ViewModel Base")
class UMetaSoundTemplateViewModelBase : public UMVVMViewModelBase
{
	GENERATED_BODY()

public:
	// Constructs a view model
	static void ConstructWidgetViewModel(
		UUserWidget* UserWidget,
		UMetaSoundBuilderBase& InBuilder,
		TConstStructView<FMetaSoundFrontendDocumentTemplate> Config,
		TSubclassOf<UMetaSoundTemplateViewModelBase> VMClass);

	UE_API virtual bool IsSupportedTemplate(const UScriptStruct& InStruct) const;

protected:
	template <typename TemplateType>
	const TemplateType* GetConstTemplate() const
	{
		if (Builder)
		{
			return Builder->GetConstBuilder().GetConstTemplateAs<TemplateType>();
		}
		return nullptr;
	}

	// Enables responding to the builder reloading or the model view requiring a full
	// resynchronization of configuration properties to view (i.e. when user transactions
	// are applied that require a full resync of the view such as undo, redo, etc.)
	UE_API virtual void Reload();

	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundBuilderBase> Builder;

private:
	void Initialize(UMetaSoundBuilderBase* InBuilder);

	class FBuilderListener : public Metasound::Frontend::IDocumentBuilderTransactionListener
	{
		TWeakObjectPtr<UMetaSoundTemplateViewModelBase> Parent;

	public:
		FBuilderListener() = default;
		FBuilderListener(UMetaSoundTemplateViewModelBase* InParent)
			: Parent(InParent)
		{
		}

		virtual ~FBuilderListener() = default;

	private:
		virtual void OnBuilderReloaded(Metasound::Frontend::FDocumentModifyDelegates& OutDelegates) override;
		void OnTemplateChanged(const Metasound::Frontend::FDocumentTemplateChangedArgs& Args);

		FDelegateHandle TemplateChangeDelegateHandle;
	};
	TSharedPtr<FBuilderListener> BuilderListener;
};
#undef UE_API // METASOUNDEDITOR_API
