// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundTemplateViewModelBase.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "Sound/SoundWave.h"

#include "RandomizerConfigurationViewModel.generated.h"

#define UE_API METASOUNDEDITOR_API


// Forward Declarations
struct FMetaSoundRandomizerTemplate;


UCLASS(MinimalAPI, DisplayName = "MetaSound Randomizer ViewModel")
class UMetaSoundRandomizerViewModel : public UMetaSoundTemplateViewModelBase
{
	GENERATED_BODY()

public:
	UE_API bool GetIsOneShot() const;
	UE_API FVector2f GetPitch() const;

	UE_API const TArray<TObjectPtr<USoundWave>>& GetSounds() const;

	UFUNCTION(BlueprintPure, FieldNotify, Category = "MetaSound|Randomizer")
	UE_API float GetPitchMin() const;

	UFUNCTION(BlueprintPure, FieldNotify, Category = "MetaSound|Randomizer")
	UE_API float GetPitchMax() const;

	UFUNCTION(BlueprintCallable, Category = "MetaSound|Randomizer")
	UE_API void SetNumSounds(int32 NumSounds);

	UFUNCTION(BlueprintCallable, Category = "MetaSound|Randomizer")
	UE_API void SetPitchMin(float PitchMin);

	UFUNCTION(BlueprintCallable, Category = "MetaSound|Randomizer")
	UE_API void SetPitchMax(float PitchMax);

	UE_API void SetIsOneShot(bool bInIsOneShot);
	UE_API void SetPitch(const FVector2f& InPitch);

	UE_API void SetSounds(TArray<TObjectPtr<USoundWave>> InSounds);

	UE_API virtual bool IsSupportedTemplate(const UScriptStruct& InStruct) const override;
	UE_API virtual void Reload() override;

private:
	void OnPropertyChanged(const FPropertyChangedEvent& InEvent);

	FDelegateHandle PropertyChangedHandle;

	// If randomizer is one shot (looping if false).
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "GetIsOneShot", Setter = "SetIsOneShot", Category = "MetaSound|Randomizer", meta = (AllowPrivateAccess))
	bool bIsOneShot = false;

	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "GetSounds", Setter = "SetSounds", Category = "MetaSound|Randomizer", meta = (
		AllowPrivateAccess,
		DisallowedClasses = "/Script/MetasoundEngine.MetaSoundSource, /Script/Engine.SoundSourceBus"
	))
	TArray<TObjectPtr<USoundWave>> Sounds;

	// Sound's pitch range in semitones.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "GetPitch", Setter = "SetPitch", Category = "MetaSound|Randomizer", meta = (AllowPrivateAccess))
	FVector2f Pitch { 0.f, 0.f };
};
#undef UE_API
