// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundEngine : ModuleRules
	{
		public MetasoundEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePathModuleNames.AddRange
			(
				new string[]
				{
					"AVEncoder",
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetRegistry",
					"AudioExtensions",
					"AudioMixer",
					"MetasoundGraphCore",
					"MetasoundGenerator",
					"SignalProcessing",
					"AudioChannelAgnosticCore",
					"Projects"
				}
			);

			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"MetasoundFrontend",
					"MetasoundStandardNodes",
					"Serialization",
					"AudioPlatformConfiguration",
					"WaveTable"
				}
			);

			PrivateDefinitions.Add("METASOUND_PLUGIN=Metasound");
			PrivateDefinitions.Add("METASOUND_MODULE=MetasoundEngine");
		}
	}
}
