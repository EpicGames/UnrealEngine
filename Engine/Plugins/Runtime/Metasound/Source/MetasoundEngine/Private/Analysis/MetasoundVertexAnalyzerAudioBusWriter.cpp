// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundVertexAnalyzerAudioBusWriter.h"

#include "AudioBusSubsystem.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "MetasoundAudioBuffer.h"


namespace Metasound::Engine
{
	const FName& FVertexAnalyzerAudioBusWriter::GetAnalyzerName()
	{
		static const FName AnalyzerName = "UE.Audio.AudioBusWriter";
		return AnalyzerName;
	}

	const FName& FVertexAnalyzerAudioBusWriter::GetDataType()
	{
		return GetMetasoundDataTypeName<FAudioBuffer>();
	}

	FName FVertexAnalyzerAudioBusWriter::GetAnalyzerMemberName(const Audio::FDeviceId InDeviceID, const uint32 InAudioBusID)
	{
		const FBusAddress BusAddress
		{
			.DeviceID = InDeviceID,
			.AudioBusID = InAudioBusID,
		};
		return FName(BusAddress.ToString());
	}

	FVertexAnalyzerAudioBusWriter::FVertexAnalyzerAudioBusWriter(const Frontend::FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
	{
		// AnalyzerMemberName is used as an input to modify analyzer settings:
		const FBusAddress BusAddress = FBusAddress::FromString(InParams.AnalyzerAddress.AnalyzerMemberName.ToString());

		if (FAudioDeviceManager* ADM = FAudioDeviceManager::Get())
		{
			if (const FAudioDevice* AudioDevice = ADM->GetAudioDeviceRaw(BusAddress.DeviceID))
			{
				UAudioBusSubsystem* AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();

				const int32 Frames = InParams.OperatorSettings.GetNumFramesPerBlock();
				constexpr int32 Channels = 1;
				AudioBusPatchInput = AudioBusSubsystem->AddPatchInputForAudioBus(BusAddress.AudioBusID, Frames, Channels);
			}
		}
	}

	void FVertexAnalyzerAudioBusWriter::Execute()
	{
		const FAudioBuffer& AudioBuffer = GetVertexData<FAudioBuffer>();
		AudioBusPatchInput.PushAudio(AudioBuffer.GetData(), AudioBuffer.Num());
	}

	FString FVertexAnalyzerAudioBusWriter::FBusAddress::ToString() const
	{
		return FString::Join(TArray<FString>
		{
			FString::FromInt(DeviceID),
			FString::FromInt(AudioBusID),
		}
		, TEXT(","));
	}

	FVertexAnalyzerAudioBusWriter::FBusAddress FVertexAnalyzerAudioBusWriter::FBusAddress::FromString(const FString& InAnalyzerMemberName)
	{
		FBusAddress BusAddress;

		TArray<FString> Tokens;
		if (ensure(InAnalyzerMemberName.ParseIntoArray(Tokens, TEXT(",")) == 2))
		{
			BusAddress.DeviceID = FCString::Atoi(*Tokens[0]);
			BusAddress.AudioBusID = FCString::Atoi(*Tokens[1]);
		}

		return BusAddress;
	}
} // namespace Metasound::Engine
