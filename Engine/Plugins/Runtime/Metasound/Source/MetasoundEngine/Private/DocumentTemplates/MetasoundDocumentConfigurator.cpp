// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentTemplates/MetasoundDocumentConfigurator.h"

#include "Algo/Transform.h"
#include "AudioParameter.h"
#include "Templates/Function.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundSettings.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "UObject/SoftObjectPath.h"


#if WITH_EDITORONLY_DATA
namespace Metasound::Engine
{
	namespace ConfigCodeReflectorPrivate
	{
		struct FConfigClassInfo
		{
			const FMetasoundFrontendClass* Class = nullptr;
		};

		struct FConfigNodeInfo
		{
			FString InstanceName;
			const FMetasoundFrontendNode* Node = nullptr;
		};

		template <typename EntryType>
		void FormatLiteralArray(const FMetasoundFrontendLiteral& Literal, FStringBuilderBase& OutBuilder)
		{
			if (Literal.GetArrayNum() == 0)
			{
				OutBuilder += TEXT(" )");
				return;
			}

			OutBuilder += TEXT("\n\t\t{");

			TArray<EntryType> ArrayEntries;
			TArray<int32> Entries;
			ensure(Literal.TryGet(ArrayEntries));
			for (EntryType& Entry : ArrayEntries)
			{
				OutBuilder.Appendf(TEXT("\n\t\t\t%s,"), *LexToString(Entry));
			}

			OutBuilder += TEXT("\n\t\t})\n\t");
		}

		template <typename EntryType>
		void FormatLiteralArray(const FMetasoundFrontendLiteral& Literal, FStringBuilderBase& OutBuilder, TFunctionRef<FString(EntryType&)> GetEntryString)
		{
			if (Literal.GetArrayNum() == 0)
			{
				OutBuilder += TEXT(" )");
				return;
			}

			OutBuilder += TEXT("\n\t\t{");

			TArray<EntryType> ArrayEntries;
			ensure(Literal.TryGet(ArrayEntries));
			for (EntryType& Entry : ArrayEntries)
			{
				const FString EntryString = GetEntryString(Entry);
				OutBuilder.Appendf(TEXT("\n\t\t\t%s,"), *EntryString);
			}

			OutBuilder += TEXT("\n\t\t})\n\t");
		}

		class FConfigReflector
		{
			// non-const to enable harvesting input defaults via move semantics.
			// Otherwise, structure left alone through reflection execution.
			FMetasoundFrontendGraphLayer Layer;

			TArray<FMetasoundFrontendClassOutput> InterfaceOutputs;

			TSet<FString> NodeInstanceNames;

			TMap<FGuid, FName> TemplateNodeIDToInputName;
			TMap<FGuid, FConfigClassInfo> ClassIDToInfo;
			TMap<FGuid, FConfigNodeInfo> NodeIDToInfo;

			TStringBuilder<0> NodeCode;
			TStringBuilder<0> NodeInputDefaultsCode;

			TStringBuilder<0> InterfaceCode;

			TStringBuilder<0> InputCode;

#if WITH_EDITORONLY_DATA
			TStringBuilder<0> InputNodeCode; // Input Template nodes
#endif // WITH_EDITORONLY_DATA

			TStringBuilder<0> VariableCode;
			TStringBuilder<0> VariableSetNodeCode;
			TStringBuilder<0> VariableGetNodeCode;
			TStringBuilder<0> VariableGetDelayedNodeCode;

			TStringBuilder<0> OutputCode;
			TStringBuilder<0> OutputEdgesCode;

			TStringBuilder<0> EdgesCode;

			void CacheDependencies()
			{
				Algo::Transform(Layer.Dependencies, ClassIDToInfo, [](const FMetasoundFrontendClass& Dependency)
				{
					return TPair<FGuid, FConfigClassInfo>(Dependency.ID, FConfigClassInfo { &Dependency });
				});
			}

			FName FindConnectedInputName(const FGuid& InTemplateNodeID)
			{
				for (const FMetasoundFrontendEdge& Edge : Layer.Graph.Edges)
				{
					if (Edge.ToNodeID == InTemplateNodeID)
					{
						auto IsInput = [&Edge](const FMetasoundFrontendClassInput& Input)
						{
							return Input.NodeID == Edge.FromNodeID;
						};
						if (const FMetasoundFrontendClassInput* Input = Layer.Inputs.FindByPredicate(IsInput))
						{
							return Input->Name;
						}
					}
				}

				return NAME_None;
			}

#if WITH_EDITORONLY_DATA
			void CacheInputNodes()
			{
				using namespace Frontend;

				for (const FMetasoundFrontendNode& Node: Layer.Graph.Nodes)
				{
					if (const FConfigClassInfo* ClassInfo = ClassIDToInfo.Find(Node.ClassID))
					{
						if (ClassInfo->Class->Metadata.GetClassName() == FInputNodeTemplate::GetChecked().GetClassName())
						{
							const FName InputName = FindConnectedInputName(Node.GetID());
							if (!InputName.IsNone())
							{
								TemplateNodeIDToInputName.Add(Node.GetID(), InputName);
							}
						}
					}
				}
			}
#endif // WITH_EDITORONLY_DATA

			const FMetasoundFrontendClassOutput* FindOutput(const FGuid& InNodeID) const
			{
				auto IsOutput = [&InNodeID](const FMetasoundFrontendClassOutput& Output) { return Output.NodeID == InNodeID; };
				if (const FMetasoundFrontendClassOutput* Output = Layer.Outputs.FindByPredicate(IsOutput))
				{
					return Output;
				}

				if (const FMetasoundFrontendClassOutput* Output = InterfaceOutputs.FindByPredicate(IsOutput))
				{
					return Output;
				}

				return nullptr;
			}

			static void FormatDefaultCode(const FMetasoundFrontendLiteral& Literal, FStringBuilderBase& OutBuilder)
			{
				OutBuilder.Append(TEXT("MakeFrontendLiteral("));

				auto FormatAsLoadedPath = [](const FString& InPathString)
				{
					return FString::Printf(TEXT("FSoftObjectPath(TEXT(\"%s\")).TryLoad()"), *InPathString);
				};

				switch (Literal.GetType())
				{
					case EMetasoundFrontendLiteralType::Boolean:
					case EMetasoundFrontendLiteralType::Integer:
					{
						OutBuilder.Appendf(TEXT("%s) "), *Literal.ToString());
					}
					break;

					case EMetasoundFrontendLiteralType::Float:
					{
						float Value = 0.f;
						ensure(Literal.TryGet(Value));
						OutBuilder.Appendf(TEXT("%sf) "), *FString::SanitizeFloat(Value));
					}
					break;

					case EMetasoundFrontendLiteralType::String:
					{
						OutBuilder.Appendf(TEXT("TEXT(\"%s\")) "), *Literal.ToString());
					}
					break;

					case EMetasoundFrontendLiteralType::UObject:
					{
						OutBuilder.Appendf(TEXT("%s) "), *FormatAsLoadedPath(Literal.ToString()));
					}
					break;

					case EMetasoundFrontendLiteralType::BooleanArray:
					{
						FormatLiteralArray<bool>(Literal, OutBuilder);
					}
					break;

					case EMetasoundFrontendLiteralType::IntegerArray:
					{
						FormatLiteralArray<int32>(Literal, OutBuilder);
					}
					break;

					case EMetasoundFrontendLiteralType::FloatArray:
					{
						FormatLiteralArray<float>(Literal, OutBuilder, [&](float& Value)
						{
							return FString::SanitizeFloat(Value);
						});
					}
					break;

					case EMetasoundFrontendLiteralType::StringArray:
					{
						FormatLiteralArray<FString>(Literal, OutBuilder);
					}
					break;

					case EMetasoundFrontendLiteralType::UObjectArray:
					{
						FormatLiteralArray<UObject*>(Literal, OutBuilder, [&](UObject*& Object)
						{
							const FSoftObjectPath Path(Object);
							return FormatAsLoadedPath(Path.ToString());
						});
					}
					break;

					case EMetasoundFrontendLiteralType::None:
					case EMetasoundFrontendLiteralType::NoneArray:
					case EMetasoundFrontendLiteralType::Invalid:
					{
						OutBuilder += TEXT(")");
					}
					default:
					break;
				}
			}

			void ReflectVariables()
			{
				if (Layer.Graph.Variables.IsEmpty())
				{
					return;
				}

				VariableCode = TEXT("\n// Variables\n");
				VariableCode += TEXT(".Add<FDC::FVariable>(\n{\n");

				for (const FMetasoundFrontendVariable& Variable : Layer.Graph.Variables)
				{
					// Configurations/MetaSound Builder Subsystem currently do not support pages,
					// so attempt to harvest literal with Default PageID.
					VariableCode += TEXT("\t{ ");
					{
						VariableCode.Appendf(TEXT(".Name = \"%s\", "), *Variable.Name.ToString());
						VariableCode.Appendf(TEXT(".DataType = \"%s\", "), *Variable.TypeName.ToString());
						VariableCode.Appendf(TEXT(".DefaultValue = ")); FormatDefaultCode(Variable.Literal, VariableCode);
					}
					VariableCode += TEXT("},\n");
				}

				VariableCode += TEXT("})");

				// Build a map from variable node IDs to variable names for use in variable node reflection
				bool bHasSetNodes = false;
				bool bHasGetNodes = false;
				bool bHasGetDelayedNodes = false;

				VariableSetNodeCode = TEXT("\n// Variable Set Nodes\n");
				VariableSetNodeCode += TEXT(".Add<FDC::FVariableSetNode>(\n{\n");

				VariableGetNodeCode = TEXT("\n// Variable Get Nodes\n");
				VariableGetNodeCode += TEXT(".Add<FDC::FVariableGetNode>(\n{\n");

				VariableGetDelayedNodeCode = TEXT("\n// Variable Get Delayed Nodes\n");
				VariableGetDelayedNodeCode += TEXT(".Add<FDC::FVariableGetDelayedNode>(\n{\n");

				auto CacheNodeInfo = [this](const FString& BaseName, const FMetasoundFrontendNode& Node)
				{
					int32 Index = 0;
					FString FullName = BaseName;
					while (NodeInstanceNames.Contains(FullName))
					{
						FullName = FString::Printf(TEXT("%s_%i"), *BaseName, ++Index);
					}
					NodeInstanceNames.Add(FullName);
					NodeIDToInfo.Add(Node.GetID(), FConfigNodeInfo { .InstanceName = FullName, .Node = &Node });
					return FullName;
				};

				for (const FMetasoundFrontendVariable& Variable : Layer.Graph.Variables)
				{
					auto ReflectVariableNode = [&](const FGuid& InNodeID, FStringBuilderBase& OutCode, bool& bOutHasNodes, const TCHAR* StructName)
					{
						auto IsNodeWithID = [&InNodeID](const FMetasoundFrontendNode& Node) { return Node.GetID() == InNodeID; };
						if (const FMetasoundFrontendNode* Node = Layer.Graph.Nodes.FindByPredicate(IsNodeWithID))
						{
							FVector2D Location = FVector2D::Zero();
							if (!Node->Style.Display.Locations.IsEmpty())
							{
								Location = Node->Style.Display.Locations.Array().Last().Value;
							}

							const FString NodeName = CacheNodeInfo(FString::Printf(TEXT("%s_%s"), StructName, *Variable.Name.ToString()), *Node);
							bOutHasNodes = true;
							OutCode.Appendf(
								TEXT("\t{ .Name = \"%s\", .VariableName = \"%s\", .Location = FVector2D(%sf, %sf) },\n"),
								*NodeName,
								*Variable.Name.ToString(),
								*FString::SanitizeFloat(Location.X),
								*FString::SanitizeFloat(Location.Y)
							);
						}
					};

					// Mutator (Set) node
					if (Variable.MutatorNodeID.IsValid())
					{
						ReflectVariableNode(Variable.MutatorNodeID, VariableSetNodeCode, bHasSetNodes, TEXT("Set"));
					}

					// Accessor (Get) nodes
					for (const FGuid& AccessorID : Variable.AccessorNodeIDs)
					{
						ReflectVariableNode(AccessorID, VariableGetNodeCode, bHasGetNodes, TEXT("Get"));
					}

					// Deferred Accessor (Get Delayed) nodes
					for (const FGuid& DeferredID : Variable.DeferredAccessorNodeIDs)
					{
						ReflectVariableNode(DeferredID, VariableGetDelayedNodeCode, bHasGetDelayedNodes, TEXT("GetDelayed"));
					}
				}

				auto FinishCode = [](bool bHadContent, FStringBuilderBase& CodeBuilder)
				{
					if (bHadContent)
					{
						CodeBuilder += TEXT("})");
					}
					else
					{
						CodeBuilder = { };
					}
				};

				FinishCode(bHasSetNodes, VariableSetNodeCode);
				FinishCode(bHasGetNodes, VariableGetNodeCode);
				FinishCode(bHasGetDelayedNodes, VariableGetDelayedNodeCode);
			}

			// Edges from inputs are not distinct in reflected code as inputs are characterized
			// visually by template nodes which are included in the typical edge addition case.
			void ReflectEdges()
			{
				if (Layer.Graph.Edges.IsEmpty())
				{
					return;
				}

				bool bHasNodeEdges = false;
				EdgesCode = TEXT("\n// Edges\n");
				EdgesCode += TEXT(".Add<FDC::FEdge>(\n{\n");

				bool bHasOutputEdges = false;
				OutputEdgesCode = TEXT("\n// Output Edges\n");
				OutputEdgesCode += TEXT(".Add<FDC::FEdgeToOutput>(\n{\n");

				for (const FMetasoundFrontendEdge& Edge : Layer.Graph.Edges)
				{
					// Edges between template nodes and inputs are "hidden" to mirror
					// design characterized in editor layer and make config code more
					// readable. Edges between associated inputs and template nodes
					// are therefore added via the Add(FInput) function and not here,
					// so this case early outs.
					if (TemplateNodeIDToInputName.Contains(Edge.ToNodeID))
					{
						continue;
					}

					if (const FConfigNodeInfo* FromNodeInfo = NodeIDToInfo.Find(Edge.FromNodeID))
					{
						const FMetasoundFrontendNode* FromNode = FromNodeInfo->Node;
						check(FromNode);

						auto IsFromVertex = [&Edge](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == Edge.FromVertexID; };
						const FMetasoundFrontendVertex* FromVertex = FromNode->Interface.Outputs.FindByPredicate(IsFromVertex);
						if (!FromVertex)
						{
							continue;
						}

						if (const FConfigNodeInfo* ToNodeInfo = NodeIDToInfo.Find(Edge.ToNodeID))
						{
							auto GetClassInfoIfOfType = [this](const FConfigNodeInfo& NodeInfo, EMetasoundFrontendClassType Type) -> const FConfigClassInfo*
							{
								check(NodeInfo.Node);
								if (const FConfigClassInfo* ClassInfo = ClassIDToInfo.Find(NodeInfo.Node->ClassID))
								{
									const FMetasoundFrontendClass* Class = ClassInfo->Class;
									check(Class);
									if (Class && Class->Metadata.GetType() == Type)
									{
										return ClassInfo;
									}
								}
								return nullptr;
							};
							if (const FConfigClassInfo* ClassInfo = GetClassInfoIfOfType(*ToNodeInfo, EMetasoundFrontendClassType::Output))
							{
								if (const FMetasoundFrontendClassOutput* Output = FindOutput(Edge.ToNodeID))
								{
									bHasOutputEdges = true;
									OutputEdgesCode.Appendf(
										TEXT("\t{ .Node = \"%s\", .NodeOutput = \"%s\", .GraphOutput = \"%s\" },\n"),
										*ToNodeInfo->InstanceName,
										*FromVertex->Name.ToString(),
										*Output->Name.ToString()
									);
								}
								continue;
							}

							const FMetasoundFrontendNode* ToNode = ToNodeInfo->Node;
							check(ToNode);

							auto IsToVertex = [&Edge](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == Edge.ToVertexID; };
							const FMetasoundFrontendVertex* ToVertex = ToNode->Interface.Inputs.FindByPredicate(IsToVertex);
							if (!ToVertex)
							{
								continue;
							}

							bHasNodeEdges = true;
							EdgesCode.Appendf(
								TEXT("\t{ .OutputNode = \"%s\", .Output = \"%s\", .InputNode = \"%s\", .Input = \"%s\" },\n"),
								*FromNodeInfo->InstanceName,
								*FromVertex->Name.ToString(),
								*ToNodeInfo->InstanceName,
								*ToVertex->Name.ToString()
							);
						}
					}
				}

				auto FinishEdgeCode = [](bool bHadContent, FStringBuilderBase& CodeBuilder)
				{
					if (bHadContent)
					{
						CodeBuilder += TEXT("})");
					}
					else
					{
						CodeBuilder = { };
					}
				};

				FinishEdgeCode(bHasNodeEdges, EdgesCode);
				FinishEdgeCode(bHasOutputEdges, OutputEdgesCode);
			}

			void ReflectInterfaces()
			{
				using namespace Frontend;

				if (Layer.Inputs.IsEmpty() && Layer.Outputs.IsEmpty())
				{
					return;
				}

				TSet<FMetasoundFrontendVersion> Interfaces;

				IInterfaceRegistry& InterfaceRegistry = IInterfaceRegistry::Get();

				auto GetMemberInterface = [&InterfaceRegistry](FName MemberName) -> TOptional<FMetasoundFrontendVersion>
				{
					FName InterfaceNamespace;
					FName ParamName;
					Audio::FParameterPath::SplitName(MemberName, InterfaceNamespace, ParamName);

					if (!InterfaceNamespace.IsNone())
					{
						FMetasoundFrontendVersion InterfaceVersion;
						const bool bFoundVersion = ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceNamespace, InterfaceVersion);
						if (bFoundVersion)
						{
							return InterfaceVersion;
						}
					}

					return { };
				};

				for (int32 Index = Layer.Inputs.Num() - 1; Index >= 0; --Index)
				{
					FMetasoundFrontendClassInput& Input = Layer.Inputs[Index];
					TOptional<FMetasoundFrontendVersion> InterfaceVersion = GetMemberInterface(Input.Name);
					if (InterfaceVersion.IsSet() && InterfaceRegistry.ContainsInput(InterfaceVersion.GetValue(), Input))
					{
						Interfaces.Add(InterfaceVersion.GetValue());
						Layer.Inputs.RemoveAtSwap(Index, EAllowShrinking::No);
					}
				}
				Layer.Inputs.Shrink();

				for (int32 Index = Layer.Outputs.Num() - 1; Index >= 0; --Index)
				{
					FMetasoundFrontendClassOutput& Output = Layer.Outputs[Index];
					TOptional<FMetasoundFrontendVersion> InterfaceVersion = GetMemberInterface(Output.Name);
					if (InterfaceVersion.IsSet() && InterfaceRegistry.ContainsOutput(InterfaceVersion.GetValue(), Output))
					{
						Interfaces.Add(InterfaceVersion.GetValue());
						InterfaceOutputs.Add(MoveTemp(Layer.Outputs[Index]));
						Layer.Outputs.RemoveAtSwap(Index, EAllowShrinking::No);
					}
				}
				Layer.Outputs.Shrink();

				if (!Interfaces.IsEmpty())
				{
					InterfaceCode = TEXT("\n// Interfaces\n");
					InterfaceCode += TEXT(".Add<FDC::FInterface>(\n{\n");
					for (const FMetasoundFrontendVersion& Version : Interfaces)
					{
						InterfaceCode += FString::Format(TEXT("\t{ .Version = { .Name = \"{0}\", .Number = { .Major = {1}, .Minor = {2} } } },\n"),
						{
							*Version.Name.ToString(),
							Version.Number.Major,
							Version.Number.Minor
						});
					}

					InterfaceCode += TEXT("})");
				}
			}

			void ReflectInputs()
			{
				if (Layer.Inputs.IsEmpty())
				{
					return;
				}

				InputCode = TEXT("\n// Inputs\n");
				InputCode += TEXT(".Add<FDC::FInput>(\n{\n");

				FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
				for (FMetasoundFrontendClassInput& Input : Layer.Inputs)
				{
					// Configurations/MetaSound Builder Subsystem currently do not support pages,
					// so attempt to harvest literal with Default PageID.  Otherwise, take any value
					// (i.e. undefined behavior).
					FMetasoundFrontendLiteral InitValue = FMetasoundFrontendLiteral::GetInvalid();
					Input.IterateDefaults([&InitValue](const FGuid& PageID, FMetasoundFrontendLiteral& Literal)
					{
						if (PageID == Frontend::DefaultPageID || !InitValue.IsValid())
						{
							InitValue = MoveTemp(Literal);
						}
					});

					InputCode += TEXT("\t{ ");
					{
						InputCode.Appendf(TEXT(".Name = \"%s\", "), *Input.Name.ToString());
						InputCode.Appendf(TEXT(".DataType = \"%s\", "), *Input.TypeName.ToString());
						InputCode.Appendf(TEXT(".DefaultValue = ")); FormatDefaultCode(InitValue, InputCode);
						InputCode.Appendf(TEXT(".bIsConstructorInput = %s"), Input.AccessType == EMetasoundFrontendVertexAccessType::Value ? TEXT("true") : TEXT("false"));
					}
					InputCode += TEXT("},\n");

				}

				InputCode += TEXT("})");
			}

			void ReflectNodeInputDefaults()
			{
				TStringBuilder<0> LiteralsCode;

				for (const FMetasoundFrontendNode& Node : Layer.Graph.Nodes)
				{
					if (const FConfigNodeInfo* NodeInfo = NodeIDToInfo.Find(Node.GetID()))
					{
						for (const FMetasoundFrontendVertexLiteral& VertexLiteral : Node.InputLiterals)
						{
							auto IsVertexWithID = [&VertexLiteral](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == VertexLiteral.VertexID; };
							if (const FMetasoundFrontendVertex* InputVertex = Node.Interface.Inputs.FindByPredicate(IsVertexWithID))
							{
								LiteralsCode += TEXT("\t{ ");
								{
									LiteralsCode.Appendf(TEXT(".Node = \"%s\", "), *NodeInfo->InstanceName);
									LiteralsCode.Appendf(TEXT(".Input = \"%s\", "), *InputVertex->Name.ToString());
									LiteralsCode.Appendf(TEXT(".Value = ")); FormatDefaultCode(VertexLiteral.Value, LiteralsCode);
								}
								LiteralsCode += TEXT("},\n");
							}
						}
					}
				}

				FStringView LiteralsView = LiteralsCode.ToView();
				if (!LiteralsView.IsEmpty())
				{
					NodeInputDefaultsCode = TEXT("\n// Node Input Defaults\n");
					NodeInputDefaultsCode += TEXT(".Set<FDC::FNodeInputDefault>(\n{\n");
					NodeInputDefaultsCode += LiteralsView;
					NodeInputDefaultsCode += TEXT("})");
				}
			}

			void ReflectNodes()
			{
				using namespace Frontend;

				if (Layer.Graph.Nodes.IsEmpty())
				{
					return;
				}

				auto CacheNodeInfo = [this](const FString& BaseName, const FMetasoundFrontendNode& Node)
				{
					int32 Index = 0;
					FString FullName = BaseName;
					while (NodeInstanceNames.Contains(FullName))
					{
						FullName = FString::Printf(TEXT("%s_%i"), *BaseName, ++Index);
					}
					NodeInstanceNames.Add(FullName);
					NodeIDToInfo.Add(Node.GetID(), FConfigNodeInfo { .InstanceName = FullName, .Node = &Node });

					return FullName;
				};

				NodeCode = TEXT("\n// Nodes\n");
				NodeCode += TEXT(".Add<FDC::FNode>(\n{\n");

#if WITH_EDITORONLY_DATA
				InputNodeCode = TEXT("\n// Input (Template) Nodes\n");
				InputNodeCode += TEXT(".Add<FDC::FInputNode>(\n{\n");
#endif // WITH_EDITORONLY_DATA

				bool bHasNodes = false;
				bool bHasInputNodes = false;
				FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
				for (const FMetasoundFrontendNode& Node : Layer.Graph.Nodes)
				{
					if (const FConfigClassInfo* ClassInfo = ClassIDToInfo.Find(Node.ClassID))
					{
						const FMetasoundFrontendClass* Class = ClassInfo->Class;
						check(Class);

						FVector2D Location = FVector2D::Zero();
						if (!Node.Style.Display.Locations.IsEmpty())
						{
							Location = Node.Style.Display.Locations.Array().Last().Value;
						}

						const FMetasoundFrontendClassName& ClassName = Class->Metadata.GetClassName();
						const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
						switch (ClassType)
						{
							case EMetasoundFrontendClassType::Template:
							{
								if (ClassName != FInputNodeTemplate::GetChecked().ClassName)
								{
									UE_LOGF(LogMetaSound, Error,
										"Template node of class '%ls' currently not supported for configuration reflection"
										, *Class->Metadata.GetClassName().ToString());
									continue;
								}

								if (const FName* InputName = TemplateNodeIDToInputName.Find(Node.GetID()))
								{
									bHasInputNodes = true;
									InputNodeCode.Appendf(
										TEXT("\t{ .NodeName = \"%s\", .InputName = \"%s\", .Location = FVector2D(%sf, %sf) },\n"),
										*CacheNodeInfo(InputName->ToString(), Node),
										*InputName->ToString(),
										*FString::SanitizeFloat(Location.X),
										*FString::SanitizeFloat(Location.Y)
									);
								}
							}
							break;

							case EMetasoundFrontendClassType::External:
							{
								FString ClassString;
								FString BaseName;
								if (AssetManager.IsAssetClass(Class->Metadata))
								{
									const FTopLevelAssetPath AssetPath = AssetManager.FindAssetPath(FMetaSoundAssetKey(Class->Metadata));
									check(AssetPath.IsValid());

									BaseName = AssetPath.GetAssetName().ToString();
									ClassString = FString::Printf(TEXT(".AssetPath = FTopLevelAssetPath(TEXT(\"%s\"))"), *AssetPath.ToString());
								}
								else
								{
									BaseName = ClassName.Name.ToString();
									ClassString = FString::Printf(TEXT(".ClassName = { \"%s\", \"%s\", \"%s\" }"),
										*ClassName.Namespace.ToString(),
										*ClassName.Name.ToString(),
										*ClassName.Variant.ToString()
									);
								}

								bHasNodes = true;
								NodeCode.Appendf(
									TEXT("\t{ .Name = \"%s\", %s, .Location = FVector2D(%sf, %sf) },\n"),
									*CacheNodeInfo(BaseName, Node),
									*ClassString,
									*FString::SanitizeFloat(Location.X),
									*FString::SanitizeFloat(Location.Y)
								);
								break;
							}

							case EMetasoundFrontendClassType::Input:
							case EMetasoundFrontendClassType::Output:
							{
								FString NodeName = FString::Printf(TEXT("%s_%s"), LexToString(ClassType), *Node.Name.ToString());
								// Don't reflect as code as input/output is included in either a. the analogous
								// reflection call OR reflecting associated interfaces. However, node info is
								// cached for use when reflecting inputs/outputs.
								CacheNodeInfo(NodeName, Node);
							}
							break;

							case EMetasoundFrontendClassType::Variable:
							case EMetasoundFrontendClassType::VariableAccessor:
							case EMetasoundFrontendClassType::VariableDeferredAccessor:
							case EMetasoundFrontendClassType::VariableMutator:
							{
								// Variable nodes are reflected via ReflectVariables, not here.
							}
							break;

							default:
							{
								break;
							}
						}
					}
				}

				auto FinishNodeCode = [](bool bHadContent, FStringBuilderBase& CodeBuilder)
				{
					if (bHadContent)
					{
						CodeBuilder += TEXT("})");
					}
					else
					{
						CodeBuilder = { };
					}
				};

				FinishNodeCode(bHasNodes, NodeCode);
				FinishNodeCode(bHasInputNodes, InputNodeCode);
			}

			void ReflectOutputs()
			{
				if (Layer.Outputs.IsEmpty())
				{
					return;
				}

				OutputCode = TEXT("\n// Outputs\n");
				OutputCode += TEXT(".Add<FDC::FOutput>(\n{\n");

				FMetaSoundAssetManager& AssetManager = FMetaSoundAssetManager::GetChecked();
				for (const FMetasoundFrontendClassOutput& Output : Layer.Outputs)
				{
					FVector2D Location = FVector2D::Zero();

					if (const FConfigNodeInfo* NodeInfo = NodeIDToInfo.Find(Output.NodeID))
					{
						const FMetasoundFrontendNode* Node = NodeInfo->Node;
						check(Node);

						if (!Node->Style.Display.Locations.IsEmpty())
						{
							Location = Node->Style.Display.Locations.Array().Last().Value;
						}

						OutputCode.Appendf(
							TEXT("\t{ .Name = \"%s\", .DataType = \"%s\", .Location = FVector2D(%sf, %sf), .bIsConstructorOutput = %s }\n"),
							*Output.Name.ToString(),
							*Output.TypeName.ToString(),
							*FString::SanitizeFloat(FMath::RoundHalfToEven(Location.X)),
							*FString::SanitizeFloat(FMath::RoundHalfToEven(Location.Y)),
							Output.AccessType == EMetasoundFrontendVertexAccessType::Value ? TEXT("true") : TEXT("false")
						);
					}
				}

				OutputCode += TEXT("})");
			}

		public:
			FConfigReflector(FMetasoundFrontendGraphLayer InLayer)
				: Layer(MoveTemp(InLayer))
			{
			}

			FString Execute()
			{
				FString SetBuildPageCode;
				{
					const FName PageName = UMetaSoundSettings::GetPageName(Layer.Graph.PageID);
					if (PageName != Frontend::DefaultPageName)
					{
						SetBuildPageCode = FString::Printf(TEXT("\n// Page Setup\n.SetBuildPage(\"%s\")\n"), *PageName.ToString());
					}
				}

				CacheDependencies();

#if WITH_EDITORONLY_DATA
				CacheInputNodes();
#endif // WITH_EDITORONLY_DATA

				// Strips inputs/outputs found to be registered to interfaces and directly adds applicable interfaces
				ReflectInterfaces();

				// Nodes reflected first to cache node data used by input/output reflection
				ReflectNodes();

				ReflectInputs();
				ReflectOutputs();
				ReflectVariables();
				ReflectEdges();

				ReflectNodeInputDefaults();

				TStringBuilder<0> LayerCode;
				auto AddSection = [&LayerCode](FStringBuilderBase& SectionCode, bool bNextLine = true)
				{
					FStringView SectionView = SectionCode.ToView();
					if (!SectionView.IsEmpty())
					{
						LayerCode.Append(SectionView);
						if (bNextLine)
						{
							LayerCode.AppendChar('\n');
						}
					}
					SectionCode = { };
				};

				LayerCode.Append(TEXT("using namespace Metasound::Frontend;\n"));
				LayerCode.Append(TEXT("using FDC = Metasound::Engine::FDocumentConfigurator;\n"));
				LayerCode.Append(TEXT("FDC Configurator(FDC::FArgs { .Builder = &OutBuilder });\n"));
				LayerCode.Append(TEXT("return Configurator\n"));

				if (!SetBuildPageCode.IsEmpty())
				{
					LayerCode.Append(SetBuildPageCode);
				}

				AddSection(InterfaceCode);
				AddSection(InputCode);
				AddSection(OutputCode);

				AddSection(VariableCode);
				AddSection(VariableSetNodeCode);
				AddSection(VariableGetNodeCode);
				AddSection(VariableGetDelayedNodeCode);

				AddSection(InputNodeCode);
				AddSection(NodeCode);
				AddSection(NodeInputDefaultsCode);

				AddSection(EdgesCode);
				AddSection(OutputEdgesCode);

				LayerCode.Append(TEXT(".Succeeded();\n"));
				return FString(LayerCode);
			}
		};
	} // namespace ConfigCodeReflectorPrivate

#if WITH_EDITORONLY_DATA
	FDocumentConfigurator::FDocumentConfigurator(UMetaSoundBuilderBase& InBuilder)
	{
		Builder.Reset(&InBuilder);
	}
#endif // WITH_EDITORONLY_DATA

	FDocumentConfigurator::FDocumentConfigurator(const FDocumentConfigurator::FArgs& Args)
#if WITH_EDITORONLY_DATA
		: Offset(Args.Offset)
#endif // WITH_EDITORONLY_DATA
	{
		checkf(Args.Builder, TEXT("Builder must be supplied via document configurator arguments"));

		UObject& MetaSoundObject = Args.Builder->CastDocumentObjectChecked<UObject>();
		Builder.Reset(&FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSoundObject));

		checkf(Args.Builder == &Builder->GetConstBuilder(), TEXT("Builder provided in arguments differs from that registered with the DocumentBuilderRegistry."));

		if (Args.bResetDoc)
		{
			ResetDocument();
		}
	}

	FDocumentConfigurator::~FDocumentConfigurator()
	{
		// Inject result is ignored because 1. the configurator has already reported success if its destroying,
		// and 2. the inject can fail if there are no inputs which is technically a valid situation.
		EMetaSoundBuilderResult InjectResult = EMetaSoundBuilderResult::Failed;
		constexpr bool bForceNodeCreation = false;
		Builder->InjectInputTemplateNodes(bForceNodeCreation, InjectResult);
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FInterface Interface)
	{
		const bool bSucceeded = Builder->GetBuilder().ModifyInterfaces(Frontend::FModifyInterfaceOptions(
			{ }, // None removed
			{ Interface.Version }
		));

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = bSucceeded ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FInput Input)
	{
		using namespace Frontend;

		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		FMetaSoundBuilderNodeOutputHandle OutputHandle = Builder->AddGraphInputNode(Input.Name, Input.DataType, MoveTemp(Input.DefaultValue), StepResult, Input.bIsConstructorInput);
		if (StepResult == EMetaSoundBuilderResult::Succeeded)
		{
			Builder->SetNodeLocation(OutputHandle.NodeID, Input.Location + Offset, StepResult);
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

#if WITH_EDITORONLY_DATA
	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FInputNode InputNode)
	{
		using namespace Frontend;

		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		FMetaSoundNodeHandle NodeHandle;
		if (NodeNameToHandle.Contains(InputNode.NodeName))
		{
			UE_LOGF(LogMetaSound, Error,
				"Node with name '%ls' already exists. Failed to add node instance to MetaSound graph",
				*InputNode.InputName.ToString());
			StepResult = EMetaSoundBuilderResult::Failed;
		}
		else
		{
			if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FInputNodeTemplate::ClassName); ensure(Template))
			{
				if (const FMetasoundFrontendClassInput* Input = Builder->GetConstBuilder().FindGraphInput(InputNode.InputName))
				{
					FNodeTemplateGenerateInterfaceParams Params { .InputsToConnect = { Input->TypeName }, .OutputsToConnect = { Input->TypeName } };
					if (const FMetasoundFrontendNode* TemplateNode = Builder->GetBuilder().AddNodeByTemplate(*Template, MoveTemp(Params), FGuid::NewGuid()))
					{
						Builder->ConnectGraphInputToNode(InputNode.InputName, FMetaSoundNodeHandle(TemplateNode->GetID()), "Value", StepResult);
						if (StepResult == EMetaSoundBuilderResult::Succeeded)
						{
							NodeHandle.NodeID = TemplateNode->GetID();
							NodeNameToHandle.Add(InputNode.NodeName, NodeHandle);
							Builder->SetNodeLocation(NodeHandle, InputNode.Location + Offset, StepResult);
						}
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Error,
						"Input Node could not be added: Input '%ls' not found (must add input first)",
						*InputNode.InputName.ToString());
				}
			}
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}
#endif // WITH_EDITORONLY_DATA

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FOutput Output)
	{
		using namespace Frontend;

		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		Builder->AddGraphOutputNode(Output.Name, Output.DataType, MoveTemp(Output.DefaultValue), StepResult, Output.bIsConstructorOutput);
		if (StepResult == EMetaSoundBuilderResult::Failed)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FVariable Variable)
	{
		const FMetasoundFrontendVariable* NewVariable = Builder->GetBuilder().AddGraphVariable(Variable.Name, Variable.DataType, &Variable.DefaultValue);

		if (!NewVariable)
		{
			Result = EMetaSoundBuilderResult::Failed;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FVariableSetNode Node)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (NodeNameToHandle.Contains(Node.Name))
		{
			UE_LOGF(LogMetaSound, Error, "Node with name '%ls' already exists. Failed to add variable set node to MetaSound graph", *Node.Name.ToString());
		}
		else
		{
			if (const FMetasoundFrontendNode* MutatorNode = Builder->GetBuilder().AddGraphVariableMutatorNode(Node.VariableName, FGuid::NewGuid()))
			{
				FMetaSoundNodeHandle NodeHandle;
				NodeHandle.NodeID = MutatorNode->GetID();
				NodeNameToHandle.Add(Node.Name, NodeHandle);
				Builder->SetNodeLocation(NodeHandle, Node.Location + Offset, StepResult);
			}
			else
			{
				StepResult = EMetaSoundBuilderResult::Failed;
			}
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FVariableGetNode Node)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (NodeNameToHandle.Contains(Node.Name))
		{
			UE_LOGF(LogMetaSound, Error, "Node with name '%ls' already exists. Failed to add variable get node to MetaSound graph", *Node.Name.ToString());
		}
		else
		{
			if (const FMetasoundFrontendNode* AccessorNode = Builder->GetBuilder().AddGraphVariableAccessorNode(Node.VariableName, FGuid::NewGuid()))
			{
				FMetaSoundNodeHandle NodeHandle;
				NodeHandle.NodeID = AccessorNode->GetID();
				NodeNameToHandle.Add(Node.Name, NodeHandle);
				Builder->SetNodeLocation(NodeHandle, Node.Location + Offset, StepResult);
			}
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FVariableGetDelayedNode Node)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (NodeNameToHandle.Contains(Node.Name))
		{
			UE_LOGF(LogMetaSound, Error, "Node with name '%ls' already exists. Failed to add variable get delayed node to MetaSound graph", *Node.Name.ToString());
		}
		else
		{
			if (const FMetasoundFrontendNode* AccessorNode = Builder->GetBuilder().AddGraphVariableDeferredAccessorNode(Node.VariableName, FGuid::NewGuid()))
			{
				FMetaSoundNodeHandle NodeHandle;
				NodeHandle.NodeID = AccessorNode->GetID();
				NodeNameToHandle.Add(Node.Name, NodeHandle);
				Builder->SetNodeLocation(NodeHandle, Node.Location + Offset, StepResult);
			}
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FNode Node)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		FMetaSoundNodeHandle NodeHandle;
		if (NodeNameToHandle.Contains(Node.Name))
		{
			UE_LOGF(LogMetaSound, Error, "Node with name '%ls' already exists. Failed to add node instance to MetaSound graph", *Node.Name.ToString());
			Result = EMetaSoundBuilderResult::Failed;
		}
		else if (Node.ClassName.IsValid())
		{
			if (const FMetasoundFrontendNode* NewNode = Builder->GetBuilder().AddNodeByClassName(Node.ClassName, 1, FGuid::NewGuid()))
			{
				NodeHandle.NodeID = NewNode->GetID();
				StepResult = EMetaSoundBuilderResult::Succeeded;
			}
		}
		else if (Node.AssetPath.IsValid())
		{
			const FSoftObjectPath MetaSoundPath(Node.AssetPath);
			if (TScriptInterface<IMetaSoundDocumentInterface> DocInterface = MetaSoundPath.TryLoad(); DocInterface.GetObject())
			{
				NodeHandle = Builder->AddNode(DocInterface, StepResult);
			}
		}

		if (StepResult == EMetaSoundBuilderResult::Succeeded)
		{
			NodeNameToHandle.Add(Node.Name, NodeHandle);
			Builder->SetNodeLocation(NodeHandle, Node.Location + Offset, StepResult);
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FEdge Edge)
	{
		using namespace Frontend;

		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (const FMetaSoundNodeHandle* OutputNodeHandle = NodeNameToHandle.Find(Edge.OutputNode))
		{
			if (const FMetaSoundNodeHandle* InputNodeHandle = NodeNameToHandle.Find(Edge.InputNode))
			{
				TSet<FNamedEdge> Edges;
				Edges.Add(FNamedEdge { OutputNodeHandle->NodeID, Edge.Output, InputNodeHandle->NodeID, Edge.Input });
				StepResult = Builder->GetBuilder().AddNamedEdges(Edges, nullptr, true)
					? EMetaSoundBuilderResult::Succeeded
					: EMetaSoundBuilderResult::Failed;
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Failed to find input node with name '%ls' to connect", *Edge.InputNode.ToString());
				StepResult = EMetaSoundBuilderResult::Failed;
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Failed to find output node with name '%ls' to connect", *Edge.OutputNode.ToString());
			StepResult = EMetaSoundBuilderResult::Failed;
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FEdgeFromInput Edge)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (const FMetaSoundNodeHandle* DestNodeHandle = NodeNameToHandle.Find(Edge.Node))
		{
			const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
			if (const FMetasoundFrontendNode* GraphInputNode = ConstBuilder.FindGraphInputNode(Edge.GraphInput))
			{
				using namespace Frontend;
				const FMetasoundFrontendVertex& OutputVertex = GraphInputNode->Interface.Outputs.Last();
				TSet<FNamedEdge> Edges;
				Edges.Add(FNamedEdge { GraphInputNode->GetID(), OutputVertex.Name, DestNodeHandle->NodeID, Edge.NodeInput });
				StepResult = Builder->GetBuilder().AddNamedEdges(Edges, nullptr, true)
					? EMetaSoundBuilderResult::Succeeded
					: EMetaSoundBuilderResult::Failed;
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Failed to find node with name '%ls': Graph input to node input connection failed", *Edge.Node.ToString());
			StepResult = EMetaSoundBuilderResult::Failed;
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Add(FDocumentConfigurator::FEdgeToOutput Edge)
	{
		using namespace Frontend;

		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;

		if (const FMetaSoundNodeHandle* SourceNodeHandle = NodeNameToHandle.Find(Edge.Node))
		{
			const FMetaSoundFrontendDocumentBuilder& ConstBuilder = Builder->GetConstBuilder();
			if (const FMetasoundFrontendNode* GraphOutputNode = ConstBuilder.FindGraphOutputNode(Edge.GraphOutput))
			{
				const FMetasoundFrontendVertex& InputVertex = GraphOutputNode->Interface.Inputs.Last();
				TSet<FNamedEdge> Edges;
				Edges.Add(FNamedEdge { SourceNodeHandle->NodeID, Edge.NodeOutput, GraphOutputNode->GetID(), InputVertex.Name });
				StepResult = Builder->GetBuilder().AddNamedEdges(Edges, nullptr, true)
					? EMetaSoundBuilderResult::Succeeded
					: EMetaSoundBuilderResult::Failed;
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Failed to find node with name '%ls': Graph output to node input connection failed", *Edge.Node.ToString());
			StepResult = EMetaSoundBuilderResult::Failed;
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::AddInternal(TArray<FDocumentConfigurator::FInterface> Interfaces)
	{
		TArray<FMetasoundFrontendVersion> Versions;
		Algo::Transform(Interfaces, Versions, [](const FInterface& Interface) { return Interface.Version; });
		const bool bSucceeded = Builder->GetBuilder().ModifyInterfaces(Frontend::FModifyInterfaceOptions(
			{ }, // None removed
			{ Versions }
		));

		if (!bSucceeded)
		{
			Result = EMetaSoundBuilderResult::Failed;
		}

		return *this;
	}

	UMetaSoundBuilderBase& FDocumentConfigurator::GetBuilder()
	{
		return *Builder.Get();
	}
	
#if WITH_EDITORONLY_DATA
	void FDocumentConfigurator::Reflect(FMetasoundFrontendGraphLayer InLayer, FString& OutCode)
	{
		using namespace ConfigCodeReflectorPrivate;
		FConfigReflector Reflector(MoveTemp(InLayer));
		OutCode = Reflector.Execute();
	}
#endif // WITH_EDITORONLY_DATA

	FDocumentConfigurator& FDocumentConfigurator::ResetDocument()
	{
		using namespace Frontend;

		FMetaSoundFrontendDocumentBuilder& FrontendBuilder = Builder->GetBuilder();
		TSharedRef<FDocumentModifyDelegates> ModifyDelegates = MakeShared<FDocumentModifyDelegates>(FrontendBuilder.GetDocumentDelegates());
		FrontendBuilder.ClearDocument(ModifyDelegates);

		FrontendBuilder.SetIsGraphEditable(false);

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::SetBuildPage(FName PageName)
	{
		if (const FGuid* PageID = UMetaSoundSettings::GetPageID(PageName))
		{
			Builder->GetBuilder().SetBuildPageID(*PageID);
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "FDocumentConfigurator::SetBuildPage: Page '%ls' not found in MetaSound settings", *PageName.ToString());
			Result = EMetaSoundBuilderResult::Failed;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Set(FDocumentConfigurator::FInputLocation Location)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;
		if (const FMetaSoundNodeHandle InputNode = Builder->FindGraphInputNode(Location.Name, StepResult); StepResult == EMetaSoundBuilderResult::Succeeded)
		{
			Builder->SetNodeLocation(InputNode, Location.Location + Offset, StepResult);
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	FDocumentConfigurator& FDocumentConfigurator::Set(FDocumentConfigurator::FNodeInputDefault Input)
	{
		EMetaSoundBuilderResult StepResult = EMetaSoundBuilderResult::Failed;
		if (const FMetaSoundNodeHandle* NodeHandle = NodeNameToHandle.Find(Input.Node))
		{
			FMetaSoundBuilderNodeInputHandle InputHandle = Builder->FindNodeInputByName(*NodeHandle, Input.Input, StepResult);
			if (StepResult == EMetaSoundBuilderResult::Succeeded)
			{
				Builder->SetNodeInputDefault(InputHandle, Input.Value, StepResult);
			}
		}

		if (Result == EMetaSoundBuilderResult::Succeeded)
		{
			Result = StepResult;
		}

		return *this;
	}

	bool FDocumentConfigurator::Succeeded() const
	{
		return Result == EMetaSoundBuilderResult::Succeeded;
	}
} // namespace Metasound::Engine
#endif // WITH_EDITORONLY_DATA
