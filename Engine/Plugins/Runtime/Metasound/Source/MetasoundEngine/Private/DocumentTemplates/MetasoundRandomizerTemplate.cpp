// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentTemplates/MetasoundRandomizerTemplate.h"

#include "Algo/MaxElement.h"
#include "Algo/Transform.h"
#include "Containers/ContainersFwd.h"
#include "DocumentTemplates/MetasoundDocumentConfigurator.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDataTypeGetTraits.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundWave.h"
#include "Sound/SoundWave.h"
#include "UObject/SoftObjectPath.h"


#define LOCTEXT_NAMESPACE "MetaSound"

namespace Metasound::Engine
{
	namespace RandomizerPrivate
	{
		const FMetasoundFrontendNode* CreateWaveInput(FMetaSoundFrontendDocumentBuilder& OutBuilder, const TArray<TObjectPtr<USoundWave>>& InSounds)
		{
			using namespace Frontend;

			const FMetasoundFrontendDocument& Doc = OutBuilder.GetConstDocumentChecked();

			FMetasoundFrontendClassInput WaveInput;
			WaveInput.Name = "Sounds";
			WaveInput.TypeName = GetMetasoundDataTypeName<TArray<FWaveAsset>>();
			WaveInput.NodeID = FDocumentIDGenerator::Get().CreateNodeID(Doc);
			WaveInput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(Doc);
			WaveInput.AccessType = EMetasoundFrontendVertexAccessType::Value;

			TArray<UObject*> SoundObjects;
			Algo::TransformIf(InSounds, SoundObjects,
				[](const TObjectPtr<USoundWave> Sound) { return Sound != nullptr; },
				[](const TObjectPtr<USoundWave> Sound) { return Sound.Get(); });

			FMetasoundFrontendLiteral DefaultValue;
			DefaultValue.Set(MoveTemp(SoundObjects));
			WaveInput.InitDefault(MoveTemp(DefaultValue));

			return OutBuilder.AddGraphInput(MoveTemp(WaveInput));
		}

		int32 GetMaxChannels(const TArray<TObjectPtr<USoundWave>>& InSounds)
		{
			auto GetNumChannels = [](const TObjectPtr<USoundWave>& Sound)
			{
				if (const USoundWave* SoundPtr = Sound.Get())
				{
					return SoundPtr->NumChannels;
				}

				return 0;
			};

			if (const TObjectPtr<USoundWave>* MaxElement = Algo::MaxElementBy(InSounds, GetNumChannels))
			{
				return GetNumChannels(*MaxElement);
			}

			return 0;
		}

		FMetasoundFrontendClassName GetPlayerClassName(int32 NumChannels)
		{
			FMetasoundFrontendClassName ClassName(EngineNodes::Namespace, "Wave Player");
			if (NumChannels == 2)
			{
				ClassName.Variant = EngineNodes::StereoVariant;
			}
			else if (NumChannels == 4)
			{
				ClassName.Variant = EngineNodes::QuadVariant;
			}
			else if (NumChannels == 6)
			{
				ClassName.Variant = EngineNodes::FiveDotOneVariant;
			}
			else if (NumChannels == 8)
			{
				ClassName.Variant = EngineNodes::SevenDotOneVariant;
			}
			else
			{
				ClassName.Variant = EngineNodes::MonoVariant;

				if (NumChannels != 1)
				{
					UE_LOGF(LogMetaSound, Warning, "SoundWave format with %i NumChannels not recognized: using Mono MetaSound WavePlayer", NumChannels);
				}
			}
			return ClassName;
		}

		FMetasoundFrontendVersion GetOutputVersion(int32 NumChannels)
		{
			if (NumChannels == 2)
			{
				return OutputFormatStereoInterface::GetVersion();
			}

			if (NumChannels == 4)
			{
				return OutputFormatQuadInterface::GetVersion();
			}

			if (NumChannels == 6)
			{
				return OutputFormatFiveDotOneInterface::GetVersion();
			}

			if (NumChannels == 8)
			{
				return OutputFormatSevenDotOneInterface::GetVersion();
			}

			return OutputFormatMonoInterface::GetVersion();
		}

		TArray<FDocumentConfigurator::FEdgeToOutput> GetAudioOutputEdges(int32 NumChannels)
		{
			using FDC = FDocumentConfigurator;

			if (NumChannels == 2)
			{
				return
				{
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Left", .GraphOutput = OutputFormatStereoInterface::Outputs::LeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Right", .GraphOutput = OutputFormatStereoInterface::Outputs::RightOut }
				};
			}

			if (NumChannels == 4)
			{
				return
				{
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Left", .GraphOutput = OutputFormatQuadInterface::Outputs::FrontLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Right", .GraphOutput = OutputFormatQuadInterface::Outputs::FrontRightOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Left", .GraphOutput = OutputFormatQuadInterface::Outputs::SideLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Right", .GraphOutput = OutputFormatQuadInterface::Outputs::SideRightOut }
				};
			}

			if (NumChannels == 6)
			{
				return
				{
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Left", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Right", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::FrontRightOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Center", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::FrontCenterOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Low Frequency", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::LowFrequencyOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Left", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::SideLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Right", .GraphOutput = OutputFormatFiveDotOneInterface::Outputs::SideRightOut }
				};
			}

			if (NumChannels == 8)
			{
				return
				{
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Left", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Right", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::FrontRightOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Front Center", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::FrontCenterOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Low Frequency", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::LowFrequencyOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Left", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::SideLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Side Right", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::SideRightOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Back Left", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::BackLeftOut },
					FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Back Right", .GraphOutput = OutputFormatSevenDotOneInterface::Outputs::BackRightOut }
				};
			}

			if (NumChannels != 1)
			{
				UE_LOGF(LogMetaSound, Warning, "SoundWave format with %i NumChannels not recognized: Connecting only first channel as mono.", NumChannels);
			}

			return
			{
				FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "Out Mono", .GraphOutput = OutputFormatMonoInterface::Outputs::MonoOut }
			};
		}
	} // namespace RandomizerPrivate
} // namespace Metasound::Engine

#if WITH_EDITORONLY_DATA
bool FMetaSoundRandomizerTemplate::ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	using FDC = FDocumentConfigurator;

	// If max channels cannot be deduced, then return clear document and consider it valid as
	// user may not be supplying waves for a reason (or may be amidst editing array of sounds).
	const int32 MaxNumChannels = RandomizerPrivate::GetMaxChannels(Sounds);

	FDC Configurator(FDC::FArgs { .Builder = &OutBuilder });

	// Add interfaces first.  Done even if no sounds supplied to make sure resulting document is valid source.
	Configurator.Add<FDC::FInterface>(
	{
		{ .Version = SourceInterface::GetVersion() },
		{ .Version = RandomizerPrivate::GetOutputVersion(MaxNumChannels) }
	});

	if (bIsOneShot)
	{
		Configurator.Add(FDC::FInterface { .Version = SourceOneShotInterface::GetVersion() });
	}

	if (MaxNumChannels <= 0)
	{
		return true;
	}

	const FName WaveArrayTypeName = GetMetasoundDataTypeName<TArray<FWaveAsset>>();
	const FMetasoundFrontendClassName PlayerClassName = RandomizerPrivate::GetPlayerClassName(MaxNumChannels);
	const FMetasoundFrontendClassName RandomClassName = FMetasoundFrontendClassName("Array", "Random Get", WaveArrayTypeName);

	Configurator

	// Add Non-Interface Graph Inputs
	.Add(FDC::FInput { .Name = "Sounds", .DataType = WaveArrayTypeName, .DefaultValue = MakeFrontendLiteral<USoundWave>(Sounds), .Location = FVector2D(-500, 250) })

	// Set Interface Graph Input locations
	.Set(FDC::FInputLocation { .Name = "UE.Source.OnPlay", .Location = FVector2D(-500, 0)})

	// Add Nodes
	.Add<FDC::FNode>(
	{
		{ .Name = "Pitch", .ClassName = FMetasoundFrontendClassName("UE", "RandomFloat"), .Location = FVector2D(250, 400) },
		{ .Name = "RandomWave", .ClassName = RandomClassName, .Location = FVector2D(0, 0) },
		{ .Name = "WavePlayer", .ClassName = PlayerClassName, .Location = FVector2D(500, 0) }
	})

	.Set(FDC::FNodeInputDefault { .Node = "Pitch", .Input = "Min", .Value = MakeFrontendLiteral(FMath::Min(Pitch.X, Pitch.Y)) })
	.Set(FDC::FNodeInputDefault { .Node = "Pitch", .Input = "Max", .Value = MakeFrontendLiteral(FMath::Max(Pitch.X, Pitch.Y)) })

	// Connect Graph Inputs to Nodes
	.Add<FDC::FEdgeFromInput>(
	{
		{ .GraphInput = "Sounds",			.Node = "RandomWave", .NodeInput = "In Array" },
		{ .GraphInput = "UE.Source.OnPlay", .Node = "RandomWave", .NodeInput = "Next" }
	})

	// Connect Nodes Together
	.Add<FDC::FEdge>(
	{
		{ .OutputNode = "RandomWave", .Output = "On Next",	.InputNode = "WavePlayer", .Input = "Play" },
		{ .OutputNode = "RandomWave", .Output = "Value",	.InputNode = "WavePlayer", .Input = "Wave Asset" },
		{ .OutputNode = "Pitch",	  .Output = "Value",	.InputNode = "WavePlayer", .Input = "Pitch Shift" },
	})

	// Connect Nodes to Graph Outputs
	.Add<FDC::FEdgeToOutput>(RandomizerPrivate::GetAudioOutputEdges(MaxNumChannels));

	if (bIsOneShot)
	{
		Configurator.Add(FDC::FEdgeToOutput { .Node = "WavePlayer", .NodeOutput = "On Finished", .GraphOutput = SourceOneShotInterface::Outputs::OnFinished });
	}
	else
	{
		Configurator.Set(FDC::FNodeInputDefault { .Node = "WavePlayer", .Input = "Loop", .Value = MakeFrontendLiteral(true) });
	}

	return Configurator.Succeeded();
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
const FMetaSoundFrontendDocumentTemplate::FEditorOptions& FMetaSoundRandomizerTemplate::GetEditorOptions() const
{
	static const FEditorOptions RandomizerOptions
	{
		.AssetActionClasses = { USoundWave::StaticClass() },
		.bTemplatePropertiesVisible = true
	};
	return RandomizerOptions;
}

void FMetaSoundRandomizerTemplate::OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	Sounds.Reset();
	Algo::TransformIf(SelectedObjects, Sounds,
		[](const UObject* Object) { return Object->GetClass() == USoundWave::StaticClass(); }, // Has to match type exactly so procedural sounds, SourceBuses, MetaSounds, etc. aren't used
		[](UObject* Object) { return TObjectPtr<USoundWave>(CastChecked<USoundWave>(Object)); });
}

void FMetaSoundRandomizerTemplate::OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	if (InEvent.Property && InEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FMetaSoundRandomizerTemplate, Sounds))
	{
		RemoveInvalidSounds(OutBuilder, false /* bRemoveNullEntries */, Sounds);
	}

	Super::OnPropertyChanged(InEvent, OutBuilder);
}

bool FMetaSoundRandomizerTemplate::RemoveInvalidSounds(const FMetaSoundFrontendDocumentBuilder& ParentBuilder, bool bRemoveNullEntries, TArray<TObjectPtr<USoundWave>>& OutSounds)
{
	return OutSounds.RemoveAll([&bRemoveNullEntries](const TObjectPtr<USoundWave>& Wave)
	{
		if (bRemoveNullEntries && Wave == nullptr)
		{
			return true;
		}

		return Wave && Wave->GetClass() != USoundWave::StaticClass();
	}) > 0;
}
#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE
