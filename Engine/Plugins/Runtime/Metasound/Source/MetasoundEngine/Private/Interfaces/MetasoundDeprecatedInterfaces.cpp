// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundDeprecatedInterfaces.h"

#include "AudioParameter.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Interfaces/MetasoundInterface.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "MetasoundAudioFormats.h"
#include "MetasoundDataReference.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundInterface.h"
#include "MetasoundRouter.h"
#include "MetasoundSource.h"
#include "MetasoundUObjectRegistry.h"
#include "UObject/Class.h"
#include "UObject/NoExportTypes.h"

#define LOCTEXT_NAMESPACE "MetasoundEngine"

namespace Metasound::Engine
{
	namespace DeprecatedInterfacesPrivate
	{
		const TArray<FMetasoundFrontendInterfaceUClassOptions>& GetDeprecatedClassOptions()
		{
			auto GetOptions = []()
			{
				constexpr bool bIsModifiable = false;
				TArray<FMetasoundFrontendInterfaceUClassOptions> Options;
				Options.Add(FMetasoundFrontendInterfaceUClassOptions(UMetaSoundPatch::StaticClass()->GetClassPathName(), bIsModifiable));
				Options.Add(FMetasoundFrontendInterfaceUClassOptions(UMetaSoundSource::StaticClass()->GetClassPathName(), bIsModifiable));
				return Options;
			};
			static const TArray<FMetasoundFrontendInterfaceUClassOptions> Options = GetOptions();
			return Options;
		}

		auto MatchMemberNamesIgnoreSpaces = [] (FName InNameA, FName InNameB)
		{
			FName ParamA;
			FName ParamB;
			FName Namespace;
			Audio::FParameterPath::SplitName(InNameA, Namespace, ParamA);
			Audio::FParameterPath::SplitName(InNameB, Namespace, ParamB);

			auto GetStringNoWhitespace = [](FName InName) { return InName.ToString().Replace(TEXT(" "), TEXT("")); };
			return GetStringNoWhitespace(ParamA) == GetStringNoWhitespace(ParamB);
		};

		const FVertexName& GetOnPlayInputName()
		{
			static const FVertexName OnPlayInputName = "On Play";
			return OnPlayInputName;
		}

		const FVertexName& GetIsFinishedOutputName()
		{
			static const FVertexName OnFinishedOutputName = "On Finished";
			return OnFinishedOutputName;
		}

		const FVertexName& GetAudioDeviceIDVariableName()
		{
			static const FVertexName AudioDeviceIDVarName = "AudioDeviceID";
			return AudioDeviceIDVarName;
		}

		const FVertexName& GetGraphName()
		{
			static const FVertexName SoundGraphName = "GraphName";
			return SoundGraphName;
		}
	}

	// MetasoundV1_0 is a metasound without any required inputs or outputs.
	namespace MetasoundV1_0
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version { "MetaSound", FMetasoundFrontendVersionNumber {1, 0} };
			return Version;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();
			return Interface;
		}
	}

	namespace MetasoundOutputFormatMonoV1_0
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version { "MonoSource", FMetasoundFrontendVersionNumber {1, 0} };
			return Version;
		}

		const FVertexName& GetAudioOutputName()
		{
			static const FVertexName AudioOutputName = "Generated Audio";
			return AudioOutputName;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();

			// Inputs
			FMetasoundFrontendClassVertex OnPlayTrigger;
			OnPlayTrigger.Name = DeprecatedInterfacesPrivate::GetOnPlayInputName();
			OnPlayTrigger.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnPlayTrigger.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnPlayTrigger.Metadata.SetDisplayName(LOCTEXT("OnPlay", "On Play"));
			OnPlayTrigger.Metadata.SetDescription(LOCTEXT("OnPlayTriggerToolTip", "Trigger executed when this source is played."));
#endif // WITH_EDITOR

			Interface.Inputs.Add(OnPlayTrigger);

			// Outputs 
			FMetasoundFrontendClassVertex OnFinished;
			OnFinished.Name = DeprecatedInterfacesPrivate::GetIsFinishedOutputName();
			OnFinished.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnFinished.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnFinished.Metadata.SetDisplayName(LOCTEXT("OnFinished", "On Finished"));
			OnFinished.Metadata.SetDescription(LOCTEXT("OnFinishedToolTip", "Trigger executed to initiate stopping the source."));
#endif // WITH_EDITOR

			Interface.Outputs.Add(OnFinished);

			FMetasoundFrontendClassVertex GeneratedAudio;
			GeneratedAudio.Name = GetAudioOutputName();
			GeneratedAudio.TypeName = GetMetasoundDataTypeName<FMonoAudioFormat>();
			GeneratedAudio.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			GeneratedAudio.Metadata.SetDisplayName(LOCTEXT("GeneratedMono", "Audio"));
			GeneratedAudio.Metadata.SetDescription(LOCTEXT("GeneratedAudioToolTip", "The resulting output audio from this source."));
#endif // WITH_EDITOR

			Interface.Outputs.Add(GeneratedAudio);

			// Environment 
			FMetasoundFrontendClassEnvironmentVariable AudioDeviceID;
			AudioDeviceID.Name = DeprecatedInterfacesPrivate::GetAudioDeviceIDVariableName();

			Interface.Environment.Add(AudioDeviceID);

			return Interface;
		}
	}

	namespace MetasoundOutputFormatStereoV1_0
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version { "StereoSource", FMetasoundFrontendVersionNumber { 1, 0 } };
			return Version;
		}

		const FVertexName& GetAudioOutputName()
		{
			static const FVertexName AudioOutputName = "Generated Audio";
			return AudioOutputName;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();

			// Inputs
			FMetasoundFrontendClassVertex OnPlayTrigger;
			OnPlayTrigger.Name = DeprecatedInterfacesPrivate::GetOnPlayInputName();
			OnPlayTrigger.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnPlayTrigger.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnPlayTrigger.Metadata.SetDisplayName(LOCTEXT("OnPlay", "On Play"));
			OnPlayTrigger.Metadata.SetDescription(LOCTEXT("OnPlayTriggerToolTip", "Trigger executed when this source is played."));
#endif // WITH_EDITOR

			Interface.Inputs.Add(OnPlayTrigger);

			// Outputs
			FMetasoundFrontendClassVertex OnFinished;
			OnFinished.Name = DeprecatedInterfacesPrivate::GetIsFinishedOutputName();
			OnFinished.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnFinished.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnFinished.Metadata.SetDisplayName(LOCTEXT("OnFinished", "On Finished"));
			OnFinished.Metadata.SetDescription(LOCTEXT("OnFinishedToolTip", "Trigger executed to initiate stopping the source."));
#endif // WITH_EDITOR

			Interface.Outputs.Add(OnFinished);

			FMetasoundFrontendClassVertex GeneratedAudio;
			GeneratedAudio.Name = GetAudioOutputName();
			GeneratedAudio.TypeName = GetMetasoundDataTypeName<FStereoAudioFormat>();
			GeneratedAudio.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			GeneratedAudio.Metadata.SetDisplayName(LOCTEXT("GeneratedStereo", "Audio"));
			GeneratedAudio.Metadata.SetDescription(LOCTEXT("GeneratedAudioToolTip", "The resulting output audio from this source."));
#endif // WITH_EDITOR

			Interface.Outputs.Add(GeneratedAudio);

			// Environment
			FMetasoundFrontendClassEnvironmentVariable AudioDeviceID;
			AudioDeviceID.Name = DeprecatedInterfacesPrivate::GetAudioDeviceIDVariableName();

			Interface.Environment.Add(AudioDeviceID);

			return Interface;
		}
	}

	namespace MetasoundOutputFormatMonoV1_1
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version { "MonoSource", FMetasoundFrontendVersionNumber { 1, 1} };
			return Version;
		}

		const FVertexName& GetAudioOutputName()
		{
			static const FVertexName AudioOutputName = "Audio:0";
			return AudioOutputName;
		}

		FMetasoundFrontendClassVertex GetClassAudioOutput()
		{
			FMetasoundFrontendClassVertex GeneratedAudio;
			GeneratedAudio.Name = GetAudioOutputName();
			GeneratedAudio.TypeName = GetMetasoundDataTypeName<FAudioBuffer>();
			GeneratedAudio.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			GeneratedAudio.Metadata.SetDisplayName(LOCTEXT("GeneratedMono", "Audio"));
			GeneratedAudio.Metadata.SetDescription(LOCTEXT("GeneratedAudioToolTip", "The resulting output audio from this source."));
#endif // WITH_EDITOR

			return GeneratedAudio;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();

			// Inputs
			FMetasoundFrontendClassVertex OnPlayTrigger;
			OnPlayTrigger.Name = DeprecatedInterfacesPrivate::GetOnPlayInputName();
			OnPlayTrigger.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnPlayTrigger.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnPlayTrigger.Metadata.SetDisplayName(LOCTEXT("OnPlay", "On Play"));
			OnPlayTrigger.Metadata.SetDescription(LOCTEXT("OnPlayTriggerToolTip", "Trigger executed when this source is played."));
#endif // WITH_EDITOR

			Interface.Inputs.Add(OnPlayTrigger);

			// Outputs
			FMetasoundFrontendClassVertex OnFinished;
			OnFinished.Name = DeprecatedInterfacesPrivate::GetIsFinishedOutputName();
			OnFinished.TypeName = GetMetasoundDataTypeName<FTrigger>();
			OnFinished.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			OnFinished.Metadata.SetDisplayName(LOCTEXT("OnFinished", "On Finished"));
			OnFinished.Metadata.SetDescription(LOCTEXT("OnFinishedToolTip", "Trigger executed to initiate stopping the source."));
#endif // WITH_EDITOR

			Interface.Outputs.Add(OnFinished);

			FMetasoundFrontendClassVertex GeneratedAudio = GetClassAudioOutput();
			Interface.Outputs.Add(GeneratedAudio);

			// Environment
			FMetasoundFrontendClassEnvironmentVariable AudioDeviceID;
			AudioDeviceID.Name = DeprecatedInterfacesPrivate::GetAudioDeviceIDVariableName();

			Interface.Environment.Add(AudioDeviceID);

			return Interface;
		}

		bool FUpdateInterface::Transform(Frontend::FDocumentHandle InDocument) const
			{
				// Swap FMonoAudioFormat output node to an FAudioBuffer output node.
				using namespace Frontend;

				FGraphHandle Graph = InDocument->GetRootGraph();
				if (!Graph->IsValid())
				{
					return false;
				}

				InDocument->RemoveInterfaceVersion(MetasoundOutputFormatMonoV1_0::GetInterface().Metadata.Version);
				InDocument->AddInterfaceVersion(MetasoundOutputFormatMonoV1_1::GetInterface().Metadata.Version);

				FNodeHandle MonoFormatOutput = Graph->GetOutputNodeWithName(MetasoundOutputFormatMonoV1_0::GetAudioOutputName());
#if WITH_EDITOR
				FVector2D MonoFormatLocation;
#endif // WITH_EDITOR

				FOutputHandle OutputToReconnect = IOutputController::GetInvalidHandle();
				if (MonoFormatOutput->IsValid())
				{
#if WITH_EDITOR
					// Get the first location.
					for (auto Location : MonoFormatOutput->GetNodeStyle().Display.Locations)
					{
						MonoFormatLocation = Location.Value;
					}
#endif // WITH_EDITOR

					// Get connections
					TArray<FInputHandle> Inputs = MonoFormatOutput->GetInputs();
					if (ensure(Inputs.Num() == 1))
					{
						OutputToReconnect = Inputs[0]->GetConnectedOutput();
					}

					Graph->RemoveOutputVertex(MetasoundOutputFormatMonoV1_0::GetAudioOutputName());
				}

				// Create output
				FNodeHandle BufferOutput = Graph->AddOutputVertex(GetClassAudioOutput());
				if (ensure(BufferOutput->IsValid()))
				{
#if WITH_EDITOR
					FMetasoundFrontendNodeStyle Style = BufferOutput->GetNodeStyle();
					Style.Display.Locations.Add(FGuid::NewGuid(), MonoFormatLocation);
					BufferOutput->SetNodeStyle(Style);
#endif // WITH_EDITOR

					if (OutputToReconnect->IsValid())
					{
						// Reconnect
						TArray<FInputHandle> Inputs = BufferOutput->GetInputs();
						if (ensure(Inputs.Num() == 1))
						{
							ensure(OutputToReconnect->Connect(*Inputs[0]));
						}
					}
				}

				return true;
			}
	}

	namespace MetasoundOutputFormatStereoV1_1
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version { "StereoSource", FMetasoundFrontendVersionNumber { 1, 1 } };
			return Version;
		}

		const FVertexName& GetOnPlayInputName()
		{
			static const FVertexName TriggerInputName = "On Play";
			return TriggerInputName;
		}

		const FVertexName& GetLeftAudioOutputName()
		{
			static const FVertexName AudioOutputName = "Audio:0";
			return AudioOutputName;
		}

		const FVertexName& GetRightAudioOutputName()
		{
			static const FVertexName AudioOutputName = "Audio:1";
			return AudioOutputName;
		}

		FMetasoundFrontendClassVertex GetClassLeftAudioOutput()
		{
			FMetasoundFrontendClassVertex GeneratedLeftAudio;
			GeneratedLeftAudio.Name = GetLeftAudioOutputName();
			GeneratedLeftAudio.TypeName = GetMetasoundDataTypeName<FAudioBuffer>();
			GeneratedLeftAudio.VertexID = FGuid::NewGuid();

#if WITH_EDITOR
			GeneratedLeftAudio.Metadata.SetDisplayName(LOCTEXT("GeneratedStereoLeft", "Left Audio"));
			GeneratedLeftAudio.Metadata.SetDescription(LOCTEXT("GeneratedLeftAudioToolTip", "The resulting output audio from this source."));
#endif // WITH_EDITOR

			return GeneratedLeftAudio;
		}

		FMetasoundFrontendClassVertex GetClassRightAudioOutput()
		{
			FMetasoundFrontendClassVertex GeneratedRightAudio;
			GeneratedRightAudio.Name = GetRightAudioOutputName();
			GeneratedRightAudio.TypeName = GetMetasoundDataTypeName<FAudioBuffer>();

#if WITH_EDITOR
			GeneratedRightAudio.Metadata.SetDisplayName(LOCTEXT("GeneratedStereoRight", "Right Audio"));
			GeneratedRightAudio.Metadata.SetDescription(LOCTEXT("GeneratedRightAudioToolTip", "The resulting output audio from this source."));
#endif // WITH_EDITOR

			GeneratedRightAudio.VertexID = FGuid::NewGuid();

			return GeneratedRightAudio;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();

			// Inputs
			FMetasoundFrontendClassVertex OnPlayTrigger;
			OnPlayTrigger.Name = DeprecatedInterfacesPrivate::GetOnPlayInputName();
			OnPlayTrigger.TypeName = GetMetasoundDataTypeName<FTrigger>();

#if WITH_EDITOR
			OnPlayTrigger.Metadata.SetDisplayName(LOCTEXT("OnPlay", "On Play"));
			OnPlayTrigger.Metadata.SetDescription(LOCTEXT("OnPlayTriggerToolTip", "Trigger executed when this source is played."));
#endif // WITH_EDITOR

			OnPlayTrigger.VertexID = FGuid::NewGuid();

			Interface.Inputs.Add(OnPlayTrigger);

			// Outputs
			FMetasoundFrontendClassVertex OnFinished;
			OnFinished.Name = DeprecatedInterfacesPrivate::GetIsFinishedOutputName();

			OnFinished.TypeName = GetMetasoundDataTypeName<FTrigger>();

#if WITH_EDITOR
			OnFinished.Metadata.SetDisplayName(LOCTEXT("OnFinished", "On Finished"));
			OnFinished.Metadata.SetDescription(LOCTEXT("OnFinishedToolTip", "Trigger executed to initiate stopping the source."));
#endif // WITH_EDITOR

			OnFinished.VertexID = FGuid::NewGuid();

			Interface.Outputs.Add(OnFinished);

			FMetasoundFrontendClassVertex GeneratedLeftAudio = GetClassLeftAudioOutput();
			Interface.Outputs.Add(GeneratedLeftAudio);

			FMetasoundFrontendClassVertex GeneratedRightAudio = GetClassRightAudioOutput();
			Interface.Outputs.Add(GeneratedRightAudio);

			// Environment
			FMetasoundFrontendClassEnvironmentVariable AudioDeviceID;
			AudioDeviceID.Name = DeprecatedInterfacesPrivate::GetAudioDeviceIDVariableName();
			Interface.Environment.Add(AudioDeviceID);

			return Interface;
		};

		bool FUpdateInterface::Transform(Frontend::FDocumentHandle InDocument) const
			{
				using namespace Frontend;

				FGraphHandle Graph = InDocument->GetRootGraph();

				if (!Graph->IsValid())
				{
					return false;
				}

				InDocument->RemoveInterfaceVersion(MetasoundOutputFormatStereoV1_0::GetInterface().Metadata.Version);
				InDocument->AddInterfaceVersion(MetasoundOutputFormatStereoV1_1::GetInterface().Metadata.Version);

				FNodeHandle StereoFormatOutput = Graph->GetOutputNodeWithName(MetasoundOutputFormatStereoV1_0::GetAudioOutputName());
				FOutputHandle LeftOutputToReconnect = IOutputController::GetInvalidHandle();
				FOutputHandle RightOutputToReconnect = IOutputController::GetInvalidHandle();

#if WITH_EDITOR
				FVector2D StereoFormatLocation = FVector2D::ZeroVector;
#endif // WITH_EDITOR

				if (StereoFormatOutput->IsValid())
				{
#if WITH_EDITOR
					// Get the first location.
					for (auto Location : StereoFormatOutput->GetNodeStyle().Display.Locations)
					{
						StereoFormatLocation = Location.Value;
					}
#endif // WITH_EDITOR

					FInputHandle LeftInput = StereoFormatOutput->GetInputWithVertexName("Left");
					LeftOutputToReconnect = LeftInput->GetConnectedOutput();

					FInputHandle RightInput = StereoFormatOutput->GetInputWithVertexName("Right");
					RightOutputToReconnect = RightInput->GetConnectedOutput();

					Graph->RemoveOutputVertex(MetasoundOutputFormatStereoV1_0::GetAudioOutputName());
				}

				FNodeHandle LeftBufferOutput = Graph->AddOutputVertex(GetClassLeftAudioOutput());
				if (ensure(LeftBufferOutput->IsValid()))
				{
#if WITH_EDITOR
					FMetasoundFrontendNodeStyle Style = LeftBufferOutput->GetNodeStyle();
					Style.Display.Locations.Add(FGuid::NewGuid(), StereoFormatLocation);
					LeftBufferOutput->SetNodeStyle(Style);
#endif // WITH_EDITOR

					if (LeftOutputToReconnect->IsValid())
					{
						TArray<FInputHandle> Inputs = LeftBufferOutput->GetInputs();
						if (ensure(Inputs.Num() == 1))
						{
							ensure(LeftOutputToReconnect->Connect(*Inputs[0]));
						}
					}
				}

				FNodeHandle RightBufferOutput = Graph->AddOutputVertex(GetClassRightAudioOutput());
				if (ensure(RightBufferOutput->IsValid()))
				{
					if (RightOutputToReconnect->IsValid())
					{
#if WITH_EDITOR
						FMetasoundFrontendNodeStyle Style = RightBufferOutput->GetNodeStyle();
						Style.Display.Locations.Add(FGuid::NewGuid(), StereoFormatLocation + FVector2D{ 0, 100.f });
						RightBufferOutput->SetNodeStyle(Style);
#endif // WITH_EDITOR

						TArray<FInputHandle> Inputs = RightBufferOutput->GetInputs();
						if (ensure(Inputs.Num() == 1))
						{
							ensure(RightOutputToReconnect->Connect(*Inputs[0]));
						}
					}
				}

				return true;
			}
	}

	namespace MetasoundOutputFormatMonoV1_2
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			const static FMetasoundFrontendVersion Version = { "MonoSource", FMetasoundFrontendVersionNumber { 1, 2 } };
			return Version;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();
			return Interface;
		}

		bool FUpdateInterface::Transform(Frontend::FDocumentHandle InDocument) const
		{
			using namespace DeprecatedInterfacesPrivate;
			using namespace Frontend;

			const TArray<FMetasoundFrontendVersion> InterfacesToRemove
			{
				MetasoundOutputFormatMonoV1_1::GetVersion()
			};

			const TArray<FMetasoundFrontendVersion> InterfacesToAdd
			{
				SourceInterfaceV1_0::GetVersion(),
				OutputFormatMonoInterface::GetVersion()
			};

			FModifyRootGraphInterfaces InterfaceTransform(InterfacesToRemove, InterfacesToAdd);
			InterfaceTransform.SetNamePairingFunction(MatchMemberNamesIgnoreSpaces);
			return InterfaceTransform.Transform(InDocument);
		}
	}

	namespace MetasoundOutputFormatStereoV1_2
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			const static FMetasoundFrontendVersion Version = { "StereoSource", FMetasoundFrontendVersionNumber { 1, 2 } };
			return Version;
		}

		FMetasoundFrontendInterface GetInterface()
		{
			FMetasoundFrontendInterface Interface;
			Interface.Metadata.Version = GetVersion();
			Interface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();
			return Interface;
		}

		bool FUpdateInterface::Transform(Frontend::FDocumentHandle InDocument) const
		{
			using namespace DeprecatedInterfacesPrivate;
			using namespace Frontend;

			const TArray<FMetasoundFrontendVersion> InterfacesToRemove
			{
				MetasoundOutputFormatStereoV1_1::GetVersion()
			};

			const TArray<FMetasoundFrontendVersion> InterfacesToAdd
			{
				SourceInterfaceV1_0::GetVersion(),
				OutputFormatStereoInterface::GetVersion()
			};

			FModifyRootGraphInterfaces InterfaceTransform(InterfacesToRemove, InterfacesToAdd);
			InterfaceTransform.SetNamePairingFunction(MatchMemberNamesIgnoreSpaces);
			return InterfaceTransform.Transform(InDocument);
		}
	}

	void RegisterDeprecatedInterfaces()
	{
		using namespace Frontend;

		constexpr bool bDeprecated = true;
		const FName RouterName = IDataReference::RouterName;
		IInterfaceRegistry& Reg = IInterfaceRegistry::Get();

		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundV1_0::GetInterface(), RouterName, bDeprecated));
		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(SourceInterfaceV1_0::CreateInterface(*UMetaSoundSource::StaticClass()), RouterName, bDeprecated));

		// Set default interface with unset version to use base UMetaSoundPatch class implementation (legacy requirement for 5.0 alpha).
		{
			FMetasoundFrontendInterface DeprecatedInterface;
			DeprecatedInterface.Metadata.UClassOptions = DeprecatedInterfacesPrivate::GetDeprecatedClassOptions();
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MoveTemp(DeprecatedInterface), RouterName, bDeprecated));
		}

		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatStereoV1_0::GetInterface(), RouterName, bDeprecated));
		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatStereoV1_1::GetInterface(), MakeUnique<MetasoundOutputFormatStereoV1_1::FUpdateInterface>(), RouterName, bDeprecated));
		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatStereoV1_2::GetInterface(), MakeUnique<MetasoundOutputFormatStereoV1_2::FUpdateInterface>(), RouterName, bDeprecated));

		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatMonoV1_0::GetInterface(), RouterName, bDeprecated));
		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatMonoV1_1::GetInterface(), MakeUnique<MetasoundOutputFormatMonoV1_1::FUpdateInterface>(), RouterName, bDeprecated));
		Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(MetasoundOutputFormatMonoV1_2::GetInterface(), MakeUnique<MetasoundOutputFormatMonoV1_2::FUpdateInterface>(), RouterName, bDeprecated));
	}
} // namespace Metasound::Engine
#undef LOCTEXT_NAMESPACE // MetasoundEngineDeprecatedInterfaces
