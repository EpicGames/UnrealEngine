// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundVertex.h"


// DEPRECATED: DeprecatedInterfaces that corrispond with archetypes
// that preceded the Interface System.  Only remain for purposes of
// upgrading content generated prior to 5.0 release.
namespace Metasound::Engine
{
	// Base interface without any required inputs or outputs (TODO: Version & Remove)
	namespace MetasoundV1_0
	{
		const FMetasoundFrontendVersion& GetVersion();
		FMetasoundFrontendInterface GetInterface();
	}

	// V1.0 of a Metasound Mono output format. Uses FMonoAudioFormat as output.
	namespace MetasoundOutputFormatMonoV1_0
	{
		const FMetasoundFrontendVersion& GetVersion();
		const FVertexName& GetAudioOutputName();
		FMetasoundFrontendInterface GetInterface();
	}

	// V1.0 of a Metasound Stereo output format. Uses FStereoAudioFormat as output.
	namespace MetasoundOutputFormatStereoV1_0
	{
		const FMetasoundFrontendVersion& GetVersion();
		const FVertexName& GetAudioOutputName();
		FMetasoundFrontendInterface GetInterface();
	}

	// V1.1 of a Metasound Mono output format. Uses FAudioBuffer as output.
	namespace MetasoundOutputFormatMonoV1_1
	{
		const FMetasoundFrontendVersion& GetVersion();
		const FVertexName& GetAudioOutputName();
		FMetasoundFrontendInterface GetInterface();

		class FUpdateInterface : public Frontend::IDocumentTransform
		{
		public:
			virtual bool Transform(Frontend::FDocumentHandle InDocument) const override;
		};
	}

	// V1.2 of a Metasound Mono output format.
	// Deprecate path from MetasoundOutputFormatStereoV1_1 to SourceInterface & respective OutputFormatMonoInterface
	namespace MetasoundOutputFormatMonoV1_2
	{
		const FMetasoundFrontendVersion& GetVersion();
		FMetasoundFrontendInterface GetInterface();

		class FUpdateInterface : public Frontend::IDocumentTransform
		{
		public:
			virtual bool Transform(Frontend::FDocumentHandle InDocument) const override;
		};
	}

	// V1.1 of a Metasound Stereo output format. Uses two FAudioBuffers as outputs.
	namespace MetasoundOutputFormatStereoV1_1
	{
		const FMetasoundFrontendVersion& GetVersion();
		const FVertexName& GetLeftAudioOutputName();
		const FVertexName& GetRightAudioOutputName();
		FMetasoundFrontendInterface GetInterface();

		class FUpdateInterface : public Frontend::IDocumentTransform
		{
		public:
			virtual bool Transform(Frontend::FDocumentHandle InDocument) const override;
		};
	}

	// V1.2 of a Metasound Stereo output format. Uses two FAudioBuffers as outputs.
	// Deprecate path from MetasoundOutputFormatStereoV1_1 to SourceInterface & respective OutputFormatStereoInterface
	namespace MetasoundOutputFormatStereoV1_2
	{
		const FMetasoundFrontendVersion& GetVersion();
		FMetasoundFrontendInterface GetInterface();

		class FUpdateInterface : public Frontend::IDocumentTransform
		{
		public:
			virtual bool Transform(Frontend::FDocumentHandle InDocument) const override;
		};
	}

	void RegisterDeprecatedInterfaces();
} // namespace Metasound::Engine
