// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundInputFormatInterfaces.h"

#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Metasound.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundSource.h"
#include "MetasoundTrigger.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"


#define LOCTEXT_NAMESPACE "MetasoundEngine"
namespace Metasound::Engine
{
	namespace InputFormatPrivate
	{
		TArray<Audio::FParameterInterface::FClassOptions> GetUClassOptions()
		{
			return
			{
				{ UMetaSoundPatch::StaticClass()->GetClassPathName(), true /* bIsModifiable */, false /* bIsDefault */ },
				{ UMetaSoundSource::StaticClass()->GetClassPathName(), false /* bIsModifiable */ , false /* bIsDefault */ }
			};
		};
	} // namespace InputFormatPrivate

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.Mono"
	namespace InputFormatMonoInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName MonoIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatMonoInterface::GetVersion().Name, InputFormatMonoInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputFormatMonoInterfaceInputName", "In Mono"),
							LOCTEXT("OutputFormatStereoInterface_RightDescription", "Mono input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::MonoIn
						}
					};
				}
			};
			return MakeShared<FInterface>();
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatMonoInterface::GetVersion(),
					InputFormatMonoInterface::GetVersion(),
					0, // Mono-to-mono takes priority
					{
						{
							OutputFormatMonoInterface::Outputs::MonoOut, InputFormatMonoInterface::Inputs::MonoIn,
						}
					},
				},
				{
					OutputFormatStereoInterface::GetVersion(),
					InputFormatMonoInterface::GetVersion(),
					10,
					{
						{
							OutputFormatStereoInterface::Outputs::LeftOut, InputFormatMonoInterface::Inputs::MonoIn,
						}
					}
				},
				{
					OutputFormatQuadInterface::GetVersion(),
					InputFormatMonoInterface::GetVersion(),
					20,
					{
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut, InputFormatMonoInterface::Inputs::MonoIn,
						}
					}
				},
				{
					OutputFormatFiveDotOneInterface::GetVersion(),
					InputFormatMonoInterface::GetVersion(),
					30,
					{
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut, InputFormatMonoInterface::Inputs::MonoIn,
						}
					}
				},
				{
					OutputFormatSevenDotOneInterface::GetVersion(),
					InputFormatMonoInterface::GetVersion(),
					40,
					{
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut, InputFormatMonoInterface::Inputs::MonoIn,
						}
					}
				}
			};
		}
	} // namespace InputFormatMonoInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.Stereo"
	namespace InputFormatStereoInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName LeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName RightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatStereoInterface::GetVersion().Name, InputFormatStereoInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputFormatStereoInterfaceInputLeftName", "In Left"),
							LOCTEXT("StereoIn_Left_AudioDescription", "Left stereo input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::LeftIn,
							FText::GetEmpty(), // Required Text
							100
						},
						{
							LOCTEXT("InputFormatStereoInterfaceInputRightName", "In Right"),
							LOCTEXT("StereoIn_Right_AudioDescription", "Right stereo input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::RightIn,
							FText::GetEmpty(), // Required Text
							101
						}
					};
				}
			};

			return MakeShared<FInterface>();
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatMonoInterface::GetVersion(),
					InputFormatStereoInterface::GetVersion(),
					10,
					{
						{
							OutputFormatMonoInterface::Outputs::MonoOut, InputFormatStereoInterface::Inputs::LeftIn
						}
					}
				},
				{
					OutputFormatStereoInterface::GetVersion(),
					InputFormatStereoInterface::GetVersion(),
					0,	// Stereo to stereo takes priority
					{
						{
							OutputFormatStereoInterface::Outputs::LeftOut, InputFormatStereoInterface::Inputs::LeftIn,
						},
						{
							OutputFormatStereoInterface::Outputs::RightOut, InputFormatStereoInterface::Inputs::RightIn
						}
					}
				},
				{
					OutputFormatQuadInterface::GetVersion(),
					InputFormatStereoInterface::GetVersion(),
					30,
					{
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut, InputFormatStereoInterface::Inputs::LeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::FrontRightOut, InputFormatStereoInterface::Inputs::RightIn
						}
					}
				},
				{
					OutputFormatFiveDotOneInterface::GetVersion(),
					InputFormatStereoInterface::GetVersion(),
					40,
					{
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut, InputFormatStereoInterface::Inputs::LeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontRightOut, InputFormatStereoInterface::Inputs::RightIn
						}
					}
				},
				{
					OutputFormatSevenDotOneInterface::GetVersion(),
					InputFormatStereoInterface::GetVersion(),
					50,
					{
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut, InputFormatStereoInterface::Inputs::LeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontRightOut, InputFormatStereoInterface::Inputs::RightIn
						}
					}
				}
			};
		}
	} // namespace InputFormatStereoInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.Quad"
	namespace InputFormatQuadInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName FrontLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName SideLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatQuadInterface::GetVersion().Name, InputFormatQuadInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputFormatQuadInterfaceInputFrontLeftName", "In Front Left"),
							LOCTEXT("QuadIn_FrontLeft_AudioDescription", "Front Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontLeftIn,
							FText::GetEmpty(), // Required Text
							100
						},
						{
							LOCTEXT("InputFormatQuadInterfaceInputFrontRightName", "In Front Right"),
							LOCTEXT("QuadIn_FrontRight_AudioDescription", "Front Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontRightIn,
							FText::GetEmpty(), // Required Text
							101
						},
						{
							LOCTEXT("InputFormatQuadInterfaceInputSideLeftName", "In Side Left"),
							LOCTEXT("QuadIn_SideLeft_AudioDescription", "Side Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideLeftIn,
							FText::GetEmpty(), // Required Text
							104
						},
						{
							LOCTEXT("InputFormatQuadInterfaceInputSideRightName", "In Side Right"),
							LOCTEXT("QuadIn_SideRight_AudioDescription", "Side Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideRightIn,
							FText::GetEmpty(), // Required Text
							105
						},
					};
				}
			};

			return MakeShared<FInterface>();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatMonoInterface::GetVersion(),
					InputFormatQuadInterface::GetVersion(),
					10,
					{
						{
							OutputFormatMonoInterface::Outputs::MonoOut, InputFormatQuadInterface::Inputs::FrontLeftIn
						}
					}
				},
				{
					OutputFormatStereoInterface::GetVersion(),
					InputFormatQuadInterface::GetVersion(),
					20,
					{
						{
							OutputFormatStereoInterface::Outputs::LeftOut, InputFormatQuadInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatStereoInterface::Outputs::RightOut, InputFormatQuadInterface::Inputs::FrontRightIn
						}
					}
				},
				{
					OutputFormatQuadInterface::GetVersion(),
					InputFormatQuadInterface::GetVersion(),
					0, // Quad to quad takes priority
					{
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut, InputFormatQuadInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::FrontRightOut, InputFormatQuadInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideLeftOut, InputFormatQuadInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideRightOut, InputFormatQuadInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatFiveDotOneInterface::GetVersion(),
					InputFormatQuadInterface::GetVersion(),
					40,
					{
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut, InputFormatQuadInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontRightOut, InputFormatQuadInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideLeftOut, InputFormatQuadInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideRightOut, InputFormatQuadInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatSevenDotOneInterface::GetVersion(),
					InputFormatQuadInterface::GetVersion(),
					50,
					{
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut, InputFormatQuadInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontRightOut, InputFormatQuadInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideLeftOut, InputFormatQuadInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideRightOut, InputFormatQuadInterface::Inputs::SideRightIn
						}
					}
				}
			};
		}
	} // namespace InputFormatQuadInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.5dot1"
	namespace InputFormatFiveDotOneInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName FrontLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName FrontCenterIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:2");
			const FName LowFrequencyIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:3");
			const FName SideLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatFiveDotOneInterface::GetVersion().Name, InputFormatFiveDotOneInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputFrontLeftName", "In Front Left"),
							LOCTEXT("FiveDotOneIn_FrontLeft_AudioDescription", "Front Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontLeftIn,
							FText::GetEmpty(), // Required Text
							100
						},
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputFrontRightName", "In Front Right"),
							LOCTEXT("FiveDotOneIn_FrontRight_AudioDescription", "Front Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontRightIn,
							FText::GetEmpty(), // Required Text
							101
						},
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputFrontCenterName", "In Front Center"),
							LOCTEXT("FiveDotOneIn_FrontCenter_AudioDescription", "Front Center input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontCenterIn,
							FText::GetEmpty(), // Required Text
							102
						},
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputLowFrequencyName", "In Low Frequency"),
							LOCTEXT("FiveDotOneIn_LowFrequency_AudioDescription", "Low Frequency input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::LowFrequencyIn,
							FText::GetEmpty(), // Required Text
							103
						},
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputSideLeftName", "In Side Left"),
							LOCTEXT("FiveDotOneIn_SideLeft_AudioDescription", "Side Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideLeftIn,
							FText::GetEmpty(), // Required Text
							104
						},
						{
							LOCTEXT("InputFormatFiveDotOneInterfaceInputSideRightName", "In Side Right"),
							LOCTEXT("FiveDotOneIn_SideRight_AudioDescription", "Side Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideRightIn,
							FText::GetEmpty(), // Required Text
							105
						},
					};
				}
			};

			return MakeShared<FInterface>();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatMonoInterface::GetVersion(),
					InputFormatFiveDotOneInterface::GetVersion(),
					10,
					{
						{
							OutputFormatMonoInterface::Outputs::MonoOut, InputFormatFiveDotOneInterface::Inputs::FrontLeftIn
						}
					}
				},
				{
					OutputFormatStereoInterface::GetVersion(),
					InputFormatFiveDotOneInterface::GetVersion(),
					20,
					{
						{
							OutputFormatStereoInterface::Outputs::LeftOut, InputFormatFiveDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatStereoInterface::Outputs::RightOut, InputFormatFiveDotOneInterface::Inputs::FrontRightIn
						}
					}
				},
				{
					OutputFormatQuadInterface::GetVersion(),
					InputFormatFiveDotOneInterface::GetVersion(),
					30,
					{
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut, InputFormatFiveDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::FrontRightOut, InputFormatFiveDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideLeftOut, InputFormatFiveDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideRightOut, InputFormatFiveDotOneInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatFiveDotOneInterface::GetVersion(),
					InputFormatFiveDotOneInterface::GetVersion(),
					0, // 5.1 to 5.1 takes priority
					{
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut, InputFormatFiveDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontRightOut, InputFormatFiveDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontCenterOut, InputFormatFiveDotOneInterface::Inputs::FrontCenterIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::LowFrequencyOut, InputFormatFiveDotOneInterface::Inputs::LowFrequencyIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideLeftOut, InputFormatFiveDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideRightOut, InputFormatFiveDotOneInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatSevenDotOneInterface::GetVersion(),
					InputFormatFiveDotOneInterface::GetVersion(),
					50,
					{
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut, InputFormatFiveDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontRightOut, InputFormatFiveDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontCenterOut, InputFormatFiveDotOneInterface::Inputs::FrontCenterIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::LowFrequencyOut, InputFormatFiveDotOneInterface::Inputs::LowFrequencyIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideLeftOut, InputFormatFiveDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideRightOut, InputFormatFiveDotOneInterface::Inputs::SideRightIn
						}
					}
				}
			};
		}
	} // namespace InputFormatFiveDotOneInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.7dot1"
	namespace InputFormatSevenDotOneInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName FrontLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName FrontCenterIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:2");
			const FName LowFrequencyIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:3");
			const FName SideLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
			const FName BackLeftIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:6");
			const FName BackRightIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:7");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatSevenDotOneInterface::GetVersion().Name, InputFormatSevenDotOneInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputFrontLeftName", "In Front Left"),
							LOCTEXT("SevenDotOneIn_FrontLeft_AudioDescription", "Front Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontLeftIn,
							FText::GetEmpty(), // Required Text
							100
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputFrontRightName", "In Front Right"),
							LOCTEXT("SevenDotOneIn_FrontRight_AudioDescription", "Front Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontRightIn,
							FText::GetEmpty(), // Required Text
							101
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputFrontCenterName", "In Front Center"),
							LOCTEXT("SevenDotOneIn_FrontCenter_AudioDescription", "Front Center input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::FrontCenterIn,
							FText::GetEmpty(), // Required Text
							102
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputLowFrequencyName", "In Low Frequency"),
							LOCTEXT("SevenDotOneIn_LowFrequency_AudioDescription", "Low Frequency input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::LowFrequencyIn,
							FText::GetEmpty(), // Required Text
							103
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputSideLeftName", "In Side Left"),
							LOCTEXT("SevenDotOneIn_SideLeft_AudioDescription", "Side Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideLeftIn,
							FText::GetEmpty(), // Required Text
							104
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputSideRightName", "In Side Right"),
							LOCTEXT("SevenDotOneIn_SideRight_AudioDescription", "Side Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::SideRightIn,
							FText::GetEmpty(), // Required Text
							105
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputBackLeftName", "In Back Left"),
							LOCTEXT("SevenDotOneIn_BackLeft_AudioDescription", "Back Left input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::BackLeftIn,
							FText::GetEmpty(), // Required Text
							106
						},
						{
							LOCTEXT("InputFormatSevenDotOneInterfaceInputBackRightName", "In Back Right"),
							LOCTEXT("SevenDotOneIn_BackRight_AudioDescription", "Back Right input audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Inputs::BackRightIn,
							FText::GetEmpty(), // Required Text
							107
						},
					};
				}
			};

			return MakeShared<FInterface>();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatMonoInterface::GetVersion(),
					InputFormatSevenDotOneInterface::GetVersion(),
						10,
					{
						{
							OutputFormatMonoInterface::Outputs::MonoOut, InputFormatSevenDotOneInterface::Inputs::FrontLeftIn
						}
					}
				},
				{
					OutputFormatStereoInterface::GetVersion(),
					InputFormatSevenDotOneInterface::GetVersion(),
					20,
					{
						{
							OutputFormatStereoInterface::Outputs::LeftOut, InputFormatSevenDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatStereoInterface::Outputs::RightOut, InputFormatSevenDotOneInterface::Inputs::FrontRightIn
						}
					}
				},
				{
					OutputFormatQuadInterface::GetVersion(),
					InputFormatSevenDotOneInterface::GetVersion(),
					30,
					{
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut, InputFormatSevenDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::FrontRightOut, InputFormatSevenDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideLeftOut, InputFormatSevenDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatQuadInterface::Outputs::SideRightOut, InputFormatSevenDotOneInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatFiveDotOneInterface::GetVersion(),
					InputFormatSevenDotOneInterface::GetVersion(),
					40,
					{
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut, InputFormatSevenDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontRightOut, InputFormatSevenDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontCenterOut, InputFormatSevenDotOneInterface::Inputs::FrontCenterIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::LowFrequencyOut, InputFormatSevenDotOneInterface::Inputs::LowFrequencyIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideLeftOut, InputFormatSevenDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatFiveDotOneInterface::Outputs::SideRightOut, InputFormatSevenDotOneInterface::Inputs::SideRightIn
						}
					}
				},
				{
					OutputFormatSevenDotOneInterface::GetVersion(),
					InputFormatSevenDotOneInterface::GetVersion(),
					0, // 7.1 to 7.1 takes priority
					{
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut, InputFormatSevenDotOneInterface::Inputs::FrontLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontRightOut, InputFormatSevenDotOneInterface::Inputs::FrontRightIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontCenterOut, InputFormatSevenDotOneInterface::Inputs::FrontCenterIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::LowFrequencyOut, InputFormatSevenDotOneInterface::Inputs::LowFrequencyIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideLeftOut, InputFormatSevenDotOneInterface::Inputs::SideLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::SideRightOut, InputFormatSevenDotOneInterface::Inputs::SideRightIn
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::BackLeftOut, InputFormatSevenDotOneInterface::Inputs::BackLeftIn,
						},
						{
							OutputFormatSevenDotOneInterface::Outputs::BackRightOut, InputFormatSevenDotOneInterface::Inputs::BackRightIn
						}
					}
				}
			};
		}
	} // namespace InputFormatSevenDotOneInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE
	
	
#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.InputFormat.Cat"
	namespace InputFormatChannelAgnosticInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName CatIn = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
		} // namespace Inputs

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(InputFormatChannelAgnosticInterface::GetVersion().Name, InputFormatChannelAgnosticInterface::GetVersion().Number.ToInterfaceVersion())
				{
					UClassOptions = InputFormatPrivate::GetUClassOptions();
					Inputs =
					{
						{
							LOCTEXT("InputChannelAgnosticInterfaceInputCatName", "Channel Agnostic"),
							LOCTEXT("ChannelAgnostic_Cat_AudioDescription", "Channel Agnostic Input."),
							GetMetasoundDataTypeName<FChannelAgnosticType>(),
							Inputs::CatIn,
							FText::GetEmpty(), // Required Text
							100
						}
					};
				}
			};

			return MakeShared<FInterface>();
		}

		TArray<FMetasoundFrontendInterfaceBinding> CreateBindings()
		{
			return TArray<FMetasoundFrontendInterfaceBinding>
			{
				{
					OutputFormatChannelAgnosticInterface::GetVersion(),
					GetVersion(),
						10,
					{
						{
							OutputFormatChannelAgnosticInterface::Outputs::CatOut, Inputs::CatIn
						}
					}
				}
			};
		}
	} // namespace InputFormatChannelAgnosticInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

} // namespace Metasound::Engine
#undef LOCTEXT_NAMESPACE
