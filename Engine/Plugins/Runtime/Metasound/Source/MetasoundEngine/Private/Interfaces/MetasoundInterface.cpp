// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundInterface.h"

#include "Algo/Transform.h"
#include "AudioParameterControllerInterface.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "IAudioParameterTransmitter.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Interfaces/MetasoundInputFormatInterfaces.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Metasound.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLog.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundSource.h"
#include "MetasoundUObjectRegistry.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/NoExportTypes.h"


namespace Metasound::Engine
{
	namespace InterfacePrivate
	{
		struct FDefaultTransform : public Frontend::IBuilderVersionTransform
		{
			const FMetasoundFrontendVersion Version;

			FDefaultTransform(FMetasoundFrontendVersion InVersion)
				: Version(InVersion)
			{
			}

			virtual ~FDefaultTransform() = default;

			virtual const FMetasoundFrontendVersion& GetVersion() const override
			{
				return Version;
			}

			virtual bool Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilder) const override
			{
				InOutBuilder.AddInterface(Version.Name);
				return true;
			}
		};
	}

	FInterfaceRegistryEntry::FInterfaceRegistryEntry(FMetasoundFrontendInterface&& InInterface, FName InRouterName, bool bInIsDeprecated)
		: Interface(MoveTemp(InInterface))
		, UpdateTransform(MakeShareable(new InterfacePrivate::FDefaultTransform(Interface.Metadata.Version)))
		, RouterName(InRouterName)
		, bIsDeprecated(bInIsDeprecated)
	{
	}

	FInterfaceRegistryEntry::FInterfaceRegistryEntry(const FMetasoundFrontendInterface& InInterface, FName InRouterName, bool bInIsDeprecated)
		: Interface(InInterface)
		, UpdateTransform(MakeShareable(new InterfacePrivate::FDefaultTransform(Interface.Metadata.Version)))
		, RouterName(InRouterName)
		, bIsDeprecated(bInIsDeprecated)
	{
	}

	FInterfaceRegistryEntry::FInterfaceRegistryEntry(FMetasoundFrontendInterface InInterface, TUniquePtr<Frontend::IBuilderVersionTransform>&& InUpdateTransform, FName InRouterName, bool bInIsDeprecated)
		: Interface(MoveTemp(InInterface))
		, UpdateTransform(MakeShareable(InUpdateTransform.Release()))
		, RouterName(InRouterName)
		, bIsDeprecated(bInIsDeprecated)
	{
		check(UpdateTransform.IsValid());
	}

	FInterfaceRegistryEntry::FInterfaceRegistryEntry(const FMetasoundFrontendInterface& InInterface, TUniquePtr<Frontend::IDocumentTransform>&& InUpdateTransform, FName InRouterName, bool bInIsDeprecated)
		: Interface(InInterface)
		, RouterName(InRouterName)
		, bIsDeprecated(bInIsDeprecated)
	{
		using FDocTransform = TUniquePtr<Frontend::IDocumentTransform>;

		struct FDefaultBuilderTransform : public Frontend::IBuilderVersionTransform
		{
			FDocTransform DocTransform;
			FMetasoundFrontendVersion Version;

			FDefaultBuilderTransform(const FMetasoundFrontendVersion& UpdateVersion, FDocTransform&& InTransform)
				: DocTransform(MoveTemp(InTransform))
				, Version(UpdateVersion)
			{
			}

			virtual ~FDefaultBuilderTransform() = default;

			virtual const FMetasoundFrontendVersion& GetVersion() const
			{
				return Version;
			}

			virtual bool Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilder) const override
			{
				DocTransform->Transform(InOutBuilder.GetMetasoundAsset().GetDocumentHandle());
				return true;
			}
		};
		UpdateTransform = MakeShareable(new FDefaultBuilderTransform(Interface.Metadata.Version, MoveTemp(InUpdateTransform)));
	}

	FName FInterfaceRegistryEntry::GetRouterName() const
	{
		return RouterName;
	}

	const FMetasoundFrontendInterface& FInterfaceRegistryEntry::GetInterface() const
	{
		return Interface;
	}

	bool FInterfaceRegistryEntry::IsDeprecated() const
	{
		return bIsDeprecated;
	}

	bool FInterfaceRegistryEntry::UpdateRootGraphInterface(Frontend::FDocumentHandle InDocument) const
	{		
		return false;
	}

	bool FInterfaceRegistryEntry::UpdateRootGraphInterface(FMetaSoundFrontendDocumentBuilder& InDocumentBuilder) const
	{
		return false;
	}

	TSharedRef<Frontend::IBuilderVersionTransform> FInterfaceRegistryEntry::GetUpdateTransform() const
	{
		return UpdateTransform.ToSharedRef();
	}

	void RegisterAudioFormatInterfaces()
	{
		using namespace Frontend;

		IInterfaceRegistry& Reg = IInterfaceRegistry::Get();

		// Input Formats
		{
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatMonoInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatStereoInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatQuadInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatFiveDotOneInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatSevenDotOneInterface::CreateInterface()));
		}

		// Output Formats
		{
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatMonoInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatStereoInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatQuadInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatFiveDotOneInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatSevenDotOneInterface::CreateInterface()));
		}

		// Channel Agnostic (CAT) format interfaces — only registered when MetasoundExperimental plugin is enabled.
		// Without this gate, CAT output would appear as an option even without the experimental plugin loaded.
		if (IPluginManager::Get().FindEnabledPlugin(TEXT("MetasoundExperimental")))
		{
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(InputFormatChannelAgnosticInterface::CreateInterface()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(OutputFormatChannelAgnosticInterface::CreateInterface()));
		}
	}

	void RegisterExternalInterfaces()
	{
		// Register External Interfaces (Interfaces defined externally & can be managed directly by end-user).
		auto RegisterExternalInterface = [](Audio::FParameterInterfacePtr Interface)
		{
			using namespace Frontend;

			bool bSupportedInterface = false;
			IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&bSupportedInterface, &Interface](UClass& InRegisteredClass)
			{
				const TArray<const UClass*> SupportedUClasses = Interface->FindSupportedUClasses();
				bSupportedInterface |= SupportedUClasses.IsEmpty();
				for (const UClass* SupportedUClass : SupportedUClasses)
				{
					check(SupportedUClass);
					bSupportedInterface |= SupportedUClass->IsChildOf(&InRegisteredClass);
				}
			});

			if (bSupportedInterface)
			{
				IInterfaceRegistry::Get().RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(FMetasoundFrontendInterface(Interface), Audio::IParameterTransmitter::RouterName));
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Parameter interface '%ls' not supported by MetaSounds", *Interface->GetName().ToString());
			}
		};

		Audio::IAudioParameterInterfaceRegistry::Get().IterateInterfaces(RegisterExternalInterface);
		Audio::IAudioParameterInterfaceRegistry::Get().OnRegistration(MoveTemp(RegisterExternalInterface));
	}

	void RegisterInterfaces()
	{
		using namespace Frontend;

		IInterfaceRegistry& Reg = IInterfaceRegistry::Get();

		// Default Source Interfaces
		{
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(SourceInterface::CreateInterface(*UMetaSoundSource::StaticClass()), MakeUnique<SourceInterface::FUpdateInterface>()));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(SourceOneShotInterface::CreateInterface(*UMetaSoundSource::StaticClass())));
			Reg.RegisterInterface(MakeUnique<FInterfaceRegistryEntry>(SourceStartTimeInterface::CreateInterface(*UMetaSoundSource::StaticClass())));
		}

		RegisterAudioFormatInterfaces();
		RegisterExternalInterfaces();
	}
} // namespace Metasound::Engine
