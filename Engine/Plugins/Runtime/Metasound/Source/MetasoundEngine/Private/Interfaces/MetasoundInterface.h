// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendTransform.h"


namespace Metasound::Engine
{
	// Entry for registered interface.
	class FInterfaceRegistryEntry : public Frontend::IInterfaceRegistryEntry
	{
	public:
		FInterfaceRegistryEntry(FMetasoundFrontendInterface&& InInterface, FName InRouterName = IDataReference::RouterName, bool bInIsDeprecated = false);
		FInterfaceRegistryEntry(const FMetasoundFrontendInterface& InInterface, FName InRouterName = IDataReference::RouterName, bool bInIsDeprecated = false);
		FInterfaceRegistryEntry(FMetasoundFrontendInterface InInterface, TUniquePtr<Frontend::IBuilderVersionTransform>&& InUpdateTransform, FName InRouterName, bool bInIsDeprecated = false);

		// Ctor is here for back compat.  Future registry entries that provide update transforms should use overload providing builder version transform
		FInterfaceRegistryEntry(const FMetasoundFrontendInterface& InInterface, TUniquePtr<Frontend::IDocumentTransform>&& InUpdateTransform, FName InRouterName = IDataReference::RouterName, bool bInIsDeprecated = false);

		virtual FName GetRouterName() const override;
		virtual const FMetasoundFrontendInterface& GetInterface() const override;
		virtual bool IsDeprecated() const override;
		virtual bool UpdateRootGraphInterface(Frontend::FDocumentHandle InDocument) const override;
		virtual bool UpdateRootGraphInterface(FMetaSoundFrontendDocumentBuilder& InDocumentBuilder) const override;
		virtual TSharedRef<Frontend::IBuilderVersionTransform> GetUpdateTransform() const override;

	private:
		FMetasoundFrontendInterface Interface;
		TSharedPtr<Frontend::IBuilderVersionTransform> UpdateTransform;
		FName RouterName;
		bool bIsDeprecated = false;
	};

	void RegisterInterfaces();
} // namespace Metasound::Engine
