// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundInterfaceBindings.h"

#include "Algo/Transform.h"
#include "Interfaces/MetasoundFrontendInterfaceBindingRegistry.h"
#include "Interfaces/MetasoundInputFormatInterfaces.h"
#include "MetasoundFrontendDocument.h"


namespace Metasound::Engine
{
	void RegisterInternalInterfaceBindings()
	{
		RegisterInterfaceBindings(InputFormatMonoInterface::CreateBindings());
		RegisterInterfaceBindings(InputFormatStereoInterface::CreateBindings());
	}

	void RegisterInterfaceBindings(const TArray<FMetasoundFrontendInterfaceBinding>& InBindings)
	{
		using namespace Metasound::Frontend;

		for (const FMetasoundFrontendInterfaceBinding& Binding : InBindings)
		{
			TUniquePtr<FInterfaceBindingRegistryEntry> Entry = MakeUnique<FInterfaceBindingRegistryEntry>(Binding);
			IInterfaceBindingRegistry::Get().RegisterInterfaceBinding(Binding.InputInterfaceVersion, MoveTemp(Entry));
		}
	}

	bool UnregisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, const FMetasoundFrontendVersion& InOutputInterfaceVersion)
	{
		using namespace Metasound::Frontend;
		return IInterfaceBindingRegistry::Get().UnregisterInterfaceBinding(InInputInterfaceVersion, InOutputInterfaceVersion);
	}

	bool UnregisterAllInterfaceBindings(const FMetasoundFrontendVersion& InInputInterfaceVersion)
	{
		using namespace Metasound::Frontend;
		return IInterfaceBindingRegistry::Get().UnregisterAllInterfaceBindings(InInputInterfaceVersion);
	}
} // namespace Metasound::Engine
