// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


namespace Metasound::Engine
{
	void RegisterInternalInterfaceBindings();
} // namespace Metasound::Engine
