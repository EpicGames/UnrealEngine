// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundOutputFormatInterfaces.h"

#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Metasound.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundSource.h"
#include "MetasoundTrigger.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundOutputFormatInterfaces)

#define LOCTEXT_NAMESPACE "MetasoundFrontend"


namespace Metasound::Engine
{
	namespace OutputFormatPrivate
	{
		Audio::FParameterInterface::FOutput GetFrontLeftOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedFrontLeftDisplayName", "Out Front Left"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedFrontLeftDescription", "The resulting front left channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				100 // SortOrder
			};
		}

		Audio::FParameterInterface::FOutput GetFrontRightOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedFrontRightDisplayName", "Out Front Right"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedFrontRightDescription", "The resulting front right channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				101
			};
		}

		Audio::FParameterInterface::FOutput GetFrontCenterOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedCenterDisplayName", "Out Front Center"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedCenterDescription", "The resulting front center channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				102
			};
		}

		Audio::FParameterInterface::FOutput GetLowFrequencyOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedLowFrequencyDisplayName", "Out Low Frequency"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedLowFrequencyDescription", "The resulting low frequency channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				103
			};
		}

		Audio::FParameterInterface::FOutput GetSideLeftOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedSideLeftDisplayName", "Out Side Left"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedSideLeftDescription", "The resulting side left channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				104
			};
		}

		Audio::FParameterInterface::FOutput GetSideRightOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedSideRightDisplayName", "Out Side Right"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedSideRightDescription", "The resulting side right channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				105
			};
		}

		Audio::FParameterInterface::FOutput GetBackLeftOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedBackLeftDisplayName", "Out Back Left"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedBackLeftDescription", "The resulting back left channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				106
			};
		}

		Audio::FParameterInterface::FOutput GetBackRightOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedBackRightDisplayName", "Out Back Right"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedBackRightDescription", "The resulting back right channel output audio."),
				GetMetasoundDataTypeName<FAudioBuffer>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				107
			};
		}
		
		Audio::FParameterInterface::FOutput GetChannelAgnosticOutput(const FName& InVertexName)
		{
			return
			{
				LOCTEXT("OutputFormatSurroundInterface_GeneratedChannelAgnosticDisplayName", "Out Channel Agnostic"),
				LOCTEXT("OutputFormatSurroundInterface_GeneratedChannelAgnosticDescription", "The resulting channel agnostic output audio."),
				GetMetasoundDataTypeName<FChannelAgnosticType>(),
				InVertexName,
				FText::GetEmpty(), // RequiredText
				EAudioParameterType::None,
				107
			};
		}	

		TArray<Audio::FParameterInterface::FClassOptions> CreateUClassOptions(bool bIsSourceDefault = false)
		{
			return
			{
				{ UMetaSoundPatch::StaticClass()->GetClassPathName(), true /* bIsModifiable */, false /* bIsDefault */ },
				{ UMetaSoundSource::StaticClass()->GetClassPathName(),  false /* bIsModifiable */, bIsSourceDefault }
			};
		};
	} // namespace OutputFormatPrivate

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.Mono"
	namespace OutputFormatMonoInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName MonoOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatMonoInterface::GetVersion().Name, OutputFormatMonoInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;

					constexpr bool bIsSourceDefault = true;
					UClassOptions = CreateUClassOptions(bIsSourceDefault);

					Outputs =
					{
						{
							LOCTEXT("GeneratedAudioDisplayName", "Out Mono"),
							LOCTEXT("GeneratedAudioDescription", "The resulting mono output."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Outputs::MonoOut,
							FText::GetEmpty(), // RequiredText
							EAudioParameterType::None,
							100
						}
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatMonoInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.Stereo"
	namespace OutputFormatStereoInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName LeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName RightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatStereoInterface::GetVersion().Name, OutputFormatStereoInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;
					
					UClassOptions = CreateUClassOptions();

					Outputs =
					{
						{
							LOCTEXT("OutputFormatStereoInterface_GeneratedLeftDisplayName", "Out Left"),
							LOCTEXT("OutputFormatStereoInterface_GeneratedLeftDescription", "The resulting left channel output audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Outputs::LeftOut,
							FText::GetEmpty(), // RequiredText
							EAudioParameterType::None,
							100 // Sort order
						},
						{
							LOCTEXT("OutputFormatStereoInterface_GeneratedRightDisplayName", "Out Right"),
							LOCTEXT("OutputFormatStereoInterface_GeneratedRightDescription", "The resulting right channel output audio."),
							GetMetasoundDataTypeName<FAudioBuffer>(),
							Outputs::RightOut,
							FText::GetEmpty(), // RequiredText
							EAudioParameterType::None,
							101 // Sort order
						}
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatStereoInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.Quad"
	namespace OutputFormatQuadInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName FrontLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName SideLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatQuadInterface::GetVersion().Name, OutputFormatQuadInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;

					UClassOptions = CreateUClassOptions();

					Outputs =
					{
						GetFrontLeftOutput(Outputs::FrontLeftOut),
						GetFrontRightOutput(Outputs::FrontRightOut),
						GetSideLeftOutput(Outputs::SideLeftOut),
						GetSideRightOutput(Outputs::SideRightOut)
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatQuadInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.5dot1"
	namespace OutputFormatFiveDotOneInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName FrontLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName FrontCenterOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:2");
			const FName LowFrequencyOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:3");
			const FName SideLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatFiveDotOneInterface::GetVersion().Name, OutputFormatFiveDotOneInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;

					UClassOptions = CreateUClassOptions();

					Outputs =
					{
						GetFrontLeftOutput(Outputs::FrontLeftOut),
						GetFrontRightOutput(Outputs::FrontRightOut),
						GetFrontCenterOutput(Outputs::FrontCenterOut),
						GetLowFrequencyOutput(Outputs::LowFrequencyOut),
						GetSideLeftOutput(Outputs::SideLeftOut),
						GetSideRightOutput(Outputs::SideRightOut)
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatFiveDotOneInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.7dot1"
	namespace OutputFormatSevenDotOneInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName FrontLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
			const FName FrontRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:1");
			const FName FrontCenterOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:2");
			const FName LowFrequencyOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:3");
			const FName SideLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:4");
			const FName SideRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:5");
			const FName BackLeftOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:6");
			const FName BackRightOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:7");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			return CreateInterface();
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatSevenDotOneInterface::GetVersion().Name, OutputFormatSevenDotOneInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;

					UClassOptions = CreateUClassOptions();

					Outputs =
					{
						GetFrontLeftOutput(Outputs::FrontLeftOut),
						GetFrontRightOutput(Outputs::FrontRightOut),
						GetFrontCenterOutput(Outputs::FrontCenterOut),
						GetLowFrequencyOutput(Outputs::LowFrequencyOut),
						GetSideLeftOutput(Outputs::SideLeftOut),
						GetSideRightOutput(Outputs::SideRightOut),
						GetBackLeftOutput(Outputs::BackLeftOut),
						GetBackRightOutput(Outputs::BackRightOut)
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatSevenDotOneInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE


#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.OutputFormat.Cat"
	namespace OutputFormatChannelAgnosticInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName CatOut = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Audio:0");
		}

		Audio::FParameterInterfacePtr CreateInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(OutputFormatChannelAgnosticInterface::GetVersion().Name, OutputFormatChannelAgnosticInterface::GetVersion().Number.ToInterfaceVersion())
				{
					using namespace OutputFormatPrivate;

					UClassOptions = CreateUClassOptions();

					Outputs =
					{
						GetChannelAgnosticOutput(Outputs::CatOut),
					};
				}
			};

			return MakeShared<FInterface>();
		}
	} // namespace OutputFormatChannelAgnosticInterface 
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE
	
	const FOutputAudioFormatInfoMap& GetOutputAudioFormatInfo()
	{
		auto CreateFormatInfoMap = []()
		{
			FOutputAudioFormatInfoMap Map
			{
				{
					EMetaSoundOutputAudioFormat::Mono,
					{
						OutputFormatMonoInterface::GetVersion(),
						{
							OutputFormatMonoInterface::Outputs::MonoOut
						}
					}
				},
				{
					EMetaSoundOutputAudioFormat::Stereo,
					{
						OutputFormatStereoInterface::GetVersion(),
						{
							OutputFormatStereoInterface::Outputs::LeftOut,
							OutputFormatStereoInterface::Outputs::RightOut
						}
					}
				},
				{
					EMetaSoundOutputAudioFormat::Quad,
					{
						OutputFormatQuadInterface::GetVersion(),
						{
							OutputFormatQuadInterface::Outputs::FrontLeftOut,
							OutputFormatQuadInterface::Outputs::FrontRightOut,
							OutputFormatQuadInterface::Outputs::SideLeftOut,
							OutputFormatQuadInterface::Outputs::SideRightOut
						}
					}
				},
				{
					EMetaSoundOutputAudioFormat::FiveDotOne,
					{
						OutputFormatFiveDotOneInterface::GetVersion(),
						{
							OutputFormatFiveDotOneInterface::Outputs::FrontLeftOut,
							OutputFormatFiveDotOneInterface::Outputs::FrontRightOut,
							OutputFormatFiveDotOneInterface::Outputs::FrontCenterOut,
							OutputFormatFiveDotOneInterface::Outputs::LowFrequencyOut,
							OutputFormatFiveDotOneInterface::Outputs::SideLeftOut,
							OutputFormatFiveDotOneInterface::Outputs::SideRightOut
						}
					}
				},
				{
					EMetaSoundOutputAudioFormat::SevenDotOne,
					{
						OutputFormatSevenDotOneInterface::GetVersion(),
						{
							OutputFormatSevenDotOneInterface::Outputs::FrontLeftOut,
							OutputFormatSevenDotOneInterface::Outputs::FrontRightOut,
							OutputFormatSevenDotOneInterface::Outputs::FrontCenterOut,
							OutputFormatSevenDotOneInterface::Outputs::LowFrequencyOut,
							OutputFormatSevenDotOneInterface::Outputs::SideLeftOut,
							OutputFormatSevenDotOneInterface::Outputs::SideRightOut,
							OutputFormatSevenDotOneInterface::Outputs::BackLeftOut,
							OutputFormatSevenDotOneInterface::Outputs::BackRightOut
						}
					}
				}
			};

			// Only include CAT output format when MetasoundExperimental plugin is enabled.
			if (IPluginManager::Get().FindEnabledPlugin(TEXT("MetasoundExperimental")))
			{
				Map.Add(EMetaSoundOutputAudioFormat::ChannelAgnostic,
				{
					OutputFormatChannelAgnosticInterface::GetVersion(),
					{
						OutputFormatChannelAgnosticInterface::Outputs::CatOut,
					}
				});
			}

			return Map;
		};

		static const FOutputAudioFormatInfoMap Map = CreateFormatInfoMap();
		return Map;
	}
} // namespace Metasound::Engine
#undef LOCTEXT_NAMESPACE
