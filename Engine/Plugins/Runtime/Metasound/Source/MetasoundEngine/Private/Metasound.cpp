// Copyright Epic Games, Inc. All Rights Reserved.
#include "Metasound.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Internationalization/Text.h"
#include "Logging/TokenizedMessage.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAudioFormats.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEngineAsset.h"
#include "MetasoundEngineEnvironment.h"
#include "MetasoundEnvironment.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendRegistryContainer.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundGenerator.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundPrimitives.h"
#include "MetasoundReceiveNode.h"
#include "MetasoundSettings.h"
#include "MetasoundTrigger.h"
#include "MetasoundUObjectRegistry.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Metasound)


#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif // WITH_EDITOR


#if WITH_EDITORONLY_DATA
#include "EdGraph/EdGraph.h"
#endif // WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "MetaSound"

namespace Metasound::MetasoundPrivate
{
	Frontend::FMetaSoundAssetRegistrationOptions GetInitRegistrationOptions()
	{
		Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
		RegOptions.bForceReregister = false;
#if !WITH_EDITOR 
		// When without editor, don't AutoUpdate or remove template nodes at runtime. This only happens at cook or save.
		// When with editor, those are needed because sounds are not necessarily saved before previewing.
		RegOptions.bAutoUpdate = false;
#endif // !WITH_EDITOR
		if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
		{
			RegOptions.bAutoUpdateLogWarningOnDroppedConnection = Settings->bAutoUpdateLogWarningOnDroppedConnection;
			RegOptions.PageOrder = Settings->GetPageOrder();
		}

		return RegOptions;
	}
}

int32 UMetasoundEditorGraphBase::GetHighestMessageSeverity() const
{
	int32 HighestMessageSeverity = EMessageSeverity::Info;

	for (const UEdGraphNode* Node : Nodes)
	{
		// Lower integer value is "higher severity"
		if (Node->ErrorType < HighestMessageSeverity)
		{
			HighestMessageSeverity = Node->ErrorType;
		}
	}

	return HighestMessageSeverity;
}

UMetaSoundPatch::UMetaSoundPatch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FMetasoundAssetBase()
{
	// Default Root Graph uses static ID to distinguish between a default constructed document
	// (invalid ID) and CDO.  A MetaSoundSource asset should only be constructed using the Document
	// Builder API to avoid ID collisions, but underlying UObjects must always be deterministically
	// generated using NewObject for serialization (and for CDOs).
	RootMetaSoundDocument.RootGraph.ID = FGuid(0x4d657461, 0x536f756e, 0x64506174, 0x63680000);
}

Metasound::Frontend::FDocumentAccessPtr UMetaSoundPatch::GetDocumentAccessPtr(bool bReloadBuilder)
{
	using namespace Metasound::Frontend;

	// Mutation of a document via the soft deprecated access ptr/controller system is not tracked by
	// the builder registry, so the document cache is invalidated here.
	if (bReloadBuilder)
	{
		if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
		{
			BuilderRegistry->ReloadBuilder(RootMetaSoundDocument.RootGraph.Metadata.GetClassName());
		}
	}

	// Return document using FAccessPoint to inform the TAccessPtr when the 
	// object is no longer valid.
	return MakeAccessPtr<FDocumentAccessPtr>(RootMetaSoundDocument.AccessPoint, RootMetaSoundDocument);
}

Metasound::Frontend::FConstDocumentAccessPtr UMetaSoundPatch::GetDocumentConstAccessPtr() const
{
	using namespace Metasound::Frontend;
	// Return document using FAccessPoint to inform the TAccessPtr when the 
	// object is no longer valid.
	return MakeAccessPtr<FConstDocumentAccessPtr>(RootMetaSoundDocument.AccessPoint, RootMetaSoundDocument);
}

const UClass& UMetaSoundPatch::GetBaseMetaSoundUClass() const
{
	return *UMetaSoundPatch::StaticClass();
}

const UClass& UMetaSoundPatch::GetBuilderUClass() const
{
	return *UMetaSoundPatchBuilder::StaticClass();
}

const FMetasoundFrontendDocument& UMetaSoundPatch::GetConstDocument() const
{
	return RootMetaSoundDocument;
}

EMetasoundFrontendClassAccessFlags UMetaSoundPatch::GetDefaultAccessFlags() const
{
	return EMetasoundFrontendClassAccessFlags::Referenceable;
}

#if WITH_EDITOR
void UMetaSoundPatch::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);
	Metasound::Engine::FAssetHelper::PreDuplicate(this, DupParams);
}

void UMetaSoundPatch::PostDuplicate(EDuplicateMode::Type InDuplicateMode)
{
	Super::PostDuplicate(InDuplicateMode);

	Metasound::Engine::FAssetHelper::PostDuplicate(this, InDuplicateMode);
}

void UMetaSoundPatch::PostEditUndo()
{
	Super::PostEditUndo();

	TScriptInterface<IMetaSoundDocumentInterface> DocInterface(this);
	Metasound::Engine::FAssetHelper::PostEditUndo(DocInterface);
}

EDataValidationResult UMetaSoundPatch::IsDataValid(FDataValidationContext& Context) const
{
	const EDataValidationResult Result = Metasound::Engine::FAssetHelper::IsDataValid(*this, RootMetaSoundDocument, Context);
	return CombineDataValidationResults(Result, Super::IsDataValid(Context));
}

#endif // WITH_EDITOR

void UMetaSoundPatch::BeginDestroy()
{
	OnNotifyBeginDestroy();
	Super::BeginDestroy();
}

void UMetaSoundPatch::PreSave(FObjectPreSaveContext InSaveContext)
{
	Super::PreSave(InSaveContext);
	Metasound::Engine::FAssetHelper::PreSaveAsset(*this, InSaveContext);
}

void UMetaSoundPatch::Serialize(FArchive& InArchive)
{
	Super::Serialize(InArchive);
	Metasound::Engine::FAssetHelper::SerializeToArchive(*this, InArchive);
}

#if WITH_EDITORONLY_DATA
void UMetaSoundPatch::MigrateEditorGraph(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Graph)
	{
		Graph->MigrateEditorDocumentData(OutBuilder);
		Graph = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UEdGraph* UMetaSoundPatch::GetGraph() const
{
	return EditorGraph;
}

UEdGraph& UMetaSoundPatch::GetGraphChecked() const
{
	check(EditorGraph);
	return *EditorGraph;
}
FText UMetaSoundPatch::GetDisplayName() const
{
	FString TypeName = UMetaSoundPatch::StaticClass()->GetName();
	return FMetasoundAssetBase::GetDisplayName(MoveTemp(TypeName));
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundPatch::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	Metasound::Engine::FAssetHelper::GetAssetRegistryTags(this, Context);
}

FTopLevelAssetPath UMetaSoundPatch::GetAssetPathChecked() const
{
	return Metasound::Engine::FAssetHelper::GetAssetPathChecked(*this);
}

void UMetaSoundPatch::PostLoad() 
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Metasound::Engine::FAssetHelper::PostLoad(*this, ReferenceAssetClassCache);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UMetaSoundPatch::SetReferencedAssets(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs)
{
	Metasound::Engine::FAssetHelper::SetReferencedAssets(*this, MoveTemp(InAssetRefs));
}
#endif

TSharedPtr<Audio::IProxyData> UMetaSoundPatch::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!IsRegistered())
	{
		// Make sure the metasound is registered and ready to go before making a proxy.
		UpdateAndRegisterForExecution(Metasound::MetasoundPrivate::GetInitRegistrationOptions());
	}

	return FMetasoundAssetBase::CreateProxyData(InitParams);
}

TArray<FMetasoundAssetBase*> UMetaSoundPatch::GetReferencedAssets() const
{
	return Metasound::Engine::FAssetHelper::GetReferencedAssets(*this);
}

#if WITH_EDITORONLY_DATA
void UMetaSoundPatch::OnAsyncReferencedAssetsLoaded(const TArray<FMetasoundAssetBase*>& InAsyncReferences)
{
	Metasound::Engine::FAssetHelper::OnAsyncReferencedAssetsLoaded(*this, InAsyncReferences);
}
#endif // WITH_EDITORONLY_DATA

bool UMetaSoundPatch::IsActivelyBuilding() const
{
	return bIsBuilderActive;
}

void UMetaSoundPatch::OnBeginActiveBuilder()
{
	using namespace Metasound::Frontend;

	if (bIsBuilderActive)
	{
		UE_LOGF(LogMetaSound, Error, "OnBeginActiveBuilder() call while prior builder is still active. This may indicate that multiple builders are attempting to modify the MetaSound %ls concurrently.", *GetOwningAssetName())
	}

	// If a builder is activating, make sure any in-flight registration
	// tasks have completed. Async registration tasks use the FMetasoundFrontendDocument
	// that lives on this object. We need to make sure that registration task
	// completes so that the FMetasoundFrontendDocument does not get modified
	// by a builder while it is also being read by async registration.
	const FGraphRegistryKey GraphKey = GetGraphRegistryKey();
	if (GraphKey.IsValid())
	{
		FMetasoundFrontendRegistryContainer::Get()->WaitForAsyncGraphRegistration(GraphKey);
	}

	bIsBuilderActive = true;
}

void UMetaSoundPatch::OnFinishActiveBuilder()
{
	bIsBuilderActive = false;
}

#undef LOCTEXT_NAMESPACE // MetaSound

