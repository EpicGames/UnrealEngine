// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundAssetSubsystem.h"

#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Engine/AssetManager.h"
#include "HAL/CriticalSection.h"
#include "Metasound.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEngineAsset.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendNodeUpdateTransform.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"
#include "MetasoundUObjectRegistry.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/NoExportTypes.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundAssetSubsystem)


namespace Metasound::Engine
{
	namespace AssetSubsystemPrivate
	{
		using FAssetClassInfoMap = FMetaSoundAssetManager::FAssetClassInfoMap;

		std::atomic<bool> InitialAssetScanComplete = false;

		bool RemoveClassInfo(Frontend::IMetaSoundAssetTransactor& Transactor, FCriticalSection& MapCritSec, FAssetClassInfoMap& Map, const FMetaSoundAssetKey& AssetKey, const FTopLevelAssetPath& InAssetPath)
		{
			using namespace Frontend;

			FScopeLock Lock(&MapCritSec);
			if (TArray<FMetaSoundAssetClassInfo>* AssetInfoArray = Map.Find(AssetKey))
			{
				auto ComparePaths = [&InAssetPath](const FMetaSoundAssetClassInfo& ClassInfo)
				{
					// Compare full paths if valid
					if (ClassInfo.AssetPath.IsValid() && InAssetPath.IsValid())
					{
						return ClassInfo.AssetPath == InAssetPath;
					}
					// Package names are stripped on destruction, so only asset name is reliable
					return ClassInfo.AssetPath.GetAssetName() == InAssetPath.GetAssetName();
				};

				const int32 NumRemoved = AssetInfoArray->RemoveAllSwap(ComparePaths, EAllowShrinking::No);
				if (NumRemoved > 0)
				{
					if (NumRemoved > 1 && InAssetPath.GetPackageName().IsNone())
					{
						UE_LOGF(LogMetaSound, Display,
							"MetaSoundAssetManager: Multiple assets registered with key '%ls' and is removing all asset class info with provided asset path missing package.  "
							"Likely caused by diff and request for removal is currently amidst diff object distruction.",
							*AssetKey.ToString());
					}

					if (AssetInfoArray->IsEmpty())
					{
						// Only unregister from Transactor if empty as transaction buffer only cares if *ANY* asset is registered with the given class
						Transactor.UnregisterAsset(AssetKey);
						Map.Remove(AssetKey);
					}
					return true;
				}
			}

			return false;
		}

		FMetaSoundAssetKey AddClassInfo(Frontend::IMetaSoundAssetTransactor& Transactor, FCriticalSection& MapCritSec, FAssetClassInfoMap& Map, Frontend::FMetaSoundAssetClassInfo&& ClassInfo)
		{
			using namespace Frontend;

			const FMetaSoundAssetKey AssetKey(ClassInfo.ClassName, ClassInfo.Version);
			if (!AssetKey.IsValid())
			{
				return AssetKey;
			}

			FScopeLock Lock(&MapCritSec);
			TArray<FMetaSoundAssetClassInfo>& TagDatas = Map.FindOrAdd(AssetKey);
			if (TagDatas.IsEmpty())
			{
				// Only register with Transactor if empty as transaction buffer only cares if *ANY* asset is registered with the given class
				Transactor.RegisterAsset(AssetKey);
			}
			else
			{
				auto IsTagData = [&ClassInfo](const FMetaSoundAssetClassInfo& Iter) { return Iter.AssetPath == ClassInfo.AssetPath; };
				TagDatas.RemoveAllSwap(IsTagData);
			}

			TagDatas.Add(MoveTemp(ClassInfo));

#if !NO_LOGGING
			if (TagDatas.Num() > 1)
			{
				TArray<FString> PathStrings;
				Algo::Transform(TagDatas, PathStrings, [](const FMetaSoundAssetClassInfo& WarnInfo) { return WarnInfo.AssetPath.ToString(); });
				UE_LOGF(LogMetaSound, Warning,
					"MetaSoundAssetManager has registered multiple assets with key '%ls':\n%ls\n. Reassign the class name of colliding assets by using the Reassign Asset Class Guid asset action in the right click -> Asset Actions menu of the MetaSound.",
					*AssetKey.ToString(),
					*FString::Join(PathStrings, TEXT("\n")));
			}
#endif // !NO_LOGGING

			return AssetKey;
		}

		TOptional<Frontend::FMetaSoundAssetClassInfo> TryGetClassInfoFromLoadedAsset(const FTopLevelAssetPath& InAssetPath)
		{
			using namespace Frontend;

			const FSoftObjectPath ObjectPath(InAssetPath);
			if (const UObject* Object = ObjectPath.ResolveObject())
			{
				TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(Object);
				if (DocInterface.GetObject())
				{
					return FMetaSoundAssetClassInfo(*DocInterface.GetInterface());
				}
			}

			return { };
		}
	} // AssetSubsystemPrivate

	FMetaSoundAssetManager::~FMetaSoundAssetManager()
	{
#if !NO_LOGGING
		using namespace Frontend;

		if (bLogActiveAssetsOnShutdown)
		{
			TMap<FMetaSoundAssetKey, TArray<FMetaSoundAssetClassInfo>> InfoOnShutdown;
			{
				FScopeLock Lock(&TagDataMapCriticalSection);
				InfoOnShutdown = MoveTemp(ClassInfoMap);
				ClassInfoMap.Reset();
			}

			if (!InfoOnShutdown.IsEmpty())
			{
				UE_LOGF(LogMetaSound, Display, "AssetManager is shutting down with the following %i assets active:", InfoOnShutdown.Num());
				for (const TPair<FMetaSoundAssetKey, TArray<FMetaSoundAssetClassInfo>>& Pair : InfoOnShutdown)
				{
					for (const FMetaSoundAssetClassInfo& ClassInfo : Pair.Value)
					{
						UE_LOGF(LogMetaSound, Display, "- %ls", *ClassInfo.AssetPath.ToString());
					}
				}
			}
		}
#endif // !NO_LOGGING

	}

	void FMetaSoundAssetManager::AddReferencedObjects(FReferenceCollector& Collector)
	{
#if WITH_EDITORONLY_DATA
		for (FMetaSoundAsyncAssetDependencies& Dependencies : LoadingDependencies)
		{
			Collector.AddReferencedObject(Dependencies.MetaSound);
		}
#endif // WITH_EDITORONLY_DATA
	}

#if WITH_EDITORONLY_DATA
	bool FMetaSoundAssetManager::AddAssetReferences(FMetasoundAssetBase& InAssetBase)
	{
		using namespace Frontend;

		{
			TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = InAssetBase.GetOwningAsset();
			check(DocInterface.GetObject());
			const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
			const FMetaSoundAssetKey AssetKey(Document.RootGraph.Metadata);
			if (!ContainsKey(AssetKey))
			{
				if (AddOrUpdateFromObject(*InAssetBase.GetOwningAsset()).IsValid())
				{
					UE_LOGF(LogMetaSound, Verbose, "Adding asset '%ls' to MetaSoundAsset registry.", *InAssetBase.GetOwningAssetName());
				}
			}
		}

		bool bAddFromReferencedAssets = false;
		const TSet<FString>& ReferencedAssetClassKeys = InAssetBase.GetReferencedAssetClassKeys();
		for (const FString& KeyString : ReferencedAssetClassKeys)
		{
			FNodeRegistryKey RegistryKey;
			const bool bIsKey = FNodeRegistryKey::Parse(KeyString, RegistryKey);
			if (!bIsKey || !ContainsKey(FMetaSoundAssetKey(RegistryKey)))
			{
				UE_LOGF(LogMetaSound, Verbose, "Missing referenced class '%ls' asset entry.", *KeyString);
				bAddFromReferencedAssets = true;
			}
		}

		// All keys are loaded
		if (!bAddFromReferencedAssets)
		{
			return false;
		}

		UE_LOGF(LogMetaSound, Verbose, "Attempting preemptive reference load...");

		TArray<FMetasoundAssetBase*> ReferencedAssets = InAssetBase.GetReferencedAssets();
		for (FMetasoundAssetBase* Asset : ReferencedAssets)
		{
			if (Asset)
			{
				TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = Asset->GetOwningAsset();
				check(DocInterface.GetObject());
				const FMetasoundFrontendDocument& RefDocument = DocInterface->GetConstDocument();
				const FMetaSoundAssetKey ClassKey(RefDocument.RootGraph);
				if (!ContainsKey(ClassKey))
				{
					UE_LOGF(LogMetaSound, Verbose,
						"Preemptive load of class '%ls' due to early "
							"registration request (asset scan likely not complete).",
						*ClassKey.ToString());

					UObject* MetaSoundObject = Asset->GetOwningAsset();
					if (ensureAlways(MetaSoundObject))
					{
						AddOrUpdateFromObject(*MetaSoundObject);
					}
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Null referenced dependent asset in %ls. Resaving asset in editor may fix the issue", *InAssetBase.GetOwningAssetName());
			}
		}

		return true;
	}
#endif // WITH_EDITORONLY_DATA

	FMetaSoundAssetKey FMetaSoundAssetManager::AddOrUpdateFromObject(const UObject& InObject)
	{
		// Don't add temporary assets used for diffing
		if (const UPackage* Package = InObject.GetPackage(); !Package || Package->HasAnyPackageFlags(PKG_ForDiffing))
		{
			return FMetaSoundAssetKey::GetInvalid();
		}

		return AddOrUpdateFromObjectInternal(InObject);
	}

	Frontend::FMetaSoundAssetRegistrationOptions FMetaSoundAssetManager::GetRegistrationOptions() const
	{
		Frontend::FMetaSoundAssetRegistrationOptions RegOptions;

		if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
		{
			RegOptions.bAutoUpdateLogWarningOnDroppedConnection = Settings->bAutoUpdateLogWarningOnDroppedConnection;
			RegOptions.PageOrder = Settings->GetPageOrder();
		}

		// nrvo
		return RegOptions;
	}

	FMetaSoundAssetKey FMetaSoundAssetManager::AddOrUpdateFromObjectInternal(const UObject& InObject)
	{
		using namespace AssetSubsystemPrivate;
		using namespace Frontend;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundAssetManager::AddOrUpdateFromObjectInternal);

		// Don't add temporary assets used for diffing
		if (UPackage* Package = InObject.GetPackage(); !Package || Package->HasAnyPackageFlags(PKG_ForDiffing))
		{
			return FMetaSoundAssetKey::GetInvalid();
		}

#if WITH_EDITORONLY_DATA
		// DocInterface needs to be updated and ready for extracting class info, so wait until versioning is finished
		FVersioningManager::Get().WaitUntilVersioningComplete(InObject);
#endif // WITH_EDITORONLY_DATA

		TScriptInterface<const IMetaSoundDocumentInterface> ScriptInterface(&InObject);
		check(ScriptInterface.GetObject() != nullptr);
		const IMetaSoundDocumentInterface& DocInterface = *ScriptInterface;
		return AssetSubsystemPrivate::AddClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, FMetaSoundAssetClassInfo(DocInterface));
	}

	void FMetaSoundAssetManager::AddOrLoadAndUpdateFromObjectAsync(const FAssetData& InAssetData, Frontend::IMetaSoundAssetManager::FOnUpdatedAssetLoaded&& OnUpdatedAssetLoaded)
	{
		using namespace Frontend;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundAssetSubsystem::AddOrLoadAndUpdateFromObjectAsync);

		// Don't add temporary assets used for diffing
		if (InAssetData.HasAnyPackageFlags(PKG_ForDiffing))
		{
			OnUpdatedAssetLoaded.Reset();
			return;
		}

		const FSoftObjectPath Path = InAssetData.ToSoftObjectPath();
		if (UObject* Object = Path.ResolveObject())
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "FMetaSoundAssetManager::AddOrLoadAndUpdateFromObjectAsync called, but object ''%ls' already loaded. Updating entry using in-memory object definition.", *Path.ToString());
			const FMetaSoundAssetKey AssetKey = AddOrUpdateFromObjectInternal(*Object);
			OnUpdatedAssetLoaded(AssetKey, *Object);
			OnUpdatedAssetLoaded.Reset();
			return;
		}

#if WITH_EDITORONLY_DATA
		ActiveAsyncAssetLoadRequests++;
#endif // WITH_EDITORONLY_DATA

		UE_LOGF(LogMetaSound, VeryVerbose, "MetaSoundAssetManager requested aync loading asset '%ls'...", *InAssetData.GetSoftObjectPath().ToString());

		LoadPackageAsync(Path.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateLambda([OnAssetLoaded = MoveTemp(OnUpdatedAssetLoaded)](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result) mutable
		{
			FMetaSoundAssetManager::OnPackageLoaded(FPackageLoadedArgs
			{
				.PackageName = PackageName,
				.Package = Package,
				.Result = Result,
				.OnUpdatedAssetLoaded = MoveTemp(OnAssetLoaded)
			});
		}));
	}

	void FMetaSoundAssetManager::AddOrUpdateFromAssetData(const FAssetData& InAssetData)
	{
		using namespace Frontend;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundAssetSubsystem::AddOrUpdateAsset_AssetData);

		// Don't add temporary assets used for diffing
		if (InAssetData.HasAnyPackageFlags(PKG_ForDiffing))
		{
			return;
		}

		// If object is loaded, always use data in memory instead of asset tag data as it may be more up-to-date.
		const FSoftObjectPath Path = InAssetData.ToSoftObjectPath();
		if (UObject* Object = Path.ResolveObject())
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "AddOrUpdateFromAssetData called, but object '%ls' already loaded. Updating entry using in-memory object definition.", *Path.ToString());
			AddOrUpdateFromObjectInternal(*Object);
		}
		else
		{
			// Older document versions did not serialize asset tags. In these cases, a load of the object is required
			// in order to retrieve the appropriate asset tag data translated to asset info cached in the MetaSound Asset Manager.
			FMetaSoundAssetClassInfo ClassInfo(InAssetData);
			if (ClassInfo.bIsValid)
			{
				AssetSubsystemPrivate::AddClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, MoveTemp(ClassInfo));
			}
			else
			{
#if WITH_EDITORONLY_DATA
				ActiveAsyncAssetLoadRequests++;
#endif // WITH_EDITORONLY_DATA
		
				UE_LOGF(LogMetaSound, Verbose, "MetaSound tags require updating: MetaSoundAssetManager aync loading asset '%ls' to access tags/update entry...", *InAssetData.GetSoftObjectPath().ToString());

				LoadPackageAsync(Path.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateLambda([](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
				{
					FMetaSoundAssetManager::OnPackageLoaded(FPackageLoadedArgs { .PackageName = PackageName, .Package = Package, .Result = Result });
				}));
			}
		}
	}

	bool FMetaSoundAssetManager::ContainsKey(const FMetaSoundAssetKey& InKey) const
	{
		FScopeLock Lock(&TagDataMapCriticalSection);
		return ClassInfoMap.Contains(InKey);
	}

	bool FMetaSoundAssetManager::ContainsKey(const Frontend::FNodeRegistryKey& InRegistryKey) const
	{
		if (FMetaSoundAssetKey::IsValidType(InRegistryKey.Type))
		{
			return ContainsKey(FMetaSoundAssetKey(InRegistryKey));
		}

		return false;
	}

	void FMetaSoundAssetManager::DepthFirstVisit_AssetKey(const FMetaSoundAssetKey& InKey, TFunctionRef<TSet<FMetaSoundAssetKey>(const FMetaSoundAssetKey&)> VisitFunction)
	{
		// Non recursive depth first traversal.
		TArray<FMetaSoundAssetKey> Stack({ InKey });
		TSet<FMetaSoundAssetKey> Visited;

		while (!Stack.IsEmpty())
		{
			const FMetaSoundAssetKey CurrentKey = Stack.Pop();
			if (!Visited.Contains(CurrentKey))
			{
				TArray<FMetaSoundAssetKey> Children = VisitFunction(CurrentKey).Array();
				Stack.Append(MoveTemp(Children));
				Visited.Add(CurrentKey);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	FMetaSoundAsyncAssetDependencies* FMetaSoundAssetManager::FindLoadingDependencies(const UObject* InParentAsset)
	{
		auto IsEqualMetaSoundUObject = [InParentAsset](const FMetaSoundAsyncAssetDependencies& InDependencies) -> bool
		{
			return (InDependencies.MetaSound == InParentAsset);
		};

		return LoadingDependencies.FindByPredicate(IsEqualMetaSoundUObject);
	}

	FMetaSoundAsyncAssetDependencies* FMetaSoundAssetManager::FindLoadingDependencies(int32 InLoadID)
	{
		auto IsEqualID = [InLoadID](const FMetaSoundAsyncAssetDependencies& InDependencies) -> bool
		{
			return (InDependencies.LoadID == InLoadID);
		};

		return LoadingDependencies.FindByPredicate(IsEqualID);
	}
#endif // WITH_EDITORONLY_DATA

	FMetasoundAssetBase* FMetaSoundAssetManager::FindAsset(const FMetaSoundAssetKey& InKey) const
	{
		FTopLevelAssetPath AssetPath = FindAssetPath(InKey);
		if (AssetPath.IsValid())
		{
			if (UObject* Object = FSoftObjectPath(AssetPath, { }).ResolveObject())
			{
				return GetAsAsset(*Object);
			}
		}

		return nullptr;
	}

	Frontend::FMetaSoundAssetClassInfo FMetaSoundAssetManager::FindAssetClassInfo(const FTopLevelAssetPath& InPath) const
	{
		using namespace Frontend;

		TOptional<FMetaSoundAssetClassInfo> ClassInfo = AssetSubsystemPrivate::TryGetClassInfoFromLoadedAsset(InPath);
		if (ClassInfo.IsSet())
		{
			return *ClassInfo;
		}

		FAssetData AssetData;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		UE::AssetRegistry::EExists Exists = AssetRegistryModule.TryGetAssetByObjectPath(FSoftObjectPath(InPath), AssetData);
		if (Exists == UE::AssetRegistry::EExists::Exists)
		{
			FMetaSoundAssetKey AssetKey;
			if (FMetaSoundAssetClassInfo::TryGetAssetKey(AssetData, AssetKey))
			{
				const TArray<Frontend::FMetaSoundAssetClassInfo> CachedClassInfo = FindAssetClassInfoInternal(AssetKey);
				if (CachedClassInfo.Num() == 1)
				{
					return CachedClassInfo.Last();
				}
			}

			return FMetaSoundAssetClassInfo(AssetData);
		}

		return { };
	}

	TArray<Frontend::FMetaSoundClassInfo> FMetaSoundAssetManager::FindRegisteredClassInfo(const FMetaSoundAssetKey& InKey) const
	{
		using namespace Frontend;

		TArray<FMetaSoundAssetClassInfo> AssetClassInfo = FindAssetClassInfoInternal(InKey);
		TArray<FMetaSoundClassInfo> ClassInfo;
		Algo::Transform(AssetClassInfo, ClassInfo, [](const FMetaSoundAssetClassInfo& AssetInfo)
		{
			// If loaded, use that as its potentially "fresher" than the cached info in the AssetManager (provided at last load/save)
			TOptional<FMetaSoundAssetClassInfo> LoadedInfo = AssetSubsystemPrivate::TryGetClassInfoFromLoadedAsset(AssetInfo.AssetPath);
			if (LoadedInfo.IsSet())
			{
				return *LoadedInfo;
			}
			return AssetInfo;
		});
		return ClassInfo;
	}

	TArray<Frontend::FMetaSoundAssetClassInfo> FMetaSoundAssetManager::FindAssetClassInfoInternal(const FMetaSoundAssetKey& InKey) const
	{
		FScopeLock Lock(&TagDataMapCriticalSection);
		return ClassInfoMap.FindRef(InKey);
	}

	TScriptInterface<IMetaSoundDocumentInterface> FMetaSoundAssetManager::FindAssetAsDocumentInterface(const FMetaSoundAssetKey& InKey) const
	{
		const FTopLevelAssetPath AssetPath = FindAssetPath(InKey);
		if (AssetPath.IsValid())
		{
			if (UObject* Object = FSoftObjectPath(AssetPath, { }).ResolveObject())
			{
				return TScriptInterface<IMetaSoundDocumentInterface>(Object);
			}
		}

		return nullptr;
	}

	FTopLevelAssetPath FMetaSoundAssetManager::FindAssetPath(const FMetaSoundAssetKey& InKey) const
	{
		using namespace Frontend;

		FScopeLock Lock(&TagDataMapCriticalSection);
		if (const TArray<FMetaSoundAssetClassInfo>* TagDatas = ClassInfoMap.Find(InKey))
		{
			if (!TagDatas->IsEmpty())
			{
				return TagDatas->Last().AssetPath;
			}
		}

		return nullptr;
	}

	TArray<FTopLevelAssetPath> FMetaSoundAssetManager::FindAssetPaths(const FMetaSoundAssetKey& InKey) const
	{
		using namespace Frontend;

		FScopeLock Lock(&TagDataMapCriticalSection);
		if (const TArray<FMetaSoundAssetClassInfo>* TagDatas = ClassInfoMap.Find(InKey))
		{
			TArray<FTopLevelAssetPath> Paths;
			Algo::Transform(*TagDatas, Paths, [](const FMetaSoundAssetClassInfo& ClassInfo) { return ClassInfo.AssetPath; });
			return Paths;
		}

		return { };
	}

#if WITH_EDITOR
	int32 FMetaSoundAssetManager::GetActiveAsyncLoadRequestCount() const
	{
		return ActiveAsyncAssetLoadRequests;
	}
#endif // WITH_EDITOR

	FMetasoundAssetBase* FMetaSoundAssetManager::GetAsAsset(UObject& InObject) const
	{
		return IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InObject);
	}

	const FMetasoundAssetBase* FMetaSoundAssetManager::GetAsAsset(const UObject& InObject) const
	{
		return IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InObject);
	}

	FMetaSoundAssetKey FMetaSoundAssetManager::GetAssetKey(const FSoftObjectPath& InObjectPath) const
	{
		using namespace Frontend;

		FAssetData AssetData;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		UE::AssetRegistry::EExists Exists = AssetRegistryModule.TryGetAssetByObjectPath(InObjectPath, AssetData);
		if (Exists == UE::AssetRegistry::EExists::Exists)
		{
			FMetaSoundAssetKey AssetKey;
			if (FMetaSoundAssetClassInfo::TryGetAssetKey(AssetData, AssetKey))
			{
				return AssetKey;
			}
		}

		return { };
	}

	bool FMetaSoundAssetManager::IsInitialAssetScanComplete() const
	{
		return AssetSubsystemPrivate::InitialAssetScanComplete;
	}

#if WITH_EDITORONLY_DATA
	void FMetaSoundAssetManager::OnReferencedAssetsLoaded(int32 InLoadID)
	{
		FMetaSoundAsyncAssetDependencies* LoadedDependencies = FindLoadingDependencies(InLoadID);
		if (ensureMsgf(LoadedDependencies, TEXT("Call to async asset load complete with invalid IDs %d"), InLoadID))
		{
			if (LoadedDependencies->StreamableHandle.IsValid())
			{
				if (LoadedDependencies->MetaSound)
				{
					Metasound::IMetasoundUObjectRegistry& UObjectRegistry = Metasound::IMetasoundUObjectRegistry::Get();
					FMetasoundAssetBase* ParentAssetBase = UObjectRegistry.GetObjectAsAssetBase(LoadedDependencies->MetaSound);
					if (ensureMsgf(ParentAssetBase, TEXT("UClass of Parent MetaSound asset %s is not registered in metasound UObject Registery"), *LoadedDependencies->MetaSound->GetPathName()))
					{
						// Get all async loaded assets
						TArray<UObject*> LoadedAssets;
						LoadedDependencies->StreamableHandle->GetLoadedAssets(LoadedAssets);

						// Cast UObjects to FMetaSoundAssetBase
						TArray<FMetasoundAssetBase*> LoadedAssetBases;
						for (UObject* AssetDependency : LoadedAssets)
						{
							if (AssetDependency)
							{
								FMetasoundAssetBase* AssetDependencyBase = UObjectRegistry.GetObjectAsAssetBase(AssetDependency);
								if (ensure(AssetDependencyBase))
								{
									LoadedAssetBases.Add(AssetDependencyBase);
								}
							}
						}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
						// Update parent asset with loaded assets.
						ParentAssetBase->OnAsyncReferencedAssetsLoaded(LoadedAssetBases);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}
			}

			// Remove from active array of loading dependencies.
			RemoveLoadingDependencies(InLoadID);
		}
	}
#endif // WITH_EDITORONLY_DATA

	void FMetaSoundAssetManager::OnPackageLoaded(const FPackageLoadedArgs& PackageLoadedArgs)
	{
		FMetaSoundAssetManager* AssetManager = FMetaSoundAssetManager::Get();
		if (!AssetManager)
		{
			return; // Likely shutting down
		}

#if WITH_EDITOR
		AssetManager->ActiveAsyncAssetLoadRequests--;

		if (AssetManager->bNotifyTagDataScanComplete)
		{
			if (AssetManager->IsInitialAssetScanComplete() && AssetManager->ActiveAsyncAssetLoadRequests == 0)
			{
				AssetManager->bNotifyTagDataScanComplete = false;
				UE_LOGF(LogMetaSound, Display, "Async MetaSound Load/Asset Tag Prime Complete");
			}
		}
#endif // WITH_EDITOR

		switch (PackageLoadedArgs.Result)
		{
			case EAsyncLoadingResult::Succeeded:
			{
				check(PackageLoadedArgs.Package);
				if (UObject* MetaSoundObj = PackageLoadedArgs.Package->FindAssetInPackage())
				{
					// Tags had to be versioned if asset tags were not loaded properly so mark asset as versioned on load.
					// This flags version scripts to properly resave even if the document version remained the same.
					{
#if WITH_EDITORONLY_DATA
						if (FMetasoundAssetBase* AssetBase = AssetManager->GetAsAsset(*MetaSoundObj))
						{
							AssetBase->SetVersionedOnLoad();
						}
#endif // WITH_EDITORONLY_DATA

						const FMetaSoundAssetKey AssetKey = AssetManager->AddOrUpdateFromObject(*MetaSoundObj);
						if (AssetKey.IsValid() && PackageLoadedArgs.OnUpdatedAssetLoaded)
						{
							PackageLoadedArgs.OnUpdatedAssetLoaded(AssetKey, *MetaSoundObj);
						}
					}
				}
				break;
			}

			case EAsyncLoadingResult::Canceled:
			{
				UE_LOGF(LogMetaSound, VeryVerbose, "MetaSoundAssetManager request to aync load asset '%ls' canceled", *PackageLoadedArgs.PackageName.ToString());
				break;
			}

			default:
			{
				UE_LOGF(LogMetaSound, Warning, "MetaSoundAssetManager request to aync load asset '%ls' failed", *PackageLoadedArgs.PackageName.ToString());
				break;
			}
		}
	}

#if WITH_EDITOR
	TSet<Frontend::IMetaSoundAssetManager::FAssetRef> FMetaSoundAssetManager::GetReferencedAssets(const FMetasoundAssetBase& InAssetBase) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundAssetManager::GetReferencedAssets);
		using namespace Frontend;

		using FAssetRef = IMetaSoundAssetManager::FAssetRef;

		TSet<FAssetRef> OutAssetRefs;

		TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = InAssetBase.GetOwningAsset();
		check(DocInterface.GetObject());
		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		for (const FMetasoundFrontendClass& Class : Document.Dependencies)
		{
			if (Class.Metadata.GetType() != EMetasoundFrontendClassType::External)
			{
				continue;
			}

			const FMetaSoundAssetKey AssetKey(Class.Metadata);
			FTopLevelAssetPath ObjectPath = FindAssetPath(AssetKey);
			if (ObjectPath.IsValid())
			{
				FAssetRef AssetRef { FMetaSoundAssetKey(Class.Metadata), ObjectPath };
				OutAssetRefs.Add(MoveTemp(AssetRef));
			}
			else
			{
				const FNodeRegistryKey RegistryKey(Class.Metadata);
				const FMetasoundFrontendRegistryContainer* Registry = FMetasoundFrontendRegistryContainer::Get();
				check(Registry);

				const bool bIsRegistered = Registry->IsNodeRegistered(RegistryKey);
				if (!bIsRegistered)
				{
					// Don't report failure if a matching class with a matching major version and higher minor version exists (it will be autoupdated) 
					FMetaSoundClassInfo ClassInfo;
					const bool bDidFindClassWithName = ISearchEngine::Get().FindClassWithHighestVersion(AssetKey.ClassName.ToNodeClassName(), ClassInfo);
					bool bEligibleForAutoUpdate = ClassInfo.bIsValid &&
						AssetKey.Version.Major == ClassInfo.Version.Major &&
						AssetKey.Version.Minor < ClassInfo.Version.Minor;
					if (!bEligibleForAutoUpdate)
					{
						if (const TSharedPtr<const FBaseNodeUpdateTransform> MajorCustomUpdateTransform = INodeClassRegistry::Get()->FindCustomNodeUpdateTransform(AssetKey.ClassName, AssetKey.Version.Major))
						{
							bEligibleForAutoUpdate = MajorCustomUpdateTransform->ShouldAutoApply();
						}
					}
					
					if (!bDidFindClassWithName && !bEligibleForAutoUpdate)
					{
						if (IsInitialAssetScanComplete())
						{
							UE_LOGF(LogMetaSound, Warning, "MetaSound Node Class with registry key '%ls' not registered when gathering referenced asset classes from '%ls': Retrieving all asset classes may not be comprehensive.", *AssetKey.ToString(), *InAssetBase.GetOwningAssetName());
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Attempt to get registered dependent asset with key '%ls' from MetaSound asset '%ls' before asset scan has completed: Asset class cannot be provided", *AssetKey.ToString(), *InAssetBase.GetOwningAssetName());
						}
					}
				}
			}
		}
		return MoveTemp(OutAssetRefs);
	}
	
	bool FMetaSoundAssetManager::GetReferencedPresetHierarchy(FMetasoundAssetBase& InAsset, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const
	{
		using namespace Frontend;

		OutReferencedAssets.Reset();

		TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = InAsset.GetOwningAsset();
		check(DocInterface.GetObject());

		const UStruct* Struct = DocInterface->GetConstDocument().Template.GetScriptStruct();
		if (!Struct || !Struct->IsChildOf(FPresetTemplate::StaticStruct()))
		{
			return false;
		}

		const bool bSuccess = GetReferencedPresetHierarchyInternal(InAsset, OutReferencedAssets);
		// Remove first element (this)
		if (OutReferencedAssets.Num() > 0)
		{
			OutReferencedAssets.RemoveAtSwap(0);
		}
		return bSuccess;
	}

	bool FMetaSoundAssetManager::GetReferencedPresetHierarchyInternal(FMetasoundAssetBase& InAsset, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const
	{
		using namespace Frontend;

		OutReferencedAssets.Add(&InAsset);

		TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = InAsset.GetOwningAsset();

		// Base case (found the first non preset referenced asset)
		const UStruct* Struct = DocInterface->GetConstDocument().Template.GetScriptStruct();
		if (!Struct || !Struct->IsChildOf(FPresetTemplate::StaticStruct()))
		{
			return true;
		}

		// Presets must have a single asset reference
		FMetasoundAssetBase* ReferencedAsset = nullptr;
		if (InAsset.GetReferencedAssets().Num() == 1)
		{
			ReferencedAsset = InAsset.GetReferencedAssets().Last();
		}

		if (!ReferencedAsset)
		{
			UE_LOGF(LogMetaSound, Display, "MetaSound preset '%ls' does not provide valid parent asset. Ending referenced preset iteration.", *InAsset.GetOwningAssetName());
			return false;
		}

		return GetReferencedPresetHierarchyInternal(*ReferencedAsset, OutReferencedAssets);
	}

	bool FMetaSoundAssetManager::ReassignClassName(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
	{
		using namespace Frontend;

#if WITH_EDITORONLY_DATA
		UObject* MetaSoundObject = DocInterface.GetObject();
		if (!MetaSoundObject)
		{
			return false;
		}

		FMetasoundAssetBase* AssetBase = GetAsAsset(*MetaSoundObject);
		if (!AssetBase)
		{
			return false;
		}

		FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(DocInterface);

		const FMetasoundFrontendClassMetadata& ClassMetadata = Builder.GetConstDocumentChecked().RootGraph.Metadata;
		const FTopLevelAssetPath Path(MetaSoundObject);

		AssetBase->UnregisterGraphWithFrontend();

		{
			const FMetaSoundAssetKey OldAssetKey(ClassMetadata.GetClassName(), ClassMetadata.GetVersion());
			AssetSubsystemPrivate::RemoveClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, OldAssetKey, Path);
		}

		Builder.GenerateNewClassName();

		{
			AssetSubsystemPrivate::AddClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, FMetaSoundAssetClassInfo(*DocInterface));
		}

		AssetBase->UpdateAndRegisterForExecution(GetRegistrationOptions());
		return true;

#else // !WITH_EDITORONLY_DATA
		return false;
#endif // !WITH_EDITORONLY_DATA
	}
#endif // WITH_EDITOR

	bool FMetaSoundAssetManager::IsAssetClass(const FMetasoundFrontendClassMetadata& ClassMetadata) const
	{
		const EMetasoundFrontendClassType& ClassType = ClassMetadata.GetType();
		if (ClassType != EMetasoundFrontendClassType::External && ClassType != EMetasoundFrontendClassType::Graph)
		{
			return false;
		}

		return ContainsKey(FMetaSoundAssetKey(ClassMetadata));
	}

#if WITH_EDITOR
	void FMetaSoundAssetManager::IterateAssetTagData(TFunctionRef<void(Frontend::FMetaSoundAssetClassInfo)> Iter, bool bIterateDuplicates) const
	{
		using namespace Frontend;

		TArray<TArray<FMetaSoundAssetClassInfo>> TagDataMatrix;
		{
			FScopeLock Lock(&TagDataMapCriticalSection);
			ClassInfoMap.GenerateValueArray(TagDataMatrix);
		}

		if (bIterateDuplicates)
		{
			for (TArray<FMetaSoundAssetClassInfo>& TagDataArray : TagDataMatrix)
			{
				for (FMetaSoundAssetClassInfo& ClassInfo : TagDataArray)
				{
					Iter(MoveTemp(ClassInfo));
				}
			}
		}
		else
		{
			for (TArray<FMetaSoundAssetClassInfo>& TagDataArray : TagDataMatrix)
			{
				if (!TagDataArray.IsEmpty())
				{
					Iter(MoveTemp(TagDataArray.Last()));
				}
			}
		}
	}

	void FMetaSoundAssetManager::IterateReferences(const FMetaSoundAssetKey& InKey, TFunctionRef<void(const FMetaSoundAssetKey&)> VisitFunction) const
	{
		DepthFirstVisit_AssetKey(InKey, [&](const FMetaSoundAssetKey& ReferencedKey)
		{
			using namespace Frontend;

			TArray<FMetaSoundAssetClassInfo> TagDatas;
			{
				FScopeLock Lock(&TagDataMapCriticalSection);
				if (const TArray<FMetaSoundAssetClassInfo>* TagDataArray = ClassInfoMap.Find(ReferencedKey))
				{
					TagDatas = *TagDataArray;
				}
			}

			TSet<FMetaSoundAssetKey> Children;
			for (FMetaSoundAssetClassInfo& ClassInfo : TagDatas)
			{
				// If loaded, this is the "freshest" reference list, so use that.
				if (const FMetasoundAssetBase* LoadedAsset = FindAsset(ReferencedKey))
				{
					TScriptInterface<const IMetaSoundDocumentInterface> LoadedAssetDocInterface = LoadedAsset->GetOwningAsset();
					const FMetasoundFrontendDocument& Document = LoadedAssetDocInterface->GetConstDocument();
					for (const FMetasoundFrontendClass& Class : Document.Dependencies)
					{
						if (Class.Metadata.GetType() == EMetasoundFrontendClassType::External)
						{
							bool bIsAsset = false;
							
							{
								FScopeLock Lock(&TagDataMapCriticalSection);
								bIsAsset = ClassInfoMap.Contains(Class.Metadata);
							}

							if (bIsAsset)
							{
								const FMetaSoundAssetKey RefKey(Class.Metadata);
								VisitFunction(RefKey);
								Children.Add(RefKey);
							}
						}
					}

				}
				// Otherwise, use provided reference list from the asset tag data
				else
				{
					for (const FMetaSoundAssetKey& RefKey : ClassInfo.DocInfo.ReferencedAssetKeys)
					{
						VisitFunction(RefKey);
					}
					Children.Append(MoveTemp(ClassInfo.DocInfo.ReferencedAssetKeys));
				}
			}

			VisitFunction(ReferencedKey);
			Children.Add(ReferencedKey);
			return Children;
		});
	}
#endif // WITH_EDITOR

	void FMetaSoundAssetManager::RebuildDenyListCache(const UAssetManager& InAssetManager)
	{
	}

	void FMetaSoundAssetManager::RegisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& InDirectories)
	{
		TArray<FDirectoryPath> Directories;
		Algo::Transform(InDirectories, Directories, [](const FMetaSoundAssetDirectory& AssetDir) { return AssetDir.Directory; });

		Frontend::FMetaSoundAssetRegistrationOptions RegOptions = GetRegistrationOptions();

		SearchAndIterateDirectoryAssets(Directories, [this, RegOptions](const FAssetData& AssetData)
		{
			AddOrLoadAndUpdateFromObjectAsync(AssetData, [RegOptions](FMetaSoundAssetKey, UObject& AssetObject)
			{
				FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&AssetObject);
				check(MetaSoundAsset);
				MetaSoundAsset->UpdateAndRegisterForExecution(RegOptions);
			});
		});
	}

	void FMetaSoundAssetManager::RemoveAsset(const UObject& InObject)
	{
		using namespace Frontend;

		// Don't need to remove assets used for diffing as they can't be added.
		if (UPackage* Package = InObject.GetPackage(); Package && Package->HasAnyPackageFlags(PKG_ForDiffing))
		{
			return;
		}

		TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(&InObject);
		check(DocInterface.GetObject());
		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		const FMetasoundFrontendClassMetadata& Metadata = Document.RootGraph.Metadata;

		const FTopLevelAssetPath AssetPath(&InObject);
		const FMetaSoundAssetKey AssetKey(Metadata.GetClassName(), Metadata.GetVersion());
		AssetSubsystemPrivate::RemoveClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, AssetKey, AssetPath);
	}

	void FMetaSoundAssetManager::RemoveAsset(const FAssetData& InAssetData)
	{
		using namespace Frontend;

		const FMetaSoundAssetClassInfo ClassInfo(InAssetData);
		if (ClassInfo.bIsValid)
		{
			if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
			{
				constexpr bool bForceUnregister = true;
				BuilderRegistry->FinishBuilding(ClassInfo.ClassName, ClassInfo.AssetPath, bForceUnregister);
			}

			const FMetaSoundAssetKey AssetKey(ClassInfo.ClassName, ClassInfo.Version);
			AssetSubsystemPrivate::RemoveClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, AssetKey, ClassInfo.AssetPath);
		}
	}

#if WITH_EDITORONLY_DATA
	void FMetaSoundAssetManager::RemoveLoadingDependencies(int32 InLoadID)
	{
		auto IsEqualID = [InLoadID](const FMetaSoundAsyncAssetDependencies& InDependencies) -> bool
		{
			return (InDependencies.LoadID == InLoadID);
		};
		LoadingDependencies.RemoveAllSwap(IsEqualID);
	}
#endif // WITH_EDITORONLY_DATA

	void FMetaSoundAssetManager::RenameAsset(const FAssetData& InAssetData, const FString& InOldObjectPath)
	{
		using namespace Frontend;

		FMetasoundAssetBase* MetaSoundAsset = GetAsAsset(*InAssetData.GetAsset());
		check(MetaSoundAsset);

		FMetaSoundAssetClassInfo ClassInfo(InAssetData);
		if (ensureAlways(ClassInfo.bIsValid))
		{
			const FMetaSoundAssetKey AssetKey(ClassInfo.ClassName, ClassInfo.Version);
			const FTopLevelAssetPath OldPath(InOldObjectPath);
			AssetSubsystemPrivate::RemoveClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, AssetKey, OldPath);
			AssetSubsystemPrivate::AddClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, MoveTemp(ClassInfo));
		}
	}

#if WITH_EDITOR
	bool FMetaSoundAssetManager::ReplaceReferencesInAssetInternal(UObject* MetaSoundObject, const TMap<FNodeRegistryKey, FNodeRegistryKey>& InOldToNewReferenceKeys) const
	{
		MetaSoundObject->Modify();
		FMetaSoundFrontendDocumentBuilder& Builder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSoundObject);
		const bool bDependencyUpdated = Builder.UpdateDependencyRegistryData(InOldToNewReferenceKeys);
		if (bDependencyUpdated)
		{
			Builder.RemoveUnusedDependencies();
			if (FMetasoundAssetBase* AssetBase = GetAsAsset(*MetaSoundObject); ensure(AssetBase))
			{
				AssetBase->RebuildReferencedAssetClasses();
			}
			return true;
		}
		return false;
	}
	
	bool FMetaSoundAssetManager::ReplaceReferencesInAsset(TScriptInterface<IMetaSoundDocumentInterface> InMetaSound, const Frontend::FNodeRegistryKey& OldClassKey, const Frontend::FNodeRegistryKey& NewClassKey) const
	{
		using namespace Frontend;

		bool bReferencesReplaced = false;

#if WITH_EDITORONLY_DATA
		if (!NewClassKey.IsValid())
		{
			return bReferencesReplaced;
		}
		
		if (UObject* MetaSoundObject = InMetaSound.GetObject())
		{
			FMetaSoundClassInfo ClassInfo;
			const bool bNewClassExists = ISearchEngine::Get().FindClassWithHighestVersion(NewClassKey.ClassName, ClassInfo);
			if (bNewClassExists)
			{
				TMap<FNodeRegistryKey, FNodeRegistryKey> OldToNewReferenceKeys = { { OldClassKey, NewClassKey } };
				bReferencesReplaced = ReplaceReferencesInAssetInternal(MetaSoundObject, OldToNewReferenceKeys);
			}
			else
			{
				UE_LOGF(LogMetaSound, Display, "Cannot replace references in MetaSound asset '%ls': NewClass '%ls' does not exist", *InMetaSound.GetObject()->GetFullName(), *NewClassKey.ToString());
			}
		}
		
#endif // WITH_EDITORONLY_DATA
		return bReferencesReplaced;
	}
	
	bool FMetaSoundAssetManager::ReplaceReferencesInDirectory(const TArray<FMetaSoundAssetDirectory>& InDirectories, const Frontend::FNodeRegistryKey& OldClassKey, const Frontend::FNodeRegistryKey& NewClassKey) const
	{
		using namespace Frontend;

		bool bReferencesReplaced = false;

#if WITH_EDITORONLY_DATA
		if (!NewClassKey.IsValid())
		{
			return bReferencesReplaced;
		}

		FMetaSoundClassInfo ClassInfo;
		const bool bNewClassExists = ISearchEngine::Get().FindClassWithHighestVersion(NewClassKey.ClassName, ClassInfo);
		if (bNewClassExists)
		{
			TArray<FDirectoryPath> Directories;
			Algo::Transform(InDirectories, Directories, [](const FMetaSoundAssetDirectory& AssetDir) { return AssetDir.Directory; });

			TMap<FNodeRegistryKey, FNodeRegistryKey> OldToNewReferenceKeys = { { OldClassKey, NewClassKey } };
			SearchAndIterateDirectoryAssets(Directories, [this, &bReferencesReplaced, &OldToNewReferenceKeys](const FAssetData& AssetData)
			{
				if (UObject* MetaSoundObject = AssetData.GetAsset())
				{
					bReferencesReplaced |= ReplaceReferencesInAssetInternal(MetaSoundObject, OldToNewReferenceKeys);
				}
			});
		}
		else
		{
			UE_LOGF(LogMetaSound, Display, "Cannot replace references in MetaSound assets found in given directory/directories: NewClass '%ls' does not exist", *NewClassKey.ToString());
		}
#endif // WITH_EDITORONLY_DATA

		return bReferencesReplaced;
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	void FMetaSoundAssetManager::RequestAsyncLoadReferencedAssets(FMetasoundAssetBase& InAssetBase, const TSet<FSoftObjectPath>* Paths)
	{
		if (Paths && !Paths->IsEmpty())
		{
			if (UObject* OwningAsset = InAssetBase.GetOwningAsset())
			{
				TArray<FSoftObjectPath> PathsToLoad = Paths->Array();

				// Protect against duplicate calls to async load assets. 
				if (FMetaSoundAsyncAssetDependencies* ExistingAsyncLoad = FindLoadingDependencies(OwningAsset))
				{
					if (ExistingAsyncLoad->Dependencies == PathsToLoad)
					{
						// early out since these are already actively being loaded.
						return;
					}
				}

				int32 AsyncLoadID = AsyncLoadIDCounter++;

				auto AssetsLoadedDelegate = [AsyncLoadID]()
				{
					if (FMetaSoundAssetManager* ThisManager = FMetaSoundAssetManager::Get())
					{
						ThisManager->OnReferencedAssetsLoaded(AsyncLoadID);
					}
				};

				// Store async loading data for use when async load is complete.
				FMetaSoundAsyncAssetDependencies& AsyncDependencies = LoadingDependencies.AddDefaulted_GetRef();

				AsyncDependencies.LoadID = AsyncLoadID;
				AsyncDependencies.MetaSound = OwningAsset;
				AsyncDependencies.Dependencies = PathsToLoad;
				AsyncDependencies.StreamableHandle = StreamableManager.RequestAsyncLoad(PathsToLoad, AssetsLoadedDelegate);
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Cannot load async asset as FMetasoundAssetBase null owning UObject");
			}
		}

	}
#endif // WITH_EDITORONLY_DATA

	void FMetaSoundAssetManager::ReregisterLoadedAssets() const
	{
		using namespace Frontend;

		IMetasoundUObjectRegistry& ObjectRegistry = IMetasoundUObjectRegistry::Get();
		TSet<FMetasoundAssetBase*> ToReregister;

		{
			FScopeLock Lock(&TagDataMapCriticalSection);
			{
				for (const TPair<FMetaSoundAssetKey, TArray<FMetaSoundAssetClassInfo>>& KeyInfoPair : ClassInfoMap)
				{
					if (!KeyInfoPair.Value.IsEmpty())
					{
						const FMetaSoundAssetClassInfo& ClassInfo = KeyInfoPair.Value.Last();
						if (UObject* Object = FSoftObjectPath(ClassInfo.AssetPath, { }).ResolveObject())
						{
							if (FMetasoundAssetBase* Asset = ObjectRegistry.GetObjectAsAssetBase(Object); Asset && Asset->IsRegistered())
							{
								ToReregister.Add(Asset);
							}
						}
					}
				}
			}
		}

		const Frontend::FMetaSoundAssetRegistrationOptions RegOptions = GetRegistrationOptions();
		for (FMetasoundAssetBase* AssetToReregister : ToReregister)
		{
			check(AssetToReregister);
			AssetToReregister->UnregisterGraphWithFrontend();
			AssetToReregister->UpdateAndRegisterForExecution(RegOptions);
		}
	}

	void FMetaSoundAssetManager::SearchAndIterateDirectoryAssets(const TArray<FDirectoryPath>& InDirectories, TFunctionRef<void(const FAssetData&)> InFunction) const
	{
		if (InDirectories.IsEmpty())
		{
			return;
		}

		UAssetManager& AssetManager = UAssetManager::Get();

		FAssetManagerSearchRules Rules;
		for (const FDirectoryPath& Path : InDirectories)
		{
			Rules.AssetScanPaths.Add(*Path.Path);
		}

		Metasound::IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&](UClass& RegisteredClass)
		{
			Rules.AssetBaseClass = &RegisteredClass;
			TArray<FAssetData> MetaSoundAssets;
			AssetManager.SearchAssetRegistryPaths(MetaSoundAssets, Rules);
			for (const FAssetData& AssetData : MetaSoundAssets)
			{
				InFunction(AssetData);
			}
		});
	}
	
#if WITH_EDITOR
	void FMetaSoundAssetManager::SetCanNotifyAssetTagScanComplete()
	{
		bNotifyTagDataScanComplete = true;
	}
#endif // WITH_EDITOR

	void FMetaSoundAssetManager::SetLogActiveAssetsOnShutdown(bool bInLogActiveAssetsOnShutdown)
	{
		bLogActiveAssetsOnShutdown = bInLogActiveAssetsOnShutdown;
	}

	FMetasoundAssetBase* FMetaSoundAssetManager::TryLoadAssetFromKey(const FMetaSoundAssetKey& InAssetKey) const
	{
		FTopLevelAssetPath ObjectPath = FindAssetPath(InAssetKey);
		if (ObjectPath.IsValid())
		{
			const FSoftObjectPath SoftPath(ObjectPath);
			return TryLoadAsset(SoftPath);
		}

		return nullptr;
	}

	bool FMetaSoundAssetManager::TryGetAssetIDFromClassName(const FMetasoundFrontendClassName& InClassName, FGuid& OutGuid) const
	{
		return FGuid::Parse(InClassName.Name.ToString(), OutGuid);
	}

	FMetasoundAssetBase* FMetaSoundAssetManager::TryLoadAsset(const FSoftObjectPath& InObjectPath) const
	{
		UObject* MetaSoundObject = InObjectPath.ResolveObject();
		if (!MetaSoundObject)
		{
			MetaSoundObject = InObjectPath.TryLoad();
		}
		return Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSoundObject);
	}

	bool FMetaSoundAssetManager::TryLoadReferencedAssets(const FMetasoundAssetBase& InAssetBase, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const
	{
		using namespace Frontend;

		bool bSucceeded = true;
		OutReferencedAssets.Reset();

		TArray<FMetasoundAssetBase*> ReferencedAssets;
		const TSet<FString>& AssetClassKeys = InAssetBase.GetReferencedAssetClassKeys();
		for (const FString& KeyString : AssetClassKeys)
		{
			FNodeRegistryKey Key;
			FNodeRegistryKey::Parse(KeyString, Key);
			if (FMetasoundAssetBase* MetaSound = TryLoadAssetFromKey(Key))
			{
				OutReferencedAssets.Add(MetaSound);
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Failed to find or load referenced MetaSound asset with key '%ls'", *KeyString);
				bSucceeded = false;
			}
		}

		return bSucceeded;
	}

	void FMetaSoundAssetManager::UnregisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& InDirectories)
	{
		TArray<FDirectoryPath> Directories;
		Algo::Transform(InDirectories, Directories, [](const FMetaSoundAssetDirectory& AssetDir) { return AssetDir.Directory; });

		SearchAndIterateDirectoryAssets(Directories, [this](const FAssetData& AssetData)
		{
			using namespace Frontend;

			if (AssetData.IsAssetLoaded())
			{
				FMetasoundAssetBase* MetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(AssetData.GetAsset());
				check(MetaSoundAsset);
				MetaSoundAsset->UnregisterGraphWithFrontend();

				RemoveAsset(AssetData);
			}
			else
			{
				const FMetaSoundAssetClassInfo ClassInfo(AssetData);
				if (ensureAlways(ClassInfo.bIsValid))
				{
					const FMetaSoundAssetKey AssetKey(ClassInfo.ClassName, ClassInfo.Version);
					const FNodeRegistryKey RegistryKey = FNodeRegistryKey(AssetKey);
					const bool bIsRegistered = FMetasoundFrontendRegistryContainer::Get()->IsNodeRegistered(RegistryKey);
					if (bIsRegistered)
					{
						FMetasoundFrontendRegistryContainer::Get()->UnregisterNode(RegistryKey);
						const FTopLevelAssetPath AssetPath(AssetData.PackageName, AssetData.AssetName);
						AssetSubsystemPrivate::RemoveClassInfo(GetTransactor(), TagDataMapCriticalSection, ClassInfoMap, AssetKey, AssetPath);
					}
				}
			}
		});
	}

#if WITH_EDITORONLY_DATA
	FMetaSoundAssetManager::FVersionAssetResults FMetaSoundAssetManager::SetAccessFlagsOnAssetsInFolders(const TArray<FString>& FolderPaths, EMetasoundFrontendClassAccessFlags Flags, bool bRecursePaths) const
	{
		TArray<FTopLevelAssetPath> ClassNames;
		IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&ClassNames](UClass& InClass)
		{
			ClassNames.Add(InClass.GetClassPathName());
		});

		FARFilter Filter;
		Filter.ClassPaths = ClassNames;
		Filter.bRecursivePaths = bRecursePaths;
		Filter.bRecursiveClasses = true;
		Algo::Transform(FolderPaths, Filter.PackagePaths, [](const FString& PackagePath)
		{
			return FName(*PackagePath);
		});

		FVersionAssetResults Results;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		if (AssetRegistryModule.Get().IsGathering())
		{
			UE_LOGF(LogMetaSound, Error, "Asset Registry is still scanning, wait to update MetaSound AccessFlags in given folder(s) until initial scan is complete.");
			return Results;
		}

		AssetRegistryModule.Get().EnumerateAssets(Filter, [&Flags, &Results](const FAssetData& AssetData)
		{
			using namespace Frontend;

			const FTopLevelAssetPath PackagePath(AssetData.PackageName, AssetData.AssetName);
			const FMetaSoundAssetClassInfo ClassInfo(AssetData);
			if (ClassInfo.bIsValid)
			{
				if (ClassInfo.AccessFlags == Flags)
				{
					Results.PackagesUpToDate.Add(PackagePath);
					UE_LOGF(LogMetaSound, Display, "MetaSound '%ls' already has given flag(s), skipping update.", *AssetData.GetFullName());
					return true;
				}
			}

			UObject* MetaSoundObject = AssetData.GetAsset();
			FMetasoundAssetBase* MetaSound = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSoundObject);
			if (MetaSound)
			{
				FMetaSoundFrontendDocumentBuilder& DocBuilder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(MetaSoundObject);
				constexpr bool bAlwaysMarkDirty = true;
				MetaSoundObject->Modify(bAlwaysMarkDirty);
				Results.PackagesToReserialize.Add(AssetData.GetPackage());
				DocBuilder.SetAccessFlags(Flags);
				UE_LOGF(LogMetaSound, Display, "MetaSound '%ls' access flags updated.", *AssetData.GetFullName());
				return true;
			}

			Results.FailedPackages.Add(PackagePath);
			UE_LOGF(LogMetaSound, Error, "MetaSound asset '%ls' failed to load: asset AccessFlags not updated.", *AssetData.GetFullName());
			return true;
		});

		return Results;
	}

	FMetaSoundAssetManager::FVersionAssetResults FMetaSoundAssetManager::VersionAssetsInFolders(const TArray<FString>& FolderPaths, bool bRecursePaths) const
	{
		TArray<FTopLevelAssetPath> ClassNames;
		IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&ClassNames](UClass& InClass)
		{
			ClassNames.Add(InClass.GetClassPathName());
		});

		FARFilter Filter;
		Filter.ClassPaths = ClassNames;
		Filter.bRecursivePaths = bRecursePaths;
		Filter.bRecursiveClasses = true;
		Algo::Transform(FolderPaths, Filter.PackagePaths, [](const FString& PackagePath)
		{
			return FName(*PackagePath);
		});

		FVersionAssetResults Results;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		if (AssetRegistryModule.Get().IsGathering())
		{
			UE_LOGF(LogMetaSound, Error, "Asset Registry is still scanning, wait to version MetaSound assets in given folder(s) until initial scan is complete.");
			return Results;
		}

		AssetRegistryModule.Get().EnumerateAssets(Filter, [&Results](const FAssetData& AssetData)
		{
			using namespace Frontend;

			const FTopLevelAssetPath PackagePath(AssetData.PackageName, AssetData.AssetName);
			const FMetaSoundAssetClassInfo ClassInfo(AssetData);

			// Loaded assets will have likely versioned already, so data is already likely correct.
			const bool bWasLoaded = AssetData.IsAssetLoaded();
			if (!bWasLoaded && ClassInfo.bIsValid)
			{
				if (ClassInfo.DocInfo.DocumentVersion >= FVersioningManager::GetMaxDocumentVersion())
				{
					Results.PackagesUpToDate.Add(PackagePath);
					UE_LOGF(LogMetaSound, Display, "MetaSound '%ls' already versioned & contains valid asset tags. Skipping reserialization.", *AssetData.GetFullName());
					return true;
				}
			}

			UObject* MetaSoundObject = AssetData.GetAsset();
			FMetasoundAssetBase* MetaSound = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(MetaSoundObject);
			if (MetaSound)
			{
				FVersioningManager::Get().WaitUntilVersioningComplete(*MetaSoundObject);
				bool bVersioned = MetaSound->GetVersionedOnLoad();
				TScriptInterface<IMetaSoundDocumentInterface> DocInterface(MetaSoundObject);
				const FMetasoundFrontendDocumentMetadata& DocMetadata = DocInterface->GetConstDocument().Metadata;
				FString DocVersion;
				if (bVersioned)
				{
					DocVersion = DocMetadata.Version.Number.ToString();
				}
				else
				{
					FVersioningManager::Get().VersionAssetAsync(*MetaSoundObject, false /* bIsDeterministic */);
					FVersioningManager::Get().WaitUntilVersioningComplete(*MetaSoundObject);
					bVersioned = MetaSound->GetVersionedOnLoad();
				}

				constexpr bool bAlwaysMarkDirty = true;
				MetaSoundObject->Modify(bAlwaysMarkDirty);

				if (bVersioned)
				{
					UE_LOGF(LogMetaSound, Display, "MetaSound '%ls' document versioned to %ls", *AssetData.GetFullName(), *DocVersion);
					Results.PackagesToReserialize.Add(AssetData.GetPackage());
					return true;
				}
				else
				{
					Results.PackagesToReserialize.Add(AssetData.GetPackage());
					UE_LOGF(LogMetaSound, Display, "MetaSound '%ls' already opened but not versioned. Reserializing to ensure tags updated.", *AssetData.GetFullName());
					return true;
				}
			}

			Results.FailedPackages.Add(PackagePath);
			UE_LOGF(LogMetaSound, Error, "MetaSound asset '%ls' failed to load: asset document not versioned/tags updated.", *AssetData.GetFullName());
			return true;
		});

		return Results;
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	void FMetaSoundAssetManager::WaitUntilAsyncLoadReferencedAssetsComplete(const FMetasoundAssetBase& InAssetBase)
	{
		TSet<const FMetasoundAssetBase*> TransitiveReferences;
		TArray<const FMetasoundAssetBase*> TransitiveReferencesQueue;
		TransitiveReferences.Add(&InAssetBase);
		TransitiveReferencesQueue.Add(&InAssetBase);
		while (!TransitiveReferencesQueue.IsEmpty())
		{
			const FMetasoundAssetBase* Reference = TransitiveReferencesQueue.Pop();
			const UObject* OwningAsset = Reference->GetOwningAsset();
			if (!OwningAsset)
			{
				continue;
			}
			while (FMetaSoundAsyncAssetDependencies* LoadingDependency = FindLoadingDependencies(OwningAsset))
			{
				// Grab shared ptr to handle as LoadingDependencies may be deleted and have it's shared pointer removed. 
				TSharedPtr<FStreamableHandle> StreamableHandle = LoadingDependency->StreamableHandle;
				if (StreamableHandle.IsValid())
				{
					UE_LOGF(LogMetaSound, Verbose, "Waiting on async load (id: %d) from asset %ls", LoadingDependency->LoadID, *InAssetBase.GetOwningAssetName());

					EAsyncPackageState::Type LoadState = StreamableHandle->WaitUntilComplete();
					if (EAsyncPackageState::Complete != LoadState)
					{
						UE_LOGF(LogMetaSound, Error, "Failed to complete loading of async dependent assets from parent asset %ls", *InAssetBase.GetOwningAssetName());
						RemoveLoadingDependencies(LoadingDependency->LoadID);
					}
					else
					{
						// This will remove the loading dependencies from internal storage
						OnReferencedAssetsLoaded(LoadingDependency->LoadID);
					}

					// This will prevent OnAssetsLoaded from being called via the streamables
					// internal delegate complete callback.
					StreamableHandle->CancelHandle();
				}
			}

			for (const FMetasoundAssetBase* NextReference : Reference->GetReferencedAssets())
			{
				bool bAlreadyInSet;
				TransitiveReferences.Add(NextReference, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					TransitiveReferencesQueue.Add(NextReference);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	void DeinitializeAssetManager()
	{
		Frontend::IMetaSoundAssetManager::Deinitialize();
	}

	void InitializeAssetManager()
	{
		Frontend::IMetaSoundAssetManager::Initialize(MakeUnique<FMetaSoundAssetManager>());
	}
} // namespace Metasound::Engine

void UMetaSoundAssetSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UMetaSoundAssetSubsystem::PostEngineInitInternal);
}

void UMetaSoundAssetSubsystem::PostEngineInitInternal()
{
	using namespace Metasound::Engine;

	check(UAssetManager::IsInitialized());
	UAssetManager& AssetManager = UAssetManager::Get();
	AssetManager.CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UMetaSoundAssetSubsystem::PostInitAssetScanInternal));
}

void UMetaSoundAssetSubsystem::PostInitAssetScanInternal()
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundAssetSubsystem::PostInitAssetScanInternal);

	if (!IsRunningCookCommandlet())
	{
		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		if (ensureAlways(Settings))
		{
			FMetaSoundAssetManager& Manager = FMetaSoundAssetManager::GetChecked();
			FMetaSoundAssetRegistrationOptions RegOptions = Manager.GetRegistrationOptions();
			Manager.SearchAndIterateDirectoryAssets(Settings->DirectoriesToRegister, [&Manager, &RegOptions](const FAssetData& AssetData)
			{
				using namespace Metasound;

				Manager.AddOrLoadAndUpdateFromObjectAsync(AssetData, [RegOptions](FMetaSoundAssetKey, UObject& AssetObject)
				{
					FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&AssetObject);
					check(MetaSoundAsset);
					MetaSoundAsset->UpdateAndRegisterForExecution(RegOptions);
				});
			});
		}
	}

	AssetSubsystemPrivate::InitialAssetScanComplete.store(true);
}

#if WITH_EDITOR
bool UMetaSoundAssetSubsystem::FindAssetClassInfo(const FTopLevelAssetPath& InPath, FMetaSoundDocumentInfo& OutDocInfo, FMetaSoundClassInterfaceInfo& OutInterfaceInfo, bool bForceLoad) const
{
	using namespace Metasound;

	const FSoftObjectPath ObjectPath(InPath);
	const bool bWasLoaded = ObjectPath.ResolveObject() != nullptr;

	auto FindInternal = [](const FTopLevelAssetPath& Path, FMetaSoundDocumentInfo& DocInfo, FMetaSoundClassInterfaceInfo& InterfaceInfo)
	{
		Frontend::FMetaSoundAssetClassInfo FoundInfo = Engine::FMetaSoundAssetManager::GetChecked().FindAssetClassInfo(Path);
		if (FoundInfo.bIsValid)
		{
			DocInfo = MoveTemp(FoundInfo.DocInfo);
			InterfaceInfo = MoveTemp(FoundInfo.InterfaceInfo);
			return true;
		}

		return false;
	};

	if (FindInternal(InPath, OutDocInfo, OutInterfaceInfo))
	{
		return true;
	}

	// Attempt to load synchronously if set to force load
	if (!bWasLoaded && bForceLoad)
	{
		if (ObjectPath.TryLoad())
		{
			return FindInternal(InPath, OutDocInfo, OutInterfaceInfo);
		}
	}

	return false;
}

bool UMetaSoundAssetSubsystem::FindReferencingAssetClassInfo(TScriptInterface<IMetaSoundDocumentInterface> MetaSound, TArray<FTopLevelAssetPath>& OutPaths, TArray<FMetaSoundDocumentInfo>& OutDocInfo, TArray<FMetaSoundClassInterfaceInfo>& OutInterfaceInfo, bool bOnlyPresets, bool bForceLoad) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	IMetasoundUObjectRegistry& ObjectRegistry = IMetasoundUObjectRegistry::Get();

	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	const FTopLevelAssetPath MetaSoundPath(MetaSound.GetObject());
	TArray<FName> OnDiskReferencers;
	AssetRegistry.GetReferencers(MetaSoundPath.GetPackageName(), OnDiskReferencers);

	bool bSuccess = true;

	OutDocInfo.Empty();
	OutInterfaceInfo.Empty();
	OutPaths.Empty();

	for (const FName& OnDiskReferencer : OnDiskReferencers)
	{
		FAssetData RefAssetData;

		FString AssetName = *OnDiskReferencer.ToString();
		{
			int32 Index = INDEX_NONE;
			if (AssetName.FindLastChar('/', Index))
			{
				AssetName.RightChopInline(Index + 1);
			}
		}
		const FTopLevelAssetPath RefAssetPath(FName(*OnDiskReferencer.ToString()), FName(*AssetName));
		UE::AssetRegistry::EExists RefExists = AssetRegistry.TryGetAssetByObjectPath(FSoftObjectPath(RefAssetPath), RefAssetData);
		if (RefExists != UE::AssetRegistry::EExists::Exists)
		{
			bSuccess = false;
			continue;
		}

		const UClass* AssetClass = RefAssetData.GetClass();
		if (!AssetClass)
		{
			bSuccess = false;
			continue;
		}
		 
		if (!ObjectRegistry.IsRegisteredClass(*AssetClass))
		{
			// Could be non-MetaSound references (Blueprint, redirect, etc.), so isn't
			// a failure to succeed if referencing class isn't strictly MetaSound.
			continue;
		}

		FMetaSoundDocumentInfo DocInfo;
		FMetaSoundClassInterfaceInfo InterfaceInfo;

		const bool bRefFound = FindAssetClassInfo(RefAssetPath, DocInfo, InterfaceInfo, bForceLoad);
		bSuccess &= bRefFound;
		if (bRefFound)
		{
			if (!bOnlyPresets || (bOnlyPresets && DocInfo.bIsPreset != 0))
			{
				OutPaths.Add(RefAssetPath);
				OutDocInfo.Add(MoveTemp(DocInfo));
				OutInterfaceInfo.Add(MoveTemp(InterfaceInfo));
			}
		}
	}

	return bSuccess;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UMetaSoundAssetSubsystem::ReassignClassName(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
{
	using namespace Metasound::Engine;
	return FMetaSoundAssetManager::GetChecked().ReassignClassName(DocInterface);
}
#endif // WITH_EDITOR

void UMetaSoundAssetSubsystem::RegisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& InDirectories)
{
	using namespace Metasound::Engine;
	FMetaSoundAssetManager::GetChecked().RegisterAssetClassesInDirectories(InDirectories);
}

#if WITH_EDITOR
bool UMetaSoundAssetSubsystem::ReplaceReferencesInDirectory(
	const TArray<FMetaSoundAssetDirectory>& InDirectories,
	const FMetasoundFrontendClassName& OldClassName,
	const FMetasoundFrontendClassName& NewClassName,
	const FMetasoundFrontendVersionNumber OldVersion,
	const FMetasoundFrontendVersionNumber NewVersion)
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	return FMetaSoundAssetManager::GetChecked().ReplaceReferencesInDirectory(
		InDirectories,
		FNodeRegistryKey(EMetasoundFrontendClassType::External, OldClassName, OldVersion),
		FNodeRegistryKey(EMetasoundFrontendClassType::External, NewClassName, NewVersion)
	);
}
#endif // WITH_EDITOR

void UMetaSoundAssetSubsystem::UnregisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& InDirectories)
{
	using namespace Metasound::Engine;
	FMetaSoundAssetManager::GetChecked().UnregisterAssetClassesInDirectories(InDirectories);
}
