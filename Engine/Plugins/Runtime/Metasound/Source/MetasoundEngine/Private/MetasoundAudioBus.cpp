// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioBus.h"

#include "AudioDevice.h"

namespace Metasound
{
	FAudioBusAsset::FAudioBusAsset(const TSharedPtr<Audio::IProxyData>& InInitData)
	{
		if (InInitData.IsValid())
		{
			if (InInitData->CheckTypeCast<FAudioBusProxy>())
			{
				AudioBusProxy = MakeShared<FAudioBusProxy, ESPMode::ThreadSafe>(InInitData->GetAs<FAudioBusProxy>());
			}
		}
	}
}
