// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioBusPrivate.h"

#include "DSP/MultithreadedPatching.h"
#include "DSP/RuntimeResampler.h"
#include "HAL/IConsoleManager.h"
#include "MetasoundLog.h"

FAutoConsoleVariableRef CVarAudioBusReaderNodeEnableResampledAudioBus(
	TEXT("au.MetaSound.EnableAudioBusResampler"),
	Metasound::AudioBusPrivate::EnableResampledAudioBus,
	TEXT("Enable the use of a resampler when the AudioMixer sample rate does not match the MetaSound sample rate.\n")
	TEXT("0: disabled, 1: enabled (default)"),
	ECVF_Default);

namespace Metasound::AudioBusPrivate
{
	int32 EnableResampledAudioBus = 1;

	FResampledPatchOutput::FResampledPatchOutput(int32 InNumChannels, float InAudioBusSampleRate, float InMetaSoundSampleRate, int32 InMetaSoundBlockSize, TSharedRef<Audio::FPatchOutput> InPatchOutput)
	: NumChannels(InNumChannels)
	, Resampler(InNumChannels) 
	, PatchOutput(MoveTemp(InPatchOutput))
	{
		check(InNumChannels > 0);
		check(InAudioBusSampleRate > 0.f);
		check(InMetaSoundSampleRate> 0.f);
		check(InMetaSoundBlockSize > 0);

		// Set sample rate to read/write rate. It is assumed that all audio buses
		// read/write at the AudioBusSampleRate. MetaSounds which do not
		// render at the AudioBusSampleRate are resampled outside of the MetaSound
		// system to match the AudioBusSampleRate. This SampleRateRatio
		// accounts for the resampling that occurs outside of the MetaSound Source.
		const float SampleRateRatio = InAudioBusSampleRate / InMetaSoundSampleRate;

		Resampler.SetFrameRatio(SampleRateRatio);

		// A temporary buffer is required to interact with the FPatchOutput 
		// API. The FPatchOutput API could be reworked to not require the
		// use of a temporary buffer by providing a Peek method which returns
		// a const view of the array already existing in the FPatchOutput
		const int32 NumFramesNeededFromAudioBus = Resampler.GetNumInputFramesNeededToProduceOutputFrames(InMetaSoundBlockSize) + FMath::CeilToInt(SampleRateRatio);
		ScratchBuffer.AddUninitialized(NumFramesNeededFromAudioBus * NumChannels);

	}

	int32 FResampledPatchOutput::PopAudio(float* OutAudio, int32 InNumSamplesToPop, bool bInUseLatestAudio)
	{
		const int32 NumFramesNeededFromAudioBus = Resampler.GetNumInputFramesNeededToProduceOutputFrames(InNumSamplesToPop / NumChannels);
		const int32 NumSamplesNeededFromAudioBus = NumFramesNeededFromAudioBus * NumChannels;

		// Check that buffer allocations are not needed during rendering. 
		if (!ensureMsgf(NumSamplesNeededFromAudioBus <= ScratchBuffer.Num(), TEXT("More initial slack is needed in allocation of AudioBuBuffer. Allocated %d, Requested: %d"), ScratchBuffer.Num(), NumSamplesNeededFromAudioBus))
		{
			ScratchBuffer.Reset();
			ScratchBuffer.AddUninitialized(NumSamplesNeededFromAudioBus);
		}

		int32 NumSamplesPopped = PatchOutput->PopAudio(ScratchBuffer.GetData(), NumSamplesNeededFromAudioBus, bInUseLatestAudio);

		int32 NumFramesConsumed = -1;
		int32 NumFramesProduced = -1;

		Resampler.ProcessInterleaved(TArrayView<const float>{ScratchBuffer.GetData(), NumSamplesPopped}, TArrayView<float>{OutAudio, InNumSamplesToPop}, NumFramesConsumed, NumFramesProduced);

		// Check that all the input frames have been consumed so that they
		// do not need to be maintained here. 
		if ((NumChannels * NumFramesConsumed) < NumSamplesPopped)
		{
			UE_LOGF(LogMetaSound, Warning, "Dropping %d samples", NumSamplesPopped - (NumChannels * NumFramesConsumed));
		}
		return NumFramesProduced * NumChannels;
	};


	FResampledPatchInput::FResampledPatchInput(int32 InNumChannels, float InAudioBusSampleRate, float InMetaSoundSampleRate, int32 InMetaSoundBlockSize, Audio::FPatchInput InPatchInput)
	: NumChannels(InNumChannels)
	, Resampler(InNumChannels) 
	, PatchInput(MoveTemp(InPatchInput))
	{
		check(InNumChannels > 0);
		check(InAudioBusSampleRate > 0.f);
		check(InMetaSoundSampleRate> 0.f);
		check(InMetaSoundBlockSize > 0);

		// Set sample rate to read/write rate. It is assumed that all audio buses
		// read/write at the AudioBusSampleRate. MetaSounds which do not
		// render at the AudioBusSampleRate are resampled outside of the MetaSound
		// system to match the AudioBusSampleRate. This SampleRateRatio
		// accounts for the resampling that occurs outside of the MetaSound Source.
		const float SampleRateRatio = InMetaSoundSampleRate / InAudioBusSampleRate;

		Resampler.SetFrameRatio(SampleRateRatio);

		const int32 MaxOutputBufferNumFrames = Resampler.GetNumOutputFramesProducedByInputFrames(InMetaSoundBlockSize + 1) + FMath::CeilToInt(SampleRateRatio);
		ScratchBuffer.AddUninitialized(MaxOutputBufferNumFrames * NumChannels);
	}

	int32 FResampledPatchInput::PushAudio(float const* InAudio, int32 InNumSamplesToPush)
	{
		const int32 NumResampledFramesToPush = Resampler.GetNumOutputFramesProducedByInputFrames(InNumSamplesToPush / NumChannels);
		const int32 NumResampledSamplesToPush = NumResampledFramesToPush * NumChannels;

		// Check that buffer allocations are not needed during rendering. 
		if (!ensureMsgf(NumResampledSamplesToPush <= ScratchBuffer.Num(), TEXT("More initial slack is needed in allocation of AudioBuBuffer. Allocated %d, Requested: %d"), ScratchBuffer.Num(), NumResampledSamplesToPush))
		{
			ScratchBuffer.Reset();
			ScratchBuffer.AddUninitialized(NumResampledSamplesToPush);
		}

		int32 NumFramesConsumed = -1;
		int32 NumFramesProduced = -1;
		Resampler.ProcessInterleaved(TArrayView<const float>{InAudio, InNumSamplesToPush}, TArrayView<float>{ScratchBuffer.GetData(), NumResampledSamplesToPush}, NumFramesConsumed, NumFramesProduced);

		// Check that all the input frames have been consumed so that they
		// do not need to be maintained here. 
		if ((NumChannels * NumFramesConsumed) < InNumSamplesToPush)
		{
			UE_LOGF(LogMetaSound, Warning, "Dropping %d samples", InNumSamplesToPush - (NumChannels * NumFramesConsumed));
		}

		int32 NumSamplesPushed = PatchInput.PushAudio(ScratchBuffer.GetData(), NumFramesProduced * NumChannels);

		return NumFramesConsumed * NumChannels;
	}
}
