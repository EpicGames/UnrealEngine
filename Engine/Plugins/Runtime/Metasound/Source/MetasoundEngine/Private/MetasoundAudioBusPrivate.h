// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"
#include "DSP/MultithreadedPatching.h"
#include "DSP/RuntimeResampler.h"
#include "Templates/SharedPointer.h"

namespace Metasound::AudioBusPrivate
{
	// Enablement flag for resampling of audio when the MetaSound sample rate
	// does not match the AudioMixer sample rate. 
	extern int32 EnableResampledAudioBus;

	// Resample audio popped from an audio bus. 
	class FResampledPatchOutput
	{
	public:
		FResampledPatchOutput(int32 InNumChannels, float InAudioBusSampleRate, float InMetaSoundSampleRate, int32 InMetaSoundBlockSize, TSharedRef<Audio::FPatchOutput> InPatchOutput);

		int32 PopAudio(float* OutAudio, int32 InNumSamplesToPop, bool bInUseLatestAudio);

	private:

		int32 NumChannels = 0;
		Audio::FRuntimeResampler Resampler;
		Audio::FAlignedFloatBuffer ScratchBuffer;
		TSharedRef<Audio::FPatchOutput> PatchOutput;
	};

	// Resample audio pushed to an audio bus. 
	class FResampledPatchInput
	{
	public:
		FResampledPatchInput(int32 InNumChannels, float InAudioBusSampleRate, float InMetaSoundSampleRate, int32 InMetaSoundBlockSize, Audio::FPatchInput InPatchInput);

		int32 PushAudio(float const* InAudio, int32 InNumSamplesToPush);

	private:

		int32 NumChannels = 0;
		Audio::FRuntimeResampler Resampler;
		Audio::FAlignedFloatBuffer ScratchBuffer;
		Audio::FPatchInput PatchInput;
	};
}
