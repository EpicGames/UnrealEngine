// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioBusSubsystem.h"
#include "AudioDevice.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"
#include "Internationalization/Text.h"
#include "Sound/AudioBus.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundAudioBus.h"
#include "MetasoundAudioBusPrivate.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundAudioBusNode"

static int32 AudioBusReaderNodePatchWaitTimeout = 3;
FAutoConsoleVariableRef CVarAudioBusReaderNodePatchWaitTimeout(
	TEXT("au.BusReaderPatchWaitTimeout"),
	AudioBusReaderNodePatchWaitTimeout,
	TEXT("The maximum amount of time the audio bus reader node will wait for its patch output to receive samples."),
	ECVF_Default);

namespace Metasound
{
	namespace AudioBusReaderNode
	{
		METASOUND_PARAM(InParamAudioBusInput, "Audio Bus", "Audio Bus Asset.")

		METASOUND_PARAM(OutParamAudio, "Out {0}", "Audio bus output for channel {0}.");

	}

	int32 AudioBusReaderNodeInitialNumBlocks(int32 BlockSizeFrames, int32 AudioMixerOutputFrames)
	{
		// One extra block is required to cover the first metasound iteration.
		int32 ExtraBlocks = 1;

		// Find the number of whole blocks that fit in the mixer output.
		int32 WholeBlocks = FMath::DivideAndRoundDown(AudioMixerOutputFrames, BlockSizeFrames);

		// Determine if any frames remain.
		int32 FramesRemainder = AudioMixerOutputFrames % BlockSizeFrames;
		if (FramesRemainder > 0)
		{
			// Find the number of extra frames required to consistently cover the remainder.
			int32 ExtraFrames = FMath::DivideAndRoundUp(BlockSizeFrames, FramesRemainder) * FramesRemainder;

			// Find the number of blocks required to cover the extra frames.
			ExtraBlocks += FMath::DivideAndRoundUp(ExtraFrames, BlockSizeFrames);
		}

		// Determine the total number of blocks required.
		return WholeBlocks + ExtraBlocks;
	}

	template <uint32 NumChannels>
	class TAudioBusReaderOperator : public TExecutableOperator<TAudioBusReaderOperator<NumChannels>>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Audio Bus Reader (%d)"), NumChannels);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("AudioBusReaderDisplayNamePattern", "Audio Bus Reader ({0})", NumChannels);
				
				FNodeClassMetadata Info;
				Info.ClassName = { EngineNodes::Namespace, OperatorName, TEXT("") };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = METASOUND_LOCTEXT("AudioBusReader_Description", "Outputs audio data from the audio bus asset.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Io);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace AudioBusReaderNode;

			auto CreateVertexInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;
				InputInterface.Add(TInputDataVertex<FAudioBusAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamAudioBusInput)));

				FOutputVertexInterface OutputInterface;
				for (uint32 i = 0; i < NumChannels; ++i)
				{
					OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(OutParamAudio, i)));
				}

				return FVertexInterface(InputInterface, OutputInterface);
			};
			
			static const FVertexInterface Interface = CreateVertexInterface();
			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace Frontend;
			
			using namespace AudioBusReaderNode; 
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			bool bHasEnvironmentVars = InParams.Environment.Contains<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
			bHasEnvironmentVars &= InParams.Environment.Contains<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
			bHasEnvironmentVars &= InParams.Environment.Contains<uint64>(SourceInterface::Environment::TransmitterID);
			
			if (bHasEnvironmentVars)
			{
				FAudioBusAssetReadRef AudioBusIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBusAsset>(METASOUND_GET_PARAM_NAME(InParamAudioBusInput), InParams.OperatorSettings);
				return MakeUnique<TAudioBusReaderOperator<NumChannels>>(InParams, AudioBusIn);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus reader node requires audio device ID '%ls' and audio mixer num output frames '%ls' environment variables"
					, *SourceInterface::Environment::DeviceID.ToString(), *SourceInterface::Environment::AudioMixerNumOutputFrames.ToString());
				return nullptr;
			}

		}

		TAudioBusReaderOperator(const FBuildOperatorParams& InParams, const FAudioBusAssetReadRef& InAudioBusAsset) : AudioBusAsset(InAudioBusAsset)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				AudioOutputs.Add(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings));
			}

			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AudioBusReaderNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamAudioBusInput), AudioBusAsset);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AudioBusReaderNode;
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(OutParamAudio, ChannelIndex), AudioOutputs[ChannelIndex]);
			}
		}

		void Execute()
		{
			const FAudioBusProxyPtr& BusProxy = AudioBusAsset->GetAudioBusProxy();
			if (!BusProxy.IsValid() || BusProxy->NumChannels <= 0)
			{
				// the audio bus is invalid / uninitialized
				return;
			}

			if (BusProxy->AudioBusId != AudioBusId)
			{
				InterleavedBuffer.Reset();
			}
			
			if (InterleavedBuffer.IsEmpty())
			{
				// if environment vars & a valid audio bus have been set since starting, try to create the patch now
				if (SampleRate > 0.f && BusProxy.IsValid())
				{
					CreatePatchOutput();
				}
			}

			// If still not ready, output silence and bail
			if (InterleavedBuffer.IsEmpty())
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					AudioOutputs[ChannelIndex]->Zero();
				}
				return;
			}

			// Pop off the interleaved data from the audio bus
			int32 FramesToPop = BlockSizeFrames;
			int32 NumSamplesToPop = FramesToPop * AudioBusChannels;
			int32 SamplesPopped = -1;

			if (InterleavedBuffer.Num() < NumSamplesToPop)
			{
				InterleavedBuffer.SetNumUninitialized(NumSamplesToPop);
			}

			if (ResampledPatchOutput.IsValid())
			{
				SamplesPopped = ResampledPatchOutput->PopAudio(InterleavedBuffer.GetData(), NumSamplesToPop, false);
			}
			else
			{
				SamplesPopped = AudioBusPatchOutput->PopAudio(InterleavedBuffer.GetData(), NumSamplesToPop, false);
			}

			const uint32 MinChannels = FMath::Min(NumChannels, AudioBusChannels);
			SamplesPopped = FMath::Max(SamplesPopped, 0);

			// Copy the data that was actually popped into the output buffer	
			int32 FramesPopped = FMath::Clamp(SamplesPopped / AudioBusChannels, 0, FramesToPop);
			if (FramesPopped > 0)
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < MinChannels; ++ChannelIndex)
				{
					float* RESTRICT AudioOutputBufferPtr = AudioOutputs[ChannelIndex]->GetData();
					for (int32 FrameIndex = 0; FrameIndex < FramesPopped; ++FrameIndex)
					{
						const int32 InterleavedIdx = FrameIndex * AudioBusChannels + (int32)ChannelIndex;
						AudioOutputBufferPtr[FrameIndex] = InterleavedBuffer[InterleavedIdx];
					}
				}
			}

			// If we popped less than we requested, do a fade out and zero remainder
			if (SamplesPopped < NumSamplesToPop)
			{
				if (!bWasUnderrunReported)
				{
					UE_LOGF(LogMetaSound, Warning, "Underrun detected in audio bus reader node. Check to see if bus was stopped.");
					bWasUnderrunReported = true;
				}

				// If we have popped some frames then use those and fade out to prevent discontinutiies
				if (FramesPopped > 0)
				{
					const int32 NumFramesToZero = FramesToPop - FramesPopped;
					for (uint32 ChannelIndex = 0; ChannelIndex < MinChannels; ++ChannelIndex)
					{
						float* RESTRICT AudioOutputBufferPtr = AudioOutputs[ChannelIndex]->GetData();
						TArrayView<float> FadeView(AudioOutputBufferPtr, FramesPopped);
						Audio::ArrayFade(FadeView, 1.0f, 0.0f);

						// zero out the remainder of the buffer
						if (NumFramesToZero > 0)
						{
							FMemory::Memzero(AudioOutputBufferPtr + FramesPopped, sizeof(float) * NumFramesToZero);
						}
					}
				}
				// If we have not popped any, just make sure our buffers are zero'd
				else
				{
					for (uint32 ChannelIndex = 0; ChannelIndex < MinChannels; ++ChannelIndex)
					{
						AudioOutputs[ChannelIndex]->Zero();
					}
				}
			}
			else
			{
				bWasUnderrunReported = false;
			}

			// Zero out channels higher than our min channels here
			if (NumChannels > MinChannels)
			{
				for (uint32 ChannelIndex = MinChannels; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					AudioOutputs[ChannelIndex]->Zero();
				}
			}
		}

		void CreatePatchOutput()
		{
			using namespace AudioBusPrivate;

			const FAudioBusProxyPtr& AudioBusProxy = AudioBusAsset->GetAudioBusProxy();
			if (AudioBusProxy.IsValid())
			{
				if (AudioBusProxy->NumChannels <= 0)
				{
					UE_LOGF(LogMetaSound, Warning, "AudioBusProxy is invalid (NumChannels = %i).", AudioBusProxy->NumChannels);
					return;
				}

				UAudioBusSubsystem* AudioBusSubsystem = nullptr;
				if (FAudioDeviceManager* ADM = FAudioDeviceManager::Get())
				{
					if (FAudioDevice* AudioDevice = ADM->GetAudioDeviceRaw(AudioDeviceId))
					{
						AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();
						check(AudioBusSubsystem);
					}
				}
				if (!AudioBusSubsystem)
				{
					return;
				}

				AudioBusChannels = uint32(FMath::Min(AudioBusProxy->NumChannels, int32(EAudioBusChannels::MaxChannelCount)));
				AudioBusId = AudioBusProxy->AudioBusId;

				Audio::FAudioBusKey AudioBusKey(AudioBusId);

				const FString AudioBusReaderNodeBusName = FString::Format(TEXT("_AudioBusReaderNode_AudioBusId_{0}"), { AudioBusId });
				AudioBusSubsystem->StartAudioBus(AudioBusKey, AudioBusReaderNodeBusName, AudioBusChannels, false);

				AudioBusPatchOutput = AudioBusSubsystem->AddPatchOutputForSoundAndAudioBus(InstanceID, AudioBusKey, BlockSizeFrames, int32(AudioBusChannels));
				PatchInput = AudioBusPatchOutput;

				// Handle case of mismatched sample rate between audio mixer and
				// metasound. 
				if (EnableResampledAudioBus && (AudioMixerSampleRate != SampleRate) && (AudioMixerSampleRate > 0.f) && (SampleRate > 0.f) && AudioBusPatchOutput.IsValid())
				{
					UE_LOGF(LogMetaSound, Warning, "Using a audio bus reader node is inefficient if the MetaSound sample rate %f does not match the AudioMixer sample rate %f. Please update MetaSound SampleRate to match the AudioMixer's SampleRate", SampleRate, AudioMixerSampleRate);

					ResampledPatchOutput = MakeUnique<FResampledPatchOutput>(AudioBusChannels, AudioMixerSampleRate, SampleRate, BlockSizeFrames, AudioBusPatchOutput.ToSharedRef());

					// Sample rate matches between audio mixer and metasound. This
					// node will consume audio from the patch in approximate block
					// sizes of (BlockSize * AudioMixerSampleRate / SampleRate)
					PatchInput.PushAudio(nullptr, NumBlocksToNumSamples(InitialNumBlocks(), AudioMixerSampleRate / SampleRate));
				}
				else
				{
					// Sample rate matches between audio mixer and metasound. 
					PatchInput.PushAudio(nullptr, NumBlocksToNumSamples(InitialNumBlocks()));
				}

				InterleavedBuffer.Reset();
				InterleavedBuffer.AddUninitialized(NumBlocksToNumSamples(1));
			}
		}
		
		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace Frontend;
			using namespace AudioBusReaderNode;

			InterleavedBuffer.Reset();
			ResampledPatchOutput.Reset();
			AudioMixerOutputFrames = INDEX_NONE;
			AudioMixerSampleRate = 1.f;
			AudioDeviceId = INDEX_NONE;
			InstanceID = 0;
			AudioBusId = 0;
			SampleRate = 0.0f;
			AudioBusPatchOutput.Reset();
			PatchInput.Reset();
			AudioBusChannels = INDEX_NONE;
			SampleRate = InParams.OperatorSettings.GetSampleRate();
			BlockSizeFrames = InParams.OperatorSettings.GetNumFramesPerBlock();
			bWasUnderrunReported = false;

			bool bHasEnvironmentVars = InParams.Environment.Contains<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
			bHasEnvironmentVars &= InParams.Environment.Contains<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
			bHasEnvironmentVars &= InParams.Environment.Contains<uint64>(SourceInterface::Environment::TransmitterID);

			if (bHasEnvironmentVars)
			{
				AudioDeviceId = InParams.Environment.GetValue<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
				AudioMixerOutputFrames = InParams.Environment.GetValue<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
				InstanceID = InParams.Environment.GetValue<uint64>(SourceInterface::Environment::TransmitterID);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus reader node requires audio device ID '%ls', audio mixer num output frames '%ls' and transmitter id '%ls' environment variables"
					, *SourceInterface::Environment::DeviceID.ToString(), *SourceInterface::Environment::AudioMixerNumOutputFrames.ToString(), *SourceInterface::Environment::TransmitterID.ToString());
			}

			// Audio mixer sample rate is a newer addition to the set of required environment variables in UE 5.6.  Check separately
			// and fall back to old behavior if it does not exist. 
			if (InParams.Environment.Contains<float>(SourceInterface::Environment::AudioMixerSampleRate))
			{
				AudioMixerSampleRate = InParams.Environment.GetValue<float>(SourceInterface::Environment::AudioMixerSampleRate);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus reader node may not render correctly without the audio mixer sample rate '%ls' environment variable", *SourceInterface::Environment::AudioMixerSampleRate.ToString());
				// Assume matching sample rate if environment variable is missing.
				AudioMixerSampleRate = SampleRate;
			}
			
			for(const FAudioBufferWriteRef& Buffer : AudioOutputs)
			{
				Buffer->Zero();
			}
			
		}

	private:
		int32 InitialNumBlocks() const
		{
			return AudioBusReaderNodeInitialNumBlocks(BlockSizeFrames, AudioMixerOutputFrames);
		}

		int32 NumBlocksToNumSamples(int32 NumBlocks, float InSampleRateRatio=-1.f) const
		{
			if (InSampleRateRatio > 0.f)
			{
				return FMath::CeilToInt(NumBlocks * BlockSizeFrames * InSampleRateRatio) * AudioBusChannels;
			}
			else
			{
				return NumBlocks * BlockSizeFrames * AudioBusChannels;
			}
		}


		FAudioBusAssetReadRef AudioBusAsset;
		TArray<FAudioBufferWriteRef> AudioOutputs;

		Audio::FAlignedFloatBuffer InterleavedBuffer;
		TUniquePtr<AudioBusPrivate::FResampledPatchOutput> ResampledPatchOutput;
		int32 AudioMixerOutputFrames = INDEX_NONE;
		float AudioMixerSampleRate = -1.f;
		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		uint64 InstanceID = 0;
		uint32 AudioBusId = 0;
		float SampleRate = 0.0f;
		Audio::FPatchOutputStrongPtr AudioBusPatchOutput;
		Audio::FPatchInput PatchInput;
		uint32 AudioBusChannels = INDEX_NONE;
		int32 BlockSizeFrames = 0;
		bool bWasUnderrunReported = false;
	};

	template<uint32 NumChannels>
	using TAudioBusReaderNode = TNodeFacade<TAudioBusReaderOperator<NumChannels>>;

#define REGISTER_AUDIO_BUS_READER_NODE(ChannelCount) \
	using FAudioBusReaderNode_##ChannelCount = TAudioBusReaderNode<ChannelCount>; \
	METASOUND_REGISTER_NODE(FAudioBusReaderNode_##ChannelCount) \

	
	REGISTER_AUDIO_BUS_READER_NODE(1);
	REGISTER_AUDIO_BUS_READER_NODE(2);
	REGISTER_AUDIO_BUS_READER_NODE(4);
	REGISTER_AUDIO_BUS_READER_NODE(6);
	REGISTER_AUDIO_BUS_READER_NODE(8);
}

#undef LOCTEXT_NAMESPACE
