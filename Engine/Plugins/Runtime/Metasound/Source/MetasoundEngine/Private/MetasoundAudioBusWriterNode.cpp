// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioBusWriterNode.h"

#include "AudioMixerDevice.h"
#include "AudioBusSubsystem.h"
#include "AudioDevice.h"
#include "Internationalization/Text.h"
#include "MediaPacket.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundAudioBus.h"
#include "MetasoundAudioBusPrivate.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundAudioBusWriterNode"

namespace Metasound
{
	namespace AudioBusWriterNode
	{
		namespace Inputs
		{
			DEFINE_METASOUND_PARAM(AudioBus, "Audio Bus", "Audio Bus Asset.");
			DEFINE_METASOUND_PARAM(Audio, "In {0}", "Audio input for channel {0}.");
		}

		int32 GetCurrentMajorVersion()
		{
			return 1;
		}
	}

	int32 AudioBusWriterNodeInitialNumBlocks(int32 BlockSizeFrames, int32 AudioMixerOutputFrames)
	{
		// One less block is required because the metasound will write the final block.
		int32 MaxSizeFrames = FMath::Max(AudioMixerOutputFrames, BlockSizeFrames), MinSizeFrames = FMath::Min(AudioMixerOutputFrames, BlockSizeFrames);
		return FMath::DivideAndRoundUp(MaxSizeFrames, MinSizeFrames) - 1;
	}

	template<uint32 NumChannels>
	class TAudioBusWriterOperator : public TExecutableOperator<TAudioBusWriterOperator<NumChannels>>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("AudioBusWriterDisplayNamePattern", "Audio Bus Writer ({0})", NumChannels);

				FNodeClassMetadata Info;
				Info.ClassName = AudioBusWriterNode::GetClassName<NumChannels>();
				Info.MajorVersion = AudioBusWriterNode::GetCurrentMajorVersion();
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = METASOUND_LOCTEXT("AudioBusWriter_Description", "Sends audio data to the audio bus asset.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Io);
				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace AudioBusWriterNode;

			auto CreateVertexInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;
				InputInterface.Add(TInputDataVertex<FAudioBusAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(Inputs::AudioBus)));
				for (uint32 i = 0; i < NumChannels; ++i)
				{
					InputInterface.Add(TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(Inputs::Audio, i)));
				}

				FOutputVertexInterface OutputInterface;

				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface Interface = CreateVertexInterface();
			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace Frontend;
			using namespace AudioBusWriterNode;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			bool bHasEnvironmentVars = InParams.Environment.Contains<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
			bHasEnvironmentVars &= InParams.Environment.Contains<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
			bHasEnvironmentVars &= InParams.Environment.Contains<uint64>(SourceInterface::Environment::TransmitterID);

			if (bHasEnvironmentVars)
			{
				FAudioBusAssetReadRef AudioBusIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBusAsset>(METASOUND_GET_PARAM_NAME(Inputs::AudioBus), InParams.OperatorSettings);

				TArray<FAudioBufferReadRef> AudioInputs;
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					AudioInputs.Add(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Audio, ChannelIndex), InParams.OperatorSettings));
				}

				FString GraphName;
				if (InParams.Environment.Contains<FString>(SourceInterface::Environment::GraphName))
				{
					GraphName = InParams.Environment.GetValue<FString>(SourceInterface::Environment::GraphName);
				}
				else
				{
					GraphName = TEXT("<Unknown>");
				}

				return MakeUnique<TAudioBusWriterOperator<NumChannels>>(InParams, MoveTemp(AudioBusIn), MoveTemp(AudioInputs), MoveTemp(GraphName));
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus writer node requires audio device ID '%ls', audio mixer num output frames '%ls' and transmitter id '%ls' environment variables"
					, *SourceInterface::Environment::DeviceID.ToString(), *SourceInterface::Environment::AudioMixerNumOutputFrames.ToString(), *SourceInterface::Environment::TransmitterID.ToString());
				return nullptr;
			}
		}

		TAudioBusWriterOperator(const FBuildOperatorParams& InParams, FAudioBusAssetReadRef InAudioBusAsset, TArray<FAudioBufferReadRef> InAudioInputs, FString InGraphName)
			: AudioBusAsset(MoveTemp(InAudioBusAsset))
			, AudioInputs(MoveTemp(InAudioInputs))
		{
			Reset(InParams);
		}

		void CreatePatchInput()
		{
			using namespace AudioBusPrivate;

			const FAudioBusProxyPtr& AudioBusProxy = AudioBusAsset->GetAudioBusProxy();
			if (AudioBusProxy.IsValid())
			{
				if (AudioBusProxy->NumChannels <= 0)
				{
					UE_LOGF(LogMetaSound, Warning, "AudioBusProxy is invalid (NumChannels = %i).", AudioBusProxy->NumChannels);
					return;
				}

				UAudioBusSubsystem* AudioBusSubsystem = nullptr;
				if (FAudioDeviceManager* ADM = FAudioDeviceManager::Get())
				{
					if (FAudioDevice* AudioDevice = ADM->GetAudioDeviceRaw(AudioDeviceId))
					{
						AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();
						check(AudioBusSubsystem);
					}
				}
				if (!AudioBusSubsystem)
				{
					return;
				}

				AudioBusChannels = FMath::Min(uint32(AudioBusProxy->NumChannels), uint32(EAudioBusChannels::MaxChannelCount));
				AudioBusId = AudioBusProxy->AudioBusId;

				Audio::FAudioBusKey AudioBusKey(AudioBusId);

				const FString AudioBusWriterNodeBusName = FString::Format(TEXT("_AudioBusWriterNode_AudioBusId_{0}"), { AudioBusId });
				AudioBusSubsystem->StartAudioBus(AudioBusKey, AudioBusWriterNodeBusName, AudioBusChannels, false);

				AudioBusPatchInput = AudioBusSubsystem->AddPatchInputForSoundAndAudioBus(InstanceID, AudioBusKey, BlockSizeFrames, int32(AudioBusChannels));
				int32 NumBlocksToPush = InitialNumBlocks();

				// Handle case of mismatched sample rate between audio mixer and
				// metasound. 
				if (EnableResampledAudioBus && (AudioMixerSampleRate != SampleRate) && (AudioMixerSampleRate > 0.f) && (SampleRate > 0.f))
				{
					UE_LOGF(LogMetaSound, Warning, "Using a audio bus writer node is inefficient if the MetaSound sample rate %f does not match the AudioMixer sample rate %f. Please update MetaSound SampleRate to match the AudioMixer's SampleRate", SampleRate, AudioMixerSampleRate);

					if (NumBlocksToPush)
					{
						// Sample rate matches between audio mixer and metasound. This
						// node will produce audio to the patch in approximate block
						// sizes of (BlockSize * SampleRate / AudioMixerSampleRate)
						AudioBusPatchInput.PushAudio(nullptr, NumBlocksToNumSamples(NumBlocksToPush, SampleRate / AudioMixerSampleRate));
					}

					ResampledPatchInput = MakeUnique<FResampledPatchInput>(AudioBusChannels, AudioMixerSampleRate, SampleRate, BlockSizeFrames, AudioBusPatchInput);
				}
				else  if (NumBlocksToPush > 0)
				{
					// Sample rate matches between audio mixer and metasound. 
					AudioBusPatchInput.PushAudio(nullptr, NumBlocksToNumSamples(NumBlocksToPush));
				}

				// Allocate and fill the interleaved buffer with silence,
				// in case it contains more channels than the node supports.
				InterleavedBuffer.Reset();
				InterleavedBuffer.AddZeroed(NumBlocksToNumSamples(1));
			}
		}
		
		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace Frontend;
			using namespace AudioBusWriterNode;

			InterleavedBuffer.Reset();
			AudioMixerOutputFrames = INDEX_NONE;
			AudioMixerSampleRate = -1.f;
			AudioDeviceId = INDEX_NONE;
			SampleRate = InParams.OperatorSettings.GetSampleRate();
			AudioBusPatchInput.Reset();
			ResampledPatchInput.Reset();
			AudioBusChannels = INDEX_NONE;
			AudioBusId = 0;
			InstanceID = 0;
			SampleRate = InParams.OperatorSettings.GetSampleRate();
			BlockSizeFrames = InParams.OperatorSettings.GetNumFramesPerBlock();
			bWasUnderrunReported = false;

			bool bHasEnvironmentVars = InParams.Environment.Contains<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
			bHasEnvironmentVars &= InParams.Environment.Contains<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
			bHasEnvironmentVars &= InParams.Environment.Contains<uint64>(SourceInterface::Environment::TransmitterID);

			if (bHasEnvironmentVars)
			{
				AudioDeviceId = InParams.Environment.GetValue<Audio::FDeviceId>(SourceInterface::Environment::DeviceID);
				AudioMixerOutputFrames = InParams.Environment.GetValue<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames);
				InstanceID = InParams.Environment.GetValue<uint64>(SourceInterface::Environment::TransmitterID);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus writer node requires audio device ID '%ls', audio mixer num output frames '%ls' and transmitter id '%ls' environment variables"
					, *SourceInterface::Environment::DeviceID.ToString(), *SourceInterface::Environment::AudioMixerNumOutputFrames.ToString(), *SourceInterface::Environment::TransmitterID.ToString());
			}

			// Audio mixer sample rate is a newer addition to the set of required environment variables in UE 5.6.  Check separately
			// and fall back to old behavior if it does not exist. 
			if (InParams.Environment.Contains<float>(SourceInterface::Environment::AudioMixerSampleRate))
			{
				AudioMixerSampleRate = InParams.Environment.GetValue<float>(SourceInterface::Environment::AudioMixerSampleRate);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Audio bus writer node may not render correctly without the audio mixer sample rate '%ls' environment variable", *SourceInterface::Environment::AudioMixerSampleRate.ToString());
				// Assume matching sample rate if environment variable is missing.
				AudioMixerSampleRate = SampleRate;
			}
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AudioBusWriterNode;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Inputs::AudioBus), AudioBusAsset);

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(Inputs::Audio, ChannelIndex), AudioInputs[ChannelIndex]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
		}

		void Execute()
		{
			const FAudioBusProxyPtr& BusProxy = AudioBusAsset->GetAudioBusProxy();
			if (BusProxy.IsValid() && BusProxy->AudioBusId != AudioBusId)
			{
				InterleavedBuffer.Reset();
			}
			
			if (InterleavedBuffer.IsEmpty())
			{
				// if environment vars & a valid audio bus have been set since starting, try to create the patch now
				if (SampleRate > 0.f && BusProxy.IsValid())
				{
					CreatePatchInput();
				}
			}

			if (InterleavedBuffer.IsEmpty())
			{
				return;
			}

			// Retrieve input and interleaved buffer pointers
			const float* AudioInputBufferPtrs[NumChannels];
			for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				AudioInputBufferPtrs[ChannelIndex] = AudioInputs[ChannelIndex]->GetData();
			}
			float* InterleavedBufferPtr = InterleavedBuffer.GetData();

			if (AudioBusChannels == 1)
			{
				FMemory::Memcpy(InterleavedBufferPtr, AudioInputBufferPtrs[0], BlockSizeFrames * sizeof(float));
			}
			else
			{
				// Interleave the inputs
				// Writing the channels of the interleaved buffer sequentially should improve
				// cache utilization compared to writing each input's frames sequentially.
				// There is more likely to be a cache line for each buffer than for the
				// entirety of the interleaved buffer.
				uint32 MinChannels = FMath::Min(AudioBusChannels, NumChannels);
				for (int32 FrameIndex = 0; FrameIndex < BlockSizeFrames; ++FrameIndex)
				{
					// Fill as many channels in the interleaved buffer as possible,
					// given the number of available audio buffers.
					for (uint32 ChannelIndex = 0; ChannelIndex < MinChannels; ++ChannelIndex)
					{
						InterleavedBufferPtr[ChannelIndex] = *AudioInputBufferPtrs[ChannelIndex]++;
					}

					// The interleaved buffer has as many channels as the assigned audio bus.
					InterleavedBufferPtr += AudioBusChannels;
				}
			}

			int32 SamplesPushed = -1;
			if (ResampledPatchInput.IsValid())
			{
				// Perform resampling when pushing audio in case where audio mixer sample rate
				// does not match the metasound sample rate. 
				SamplesPushed = ResampledPatchInput->PushAudio(InterleavedBuffer.GetData(), InterleavedBuffer.Num());
			}
			else
			{
				// Pushes the interleaved data to the audio bus
				SamplesPushed = AudioBusPatchInput.PushAudio(InterleavedBuffer.GetData(), InterleavedBuffer.Num());
			}

			if (SamplesPushed < InterleavedBuffer.Num() && !bWasUnderrunReported)
			{
				UE_LOGF(LogMetaSound, Warning, "Underrun detected in audio bus writer node.");
				bWasUnderrunReported = true;
			}
		}

	private:
		int32 InitialNumBlocks() const
		{
			return AudioBusWriterNodeInitialNumBlocks(BlockSizeFrames, AudioMixerOutputFrames);
		}

		int32 NumBlocksToNumSamples(int32 NumBlocks, float InSampleRateRatio=-1.f) const
		{
			if (InSampleRateRatio > 0.f)
			{
				return FMath::CeilToInt(NumBlocks * BlockSizeFrames * InSampleRateRatio) * AudioBusChannels;
			}
			else
			{
				return NumBlocks * BlockSizeFrames * AudioBusChannels;
			}
		}

		FAudioBusAssetReadRef AudioBusAsset;
		TArray<FAudioBufferReadRef> AudioInputs;

		Audio::FAlignedFloatBuffer InterleavedBuffer;
		TUniquePtr<AudioBusPrivate::FResampledPatchInput> ResampledPatchInput;
		int32 AudioMixerOutputFrames = INDEX_NONE;
		float AudioMixerSampleRate = -1.f;
		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		float SampleRate = 0.0f;
		Audio::FPatchInput AudioBusPatchInput;
		uint64 InstanceID = 0;
		uint32 AudioBusChannels = INDEX_NONE;
		uint32 AudioBusId = 0;
		int32 BlockSizeFrames = 0;
		bool bWasUnderrunReported = false;
	};

	template<uint32 NumChannels>
	using TAudioBusWriterNode = TNodeFacade<TAudioBusWriterOperator<NumChannels>>;

#define REGISTER_AUDIO_BUS_WRITER_NODE(ChannelCount) \
	using FAudioBusWriterNode_##ChannelCount = TAudioBusWriterNode<ChannelCount>; \
	METASOUND_REGISTER_NODE(FAudioBusWriterNode_##ChannelCount) \

	REGISTER_AUDIO_BUS_WRITER_NODE(1);
	REGISTER_AUDIO_BUS_WRITER_NODE(2);
	REGISTER_AUDIO_BUS_WRITER_NODE(4);
	REGISTER_AUDIO_BUS_WRITER_NODE(6);
	REGISTER_AUDIO_BUS_WRITER_NODE(8);
}

#undef LOCTEXT_NAMESPACE
