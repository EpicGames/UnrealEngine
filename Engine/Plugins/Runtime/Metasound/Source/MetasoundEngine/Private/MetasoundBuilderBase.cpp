// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundBuilderBase.h"

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Metasound.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundBuilderPrivate.h"
#include "MetasoundDataReference.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLog.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundVertex.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundBuilderBase)


void UMetaSoundBuilderBase::BeginDestroy()
{
	using namespace Metasound::Frontend;

	// Need to finish building before destroying UPROPERTYs (Super::BeginDestroy)
	// as the Builder often holds a TScriptInterface<IMetaSoundDocumentInterface>
	// of a UPROPERTY that lives on this or derived objects. BuilderRegistry may get
	// destroyed prior to some builder objects, so for safety don't use checked registry
	// getter.
	if (Builder.IsValid())
	{
		if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
		{
			const FMetasoundFrontendClassName& MetaSoundClassName = Builder.GetConstDocumentChecked().RootGraph.Metadata.GetClassName();
			BuilderRegistry->FinishBuilding(MetaSoundClassName, Builder.GetHintPath());
		}

		// The registry may have not been active or it was and the internal weak pointer record of this object no longer accessible.
		// In either of these cases, we could still have a local, valid builder, so call finish directly here just in case.
		Builder.FinishBuilding();
	}
	Super::BeginDestroy();
}

FMetaSoundBuilderNodeOutputHandle UMetaSoundBuilderBase::AddGraphInputNode(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult, bool bIsConstructorInput)
{
	using namespace Metasound;

	FMetaSoundBuilderNodeOutputHandle NewHandle;

	if (Frontend::IDataTypeRegistry::Get().FindDataTypeRegistryEntry(DataType) == nullptr)
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphInputNode Failed on builder '%ls' when attempting to add '%ls': '%ls' is not a registered DataType", *GetName(), *Name.ToString(), *DataType.ToString());
	}
	else
	{
		const FMetasoundFrontendNode* Node = Builder.FindGraphInputNode(Name);
		if (Node)
		{
			UE_LOGF(LogMetaSound, Warning, "AddGraphInputNode Failed: Input Node already exists with name '%ls'; returning handle to existing node which may or may not match requested DataType '%ls'", *Name.ToString(), *DataType.ToString());
		}
		else
		{
			Frontend::FDocumentIDGenerator& IDGenerator = Frontend::FDocumentIDGenerator::Get();
			const FMetasoundFrontendDocument& Doc = GetConstBuilder().GetConstDocumentChecked();

			FMetasoundFrontendClassInput Description;
			Description.Name = Name;
			Description.TypeName = DataType;
			Description.NodeID = IDGenerator.CreateNodeID(Doc);
			Description.VertexID = IDGenerator.CreateVertexID(Doc);
			Description.AccessType = bIsConstructorInput ? EMetasoundFrontendVertexAccessType::Value : EMetasoundFrontendVertexAccessType::Reference;

			Description.InitDefault(MoveTemp(DefaultValue));

			Node = Builder.AddGraphInput(MoveTemp(Description));
		}

		if (Node)
		{
			const TArray<FMetasoundFrontendVertex>& Outputs = Node->Interface.Outputs;
			checkf(!Outputs.IsEmpty(), TEXT("Node should be initialized and have one output."));

			NewHandle.NodeID = Node->GetID();
			NewHandle.VertexID = Outputs.Last().VertexID;
		}
	}

	OutResult = NewHandle.IsSet() ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
	return NewHandle;
}

FMetaSoundBuilderNodeInputHandle UMetaSoundBuilderBase::AddGraphOutputNode(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult, bool bIsConstructorOutput)
{
	using namespace Metasound::Frontend;

	FMetaSoundBuilderNodeInputHandle NewHandle;

	if (IDataTypeRegistry::Get().FindDataTypeRegistryEntry(DataType) == nullptr)
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphOutputNode Failed on builder '%ls' when attempting to add '%ls': '%ls' is not a registered DataType", *GetName(), *Name.ToString(), *DataType.ToString());
	}
	else
	{
		const FMetasoundFrontendNode* Node = Builder.FindGraphOutputNode(Name);
		if (Node)
		{
			UE_LOGF(LogMetaSound, Warning, "AddGraphOutputNode Failed: Output Node already exists with name '%ls'; returning handle to existing node which may or may not match requested DataType '%ls'", *Name.ToString(), *DataType.ToString());
		}
		else
		{
			FDocumentIDGenerator& IDGenerator = FDocumentIDGenerator::Get();
			const FMetasoundFrontendDocument& Doc = GetConstBuilder().GetConstDocumentChecked();

			FMetasoundFrontendClassOutput Description;
			Description.Name = Name;
			Description.TypeName = DataType;
			Description.NodeID = IDGenerator.CreateNodeID(Doc);
			Description.VertexID = IDGenerator.CreateVertexID(Doc);
			Description.AccessType = bIsConstructorOutput ? EMetasoundFrontendVertexAccessType::Value : EMetasoundFrontendVertexAccessType::Reference;
			Node = Builder.AddGraphOutput(MoveTemp(Description));
		}

		if (Node)
		{
			const TArray<FMetasoundFrontendVertex>& Inputs = Node->Interface.Inputs;
			checkf(!Inputs.IsEmpty(), TEXT("Node should be initialized and have one input."));

			const FGuid& VertexID = Inputs.Last().VertexID;
			if (Builder.SetNodeInputDefault(Node->GetID(), VertexID, DefaultValue))
			{
				NewHandle.NodeID = Node->GetID();
				NewHandle.VertexID = VertexID;
			}
		}
	}

	OutResult = NewHandle.IsSet() ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
	return NewHandle;
}

#if WITH_EDITORONLY_DATA
void UMetaSoundBuilderBase::AddGraphPage(FName PageName, bool bDuplicateLastGraph, bool bSetAsBuildGraph, EMetaSoundBuilderResult& OutResult)
{
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageName))
		{
			if (Builder.FindGraphPageIndex(PageSettings->UniqueId) == INDEX_NONE)
			{
				Builder.AddGraphPage(PageSettings->UniqueId, bDuplicateLastGraph, bSetAsBuildGraph);
				OutResult = EMetaSoundBuilderResult::Succeeded;
				return;
			}
		}
	}

	OutResult = EMetaSoundBuilderResult::Failed;
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundBuilderBase::AddGraphVariable(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult)
{
	const FMetasoundFrontendVariable* Variable = Builder.AddGraphVariable(Name, DataType, &DefaultValue);
	OutResult = Variable ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddGraphVariableGetNode(FName Name, EMetaSoundBuilderResult& OutResult)
{
	OutResult = EMetaSoundBuilderResult::Failed;
	FMetaSoundNodeHandle NodeHandle;
	if (const FMetasoundFrontendNode* AccessorNode = Builder.AddGraphVariableAccessorNode(Name))
	{
		NodeHandle.NodeID = AccessorNode->GetID();
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}

	return NodeHandle;
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddGraphVariableGetDelayedNode(FName Name, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound;

	OutResult = EMetaSoundBuilderResult::Failed;
	FMetaSoundNodeHandle NodeHandle;
	if (const FMetasoundFrontendNode* AccessorNode = Builder.AddGraphVariableDeferredAccessorNode(Name))
	{
		NodeHandle.NodeID = AccessorNode->GetID();
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}

	return NodeHandle;
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddGraphVariableSetNode(FName Name, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound;

	OutResult = EMetaSoundBuilderResult::Failed;
	FMetaSoundNodeHandle NodeHandle;
	if (const FMetasoundFrontendNode* MutatorNode = Builder.AddGraphVariableMutatorNode(Name))
	{
		NodeHandle.NodeID = MutatorNode->GetID();
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}

	return NodeHandle;
}

void UMetaSoundBuilderBase::AddInterface(FName InterfaceName, EMetaSoundBuilderResult& OutResult)
{
	const bool bInterfaceAdded = Builder.AddInterface(InterfaceName);
	OutResult = bInterfaceAdded ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

FDelegateHandle UMetaSoundBuilderBase::AddTransactionListener(TSharedRef<Metasound::Frontend::IDocumentBuilderTransactionListener> BuilderListener)
{
	BuilderListener->OnBuilderReloaded(GetBuilderDelegates());
	return BuilderReloadDelegate.AddSP(BuilderListener, &Metasound::Frontend::IDocumentBuilderTransactionListener::OnBuilderReloaded);
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddNode(const TScriptInterface<IMetaSoundDocumentInterface>& NodeClass, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FMetaSoundNodeHandle NewHandle;

	if (NodeClass)
	{
		UObject* NodeClassObject = NodeClass.GetObject();
		check(NodeClassObject);

#if WITH_EDITOR
		// Assets that may undergo serialization cannot reference transient objects
		const bool bIsInvalidReference = !NodeClassObject->IsAsset() && Builder.CastDocumentObjectChecked<UObject>().IsAsset();
#else
		constexpr bool bIsInvalidReference = false;
#endif // WITH_EDITOR

		if (bIsInvalidReference)
		{
			UObject& ThisBuildersObject = Builder.CastDocumentObjectChecked<UObject>();
			UE_LOGF(LogMetaSound, Warning,
				"Failed to add node of transient asset '%ls' to serialized asset '%ls': "
				"Transient object node class cannot be referenced from asset node class.",
				*NodeClassObject->GetPathName(),
				*ThisBuildersObject.GetPathName());
		}
		else
		{
			RegisterGraphIfOutstandingTransactions(*NodeClassObject);

			const FMetasoundFrontendDocument& NodeClassDoc = NodeClass->GetConstDocument();
			const FMetasoundFrontendGraphClass& NodeClassGraph = NodeClassDoc.RootGraph;

			const EMetasoundFrontendClassAccessFlags AccessFlags = NodeClassGraph.Metadata.GetAccessFlags();
			if (!EnumHasAnyFlags(AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
			{
				UE_LOGF(LogMetaSound, Warning, "Failed to add new node of graph class '%ls': Class access flag '%ls' not set.",
					*NodeClassGraph.Metadata.GetClassName().ToString(),
					*LexToString(EMetasoundFrontendClassAccessFlags::Referenceable));
				OutResult = EMetaSoundBuilderResult::Failed;
				return NewHandle;
			}
			if (const FMetasoundFrontendNode* NewNode = Builder.AddGraphNode(NodeClassGraph))
			{
				NewHandle.NodeID = NewNode->GetID();
			}
		}
	}

	OutResult = NewHandle.IsSet() ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
	return NewHandle;
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, int32 MajorVersion, EMetaSoundBuilderResult& OutResult)
{
	return AddNodeByClassName(InClassName, OutResult, MajorVersion);
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, EMetaSoundBuilderResult& OutResult, int32 MajorVersion)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FMetaSoundNodeHandle NewHandle;
	FMetaSoundClassInfo ClassInfo;
	if (!ISearchEngine::Get().FindRegisteredClass(InClassName, MajorVersion, ClassInfo))
	{
		UE_LOGF(LogMetaSound, Error, "Failed to add new node by class name '%ls' and major version '%d': Class not found", *InClassName.ToString(), MajorVersion);
		OutResult = EMetaSoundBuilderResult::Failed;
		return NewHandle;
	}

	if (!EnumHasAnyFlags(ClassInfo.AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to add new node by name of class '%ls': Class access flag '%ls' not set.",
			*InClassName.ToString(),
			*LexToString(EMetasoundFrontendClassAccessFlags::Referenceable));
		OutResult = EMetaSoundBuilderResult::Failed;
		return NewHandle;
	}

	if (const FMetasoundFrontendNode* NewNode = Builder.AddNodeByClassName(InClassName, MajorVersion))
	{
		NewHandle.NodeID = NewNode->GetID();
	}

	OutResult = NewHandle.IsSet() ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
	return NewHandle;
}

#if WITH_EDITORONLY_DATA
TScriptInterface<IMetaSoundDocumentInterface> UMetaSoundBuilderBase::Build(const FMetaSoundBuilderOptions& Options) const
{
	if (Options.ExistingMetaSound)
	{
		BuildAndOverwriteMetaSoundInternal(Options.ExistingMetaSound, Options.bForceUniqueClassName);
		return Options.ExistingMetaSound;
	}

	return BuildNewMetaSound(Options.Name);
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundBuilderBase::BuildAndOverwriteMetaSound(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName)
{
	UObject* MetaSoundPtr = ExistingMetaSound.GetObject();
	if (!MetaSoundPtr)
	{
		UE_LOGF(LogMetaSound, Error, "Failed to build and overwrite MetaSound: No existing MetaSound supplied.");
		return;
	}

	if (MetaSoundPtr->IsAsset())
	{
		UE_LOGF(LogMetaSound, Error, "Failed to build and overwrite MetaSound: Cannot overwrite serialized asset "
			"(use 'BuildNewMetaSound' to create a new, transient MetaSound. Overwriting serialized asset is only "
			"supported at edit-time via UMetaSoundEditorSubsystem::BuildToAsset.");
		return;
	}

	BuildAndOverwriteMetaSoundInternal(ExistingMetaSound, bForceUniqueClassName);
}

void UMetaSoundBuilderBase::BuildInternal(TScriptInterface<IMetaSoundDocumentInterface> NewMetaSound, const FMetasoundFrontendClassName* InDocClassName) const
{
	using namespace Metasound::Engine;

	// If using existing class name, ensure that a builder does not exist for it to
	// avoid build active flag conflation with the locally generated frontend builder below.
	if (InDocClassName)
	{
		FDocumentBuilderRegistry::GetChecked().FinishBuilding(*InDocClassName);
	}

	FMetaSoundFrontendDocumentBuilder NewDocBuilder(NewMetaSound);

	constexpr bool bResetVersion = false;
	NewDocBuilder.InitDocument(&GetConstBuilder().GetConstDocumentChecked(), InDocClassName, bResetVersion);
	NewMetaSound->ConformObjectToDocument();
}

#if WITH_EDITOR
bool UMetaSoundBuilderBase::ClearMemberMetadata(const FGuid& InMemberID)
{
	return Builder.ClearMemberMetadata(InMemberID);
}
#endif // WITH_EDITOR

bool UMetaSoundBuilderBase::ConformObjectToDocument()
{
	return Builder.ConformObjectToDocument();
}

bool UMetaSoundBuilderBase::ContainsNode(const FMetaSoundNodeHandle& NodeHandle) const
{
	return Builder.ContainsNode(NodeHandle.NodeID);
}

bool UMetaSoundBuilderBase::ContainsNodeInput(const FMetaSoundBuilderNodeInputHandle& InputHandle) const
{
	return Builder.FindNodeInput(InputHandle.NodeID, InputHandle.VertexID) != nullptr;
}

bool UMetaSoundBuilderBase::ContainsNodeOutput(const FMetaSoundBuilderNodeOutputHandle& OutputHandle) const
{
	return Builder.FindNodeOutput(OutputHandle.NodeID, OutputHandle.VertexID) != nullptr;
}

void UMetaSoundBuilderBase::ConnectNodes(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;

	OutResult = EMetaSoundBuilderResult::Failed;

	FMetasoundFrontendEdge NewEdge { NodeOutputHandle.NodeID, NodeOutputHandle.VertexID, NodeInputHandle.NodeID, NodeInputHandle.VertexID };
	const EInvalidEdgeReason InvalidEdgeReason = Builder.IsValidEdge(NewEdge);

	if (InvalidEdgeReason == Metasound::Frontend::EInvalidEdgeReason::None)
	{
#if !NO_LOGGING
		const FMetasoundFrontendNode* OldOutputNode = nullptr;
		const FMetasoundFrontendVertex* OldOutputVertex = nullptr;
		if (Builder.IsNodeInputConnected(NodeInputHandle.NodeID, NodeInputHandle.VertexID))
		{
			OldOutputVertex = Builder.FindNodeOutputConnectedToNodeInput(NodeInputHandle.NodeID, NodeInputHandle.VertexID, &OldOutputNode);
		}
#endif // !NO_LOGGING

		const bool bRemovedEdge = Builder.RemoveEdgeToNodeInput(NodeInputHandle.NodeID, NodeInputHandle.VertexID);
		Builder.AddEdge(MoveTemp(NewEdge));

#if !NO_LOGGING
		if (bRemovedEdge)
		{
			checkf(OldOutputNode, TEXT("MetaSound edge was removed from output but output node not found."));
			checkf(OldOutputVertex, TEXT("MetaSound edge was removed from output but output vertex not found."));

			const FMetasoundFrontendNode* InputNode = Builder.FindNode(NodeInputHandle.NodeID);
			checkf(InputNode, TEXT("Edge was deemed valid but input parent node is missing"));

			const FMetasoundFrontendVertex* InputVertex = Builder.FindNodeInput(NodeInputHandle.NodeID, NodeInputHandle.VertexID);
			checkf(InputVertex, TEXT("Edge was deemed valid but input is missing"));

			const FMetasoundFrontendNode* OutputNode = Builder.FindNode(NodeOutputHandle.NodeID);
			checkf(OutputNode, TEXT("Edge was deemed valid but output parent node is missing"));

			const FMetasoundFrontendVertex* OutputVertex = Builder.FindNodeOutput(NodeOutputHandle.NodeID, NodeOutputHandle.VertexID);
			checkf(OutputVertex, TEXT("Edge was deemed valid but output is missing"));

			UE_LOGF(LogMetaSound, Verbose, "Removed connection from node output '%ls:%ls' to node '%ls:%ls' in order to connect to node output '%ls:%ls'",
				*OldOutputNode->Name.ToString(),
				*OldOutputVertex->Name.ToString(),
				*InputNode->Name.ToString(),
				*InputVertex->Name.ToString(),
				*OutputNode->Name.ToString(),
				*OutputVertex->Name.ToString());
		}
#endif // !NO_LOGGING

		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		UE_LOGF(LogMetaSound, Warning, "Builder '%ls' 'ConnectNodes' failed: '%ls'", *GetName(), *LexToString(InvalidEdgeReason));
	}
}

void UMetaSoundBuilderBase::ConnectNodes(const FMetaSoundNodeHandle& SourceNode, const FName& OutputName, const FMetaSoundNodeHandle& DestinationNode, const FName& InputName, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeOutputHandle Out = FindNodeOutputByName(SourceNode, OutputName, OutResult);
	if (OutResult == EMetaSoundBuilderResult::Failed) { return; }
	FMetaSoundBuilderNodeInputHandle In = FindNodeInputByName(DestinationNode, InputName, OutResult);
	if (OutResult == EMetaSoundBuilderResult::Failed) { return; }
	ConnectNodes(Out, In, OutResult);
}

void UMetaSoundBuilderBase::ConnectNodesByInterfaceBindings(const FMetaSoundNodeHandle& FromNodeHandle, const FMetaSoundNodeHandle& ToNodeHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bEdgesAdded = Builder.AddEdgesByNodeClassInterfaceBindings(FromNodeHandle.NodeID, ToNodeHandle.NodeID);
	OutResult = bEdgesAdded ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

TArray<FMetaSoundBuilderNodeInputHandle> UMetaSoundBuilderBase::ConnectNodeOutputsToMatchingGraphInterfaceOutputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult)
{
	TArray<const FMetasoundFrontendEdge*> NewEdges;
	const bool bEdgesAdded = Builder.AddEdgesFromMatchingInterfaceNodeOutputsToGraphOutputs(NodeHandle.NodeID, NewEdges);
	OutResult = bEdgesAdded ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;

	TArray<FMetaSoundBuilderNodeInputHandle> ConnectedVertices;
	Algo::Transform(NewEdges, ConnectedVertices, [this](const FMetasoundFrontendEdge* NewEdge)
	{
#if DO_CHECK
		{
			check(NewEdge);
			const FMetasoundFrontendVertex* Vertex = Builder.FindNodeInput(NewEdge->ToNodeID, NewEdge->ToVertexID);
			checkf(Vertex, TEXT("Edge node output to interface output connection reported success but vertex not found."));
		}
#endif // DO_CHECK

		return FMetaSoundBuilderNodeInputHandle(NewEdge->ToNodeID, NewEdge->ToVertexID);
	});

	return ConnectedVertices;
}

TArray<FMetaSoundBuilderNodeOutputHandle> UMetaSoundBuilderBase::ConnectNodeInputsToMatchingGraphInterfaceInputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult)
{
	TArray<const FMetasoundFrontendEdge*> NewEdges;
	const bool bEdgesAdded = Builder.AddEdgesFromMatchingInterfaceNodeInputsToGraphInputs(NodeHandle.NodeID, NewEdges);
	OutResult = bEdgesAdded ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;

	TArray<FMetaSoundBuilderNodeOutputHandle> ConnectedVertices;
	Algo::Transform(NewEdges, ConnectedVertices, [this](const FMetasoundFrontendEdge* NewEdge)
	{
#if DO_CHECK
		{
			check(NewEdge);
			const FMetasoundFrontendVertex* Vertex = Builder.FindNodeOutput(NewEdge->FromNodeID, NewEdge->FromVertexID);
			checkf(Vertex, TEXT("Edge node input to interface input connection reported success but vertex not found."));
		}
#endif // DO_CHECK

		return FMetaSoundBuilderNodeOutputHandle(NewEdge->FromNodeID, NewEdge->FromVertexID);
	});

	return ConnectedVertices;
}

void UMetaSoundBuilderBase::ConnectNodeOutputToGraphOutput(FName GraphOutputName, const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;

	OutResult = EMetaSoundBuilderResult::Failed;

	if (const FMetasoundFrontendNode* GraphOutputNode = Builder.FindGraphOutputNode(GraphOutputName))
	{
		const FMetasoundFrontendVertex& InputVertex = GraphOutputNode->Interface.Inputs.Last();
		FMetasoundFrontendEdge NewEdge { NodeOutputHandle.NodeID, NodeOutputHandle.VertexID, GraphOutputNode->GetID(), InputVertex.VertexID };
		const EInvalidEdgeReason InvalidEdgeReason = Builder.IsValidEdge(NewEdge);
		if (InvalidEdgeReason == EInvalidEdgeReason::None)
		{
			Builder.RemoveEdgeToNodeInput(GraphOutputNode->GetID(), InputVertex.VertexID);
			Builder.AddEdge(MoveTemp(NewEdge));
			OutResult = EMetaSoundBuilderResult::Succeeded;
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Builder '%ls' 'ConnectNodeOutputToGraphOutput' failed: '%ls'", *GetName(), *LexToString(InvalidEdgeReason));
		}
	}
}

void UMetaSoundBuilderBase::ConnectNodeToGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FName& GraphOutputName, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeOutputHandle NodeOut = UMetaSoundBuilderBase::FindNodeOutputByName(SourceNode, NodeOutputName, OutResult);
	if (OutResult != EMetaSoundBuilderResult::Succeeded)
	{
		return;
	}
	ConnectNodeOutputToGraphOutput(GraphOutputName, NodeOut, OutResult);
}

void UMetaSoundBuilderBase::ConnectNodeToGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FMetaSoundBuilderNodeInputHandle& GraphOutput, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeOutputHandle NodeOut = UMetaSoundBuilderBase::FindNodeOutputByName(SourceNode, NodeOutputName, OutResult);
	if (OutResult != EMetaSoundBuilderResult::Succeeded)
	{
		return;
	}
	ConnectNodes(NodeOut, GraphOutput, OutResult);
}

void UMetaSoundBuilderBase::ConnectNodeInputToGraphInput(FName GraphInputName, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;

	OutResult = EMetaSoundBuilderResult::Failed;

	if (const FMetasoundFrontendNode* GraphInputNode = Builder.FindGraphInputNode(GraphInputName))
	{
		const FMetasoundFrontendVertex& OutputVertex = GraphInputNode->Interface.Outputs.Last();
		FMetasoundFrontendEdge NewEdge { GraphInputNode->GetID(), OutputVertex.VertexID, NodeInputHandle.NodeID, NodeInputHandle.VertexID };
		const EInvalidEdgeReason InvalidEdgeReason = Builder.IsValidEdge(NewEdge);
		if (InvalidEdgeReason == EInvalidEdgeReason::None)
		{
			Builder.RemoveEdgeToNodeInput(NodeInputHandle.NodeID, NodeInputHandle.VertexID);
			Builder.AddEdge(MoveTemp(NewEdge));
			OutResult = EMetaSoundBuilderResult::Succeeded;
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Builder '%ls' 'ConnectNodeInputToGraphInput' failed: '%ls'", *GetName(), *LexToString(InvalidEdgeReason));
		}
	}
}

void UMetaSoundBuilderBase::ConnectGraphInputToNode(const FName& GraphInputName, const FMetaSoundNodeHandle& DestinationNode, const FName& InputName, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeInputHandle In = FindNodeInputByName(DestinationNode, InputName, OutResult);
	if (OutResult == EMetaSoundBuilderResult::Failed)
	{
		return;
	}
	ConnectNodeInputToGraphInput(GraphInputName, In, OutResult);
}

void UMetaSoundBuilderBase::ConvertFromPreset(EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;
	if (Builder.IsDocumentConfiguredBy<FPresetTemplate>())
	{
		Builder.SetTemplate({ });
		Builder.ConfigureDocument();

#if WITH_EDITORONLY_DATA
		Builder.SetIsGraphEditable(true);
#endif // WITH_EDITORONLY_DATA

		OutResult = EMetaSoundBuilderResult::Succeeded;
		return;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::ConvertToPreset(const TScriptInterface<IMetaSoundDocumentInterface>& ParentNodeClass, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;

	const IMetaSoundDocumentInterface* ParentInterface = ParentNodeClass.GetInterface();
	if (!ParentInterface)
	{
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}
	
	// Ensure the referenced node class isn't transient
	if (Cast<UMetaSoundBuilderDocument>(ParentInterface))
	{
		UE_LOGF(LogMetaSound, Warning, "Transient document builders cannot be referenced when converting builder '%ls' to a preset. Build the parent node class as an asset first or use an existing asset instead", *GetName());
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}

	// Ensure the parent node class is a matching object type
	const UClass& BaseMetaSoundClass = ParentInterface->GetBaseMetaSoundUClass();
	UObject* ParentObject = ParentNodeClass.GetObject();
	if (!ParentObject || (ParentObject && !ParentObject->IsA(&BaseMetaSoundClass)))
	{
		UE_LOGF(LogMetaSound, Warning, "The referenced node type must match the base MetaSound class when converting builder '%ls' to a preset (ex. source preset must reference another source)", *GetName());
		OutResult = EMetaSoundBuilderResult::Failed;
		return;
	}

	// Ensure the parent node class is registered
	if (FMetasoundAssetBase* ParentMetaSoundAsset = Metasound::IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(ParentObject))
	{
		Metasound::Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
		RegOptions.PageOrder = UMetaSoundSettings::GetPageOrder();
		ParentMetaSoundAsset->UpdateAndRegisterForExecution(RegOptions);
	}

	TSharedRef<FDocumentModifyDelegates> DocumentDelegates = MakeShared<FDocumentModifyDelegates>(ParentInterface->GetConstDocument());
	InitDelegates(*DocumentDelegates);

	TInstancedStruct<FMetaSoundFrontendPresetTemplate> NewPresetConfig = TInstancedStruct<FMetaSoundFrontendPresetTemplate>::Make();
	NewPresetConfig.GetMutable().Parent = ParentObject;

	Builder.SetTemplate(MoveTemp(NewPresetConfig));
	OutResult = Builder.ConfigureDocument() ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::DisconnectNodes(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bEdgeRemoved = Builder.RemoveEdge(FMetasoundFrontendEdge
	{
		NodeOutputHandle.NodeID,
		NodeOutputHandle.VertexID,
		NodeInputHandle.NodeID,
		NodeInputHandle.VertexID,
	});
	OutResult = bEdgeRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::DisconnectNodeInput(const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bEdgeRemoved = Builder.RemoveEdgeToNodeInput(NodeInputHandle.NodeID, NodeInputHandle.VertexID);
	OutResult = bEdgeRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::DisconnectNodeOutput(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bEdgeRemoved = Builder.RemoveEdgesFromNodeOutput(NodeOutputHandle.NodeID, NodeOutputHandle.VertexID);
	OutResult = bEdgeRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::DisconnectNodesByInterfaceBindings(const FMetaSoundNodeHandle& FromNodeHandle, const FMetaSoundNodeHandle& ToNodeHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bEdgesRemoved = Builder.RemoveEdgesByNodeClassInterfaceBindings(FromNodeHandle.NodeID, ToNodeHandle.NodeID);
	OutResult = bEdgesRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

FMetaSoundBuilderNodeInputHandle UMetaSoundBuilderBase::FindNodeInputByName(const FMetaSoundNodeHandle& NodeHandle, FName InputName, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeHandle.NodeID))
	{
		const TArray<FMetasoundFrontendVertex>& InputVertices = Node->Interface.Inputs;

		auto FindByNamePredicate = [&InputName](const FMetasoundFrontendVertex& Vertex) { return Vertex.Name == InputName; };
		if (const FMetasoundFrontendVertex* Input = InputVertices.FindByPredicate(FindByNamePredicate))
		{
			OutResult = EMetaSoundBuilderResult::Succeeded;
			return FMetaSoundBuilderNodeInputHandle(Node->GetID(), Input->VertexID);
		}

		FString NodeClassName = TEXT("N/A");
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			NodeClassName = Class->Metadata.GetClassName().ToString();
		}

		UE_LOGF(LogMetaSound, Display, "Builder '%ls' failed to find node input '%ls': Node class '%ls' contains no such input", *GetName(), *InputName.ToString(), *NodeClassName);
	}
	else
	{
		UE_LOGF(LogMetaSound, Display, "Builder '%ls' failed to find node input '%ls': Node with ID '%ls' not found", *GetName(), *InputName.ToString(), *NodeHandle.NodeID.ToString());
	}


	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

TArray<FMetaSoundBuilderNodeInputHandle> UMetaSoundBuilderBase::FindNodeInputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult)
{
	return FindNodeInputsByDataType(NodeHandle, OutResult, { });
}

TArray<FMetaSoundBuilderNodeInputHandle> UMetaSoundBuilderBase::FindNodeInputsByDataType(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, FName DataType)
{
	TArray<FMetaSoundBuilderNodeInputHandle> FoundVertices;
	if (Builder.ContainsNode(NodeHandle.NodeID))
	{
		TArray<const FMetasoundFrontendVertex*> Vertices = Builder.FindNodeInputs(NodeHandle.NodeID, DataType);
		Algo::Transform(Vertices, FoundVertices, [&NodeHandle](const FMetasoundFrontendVertex* Vertex)
		{
			return FMetaSoundBuilderNodeInputHandle(NodeHandle.NodeID, Vertex->VertexID);
		});
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		UE_LOGF(LogMetaSound, Display, "Failed to find node inputs by data type with builder '%ls'. Node of with ID '%ls' not found", *GetName(), *NodeHandle.NodeID.ToString());
		OutResult = EMetaSoundBuilderResult::Failed;
	}

	return FoundVertices;
}

FMetaSoundBuilderNodeOutputHandle UMetaSoundBuilderBase::FindNodeOutputByName(const FMetaSoundNodeHandle& NodeHandle, FName OutputName, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeHandle.NodeID))
	{
		const TArray<FMetasoundFrontendVertex>& OutputVertices = Node->Interface.Outputs;

		auto FindByNamePredicate = [&OutputName](const FMetasoundFrontendVertex& Vertex) { return Vertex.Name == OutputName; };
		if (const FMetasoundFrontendVertex* Output = OutputVertices.FindByPredicate(FindByNamePredicate))
		{
			OutResult = EMetaSoundBuilderResult::Succeeded;
			return FMetaSoundBuilderNodeOutputHandle(Node->GetID(), Output->VertexID);
		}

		FString NodeClassName = TEXT("N/A");
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			NodeClassName = Class->Metadata.GetClassName().ToString();
		}

		UE_LOGF(LogMetaSound, Display, "Builder '%ls' failed to find node output '%ls': Node class '%ls' contains no such output", *GetName(), *OutputName.ToString(), *NodeClassName);
	}
	else
	{
		UE_LOGF(LogMetaSound, Display, "Builder '%ls' failed to find node output '%ls': Node with ID '%ls' not found", *GetName(), *OutputName.ToString(), *NodeHandle.NodeID.ToString());
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

TArray<FMetaSoundBuilderNodeOutputHandle> UMetaSoundBuilderBase::FindNodeOutputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult)
{
	return FindNodeOutputsByDataType(NodeHandle, OutResult, { });
}

TArray<FMetaSoundBuilderNodeOutputHandle> UMetaSoundBuilderBase::FindNodeOutputsByDataType(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, FName DataType)
{
	TArray<FMetaSoundBuilderNodeOutputHandle> FoundVertices;
	if (Builder.ContainsNode(NodeHandle.NodeID))
	{
		TArray<const FMetasoundFrontendVertex*> Vertices = Builder.FindNodeOutputs(NodeHandle.NodeID, DataType);
		Algo::Transform(Vertices, FoundVertices, [&NodeHandle](const FMetasoundFrontendVertex* Vertex)
		{
			return FMetaSoundBuilderNodeOutputHandle(NodeHandle.NodeID, Vertex->VertexID);
		});
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		UE_LOGF(LogMetaSound, Display, "Failed to find node outputs by data type with builder '%ls'. Node of with ID '%ls' not found", *GetName(), *NodeHandle.NodeID.ToString());
		OutResult = EMetaSoundBuilderResult::Failed;
	}

	return FoundVertices;
}

#if WITH_EDITOR
const FMetaSoundFrontendGraphComment* UMetaSoundBuilderBase::FindGraphComment(const FGuid& InCommentID) const
{
	return Builder.FindGraphComment(InCommentID);
}

FMetaSoundFrontendGraphComment* UMetaSoundBuilderBase::FindGraphComment(const FGuid& InCommentID)
{
	return Builder.FindGraphComment(InCommentID);
}

FMetaSoundFrontendGraphComment& UMetaSoundBuilderBase::FindOrAddGraphComment(const FGuid& InCommentID)
{
	return Builder.FindOrAddGraphComment(InCommentID);
}
#endif // WITH_EDITOR

TArray<FMetaSoundNodeHandle> UMetaSoundBuilderBase::FindInterfaceInputNodes(FName InterfaceName, EMetaSoundBuilderResult& OutResult)
{
	TArray<FMetaSoundNodeHandle> NodeHandles;

	TArray<const FMetasoundFrontendNode*> Nodes;
	if (Builder.FindInterfaceInputNodes(InterfaceName, Nodes))
	{
		Algo::Transform(Nodes, NodeHandles, [this](const FMetasoundFrontendNode* Node)
		{
			check(Node);
			return FMetaSoundNodeHandle { Node->GetID() };
		});
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		UE_LOGF(LogMetaSound, Display, "'%ls' interface not found on builder '%ls'. No input nodes returned", *InterfaceName.ToString(), *GetName());
		OutResult = EMetaSoundBuilderResult::Failed;
	}

	return NodeHandles;
}

TArray<FMetaSoundNodeHandle> UMetaSoundBuilderBase::FindInterfaceOutputNodes(FName InterfaceName, EMetaSoundBuilderResult& OutResult)
{
	TArray<FMetaSoundNodeHandle> NodeHandles;

	TArray<const FMetasoundFrontendNode*> Nodes;
	if (Builder.FindInterfaceOutputNodes(InterfaceName, Nodes))
	{
		Algo::Transform(Nodes, NodeHandles, [this](const FMetasoundFrontendNode* Node)
		{
			check(Node);
			return FMetaSoundNodeHandle { Node->GetID() };
		});
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		OutResult = EMetaSoundBuilderResult::Failed;
	}

	return NodeHandles;
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindGraphInputNode(FName InputName, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeOutputHandle NodeOutputHandle;
	FName DataTypeName;
	return FindGraphInputNode(InputName, DataTypeName, NodeOutputHandle, OutResult);
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindGraphInputNode(FName InputName, FName& DataTypeName, FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult)
{
	const FMetasoundFrontendNode* GraphInputNode = Builder.FindGraphInputNode(InputName);
	const FMetasoundFrontendClassInput* ClassInput = Builder.FindGraphInput(InputName);

	if (GraphInputNode && ClassInput)
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		DataTypeName = ClassInput->TypeName;

		check(!GraphInputNode->Interface.Outputs.IsEmpty())
		const FMetasoundFrontendVertex& OutputVertex = GraphInputNode->Interface.Outputs[0];
		const FGuid NodeID = GraphInputNode->GetID();
		NodeOutputHandle.NodeID = NodeID;
		NodeOutputHandle.VertexID = OutputVertex.VertexID;
		return FMetaSoundNodeHandle { NodeID };
	}

	UE_LOGF(LogMetaSound, Display, "Failed to find graph input by name '%ls' with builder '%ls'", *InputName.ToString(), *GetName());
	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindGraphOutputNode(FName OutputName, FName& DataTypeName, FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult)
{
	const FMetasoundFrontendNode* GraphOutputNode = Builder.FindGraphOutputNode(OutputName);
	const FMetasoundFrontendClassOutput* ClassOutput = Builder.FindGraphOutput(OutputName);
	if (GraphOutputNode && ClassOutput)
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		DataTypeName = ClassOutput->TypeName;

		check(!GraphOutputNode->Interface.Inputs.IsEmpty())
		const FMetasoundFrontendVertex& InputVertex = GraphOutputNode->Interface.Inputs[0];
		const FGuid NodeID = GraphOutputNode->GetID();
		NodeInputHandle.NodeID = NodeID;
		NodeInputHandle.VertexID = InputVertex.VertexID;

		return FMetaSoundNodeHandle{ NodeID };
	}

	UE_LOGF(LogMetaSound, Display, "Failed to find graph output by name '%ls' with builder '%ls'", *OutputName.ToString(), *GetName());
	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindGraphOutputNode(FName OutputName, EMetaSoundBuilderResult& OutResult)
{
	FMetaSoundBuilderNodeInputHandle NodeInputHandle;
	FName DataTypeName;
	return FindGraphOutputNode(OutputName, DataTypeName, NodeInputHandle, OutResult);
}

#if WITH_EDITOR
UMetaSoundFrontendMemberMetadata* UMetaSoundBuilderBase::FindMemberMetadata(const FGuid& InMemberID)
{
	return Builder.FindMemberMetadata(InMemberID);
}
#endif // WITH_EDITOR

FMetaSoundFrontendDocumentBuilder& UMetaSoundBuilderBase::GetBuilder()
{
	return Builder;
}

Metasound::Frontend::FDocumentModifyDelegates& UMetaSoundBuilderBase::GetBuilderDelegates()
{
	return Builder.GetDocumentDelegates();
}

const FMetaSoundFrontendDocumentBuilder& UMetaSoundBuilderBase::GetConstBuilder() const
{
	return Builder;
}

FMetasoundFrontendLiteral UMetaSoundBuilderBase::GetGraphInputDefault(FName InputName, EMetaSoundBuilderResult& OutResult, FName PageName) const
{
	const FGuid* PageID = UMetaSoundSettings::GetPageID(PageName);
	if (const FMetasoundFrontendLiteral* Default = Builder.GetGraphInputDefault(InputName, PageID))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return *Default;
	}
	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetasoundFrontendLiteral UMetaSoundBuilderBase::GetGraphVariableDefault(FName VariableName, EMetaSoundBuilderResult& OutResult) const
{
	if (const FMetasoundFrontendLiteral* Default = Builder.GetGraphVariableDefault(VariableName))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return *Default;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

TArray<FName> UMetaSoundBuilderBase::GetGraphInputNames(EMetaSoundBuilderResult& OutResult) const
{
	TArray<FName> GraphInputNames;
	const FMetasoundFrontendGraphClass& RootGraph = Builder.GetConstDocumentChecked().RootGraph;
	for (const FMetasoundFrontendClassInput& Input : RootGraph.GetDefaultInterface().Inputs)
	{
		GraphInputNames.Emplace(Input.Name);
	}

	OutResult = EMetaSoundBuilderResult::Succeeded;
	return GraphInputNames;
}

TArray<FName> UMetaSoundBuilderBase::GetGraphOutputNames(EMetaSoundBuilderResult& OutResult) const
{
	TArray<FName> GraphOutputNames;
	const FMetasoundFrontendGraphClass& RootGraph = Builder.GetConstDocumentChecked().RootGraph;
	for (const FMetasoundFrontendClassOutput& Output : RootGraph.GetDefaultInterface().Outputs)
	{
		GraphOutputNames.Emplace(Output.Name);
	}

	OutResult = EMetaSoundBuilderResult::Succeeded;
	return GraphOutputNames;
}

int32 UMetaSoundBuilderBase::GetLastTransactionRegistered() const
{
	return LastTransactionRegistered;
}

UObject* UMetaSoundBuilderBase::GetReferencedPresetAsset() const
{
	using namespace Metasound::Frontend;
	const FPresetTemplate* PresetTemplate = GetConstBuilder()
		.GetConstTemplateAs<FPresetTemplate>();
	if (PresetTemplate)
	{
		return PresetTemplate->Parent.GetObject();
	}

	return nullptr;
}

void UMetaSoundBuilderBase::Initialize()
{
	using namespace Metasound::Frontend;

	const EObjectFlags NewObjectFlags = RF_Public | RF_Transient;
	TScriptInterface<IMetaSoundDocumentInterface> DocObject = NewObject<UObject>(GetTransientPackage(), &GetBaseMetaSoundUClass(), { }, NewObjectFlags);
	TSharedRef<FDocumentModifyDelegates> DocumentDelegates = MakeShared<FDocumentModifyDelegates>(DocObject->GetConstDocument());
	Builder = FMetaSoundFrontendDocumentBuilder(DocObject, DocumentDelegates);
	Builder.InitDocument();
	InitDelegates(*DocumentDelegates);
}

void UMetaSoundBuilderBase::Initialize(TScriptInterface<IMetaSoundDocumentInterface>& DocInterface)
{
	using namespace Metasound::Frontend;
	
	TSharedRef<FDocumentModifyDelegates> DocumentDelegates = MakeShared<FDocumentModifyDelegates>(DocInterface->GetConstDocument());
	Builder = FMetaSoundFrontendDocumentBuilder(DocInterface, DocumentDelegates);
	const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
	const FMetasoundFrontendClassName& DocClassName = Document.RootGraph.Metadata.GetClassName();
	if (!DocClassName.IsValid())
	{
		Builder.InitDocument();
	}

	InitDelegates(*DocumentDelegates);
}

void UMetaSoundBuilderBase::InitNodeLocations()
{
	Builder.InitNodeLocations();
}

#if WITH_EDITOR
void UMetaSoundBuilderBase::InjectInputTemplateNodes(bool bForceNodeCreation, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Frontend;
	const bool bTemplateNodesInjected = FInputNodeTemplate::GetChecked().Inject(Builder, bForceNodeCreation);
	OutResult = bTemplateNodesInjected ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}
#endif // WITH_EDITOR

bool UMetaSoundBuilderBase::InterfaceIsDeclared(FName InterfaceName) const
{
	return Builder.IsInterfaceDeclared(InterfaceName);
}

bool UMetaSoundBuilderBase::IsPreset() const
{
	return Builder.IsDocumentConfiguredBy<FMetaSoundFrontendPresetTemplate>();
}

bool UMetaSoundBuilderBase::NodesAreConnected(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, const FMetaSoundBuilderNodeInputHandle& InputHandle) const
{
	const FMetasoundFrontendEdge Edge = { OutputHandle.NodeID, OutputHandle.VertexID, InputHandle.NodeID, InputHandle.VertexID };
	return Builder.ContainsEdge(Edge);
}

bool UMetaSoundBuilderBase::NodeInputIsConnected(const FMetaSoundBuilderNodeInputHandle& InputHandle) const
{
	return Builder.IsNodeInputConnected(InputHandle.NodeID, InputHandle.VertexID);
}

bool UMetaSoundBuilderBase::NodeOutputIsConnected(const FMetaSoundBuilderNodeOutputHandle& OutputHandle) const
{
	return Builder.IsNodeOutputConnected(OutputHandle.NodeID, OutputHandle.VertexID);
}

void UMetaSoundBuilderBase::InitDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates)
{
	BuilderReloadDelegate.Broadcast(OutDocumentDelegates);
	OutDocumentDelegates.OnDependencyAdded.AddUObject(this, &UMetaSoundBuilderBase::OnDependencyAdded);
	OutDocumentDelegates.OnRemoveSwappingDependency.AddUObject(this, &UMetaSoundBuilderBase::OnRemoveSwappingDependency);
}

void UMetaSoundBuilderBase::OnDependencyAdded(int32 Index)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClass& NewDependency = Builder.GetConstDocumentChecked().Dependencies[Index];
	if (NewDependency.Metadata.GetType() == EMetasoundFrontendClassType::External)
	{
		const FMetaSoundAssetKey AssetKey(NewDependency.Metadata);
		if (TScriptInterface<IMetaSoundDocumentInterface> DocInterface = IMetaSoundAssetManager::GetChecked().FindAssetAsDocumentInterface(AssetKey))
		{
			OnAssetReferenceAdded(DocInterface);
		}
	}
}

void UMetaSoundBuilderBase::OnRemoveSwappingDependency(int32 Index, int32 LastIndex)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClass& NewDependency = Builder.GetConstDocumentChecked().Dependencies[Index];
	if (NewDependency.Metadata.GetType() == EMetasoundFrontendClassType::External)
	{
		const FMetaSoundAssetKey AssetKey(NewDependency.Metadata);
		if (TScriptInterface<IMetaSoundDocumentInterface> DocInterface = IMetaSoundAssetManager::GetChecked().FindAssetAsDocumentInterface(AssetKey))
		{
			OnRemovingAssetReference(DocInterface);
		}
	}
}

void UMetaSoundBuilderBase::RegisterGraphIfOutstandingTransactions(UObject& InMetaSound)
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();
	FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&InMetaSound);
	check(MetaSoundAsset);

	FMetaSoundAssetRegistrationOptions Options;
	Options.bForceReregister = false;
	Options.bRegisterDependencies = false; // Function handles registration via own recursive functionality below
	Options.PageOrder = UMetaSoundSettings::GetPageOrder();

	TArray<FMetasoundAssetBase*> References = MetaSoundAsset->GetReferencedAssets();
	for (FMetasoundAssetBase* Reference : References)
	{
		UObject* RefMetaSound = Reference->GetOwningAsset();
		check(RefMetaSound);
		AssetManager.AddOrUpdateFromObject(*RefMetaSound);
		RegisterGraphIfOutstandingTransactions(*RefMetaSound);
	}

	if (UMetaSoundBuilderBase* Builder = FDocumentBuilderRegistry::GetChecked().FindBuilderObject(&InMetaSound))
	{
		const int32 TransactionCount = Builder->GetConstBuilder().GetTransactionCount();

		// Force registration if transactions occurred since now and the last time the builder registered the asset.
		Options.bForceReregister = Builder->LastTransactionRegistered != TransactionCount;
		Builder->LastTransactionRegistered = TransactionCount;
	}

	MetaSoundAsset->UpdateAndRegisterForExecution(Options);
}

void UMetaSoundBuilderBase::Reload(TScriptInterface<IMetaSoundDocumentInterface> NewMetaSound, bool bPrimeCache)
{
	using namespace Metasound::Frontend;

	TSharedRef<FDocumentModifyDelegates> DocumentDelegates = MakeShared<FDocumentModifyDelegates>(GetConstBuilder().GetConstDocumentChecked());
	InitDelegates(*DocumentDelegates);
	Builder.Reload(DocumentDelegates, bPrimeCache);
}

void UMetaSoundBuilderBase::ReloadCache(bool bPrimeCache)
{
	Reload({ }, bPrimeCache);
}

void UMetaSoundBuilderBase::ResetGraphInputDefaults(FName Name, EMetaSoundBuilderResult& OutResult)
{
	const bool bReset = Builder.ResetGraphInputDefault(Name);
	OutResult = bReset ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

#if WITH_EDITORONLY_DATA
void UMetaSoundBuilderBase::ResetGraphPages(bool bClearDefaultGraph)
{
	Builder.ResetGraphPages(bClearDefaultGraph);
	Builder.RemoveUnusedDependencies();
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
bool UMetaSoundBuilderBase::RemoveGraphComment(const FGuid& InCommentID)
{
	return Builder.RemoveGraphComment(InCommentID);
}
#endif // WITH_EDITOR

void UMetaSoundBuilderBase::RemoveGraphInput(FName Name, EMetaSoundBuilderResult& OutResult)
{
	const bool bRemoved = Builder.RemoveGraphInput(Name);
	OutResult = bRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::RemoveGraphInputDefault(FName Name, EMetaSoundBuilderResult& OutResult, FName PageName)
{
	const FGuid* PageID = UMetaSoundSettings::GetPageID(PageName);
	const bool bRemoved = Builder.RemoveGraphInputDefault(Name, PageID ? *PageID : Builder.GetBuildPageID());
	OutResult = bRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::RemoveGraphOutput(FName Name, EMetaSoundBuilderResult& OutResult)
{
	const bool bRemoved = Builder.RemoveGraphOutput(Name);
	OutResult = bRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

#if WITH_EDITORONLY_DATA
void UMetaSoundBuilderBase::RemoveGraphPage(FName Name, EMetaSoundBuilderResult& OutResult)
{
	const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
	check(Settings);

	if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(Name))
	{
		Builder.RemoveGraphPage(PageSettings->UniqueId);
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundBuilderBase::RemoveGraphVariable(FName Name, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound;

	if (const FMetasoundFrontendVariable* Variable = Builder.FindGraphVariable(Name))
	{
		const bool bVariableRemoved = Builder.RemoveGraphVariable(Name);
		OutResult = bVariableRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
	}
	else
	{
		UE_LOGF(LogMetaSound, Warning, "RemoveGraphVariable Failed: Variable not found with name '%ls'", *Name.ToString());
		OutResult = EMetaSoundBuilderResult::Failed;
	}
}

void UMetaSoundBuilderBase::RemoveInterface(FName InterfaceName, EMetaSoundBuilderResult& OutResult)
{
	const bool bInterfaceRemoved = Builder.RemoveInterface(InterfaceName);
	OutResult = bInterfaceRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::RemoveNode(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, bool bRemoveUnusedDependencies)
{
	bool bRemoved = false;
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeHandle.NodeID))
	{
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
			// Special case for inputs/outputs so template nodes and interface members are cleaned up
			if (ClassType == EMetasoundFrontendClassType::Input)
			{
				bRemoved = Builder.RemoveGraphInput(Node->Name);
			}
			else if (ClassType == EMetasoundFrontendClassType::Output)
			{
				bRemoved = Builder.RemoveGraphOutput(Node->Name);
			}
			else
			{
				bRemoved = Builder.RemoveNode(NodeHandle.NodeID);
			}
		}
	}

	if (bRemoved && bRemoveUnusedDependencies)
	{
		Builder.RemoveUnusedDependencies();
	}
	OutResult = bRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::RemoveNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult)
{
	const bool bInputDefaultRemoved = Builder.RemoveNodeInputDefault(InputHandle.NodeID, InputHandle.VertexID);
	OutResult = bInputDefaultRemoved ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::RemoveTransactionListener(FDelegateHandle BuilderListenerDelegateHandle)
{
	BuilderReloadDelegate.Remove(BuilderListenerDelegateHandle);
}

void UMetaSoundBuilderBase::RemoveUnusedDependencies()
{
	Builder.RemoveUnusedDependencies();
}

void UMetaSoundBuilderBase::RenameRootGraphClass(const FMetasoundFrontendClassName& InName)
{
	checkNoEntry();
}

#if WITH_EDITOR
void UMetaSoundBuilderBase::SetAuthor(const FString& InAuthor)
{
	Builder.SetAuthor(InAuthor);
}
#endif // WITH_EDITOR

void UMetaSoundBuilderBase::SetGraphInputAccessType(FName InputName, EMetasoundFrontendVertexAccessType AccessType, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphInputAccessType(InputName, AccessType);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphInputDataType(FName InputName, FName DataType, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphInputDataType(InputName, DataType);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphInputDefault(FName InputName, const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult, FName PageName)
{
	using namespace Metasound::Frontend;
	const FGuid* PageID = UMetaSoundSettings::GetPageID(PageName);
	const bool bSet = Builder.SetGraphInputDefault(InputName, Literal, PageID);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphInputName(FName InputName, FName NewName, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphInputName(InputName, NewName);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphOutputAccessType(FName OutputName, EMetasoundFrontendVertexAccessType AccessType, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphOutputAccessType(OutputName, AccessType);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphOutputDataType(FName OutputName, FName DataType, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphOutputDataType(OutputName, DataType);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetGraphOutputName(FName OutputName, FName NewName, EMetaSoundBuilderResult& OutResult)
{
	const bool bSet = Builder.SetGraphOutputName(OutputName, NewName);
	OutResult = bSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

#if WITH_EDITOR
void UMetaSoundBuilderBase::SetMemberMetadata(UMetaSoundFrontendMemberMetadata& NewMetadata)
{
	Builder.SetMemberMetadata(NewMetadata);
}
#endif // WITH_EDITOR

void UMetaSoundBuilderBase::SetTemplate(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Configuration)
{
	Builder.SetTemplate(MoveTemp(Configuration));

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	Builder.ConfigureDocument();
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
}

void UMetaSoundBuilderBase::SetTemplateProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertyFunction)
{
	Builder.SetTemplateProperties(SetPropertyFunction);

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	Builder.ConfigureDocument();
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
}

void UMetaSoundBuilderBase::SetNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	const bool bInputDefaultSet = Builder.SetNodeInputDefault(InputHandle.NodeID, InputHandle.VertexID, Literal);
	OutResult = bInputDefaultSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

#if WITH_EDITOR
void UMetaSoundBuilderBase::SetNodeComment(const FMetaSoundNodeHandle& InNodeHandle, const FString& InNewComment, EMetaSoundBuilderResult& OutResult)
{
	FString NewComment = InNewComment;
	const bool bCommentSet = Builder.SetNodeComment(InNodeHandle.NodeID, MoveTemp(NewComment));
	OutResult = bCommentSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetNodeCommentVisible(const FMetaSoundNodeHandle& InNodeHandle, bool bIsVisible, EMetaSoundBuilderResult& OutResult)
{
	const bool bCommentSet = Builder.SetNodeCommentVisible(InNodeHandle.NodeID, bIsVisible);
	OutResult = bCommentSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetNodeLocation(const FMetaSoundNodeHandle& InNodeHandle, const FVector2D& InLocation, EMetaSoundBuilderResult& OutResult)
{
	const bool bLocationSet = Builder.SetNodeLocation(InNodeHandle.NodeID, InLocation);
	OutResult = bLocationSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

void UMetaSoundBuilderBase::SetNodeLocation(const FMetaSoundNodeHandle& InNodeHandle, const FVector2D& InLocation, const FGuid& InLocationGuid, EMetaSoundBuilderResult& OutResult)
{
	const bool bLocationSet = Builder.SetNodeLocation(InNodeHandle.NodeID, InLocation, &InLocationGuid);
	OutResult = bLocationSet ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}
#endif // WITH_EDITOR

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindNodeInputParent(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult)
{
	if (Builder.ContainsNode(InputHandle.NodeID))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return FMetaSoundNodeHandle { InputHandle.NodeID };
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetaSoundNodeHandle UMetaSoundBuilderBase::FindNodeOutputParent(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, EMetaSoundBuilderResult& OutResult)
{
	if (Builder.ContainsNode(OutputHandle.NodeID))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return FMetaSoundNodeHandle{ OutputHandle.NodeID };
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetasoundFrontendVersion UMetaSoundBuilderBase::FindNodeClassVersion(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendNode* Node = Builder.FindNode(NodeHandle.NodeID))
	{
		if (const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID))
		{
			OutResult = EMetaSoundBuilderResult::Succeeded;
			return FMetasoundFrontendVersion { Class->Metadata.GetClassName().GetFullName(), Class->Metadata.GetVersion() };
		}
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return FMetasoundFrontendVersion::GetInvalid();
}

TInstancedStruct<FMetaSoundFrontendDocumentTemplate> UMetaSoundBuilderBase::GetDocumentTemplate(EMetaSoundBuilderResult& OutResult) const
{
	using namespace Metasound::Frontend;

	OutResult = EMetaSoundBuilderResult::Failed;
	TInstancedStruct<FMetaSoundFrontendDocumentTemplate> NewInstance;

	if (TConstStructView<FMetaSoundTemplate> DocTemplate = Builder.GetConstDocumentTemplate(); DocTemplate.IsValid())
	{
		const UScriptStruct* ScriptStruct = DocTemplate.GetScriptStruct();
		check(ScriptStruct);

		NewInstance.InitializeAsScriptStruct(ScriptStruct, DocTemplate.GetMemory());
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}

	return NewInstance;
}

FMetasoundFrontendClassName UMetaSoundBuilderBase::GetRootGraphClassName() const
{
	return Builder.GetConstDocumentChecked().RootGraph.Metadata.GetClassName();
}

TScriptInterface<IMetaSoundDocumentInterface> UMetaSoundBuilderBase::GetMetaSound() const
{
	return TScriptInterface<IMetaSoundDocumentInterface>(&Builder.CastDocumentObjectChecked<UObject>());
}

void UMetaSoundBuilderBase::GetNodeInputData(const FMetaSoundBuilderNodeInputHandle& InputHandle, FName& Name, FName& DataType, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendVertex* Vertex = Builder.FindNodeInput(InputHandle.NodeID, InputHandle.VertexID))
	{
		Name = Vertex->Name;
		DataType = Vertex->TypeName;
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		Name = { };
		DataType = { };
		OutResult = EMetaSoundBuilderResult::Failed;
	}
}

FMetasoundFrontendLiteral UMetaSoundBuilderBase::GetNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendVertexLiteral* VertexLiteral = Builder.FindNodeInputDefault(InputHandle.NodeID, InputHandle.VertexID))
	{
		OutResult = EMetaSoundBuilderResult::Succeeded;
		return VertexLiteral->Value;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

FMetasoundFrontendLiteral UMetaSoundBuilderBase::GetNodeInputClassDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Engine;

	if (const FMetasoundFrontendVertex* Vertex = Builder.FindNodeInput(InputHandle.NodeID, InputHandle.VertexID))
	{
		if (const TArray<FMetasoundFrontendClassInputDefault>* ClassDefaults = Builder.FindNodeClassInputDefaults(InputHandle.NodeID, Vertex->Name))
		{
			if (const FMetasoundFrontendClassInputDefault* Default = Metasound::Engine::FindPreferredPage_ThreadSafe(*ClassDefaults))
			{
				OutResult = EMetaSoundBuilderResult::Succeeded;
				return Default->Literal;
			}
		}
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return { };
}

bool UMetaSoundBuilderBase::GetNodeInputIsConstructorPin(const FMetaSoundBuilderNodeInputHandle& InputHandle) const
{
	const EMetasoundFrontendVertexAccessType AccessType = Builder.GetNodeInputAccessType(InputHandle.NodeID, InputHandle.VertexID);
	return AccessType == EMetasoundFrontendVertexAccessType::Value;
}

void UMetaSoundBuilderBase::GetNodeOutputData(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, FName& Name, FName& DataType, EMetaSoundBuilderResult& OutResult)
{
	if (const FMetasoundFrontendVertex* Vertex = Builder.FindNodeOutput(OutputHandle.NodeID, OutputHandle.VertexID))
	{
		Name = Vertex->Name;
		DataType = Vertex->TypeName;
		OutResult = EMetaSoundBuilderResult::Succeeded;
	}
	else
	{
		Name = { };
		DataType = { };
		OutResult = EMetaSoundBuilderResult::Failed;
	}
}

bool UMetaSoundBuilderBase::GetNodeOutputIsConstructorPin(const FMetaSoundBuilderNodeOutputHandle& OutputHandle) const
{
	const EMetasoundFrontendVertexAccessType AccessType = Builder.GetNodeOutputAccessType(OutputHandle.NodeID, OutputHandle.VertexID);
	return AccessType == EMetasoundFrontendVertexAccessType::Value;
}
