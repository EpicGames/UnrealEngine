// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreGlobals.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundSettings.h"

namespace Metasound::Engine
{
	// UMetaSoundSettings::GetPageOrder() is only available on GT. This ThreadSafe
	// version is available for use in the builder when things are not on the GT.
	template<typename ElementType>
	const ElementType* FindPreferredPage_ThreadSafe(const TArray<ElementType>& InElements)
	{
		if (IsInGameThread() || IsInAudioThread())
		{
			return Frontend::FindPreferredPage(InElements, UMetaSoundSettings::GetPageOrder());
		}
		else
		{
			TArray<FGuid> PageOrder;
			if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
			{
				PageOrder = Settings->GeneratePageOrder();
			}
			return Frontend::FindPreferredPage(InElements, PageOrder);
		}
	}
}
