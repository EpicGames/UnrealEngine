// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundBuilderSubsystem.h"

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Metasound.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundBuilderPrivate.h"
#include "MetasoundDataReference.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundEngineAsset.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendGraphBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundGeneratorHandle.h"
#include "MetasoundLog.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundVertex.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "UObject/PerPlatformProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundBuilderSubsystem)


namespace Metasound::Engine
{
	FAutoConsoleCommand CVarMetaSoundSetTargetPage(
		TEXT("au.MetaSound.Pages.SetTarget"),
		TEXT("Sets the target page to that with the given name. If name not specified or not found, command is ignored.\n"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (!Args.IsEmpty())
			{
				if (UMetaSoundBuilderSubsystem* Subsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>())
				{
					Subsystem->SetTargetPage(FName { *Args.Last() });
				}
			}
		})
	);

	namespace BuilderSubsystemPrivate
	{
		template <typename TLiteralType>
		FMetasoundFrontendLiteral CreatePODMetaSoundLiteral(const TLiteralType& Value, FName& OutDataType)
		{
			OutDataType = GetMetasoundDataTypeName<TLiteralType>();

			FMetasoundFrontendLiteral Literal;
			Literal.Set(Value);
			return Literal;
		}

		const FMetasoundFrontendLiteral* TryResolveNodeInputDefault(const FMetaSoundFrontendDocumentBuilder& Builder, const FGuid& InNodeID, FName VertexName)
		{
			if (const FMetasoundFrontendVertexLiteral* InputDefault = Builder.FindNodeInputDefault(InNodeID, VertexName))
			{
				return &InputDefault->Value;
			}
			else if (const TArray<FMetasoundFrontendClassInputDefault>* ClassDefaults = Builder.FindNodeClassInputDefaults(InNodeID, VertexName))
			{
				if (const FMetasoundFrontendClassInputDefault* ClassDefault = FindPreferredPage_ThreadSafe(*ClassDefaults))
				{
					return &ClassDefault->Literal;
				}
			}

			return nullptr;
		}
	} // namespace BuilderSubsystemPrivate
} // namespace Metasound::Engine

void UMetaSoundPatchBuilder::BuildAndOverwriteMetaSoundInternal(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName) const
{
	checkf(ExistingMetaSound.GetObject(), TEXT("ExistingMetaSound interface must point to valid MetaSound object"));

	FMetaSoundBuilderOptions Options;
	Options.ExistingMetaSound = ExistingMetaSound;
	Options.bForceUniqueClassName = bForceUniqueClassName;
	constexpr UObject* Parent = nullptr;

	BuildInternal<UMetaSoundPatch>(Parent, Options);
}

TScriptInterface<IMetaSoundDocumentInterface> UMetaSoundPatchBuilder::BuildNewMetaSound(FName NameBase) const
{
	FMetaSoundBuilderOptions Options;
	Options.Name = NameBase;
	constexpr UObject* Parent = nullptr;

	return &BuildInternal<UMetaSoundPatch>(Parent, Options);
}

const UClass& UMetaSoundPatchBuilder::GetBaseMetaSoundUClass() const
{
	return *UMetaSoundPatch::StaticClass();
}

void UMetaSoundPatchBuilder::OnAssetReferenceAdded(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
{
	using namespace Metasound::Frontend;

	check(DocInterface.GetObject());
	UMetaSoundPatch& Patch = Builder.CastDocumentObjectChecked<UMetaSoundPatch>();
	Patch.ReferencedAssetClassObjects.Add(DocInterface.GetObject());

	const FNodeRegistryKey RegistryKey(DocInterface->GetConstDocument().RootGraph);
	Patch.ReferencedAssetClassKeys.Add(RegistryKey.ToString());
}

void UMetaSoundPatchBuilder::OnRemovingAssetReference(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
{
	using namespace Metasound::Frontend;

	check(DocInterface.GetObject());
	UMetaSoundPatch& Patch = Builder.CastDocumentObjectChecked<UMetaSoundPatch>();
	Patch.ReferencedAssetClassObjects.Remove(DocInterface.GetObject());

	const FNodeRegistryKey RegistryKey(DocInterface->GetConstDocument().RootGraph);
	Patch.ReferencedAssetClassKeys.Remove(RegistryKey.ToString());
}

void UMetaSoundSourceBuilder::Audition(UObject* Parent, UAudioComponent* AudioComponent, FOnCreateAuditionGeneratorHandleDelegate CreateGenerator, bool bLiveUpdatesEnabled)
{
	using namespace Metasound;
	using namespace Metasound::DynamicGraph;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSourceBuilder::Audition);

	if (!AudioComponent)
	{
		UE_LOGF(LogMetaSound, Error, "Failed to audition MetaSoundBuilder '%ls': No AudioComponent supplied", *GetFullName());
		return;
	}

	UMetaSoundSource& MetaSoundSource = GetMetaSoundSource();
	RegisterGraphIfOutstandingTransactions(MetaSoundSource);

	// Must be called post register as register ensures cached runtime data passed to transactor is up-to-date
	MetaSoundSource.SetDynamicGeneratorEnabled(bLiveUpdatesEnabled);
	MetaSoundSource.ConformObjectToDocument();

	AudioComponent->SetSound(&MetaSoundSource);

	if (CreateGenerator.IsBound())
	{
		UMetasoundGeneratorHandle* NewHandle = UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(AudioComponent);
		checkf(NewHandle, TEXT("BindToGeneratorDelegate Failed when attempting to audition MetaSoundSource builder '%s'"), *GetName());
		CreateGenerator.Execute(NewHandle);
	}

	if (bLiveUpdatesEnabled)
	{
		LiveComponentIDs.Add(AudioComponent->GetAudioComponentID());
		LiveComponentHandle = AudioComponent->OnAudioFinishedNative.AddUObject(this, &UMetaSoundSourceBuilder::OnLiveComponentFinished);
	}

	AudioComponent->Play();
}

void UMetaSoundSourceBuilder::OnLiveComponentFinished(UAudioComponent* AudioComponent)
{
	LiveComponentIDs.RemoveSwap(AudioComponent->GetAudioComponentID(), EAllowShrinking::No);
	if (LiveComponentIDs.IsEmpty())
	{
		AudioComponent->OnAudioFinishedNative.Remove(LiveComponentHandle);
		if (UMetaSoundSource* Source = Cast<UMetaSoundSource>(AudioComponent->Sound))
		{
			// This sometimes is called during teardown/resave in the editor so have to
			// make sure the builder is in a valid state before disabling the dynamic generator.
			if (Builder.IsValid() && &Builder.CastDocumentObjectChecked<UObject>() == Source)
			{
				Source->SetDynamicGeneratorEnabled(false);
			}
		}
	}
}

bool UMetaSoundSourceBuilder::AddEdgeToTransactor(const FMetasoundFrontendEdge& NewEdge, Metasound::DynamicGraph::FDynamicOperatorTransactor& Transactor) const
{
	using namespace Metasound::Frontend;

	const FMetaSoundFrontendDocumentBuilder& DocBuilder = GetConstBuilder();

	const FMetasoundFrontendVertex* FromNodeOutput = DocBuilder.FindNodeOutput(NewEdge.FromNodeID, NewEdge.FromVertexID);
	const FMetasoundFrontendVertex* ToNodeInput = DocBuilder.FindNodeInput(NewEdge.ToNodeID, NewEdge.ToVertexID);
	if (FromNodeOutput && ToNodeInput)
	{
	#if WITH_EDITOR
		const FMetasoundFrontendNode* ToNode = DocBuilder.FindNode(NewEdge.ToNodeID);
		const FMetasoundFrontendNode* FromNode = DocBuilder.FindNode(NewEdge.FromNodeID);
		if (ToNode && FromNode)
		{
			const FMetasoundFrontendClass* ToNodeClass = DocBuilder.FindDependency(ToNode->ClassID);
			const FMetasoundFrontendClass* FromNodeClass = DocBuilder.FindDependency(FromNode->ClassID);

			// If template node is connecting via an output, then the given edge may be routing data accordingly.
			if (FromNodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
			{
				bool bHandled = false;
				const FTemplateNodeEdgeTransactionContext TransactionContext
				{
					.Builder = DocBuilder,
					.PageID = DocBuilder.GetBuildPageID(),
					.FromNode = FromNode,
					.FromOutput = FromNodeOutput,
					.ToNode = ToNode,
					.ToInput = ToNodeInput
				};

				const FMetasoundFrontendClassName& FromClassName = FromNodeClass->Metadata.GetClassName();
				if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FromClassName))
				{
					bHandled = Template->OnEdgeAdded(TransactionContext, Transactor);
				}

				if (bHandled)
				{
					return true;
				}
			}

			// Adding edge to a template node input however can be safely ignored as template nodes are processed out.
			if (ToNodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
			{
				return true;
			}
		}
	#endif // WITH_EDITOR

		Transactor.AddDataEdge(NewEdge.FromNodeID, FromNodeOutput->Name, NewEdge.ToNodeID, ToNodeInput->Name);
		return true;
	}

	return false;
}

bool UMetaSoundSourceBuilder::AddNodeInputLiteralToTransactor(
	const FMetasoundFrontendNode& Node,
	const FMetasoundFrontendVertex& Input,
	const FMetasoundFrontendLiteral& InputDefault,
	Metasound::DynamicGraph::FDynamicOperatorTransactor& Transactor) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;

	Transactor.SetValue(Node.GetID(), Input.Name, InputDefault.ToLiteral(Input.TypeName), &FMetaSoundFrontendDocumentBuilder::CreateDataReference);
	return true;
}

bool UMetaSoundSourceBuilder::ExecuteAuditionableTransaction(FAuditionableTransaction Transaction) const
{
	using namespace Metasound::Engine::BuilderSubsystemPrivate;
	using namespace Metasound::DynamicGraph;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSourceBuilder::ExecuteAuditionableTransaction);

	checkf(!IsRunningCookCommandlet(),
		TEXT("ExecuteAuditionableTransaction cannot be called while running cook. ")
		TEXT("Requires resolved graph which should not be relied on while cooking as it can mutate depending on cook's ")
		TEXT("target platform, which is independent of targetable page(s)"));

	TSharedPtr<FDynamicOperatorTransactor> Transactor = GetMetaSoundSource().GetDynamicGeneratorTransactor();
	if (Transactor.IsValid())
	{
		return Transaction(*Transactor);
	}

	return false;
}

void UMetaSoundSourceBuilder::BuildAndOverwriteMetaSoundInternal(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName) const
{
	checkf(ExistingMetaSound.GetObject(), TEXT("ExistingMetaSound interface must point to valid MetaSound object"));

	FMetaSoundBuilderOptions Options;
	Options.ExistingMetaSound = ExistingMetaSound;
	Options.bForceUniqueClassName = bForceUniqueClassName;
	constexpr UObject* Parent = nullptr;

	BuildInternal<UMetaSoundSource>(Parent, Options);
}

TScriptInterface<IMetaSoundDocumentInterface> UMetaSoundSourceBuilder::BuildNewMetaSound(FName NameBase) const
{
	FMetaSoundBuilderOptions Options;
	Options.Name = NameBase;
	constexpr UObject* Parent = nullptr;

	return &BuildInternal<UMetaSoundSource>(Parent, Options);
}

const Metasound::Engine::FOutputAudioFormatInfoPair* UMetaSoundSourceBuilder::FindOutputAudioFormatInfo() const
{
	using namespace Metasound::Engine;

	const FOutputAudioFormatInfoMap& FormatInfo = GetOutputAudioFormatInfo();

	auto Predicate = [this](const FOutputAudioFormatInfoPair& Pair)
	{
		const FMetasoundFrontendDocument& Document = Builder.GetConstDocumentChecked();
		return Document.Interfaces.Contains(Pair.Value.InterfaceVersion);
	};

	return Algo::FindByPredicate(FormatInfo, Predicate);
}

const UClass& UMetaSoundSourceBuilder::GetBaseMetaSoundUClass() const
{
	return *UMetaSoundSource::StaticClass();
}

bool UMetaSoundSourceBuilder::GetLiveUpdatesEnabled() const
{
	return GetMetaSoundSource().GetDynamicGeneratorTransactor().IsValid();
}

const FMetasoundFrontendGraph& UMetaSoundSourceBuilder::GetConstTargetPageGraphChecked() const
{
	const FMetasoundFrontendGraphClass& RootGraph = Builder.GetConstDocumentChecked().RootGraph;
	return RootGraph.FindConstGraphChecked(TargetPageID);
}

UMetaSoundSource& UMetaSoundSourceBuilder::GetMetaSoundSource() const
{
	return GetConstBuilder().CastDocumentObjectChecked<UMetaSoundSource>();
}

void UMetaSoundSourceBuilder::InitDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates)
{
	Super::InitDelegates(OutDocumentDelegates);

	if (!IsRunningCookCommandlet())
	{
		OutDocumentDelegates.OnDocumentTemplateChanged.AddUObject(this, &UMetaSoundSourceBuilder::OnDocumentTemplateChanged);

		OutDocumentDelegates.PageDelegates.OnPageAdded.AddUObject(this, &UMetaSoundSourceBuilder::OnPageAdded);
		OutDocumentDelegates.PageDelegates.OnRemovingPage.AddUObject(this, &UMetaSoundSourceBuilder::OnRemovingPage);

		OutDocumentDelegates.InterfaceDelegates.OnInputAdded.AddUObject(this, &UMetaSoundSourceBuilder::OnInputAdded);
		OutDocumentDelegates.InterfaceDelegates.OnInputDataTypeChanged.AddUObject(this, &UMetaSoundSourceBuilder::OnInputDataTypeChanged);
		OutDocumentDelegates.InterfaceDelegates.OnInputDefaultChanged.AddUObject(this, &UMetaSoundSourceBuilder::OnInputDefaultChanged);
		OutDocumentDelegates.InterfaceDelegates.OnInputNameChanged.AddUObject(this, &UMetaSoundSourceBuilder::OnInputNameChanged);
		OutDocumentDelegates.InterfaceDelegates.OnOutputAdded.AddUObject(this, &UMetaSoundSourceBuilder::OnOutputAdded);
		OutDocumentDelegates.InterfaceDelegates.OnRemovingInput.AddUObject(this, &UMetaSoundSourceBuilder::OnRemovingInput);
		OutDocumentDelegates.InterfaceDelegates.OnRemovingOutput.AddUObject(this, &UMetaSoundSourceBuilder::OnRemovingOutput);

		InitTargetPageDelegates(OutDocumentDelegates);
	}
}

void UMetaSoundSourceBuilder::InitTargetPageDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates)
{
	using namespace Metasound::DynamicGraph;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	// If currently executing live audition, must call stop as provided transactions may
	// get corrupted by the fact that the executable page ID may now resolve to a different value.
	ExecuteAuditionableTransaction([this](FDynamicOperatorTransactor& Transactor)
	{
		bool bComponentsStopped = false;
		for (uint64 AudioComponentID : LiveComponentIDs)
		{
			if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
			{
				AudioComponent->Stop();
				bComponentsStopped = true;
			}
		}
		return bComponentsStopped;
	});

	OutDocumentDelegates.IterateGraphDelegates([this](FGraphModifyDelegates& GraphDelegates)
	{
		GraphDelegates.NodeDelegates.OnNodeAdded.RemoveAll(this);
		GraphDelegates.NodeDelegates.OnNodeConfigurationUpdated.RemoveAll(this);
		GraphDelegates.NodeDelegates.OnNodeInputLiteralSet.RemoveAll(this);
		GraphDelegates.NodeDelegates.OnRemoveSwappingNode.RemoveAll(this);
		GraphDelegates.NodeDelegates.OnRemovingNodeInputLiteral.RemoveAll(this);

		GraphDelegates.EdgeDelegates.OnEdgeAdded.RemoveAll(this);
		GraphDelegates.EdgeDelegates.OnRemoveSwappingEdge.RemoveAll(this);
	});

	const IMetaSoundDocumentInterface& DocInterface = GetConstBuilder().GetConstDocumentInterfaceChecked();

	// Determine which graph page is being worked on. 
	TargetPageID = Metasound::Frontend::DefaultPageID;
	if (const FMetasoundFrontendGraph* GraphPage = Metasound::Engine::FindPreferredPage_ThreadSafe(DocInterface.GetConstDocument().RootGraph.GetConstGraphPages()))
	{
		TargetPageID = GraphPage->PageID;
	}

	FGraphModifyDelegates& GraphDelegates = OutDocumentDelegates.FindGraphDelegatesChecked(TargetPageID);

	GraphDelegates.EdgeDelegates.OnEdgeAdded.AddUObject(this, &UMetaSoundSourceBuilder::OnEdgeAdded);
	GraphDelegates.EdgeDelegates.OnRemoveSwappingEdge.AddUObject(this, &UMetaSoundSourceBuilder::OnRemoveSwappingEdge);

	GraphDelegates.NodeDelegates.OnNodeAdded.AddUObject(this, &UMetaSoundSourceBuilder::OnNodeAdded);
	GraphDelegates.NodeDelegates.OnNodeConfigurationUpdated.AddUObject(this, &UMetaSoundSourceBuilder::OnNodeConfigurationUpdated);
	GraphDelegates.NodeDelegates.OnNodeInputLiteralSet.AddUObject(this, &UMetaSoundSourceBuilder::OnNodeInputLiteralSet);
	GraphDelegates.NodeDelegates.OnRemoveSwappingNode.AddUObject(this, &UMetaSoundSourceBuilder::OnRemoveSwappingNode);
	GraphDelegates.NodeDelegates.OnRemovingNodeInputLiteral.AddUObject(this, &UMetaSoundSourceBuilder::OnRemovingNodeInputLiteral);
}

void UMetaSoundSourceBuilder::OnAssetReferenceAdded(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
{
	using namespace Metasound::Frontend;

	check(DocInterface.GetObject());
	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.ReferencedAssetClassObjects.Add(DocInterface.GetObject());

	const FNodeRegistryKey RegistryKey(DocInterface->GetConstDocument().RootGraph);
	Source.ReferencedAssetClassKeys.Add(RegistryKey.ToString());
}

void UMetaSoundSourceBuilder::OnDocumentTemplateChanged(const Metasound::Frontend::FDocumentTemplateChangedArgs&) const
{
	using namespace Metasound::Frontend;

	GetMetaSoundSource().InvalidateCachedRuntimeInputData();
}

void UMetaSoundSourceBuilder::OnEdgeAdded(int32 EdgeIndex) const
{
	using namespace Metasound::DynamicGraph;

	const FMetasoundFrontendEdge& NewEdge = GetConstTargetPageGraphChecked().Edges[EdgeIndex];
	ExecuteAuditionableTransaction([this, &NewEdge](Metasound::DynamicGraph::FDynamicOperatorTransactor& Transactor)
	{
		return AddEdgeToTransactor(NewEdge, Transactor);
	});
}

void UMetaSoundSourceBuilder::OnInputAdded(int32 InputIndex)
{
	using namespace Metasound::DynamicGraph;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendDocument& Doc = Builder.GetConstDocumentChecked();
	const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;
	const FMetasoundFrontendClassInput& NewInput = GraphClass.GetDefaultInterface().Inputs[InputIndex];

	constexpr bool bCreateUObjectProxies = true;
	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.RuntimeInputData.InputMap.Add(NewInput.Name, UMetaSoundSource::CreateRuntimeInput(IDataTypeRegistry::Get(), NewInput, bCreateUObjectProxies, UMetaSoundSettings::GetPageOrder()));

	ExecuteAuditionableTransaction([this, InputIndex, &NewInput](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;


		for (uint64 AudioComponentID : LiveComponentIDs)
		{
			if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
			{
				if (FAudioDevice* AudioDevice = AudioComponent->GetAudioDevice())
				{
					AudioDevice->SendCommandToActiveSounds(AudioComponentID, [NewInputName = NewInput.Name](FActiveSound& ActiveSound)
					{
						if (FMetaSoundParameterTransmitter* Transmitter = static_cast<FMetaSoundParameterTransmitter*>(ActiveSound.GetTransmitter()))
						{
							Transmitter->AddAvailableParameter(NewInputName);
						}
					});
				}
			}
		}

		const FLiteral NewInputLiteral = NewInput.FindConstDefaultChecked(Frontend::DefaultPageID).ToLiteral(NewInput.TypeName);
		Transactor.AddInputDataDestination(NewInput.NodeID, NewInput.Name, NewInputLiteral, &FMetaSoundFrontendDocumentBuilder::CreateDataReference);
		return true;
	});
}

void UMetaSoundSourceBuilder::OnInputDataTypeChanged(int32 InputIndex)
{
	OnRemovingInput(InputIndex);
	OnInputAdded(InputIndex);
}

void UMetaSoundSourceBuilder::OnInputDefaultChanged(const Metasound::Frontend::FDocumentArrayPagedInputArgs& Args) const
{
	using namespace Metasound::DynamicGraph;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendDocument& Doc = Builder.GetConstDocumentChecked();
	const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;
	const FMetasoundFrontendClassInput& InputWithNewDefault = GraphClass.GetDefaultInterface().Inputs[Args.Index];

	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.RuntimeInputData.InputMap.Remove(InputWithNewDefault.Name);

	if (const FMetasoundFrontendClassInput* Input = GetConstBuilder().FindGraphInput(InputWithNewDefault.Name))
	{
		constexpr bool bCreateUObjectProxies = true;
		Source.RuntimeInputData.InputMap.Add(Input->Name, UMetaSoundSource::CreateRuntimeInput(IDataTypeRegistry::Get(), *Input, bCreateUObjectProxies, UMetaSoundSettings::GetPageOrder()));
	}

	// default value changes to added/removed inputs with dynamic graphs is currently unsupported
}

void UMetaSoundSourceBuilder::OnInputNameChanged(FName OldName, FName NewName)
{
	using namespace Metasound::DynamicGraph;
	using namespace Metasound::Frontend;

	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.RuntimeInputData.InputMap.Remove(OldName);

	if (const FMetasoundFrontendClassInput* Input = GetConstBuilder().FindGraphInput(NewName))
	{
		constexpr bool bCreateUObjectProxies = true;
		Source.RuntimeInputData.InputMap.Add(Input->Name, UMetaSoundSource::CreateRuntimeInput(IDataTypeRegistry::Get(), *Input, bCreateUObjectProxies, UMetaSoundSettings::GetPageOrder()));

		ExecuteAuditionableTransaction([this, OldName, &Input](FDynamicOperatorTransactor& Transactor)
		{
			using namespace Metasound;
			using namespace Metasound::Frontend;

			for (const uint64& AudioComponentID : LiveComponentIDs)
			{
				if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
				{
					if (FAudioDevice* AudioDevice = AudioComponent->GetAudioDevice())
					{
						AudioDevice->SendCommandToActiveSounds(AudioComponentID, [OldName, NewInputName = Input->Name](FActiveSound& ActiveSound)
						{
							if (FMetaSoundParameterTransmitter* Transmitter = static_cast<FMetaSoundParameterTransmitter*>(ActiveSound.GetTransmitter()))
							{
								Transmitter->RemoveAvailableParameter(OldName);
								Transmitter->AddAvailableParameter(NewInputName);
							}
						});
					}
				}
			}
			Transactor.RemoveInputDataDestination(OldName);
			const FLiteral NewInputLiteral = Input->FindConstDefaultChecked(Frontend::DefaultPageID).ToLiteral(Input->TypeName);
			Transactor.AddInputDataDestination(Input->NodeID, Input->Name, NewInputLiteral, &FMetaSoundFrontendDocumentBuilder::CreateDataReference);
			return true;
		});
	}
}

void UMetaSoundSourceBuilder::OnNodeAdded(int32 NodeIndex) const
{
	using namespace Metasound::DynamicGraph;

	ExecuteAuditionableTransaction([this, NodeIndex](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		const IMetaSoundDocumentInterface& DocInterface = Builder.GetConstDocumentInterfaceChecked();
		const FMetasoundFrontendGraphClass& OwningGraphClass = DocInterface.GetConstDocument().RootGraph;
		const FMetasoundFrontendGraph& OwningGraph = GetConstTargetPageGraphChecked();
		const FMetasoundFrontendNode& AddedNode = OwningGraph.Nodes[NodeIndex];
		const FMetasoundFrontendClass* NodeClass = Builder.FindDependency(AddedNode.ClassID);
		checkf(NodeClass, TEXT("Node successfully added to graph but document is missing associated dependency"));

#if WITH_EDITOR
		if (NodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
		{
			const FMetasoundFrontendClassName& AddedClassName = NodeClass->Metadata.GetClassName();
			if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(AddedClassName))
			{
				const FTemplateNodeTransactionContext TransactionContext { Builder, Builder.GetBuildPageID(), &AddedNode };
				const bool bHandled = Template->OnNodeAdded(TransactionContext, Transactor);
				if (bHandled)
				{
					return true;
				}
			}
		}
#endif // WITH_EDITOR

		FGraphBuilder::FCreateNodeParams CreateNodeParams
		{
			.FrontendNode = AddedNode,
			.FrontendNodeClass = *NodeClass,
			.OwningFrontendGraph = OwningGraph,
			.OwningFrontendGraphClass = OwningGraphClass,
			.OwningAssetPath = DocInterface.GetAssetPathChecked()
		};
		TUniquePtr<INode> NewNode = FGraphBuilder::CreateNode(CreateNodeParams);

		if (!NewNode.IsValid())
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' failed to create and forward added node '%ls' to live update transactor.", *Builder.GetDebugName(), *AddedNode.Name.ToString());
			return false;
		}

		Transactor.AddNode(AddedNode.GetID(), MoveTemp(NewNode));
		return true;
	});
}

void UMetaSoundSourceBuilder::OnNodeConfigurationUpdated(int32 NodeIndex) const
{
	using namespace Metasound::DynamicGraph;

	ExecuteAuditionableTransaction([this, NodeIndex](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		const IMetaSoundDocumentInterface& DocInterface = Builder.GetConstDocumentInterfaceChecked();
		const FMetasoundFrontendGraphClass& OwningGraphClass = DocInterface.GetConstDocument().RootGraph;
		const FMetasoundFrontendGraph& OwningGraph = GetConstTargetPageGraphChecked();
		const FMetasoundFrontendNode& FrontendConfigNode = OwningGraph.Nodes[NodeIndex];
		const FMetasoundFrontendClass* NodeClass = Builder.FindDependency(FrontendConfigNode.ClassID);
		checkf(NodeClass, TEXT("Node successfully configured on graph but document is missing associated dependency"));

#if WITH_EDITOR
		if (NodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
		{
			const FMetasoundFrontendClassName& AddedClassName = NodeClass->Metadata.GetClassName();
			if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(AddedClassName))
			{
				const FTemplateNodeTransactionContext TransactionContext { Builder, Builder.GetBuildPageID(), &FrontendConfigNode };
				const bool bHandled = Template->OnNodeConfigurationUpdated(TransactionContext, Transactor);
				if (bHandled)
				{
					return true;
				}
			}
		}
#endif // WITH_EDITOR

		FGraphBuilder::FCreateNodeParams ReplacementNodeParams
		{
			.FrontendNode = FrontendConfigNode,
			.FrontendNodeClass = *NodeClass,
			.OwningFrontendGraph = OwningGraph,
			.OwningFrontendGraphClass = OwningGraphClass,
			.OwningAssetPath = DocInterface.GetAssetPathChecked()
		};

		TUniquePtr<INode> ConfiguredNode = FGraphBuilder::CreateNode(ReplacementNodeParams);
		if (!ConfiguredNode.IsValid())
		{
			UE_LOGF(LogMetaSound, Error,
				"Builder '%ls' failed to create and forward reconfigured node '%ls' to live update transactor.",
				*Builder.GetDebugName(),
				*FrontendConfigNode.Name.ToString());
			return false;
		}

		const bool bRemovedNode = Transactor.RemoveNode(FrontendConfigNode.GetID());
		if (!bRemovedNode)
		{
			return false;
		}

		const bool bAddedNode = Transactor.AddNode(FrontendConfigNode.GetID(), MoveTemp(ConfiguredNode));
		if (!bAddedNode)
		{
			return false;
		}

		// Removal and then readding the given node via the transactor indirectly:
		// a. Removes edges to the node, so they must be readded
		bool bEdgesAdded = true;
		{
			auto AddConnectedEdges = [&](const TArray<FMetasoundFrontendVertex>& InVertices)
				{
					for (const FMetasoundFrontendVertex& Input : InVertices)
					{
						TArray<const FMetasoundFrontendEdge*> Edges = Builder.FindEdges(FrontendConfigNode.GetID(), Input.VertexID);
						for (const FMetasoundFrontendEdge* Edge : Edges)
						{
							check(Edge);
							bEdgesAdded &= AddEdgeToTransactor(*Edge, Transactor);
						}
					}
				};

			AddConnectedEdges(FrontendConfigNode.Interface.Inputs);
			AddConnectedEdges(FrontendConfigNode.Interface.Outputs);
		}

		// b. Removes literals, so they must be readded
		bool bLiteralsSet = true;
		for (const FMetasoundFrontendVertexLiteral& VertexLiteral : FrontendConfigNode.InputLiterals)
		{
			if (const FMetasoundFrontendVertex* Input = Builder.FindNodeInput(FrontendConfigNode.GetID(), VertexLiteral.VertexID))
			{
				bLiteralsSet &= AddNodeInputLiteralToTransactor(FrontendConfigNode, *Input, VertexLiteral.Value, Transactor);
			}
			else
			{
				bLiteralsSet = false;
			}
		}

		return bEdgesAdded && bLiteralsSet;
	});
}

void UMetaSoundSourceBuilder::OnNodeInputLiteralSet(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex) const
{
	using namespace Metasound::DynamicGraph;

	const FMetasoundFrontendNode& Node = GetConstTargetPageGraphChecked().Nodes[NodeIndex];
	const FMetasoundFrontendVertex& Input = Node.Interface.Inputs[VertexIndex];

	// Only send the literal down if not connected, as the graph core layer
	// will disconnect if a new literal is sent and edge already exists.
	if (!Builder.IsNodeInputConnected(Node.GetID(), Input.VertexID))
	{
		ExecuteAuditionableTransaction([this, &Node, &Input, &LiteralIndex](FDynamicOperatorTransactor& Transactor)
		{
			const FMetasoundFrontendLiteral& InputDefault = Node.InputLiterals[LiteralIndex].Value;
			return AddNodeInputLiteralToTransactor(Node, Input, InputDefault, Transactor);
		});
	}
}

void UMetaSoundSourceBuilder::OnOutputAdded(int32 OutputIndex) const
{
	using namespace Metasound::DynamicGraph;

	ExecuteAuditionableTransaction([this, OutputIndex](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound::Frontend;

		const FMetasoundFrontendDocument& Doc = Builder.GetConstDocumentChecked();
		const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;
		const FMetasoundFrontendClassOutput& NewOutput = GraphClass.GetDefaultInterface().Outputs[OutputIndex];

		Transactor.AddOutputDataSource(NewOutput.NodeID, NewOutput.Name);
		return true;
	});
}

void UMetaSoundSourceBuilder::OnPageAdded(const Metasound::Frontend::FDocumentMutatePageArgs& InArgs)
{
	using namespace Metasound::Frontend;

	FDocumentModifyDelegates& DocDelegates = Builder.GetDocumentDelegates();
	InitTargetPageDelegates(DocDelegates);
}

void UMetaSoundSourceBuilder::OnRemovingPage(const Metasound::Frontend::FDocumentMutatePageArgs& InArgs)
{
	using namespace Metasound::Frontend;

	FDocumentModifyDelegates& DocDelegates = Builder.GetDocumentDelegates();
	InitTargetPageDelegates(DocDelegates);
}

void UMetaSoundSourceBuilder::OnRemoveSwappingEdge(int32 SwapIndex, int32 LastIndex) const
{
	using namespace Metasound::DynamicGraph;

	const FMetasoundFrontendEdge& EdgeBeingRemoved = GetConstTargetPageGraphChecked().Edges[SwapIndex];
	ExecuteAuditionableTransaction([this, EdgeBeingRemoved](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound;
		using namespace Metasound::Engine;
		using namespace Metasound::Frontend;

		const FMetaSoundFrontendDocumentBuilder& DocBuilder = GetConstBuilder();
		const FMetasoundFrontendVertex* FromNodeOutput = DocBuilder.FindNodeOutput(EdgeBeingRemoved.FromNodeID, EdgeBeingRemoved.FromVertexID);
		const FMetasoundFrontendVertex* ToNodeInput = DocBuilder.FindNodeInput(EdgeBeingRemoved.ToNodeID, EdgeBeingRemoved.ToVertexID);
		if (FromNodeOutput && ToNodeInput)
		{
#if WITH_EDITOR
			const FMetasoundFrontendNode* ToNode = DocBuilder.FindNode(EdgeBeingRemoved.ToNodeID);
			const FMetasoundFrontendNode* FromNode = DocBuilder.FindNode(EdgeBeingRemoved.FromNodeID);
			if (ToNode && FromNode)
			{
				const FMetasoundFrontendClass* ToNodeClass = DocBuilder.FindDependency(ToNode->ClassID);
				const FMetasoundFrontendClass* FromNodeClass = DocBuilder.FindDependency(FromNode->ClassID);

				// Adding edge to a template node can be safely ignored as template nodes are processed out.
				// Once template node is connected as a source however, then the given edge can be processed accordingly.
				if (ToNodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
				{
					return true;
				}

				if (FromNodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
				{
					auto FindNodeInputDefault = [this](const FGuid& InToNodeID, const FName InputName)
					{
						return BuilderSubsystemPrivate::TryResolveNodeInputDefault(Builder, InToNodeID, InputName);
					};

					const FTemplateNodeEdgeRemoveTransactionContext TransactionContext
					{
						FTemplateNodeEdgeTransactionContext
						{
							.Builder = DocBuilder,
							.PageID = DocBuilder.GetBuildPageID(),
							.FromNode = FromNode,
							.FromOutput = FromNodeOutput,
							.ToNode = ToNode,
							.ToInput = ToNodeInput,
						},
						FindNodeInputDefault
					};

					const FMetasoundFrontendClassName& FromClassName = FromNodeClass->Metadata.GetClassName();
					if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FromClassName))
					{
						const bool bHandled = Template->OnRemoveSwappingEdge(TransactionContext, Transactor);
						if (bHandled)
						{
							return true;
						}
					}
				}
			}
#endif // WITH_EDITOR

			const FMetasoundFrontendLiteral* InputDefault = BuilderSubsystemPrivate::TryResolveNodeInputDefault(Builder, EdgeBeingRemoved.ToNodeID, ToNodeInput->Name);
			if (ensureAlwaysMsgf(InputDefault, TEXT("Could not dynamically assign default literal upon removing edge: literal should be assigned by either the frontend document's input or the class definition")))
			{
				Transactor.RemoveDataEdge(
					EdgeBeingRemoved.FromNodeID,
					FromNodeOutput->Name,
					EdgeBeingRemoved.ToNodeID,
					ToNodeInput->Name,
					InputDefault->ToLiteral(ToNodeInput->TypeName),
					&FMetaSoundFrontendDocumentBuilder::CreateDataReference);
				return true;
			}
		}

		return false;
	});
}

void UMetaSoundSourceBuilder::OnRemovingAssetReference(TScriptInterface<IMetaSoundDocumentInterface> DocInterface)
{
	using namespace Metasound::Frontend;

	check(DocInterface.GetObject());
	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.ReferencedAssetClassObjects.Remove(DocInterface.GetObject());

	const FNodeRegistryKey RegistryKey(DocInterface->GetConstDocument().RootGraph);
	Source.ReferencedAssetClassKeys.Remove(RegistryKey.ToString());
}

void UMetaSoundSourceBuilder::OnRemovingInput(int32 InputIndex)
{
	using namespace Metasound::DynamicGraph;

	const FMetasoundFrontendDocument& Doc = Builder.GetConstDocumentChecked();
	const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;
	const FMetasoundFrontendClassInput& InputBeingRemoved = GraphClass.GetDefaultInterface().Inputs[InputIndex];

	UMetaSoundSource& Source = GetMetaSoundSource();
	Source.RuntimeInputData.InputMap.Remove(InputBeingRemoved.Name);

	ExecuteAuditionableTransaction([this, InputIndex, &InputBeingRemoved](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		Transactor.RemoveInputDataDestination(InputBeingRemoved.Name);

		for (uint64 AudioComponentID : LiveComponentIDs)
		{
			if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
			{
				if (FAudioDevice* AudioDevice = AudioComponent->GetAudioDevice())
				{
					AudioDevice->SendCommandToActiveSounds(AudioComponentID, [InputRemoved = InputBeingRemoved.Name](FActiveSound& ActiveSound)
					{
						if (FMetaSoundParameterTransmitter* Transmitter = static_cast<FMetaSoundParameterTransmitter*>(ActiveSound.GetTransmitter()))
						{
							Transmitter->RemoveAvailableParameter(InputRemoved);
						}
					});
				}
			}
		}

		return true;
	});
}

void UMetaSoundSourceBuilder::OnRemoveSwappingNode(int32 SwapIndex, int32 LastIndex) const
{
	using namespace Metasound::DynamicGraph;

	// Last index will just be re-added, so this aspect of the swap is ignored by transactor
	// (i.e. no sense removing and re-adding the node that is swapped from the end as this
	// would potentially disconnect that node in the runtime graph model).
	ExecuteAuditionableTransaction([this, SwapIndex](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound::Frontend;

		const FMetasoundFrontendNode& NodeBeingRemoved = GetConstTargetPageGraphChecked().Nodes[SwapIndex];

#if WITH_EDITOR
		const FMetasoundFrontendClass* RemovingNodeClass = Builder.FindDependency(NodeBeingRemoved.ClassID);
		checkf(RemovingNodeClass, TEXT("Node to be removed from frontend graph is missing associated dependency"));
		if (RemovingNodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
		{
			const FMetasoundFrontendClassName& RemovingClassName = RemovingNodeClass->Metadata.GetClassName();
			if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(RemovingClassName))
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = GetConstBuilder();
				const FTemplateNodeTransactionContext TransactionContext { DocBuilder, DocBuilder.GetBuildPageID(), &NodeBeingRemoved };
				const bool bHandled = Template->OnRemoveSwappingNode(TransactionContext, Transactor);
				if (bHandled)
				{
					return true;
				}
			}
		}
#endif // WITH_EDITOR

		const FGuid& NodeID = NodeBeingRemoved.GetID();
		Transactor.RemoveNode(NodeID);
		return true;
	});
}

void UMetaSoundSourceBuilder::OnRemovingNodeInputLiteral(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex) const
{
	using namespace Metasound::DynamicGraph;

	const TArray<FMetasoundFrontendNode>& Nodes = GetConstTargetPageGraphChecked().Nodes;
	const FMetasoundFrontendNode& Node = Nodes[NodeIndex];
	const FMetasoundFrontendVertex& Input = Node.Interface.Inputs[VertexIndex];

	// Only send the literal down if not connected, as the graph core layer will disconnect.
	if (!Builder.IsNodeInputConnected(Node.GetID(), Input.VertexID))
	{
		ExecuteAuditionableTransaction([this, &Node, &Input, &NodeIndex, &VertexIndex, &LiteralIndex](FDynamicOperatorTransactor& Transactor)
		{
			using namespace Metasound;
			using namespace Metasound::Engine;
			using namespace Metasound::Frontend;

			const FMetasoundFrontendLiteral* InputDefault = BuilderSubsystemPrivate::TryResolveNodeInputDefault(Builder, Node.GetID(), Input.Name);
			if (ensureAlwaysMsgf(InputDefault,
				TEXT("Could not dynamically assign default literal from class definition upon removing input '%s' literal: "
					"document's dependency entry invalid and has no default assigned"),
					*Input.Name.ToString()))
			{
				Transactor.SetValue(Node.GetID(), Input.Name, InputDefault->ToLiteral(Input.TypeName), &FMetaSoundFrontendDocumentBuilder::CreateDataReference);
				return true;
			}

			return false;
		});
	}
}

void UMetaSoundSourceBuilder::OnRemovingOutput(int32 OutputIndex) const
{
	using namespace Metasound::DynamicGraph;

	ExecuteAuditionableTransaction([this, OutputIndex](FDynamicOperatorTransactor& Transactor)
	{
		using namespace Metasound::Frontend;

		const FMetasoundFrontendDocument& Doc = Builder.GetConstDocumentChecked();
		const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;
		const FMetasoundFrontendClassOutput& OutputBeingRemoved = GraphClass.GetDefaultInterface().Outputs[OutputIndex];

		Transactor.RemoveOutputDataSource(OutputBeingRemoved.Name);
		return true;
	});
}

void UMetaSoundSourceBuilder::SetBlockRateOverride(float BlockRate)
{
#if WITH_EDITORONLY_DATA
	GetMetaSoundSource().BlockRateOverride.Default = BlockRate;
#endif //WITH_EDITORONLY_DATA
}

void UMetaSoundSourceBuilder::SetFormat(EMetaSoundOutputAudioFormat OutputFormat, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	// Clear template since interface data is being altered
	Builder.SetTemplate({ });
	Builder.ConfigureDocument();

	const FOutputAudioFormatInfoMap& FormatMap = GetOutputAudioFormatInfo();

	// Determine which interfaces to add and remove from the document due to the
	// output format being changed.
	TArray<FMetasoundFrontendVersion> OutputFormatsToAdd;
	if (const FOutputAudioFormatInfo* FormatInfo = FormatMap.Find(OutputFormat))
	{
		OutputFormatsToAdd.Add(FormatInfo->InterfaceVersion);
	}

	TArray<FMetasoundFrontendVersion> OutputFormatsToRemove;

	const FMetasoundFrontendDocument& Document = GetConstBuilder().GetConstDocumentChecked();
	for (const FOutputAudioFormatInfoPair& Pair : FormatMap)
	{
		const FMetasoundFrontendVersion& FormatVersion = Pair.Value.InterfaceVersion;
		if (Document.Interfaces.Contains(FormatVersion))
		{
			if (!OutputFormatsToAdd.Contains(FormatVersion))
			{
				OutputFormatsToRemove.Add(FormatVersion);
			}
		}
	}

	FModifyInterfaceOptions Options(OutputFormatsToRemove, OutputFormatsToAdd);

#if WITH_EDITORONLY_DATA
	Options.bSetDefaultNodeLocations = true;
#endif // WITH_EDITORONLY_DATA

	const bool bSuccess = Builder.ModifyInterfaces(MoveTemp(Options));
	OutResult = bSuccess ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
}

#if WITH_EDITORONLY_DATA
void UMetaSoundSourceBuilder::SetPlatformBlockRateOverride(const FPerPlatformFloat& PlatformBlockRate)
{
	GetMetaSoundSource().BlockRateOverride = PlatformBlockRate;
}

void UMetaSoundSourceBuilder::SetPlatformSampleRateOverride(const FPerPlatformInt& PlatformSampleRate)
{
	GetMetaSoundSource().SampleRateOverride = PlatformSampleRate;
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundSourceBuilder::SetQuality(FName Quality)
{
#if WITH_EDITORONLY_DATA
	GetMetaSoundSource().QualitySetting = Quality;
#endif //WITH_EDITORONLY_DATA	
}

void UMetaSoundSourceBuilder::SetSampleRateOverride(int32 SampleRate)
{
#if WITH_EDITORONLY_DATA
	GetMetaSoundSource().SampleRateOverride.Default = SampleRate;
#endif //WITH_EDITORONLY_DATA	
}

UMetaSoundPatchBuilder* UMetaSoundBuilderSubsystem::CreatePatchBuilder(FName BuilderName, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Engine;

	OutResult = EMetaSoundBuilderResult::Succeeded;
	UMetaSoundPatchBuilder& NewBuilder = FDocumentBuilderRegistry::GetChecked().CreateTransientBuilder<UMetaSoundPatchBuilder>(BuilderName);
	return &NewBuilder;
}

UMetaSoundSourceBuilder* UMetaSoundBuilderSubsystem::CreateSourceBuilder(
	FName BuilderName,
	FMetaSoundBuilderNodeOutputHandle& OnPlayNodeOutput,
	FMetaSoundBuilderNodeInputHandle& OnFinishedNodeInput,
	TArray<FMetaSoundBuilderNodeInputHandle>& AudioOutNodeInputs,
	EMetaSoundBuilderResult& OutResult,
	EMetaSoundOutputAudioFormat OutputFormat,
	bool bIsOneShot)
{
	using namespace Metasound::Engine;
	using namespace Metasound::Engine::BuilderSubsystemPrivate;
	using namespace Metasound::Frontend;

	OnPlayNodeOutput = { };
	OnFinishedNodeInput = { };
	AudioOutNodeInputs.Reset();

	UMetaSoundSourceBuilder& NewBuilder = FDocumentBuilderRegistry::GetChecked().CreateTransientBuilder<UMetaSoundSourceBuilder>(BuilderName);
	OutResult = EMetaSoundBuilderResult::Succeeded;
	if (OutputFormat != EMetaSoundOutputAudioFormat::Mono)
	{
		NewBuilder.SetFormat(OutputFormat, OutResult);
	}
	
	if (OutResult == EMetaSoundBuilderResult::Succeeded)
	{
		TArray<FMetaSoundNodeHandle> AudioOutputNodes;
		if (const Metasound::Engine::FOutputAudioFormatInfoPair* FormatInfo = NewBuilder.FindOutputAudioFormatInfo())
		{
			AudioOutputNodes = NewBuilder.FindInterfaceOutputNodes(FormatInfo->Value.InterfaceVersion.Name, OutResult);
		}
		else
		{
			OutResult = EMetaSoundBuilderResult::Failed;
		}

		if (OutResult == EMetaSoundBuilderResult::Succeeded)
		{
			Algo::Transform(AudioOutputNodes, AudioOutNodeInputs, [&NewBuilder, &BuilderName](const FMetaSoundNodeHandle& AudioOutputNode) -> FMetaSoundBuilderNodeInputHandle
			{
				EMetaSoundBuilderResult Result;
				TArray<FMetaSoundBuilderNodeInputHandle> Inputs = NewBuilder.FindNodeInputs(AudioOutputNode, Result);
				if (!Inputs.IsEmpty())
				{
					return Inputs.Last();
				}

				UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to find expected audio output node input vertex. Returned vertices set may be incomplete.", *BuilderName.ToString());
				return { };
			});
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to find expected audio output format and/or associated output nodes.", *BuilderName.ToString());
			return nullptr;
		}
	}
	else
	{
		UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to set output format when initializing.", *BuilderName.ToString());
		return nullptr;
	}

	{
		FMetaSoundNodeHandle OnPlayNode = NewBuilder.FindGraphInputNode(SourceInterface::Inputs::OnPlay, OutResult);
		if (OutResult == EMetaSoundBuilderResult::Failed)
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to add required interface '%ls' when attempting to create MetaSound Source Builder", *BuilderName.ToString(), * SourceInterface::GetVersion().ToString());
			return nullptr;
		}

		TArray<FMetaSoundBuilderNodeOutputHandle> Outputs = NewBuilder.FindNodeOutputs(OnPlayNode, OutResult);
		if (OutResult == EMetaSoundBuilderResult::Failed)
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to find output vertex for 'OnPlay' input node when attempting to create MetaSound Source Builder", *BuilderName.ToString());
			return nullptr;
		}

		check(!Outputs.IsEmpty());
		OnPlayNodeOutput = Outputs.Last();
	}

	if (bIsOneShot)
	{
		FMetaSoundNodeHandle OnFinishedNode = NewBuilder.FindGraphOutputNode(SourceOneShotInterface::Outputs::OnFinished, OutResult);
		if (OutResult == EMetaSoundBuilderResult::Failed)
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to add '%ls' interface; interface definition may not be registered.", *BuilderName.ToString(), *SourceOneShotInterface::GetVersion().ToString());
		}

		TArray<FMetaSoundBuilderNodeInputHandle> Inputs = NewBuilder.FindNodeInputs(OnFinishedNode, OutResult);
		if (OutResult == EMetaSoundBuilderResult::Failed)
		{
			UE_LOGF(LogMetaSound, Error, "Builder '%ls' Creation Error: Failed to find input vertex for 'OnFinished' output node when attempting to create MetaSound Source Builder", *BuilderName.ToString());
			return nullptr;
		}

		check(!Inputs.IsEmpty());
		OnFinishedNodeInput = Inputs.Last();
	}
	else
	{
		NewBuilder.RemoveInterface(SourceOneShotInterface::GetVersion().Name, OutResult);
	}

	return &NewBuilder;
}

UMetaSoundPatchBuilder* UMetaSoundBuilderSubsystem::CreatePatchPresetBuilder(FName BuilderName, const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedNodeClass, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Engine;

	if (ReferencedNodeClass)
	{
		UMetaSoundPatchBuilder& Builder = FDocumentBuilderRegistry::GetChecked().CreateTransientBuilder<UMetaSoundPatchBuilder>(BuilderName);
		Builder.ConvertToPreset(ReferencedNodeClass, OutResult);
		return &Builder;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return nullptr;
}

UMetaSoundBuilderBase& UMetaSoundBuilderSubsystem::CreatePresetBuilder(FName BuilderName, const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedMetaSound, EMetaSoundBuilderResult& OutResult)
{
	check(ReferencedMetaSound.GetObject());

	const UClass& Class = ReferencedMetaSound->GetBaseMetaSoundUClass();
	if (&Class == UMetaSoundSource::StaticClass())
	{
		UMetaSoundSource* Source = CastChecked<UMetaSoundSource>(ReferencedMetaSound.GetObject());
		return *CreateSourcePresetBuilder(BuilderName, Source, OutResult);
	}
	else if (&Class == UMetaSoundPatch::StaticClass())
	{
		UMetaSoundPatch* Patch = CastChecked<UMetaSoundPatch>(ReferencedMetaSound.GetObject());
		return *CreatePatchPresetBuilder(BuilderName, Patch, OutResult);
	}
	else
	{
		checkf(false, TEXT("UClass '%s' cannot be built to a MetaSound preset"), *Class.GetFullName());
		return *NewObject<UMetaSoundPatchBuilder>();
	}
}

UMetaSoundSourceBuilder* UMetaSoundBuilderSubsystem::CreateSourcePresetBuilder(FName BuilderName, const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedNodeClass, EMetaSoundBuilderResult& OutResult)
{
	using namespace Metasound::Engine;

	if (ReferencedNodeClass)
	{
		UMetaSoundSourceBuilder& Builder = FDocumentBuilderRegistry::GetChecked().CreateTransientBuilder<UMetaSoundSourceBuilder>();
		Builder.ConvertToPreset(ReferencedNodeClass, OutResult);
		return &Builder;
	}

	OutResult = EMetaSoundBuilderResult::Failed;
	return nullptr;
}

UMetaSoundBuilderSubsystem* UMetaSoundBuilderSubsystem::Get()
{
	if (GEngine)
	{
		if (UMetaSoundBuilderSubsystem* BuilderSubsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>())
		{
			return BuilderSubsystem;
		}
	}

	return nullptr;
}

UMetaSoundBuilderSubsystem& UMetaSoundBuilderSubsystem::GetChecked()
{
	checkf(GEngine, TEXT("Cannot access UMetaSoundBuilderSubsystem without engine loaded"));
	UMetaSoundBuilderSubsystem* BuilderSubsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>();
	checkf(BuilderSubsystem, TEXT("Failed to find initialized 'UMetaSoundBuilderSubsystem"));
	return *BuilderSubsystem;
}

const UMetaSoundBuilderSubsystem* UMetaSoundBuilderSubsystem::GetConst()
{
	if (GEngine)
	{
		if (const UMetaSoundBuilderSubsystem* BuilderSubsystem = GEngine->GetEngineSubsystem<const UMetaSoundBuilderSubsystem>())
		{
			return BuilderSubsystem;
		}
	}

	return nullptr;
}

const UMetaSoundBuilderSubsystem& UMetaSoundBuilderSubsystem::GetConstChecked()
{
	checkf(GEngine, TEXT("Cannot access UMetaSoundBuilderSubsystem without engine loaded"));
	UMetaSoundBuilderSubsystem* BuilderSubsystem = GEngine->GetEngineSubsystem<UMetaSoundBuilderSubsystem>();
	checkf(BuilderSubsystem, TEXT("Failed to find initialized 'UMetaSoundBuilderSubsystem"));
	return *BuilderSubsystem;
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateBoolMetaSoundLiteral(bool Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateBoolArrayMetaSoundLiteral(const TArray<bool>& Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateFloatMetaSoundLiteral(float Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateFloatArrayMetaSoundLiteral(const TArray<float>& Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateIntMetaSoundLiteral(int32 Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateIntArrayMetaSoundLiteral(const TArray<int32>& Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateStringMetaSoundLiteral(const FString& Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateStringArrayMetaSoundLiteral(const TArray<FString>& Value, FName& OutDataType)
{
	return Metasound::Engine::BuilderSubsystemPrivate::CreatePODMetaSoundLiteral(Value, OutDataType);
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateObjectMetaSoundLiteral(UObject* Value)
{
	FMetasoundFrontendLiteral Literal;
	Literal.Set(Value);
	return Literal;
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateObjectArrayMetaSoundLiteral(const TArray<UObject*>& Value)
{
	FMetasoundFrontendLiteral Literal;
	Literal.Set(Value);
	return Literal;
}

FMetasoundFrontendLiteral UMetaSoundBuilderSubsystem::CreateMetaSoundLiteralFromParam(const FAudioParameter& Param)
{
	return FMetasoundFrontendLiteral { Param };
}

UMetaSoundBuilderBase* UMetaSoundBuilderSubsystem::FindBuilder(FName BuilderName)
{
	return NamedBuilders.FindRef(BuilderName);
}

UMetaSoundBuilderBase* UMetaSoundBuilderSubsystem::FindBuilderOfDocument(TScriptInterface<const IMetaSoundDocumentInterface> InMetaSound) const
{
	using namespace Metasound::Engine;

	return FDocumentBuilderRegistry::GetChecked().FindBuilderObject(InMetaSound);
}

UMetaSoundBuilderBase* UMetaSoundBuilderSubsystem::FindParentBuilderOfPreset(const TScriptInterface<const IMetaSoundDocumentInterface> InMetaSoundPreset, const bool bFollowPresetChain)
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	if (!InMetaSoundPreset.GetObject() || !InMetaSoundPreset.GetObject()->Implements<UMetaSoundDocumentInterface>())
	{
		UE_LOGF(LogMetaSound, Error, "FindParentBuilderOfPreset called with object that is not a MetaSound asset");
		return nullptr;
	}

	// Find the builder for the given asset to check if it is a preset.
	const FDocumentBuilderRegistry& Registry = FDocumentBuilderRegistry::GetChecked();
	const UMetaSoundBuilderBase* Builder = Registry.FindBuilderObject(InMetaSoundPreset);
	if (!Builder)
	{
		UE_LOGF(LogMetaSound, Error, "Could not find builder for MetaSound '%ls'.", *GetNameSafe(InMetaSoundPreset.GetObject()));
		return nullptr;
	}

	const FMetaSoundFrontendPresetTemplate* PresetTemplate = Builder->GetConstBuilder().GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>();
	if (!PresetTemplate)
	{
		UE_LOGF(LogMetaSound, Error, "MetaSound '%ls' is not a preset.", *GetNameSafe(InMetaSoundPreset.GetObject()));
		return nullptr;
	}

	TScriptInterface<IMetaSoundDocumentInterface> ParentAsset = PresetTemplate->Parent;
	if (!ParentAsset)
	{
		UE_LOGF(LogMetaSound, Error, "MetaSound preset '%ls' contains null reference.", *GetNameSafe(InMetaSoundPreset.GetObject()));
		return nullptr;
	}

	UMetaSoundBuilderBase* ParentBuilder = Registry.FindBuilderObject(ParentAsset);
	if (!bFollowPresetChain)
	{
		return ParentBuilder;
	}

	// Follow preset chain to the highest non-preset ancestor
	TSet<const UObject*> CheckedAssets;
	bool bIsCircular = false;
	while (ParentBuilder)
	{
		// Check for circular references
		CheckedAssets.Add(ParentAsset.GetObject(), &bIsCircular);
		if (bIsCircular)
		{
			UE_LOGF(LogMetaSound, Error, "MetaSound preset '%ls' contains circular reference.", *GetNameSafe(ParentAsset.GetObject()));
			return nullptr;
		}

		PresetTemplate = ParentBuilder->GetConstBuilder().GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>();
		if (!PresetTemplate)
		{
			break;
		}

		ParentAsset = PresetTemplate->Parent;
		if (!ParentAsset)
		{
			UE_LOGF(LogMetaSound, Warning, "MetaSound preset '%ls' contains null reference.", *GetNameSafe(ParentAsset.GetObject()));
			return nullptr;
		}

		ParentBuilder = Registry.FindBuilderObject(ParentAsset.GetObject());
	}

	return ParentBuilder;
}

UMetaSoundPatchBuilder* UMetaSoundBuilderSubsystem::FindPatchBuilder(FName BuilderName)
{
	if (UMetaSoundBuilderBase* Builder = FindBuilder(BuilderName))
	{
		return Cast<UMetaSoundPatchBuilder>(Builder);
	}

	return nullptr;
}

UMetaSoundSourceBuilder* UMetaSoundBuilderSubsystem::FindSourceBuilder(FName BuilderName)
{
	if (UMetaSoundBuilderBase* Builder = FindBuilder(BuilderName))
	{
		return Cast<UMetaSoundSourceBuilder>(Builder);
	}

	return nullptr;
}

bool UMetaSoundBuilderSubsystem::IsInterfaceRegistered(FName InInterfaceName) const
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendVersion InterfaceVersion;
	return ISearchEngine::Get().FindHighestInterfaceVersion(InInterfaceName, InterfaceVersion);
}

void UMetaSoundBuilderSubsystem::RegisterBuilder(FName BuilderName, UMetaSoundBuilderBase* Builder)
{
	if (Builder)
	{
		NamedBuilders.FindOrAdd(BuilderName) = Builder;
	}
}

void UMetaSoundBuilderSubsystem::RegisterPatchBuilder(FName BuilderName, UMetaSoundPatchBuilder* Builder)
{
	if (Builder)
	{
		NamedBuilders.FindOrAdd(BuilderName) = Builder;
	}
}

void UMetaSoundBuilderSubsystem::RegisterSourceBuilder(FName BuilderName, UMetaSoundSourceBuilder* Builder)
{
	if (Builder)
	{
		NamedBuilders.FindOrAdd(BuilderName) = Builder;
	}
}

bool UMetaSoundBuilderSubsystem::SetTargetPage(FName PageName)
{
	using namespace Metasound::Frontend;

	if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
	{
		const bool bTargetChanged = Settings->SetTargetPage(PageName);
		if (bTargetChanged)
		{
			IMetaSoundAssetManager::GetChecked().ReregisterLoadedAssets();
		}
		return bTargetChanged;
	}

	return false;
}

bool UMetaSoundBuilderSubsystem::UnregisterBuilder(FName BuilderName)
{
	return NamedBuilders.Remove(BuilderName) > 0;
}

bool UMetaSoundBuilderSubsystem::UnregisterPatchBuilder(FName BuilderName)
{
	return NamedBuilders.Remove(BuilderName) > 0;
}

bool UMetaSoundBuilderSubsystem::UnregisterSourceBuilder(FName BuilderName)
{
	return NamedBuilders.Remove(BuilderName) > 0;
}
