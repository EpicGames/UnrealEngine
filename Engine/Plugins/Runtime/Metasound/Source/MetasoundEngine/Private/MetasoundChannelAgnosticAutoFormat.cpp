// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundChannelAgnosticAutoFormat.h"

#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"

#include "MetasoundLog.h"
#include "Sound/ISoundWaveContainer.h"

#include "Sound/SoundWave.h"

namespace Metasound
{
	int32 FChannelAgnosticAutoFormatHelper::ComputeAutoChannelCountFromProxies(const TArray<FSoundWaveProxyPtr>& InProxies)
	{
		// Greatest for now.
		if (const TSharedPtr<FSoundWaveProxy>* Greatest = Algo::MaxElementBy(InProxies, [](const TSharedPtr<FSoundWaveProxy>& j) -> int32
			{
				return j->GetSoundWaveDataRef()->GetNumChannels();
			}))
		{
			return (*Greatest)->GetSoundWaveDataRef()->GetNumChannels();
		}
		
		// Fail.
		return INDEX_NONE;
	}

	TSharedPtr<const Audio::FChannelTypeFamily> FChannelAgnosticAutoFormatHelper::ComputeAutoFormatFromWaveContainer(
		const TSharedPtr<const Audio::IProxyData>& InData)
	{
		using namespace Audio; 
		if (const ISoundWaveContainer* WaveContainer = Audio::QueryInterface<ISoundWaveContainer>(InData.Get()))
		{
			if (const int32 ChannelCount = ComputeAutoChannelCountFromProxies(WaveContainer->GetContainedWaveProxies()); ChannelCount != INDEX_NONE)
			{
				if (const TSharedPtr<const Audio::FDiscreteChannelTypeFamily> Format = NumChannelsToFormat(ChannelCount); Format.IsValid())
				{
					return Format;
				}
			}
		}

		// Fail.
		return {};
	}

	FName FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
		const EMetasoundChannelAgnosticNodeFormatChooser InMethod,
		const FName InCustomFormat,
		const TSharedPtr<const Audio::IProxyData>& InContent,
		const FMetasoundEnvironment& InEnvironment)
	{
		switch (InMethod)
		{
			case EMetasoundChannelAgnosticNodeFormatChooser::Custom:
			{
				return InCustomFormat;
			}
			case EMetasoundChannelAgnosticNodeFormatChooser::Source:
			{
				static const FName SourceFormatName = TEXT("SourceFormatName");
				if (InEnvironment.Contains<FName>(SourceFormatName))
				{
					return InEnvironment.GetValue<FName>(SourceFormatName);
				}
			}
			// Fall-through.
			case EMetasoundChannelAgnosticNodeFormatChooser::Auto:
			{
				if (const TSharedPtr<const Audio::FChannelTypeFamily> Format = ComputeAutoFormatFromWaveContainer(InContent); Format.IsValid())
				{
					return Format->GetName();
				}
				break;
			}
		}

		// Fail, but return default.
		return FDiscreteChannelAgnosticType::GetDefaultCatFormat();
	}
		
	TSharedPtr<const Audio::FDiscreteChannelTypeFamily>
	FChannelAgnosticAutoFormatHelper::NumChannelsToFormat(const int32 NumChannels)
	{
		return Audio::FChannelAgnosticUtils::FindDiscreteFormatFromNumChannels(NumChannels);
	}

	
}//namespace Metasound