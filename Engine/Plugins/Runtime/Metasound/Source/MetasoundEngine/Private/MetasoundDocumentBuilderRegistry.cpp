// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDocumentBuilderRegistry.h"

#include "HAL/PlatformProperties.h"
#include "Logging/StructuredLog.h"
#include "MetasoundAssetManager.h"
#include "MetasoundEngineAsset.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundGlobals.h"
#include "MetasoundSettings.h"
#include "MetasoundTrace.h"
#include "MetasoundUObjectRegistry.h"


namespace Metasound::Engine
{
	FDocumentBuilderRegistry::~FDocumentBuilderRegistry()
	{
		TMultiMap<FMetasoundFrontendClassName, TWeakObjectPtr<UMetaSoundBuilderBase>> BuildersToFinish;
		FScopeLock Lock(&BuildersCriticalSection);
		{
			BuildersToFinish = MoveTemp(Builders);
			Builders.Reset();
		}

		UE_CLOGF(!BuildersToFinish.IsEmpty(), LogMetaSound, Display, "BuilderRegistry is shutting down with the following %i active builder entries. Forcefully shutting down:", BuildersToFinish.Num());
		int32 NumStale = 0;
		for (const TPair<FMetasoundFrontendClassName, TWeakObjectPtr<UMetaSoundBuilderBase>>& Pair : BuildersToFinish)
		{
			const bool bIsValid = Pair.Value.IsValid();
			if (bIsValid)
			{
				UE_CLOGF(bIsValid, LogMetaSound, Display, "- %ls", *Pair.Value->GetFullName());
				constexpr bool bForceUnregister = true;
				FinishBuildingInternal(*Pair.Value.Get(), bForceUnregister);
			}
			else
			{
				++NumStale;
			}
		}
		UE_CLOGF(NumStale > 0, LogMetaSound, Display, "BuilderRegistry is shutting down with %i stale entries", NumStale);
	}

#if WITH_EDITORONLY_DATA
	void FDocumentBuilderRegistry::WaitUntilVersioningComplete(UObject& InMetaSoundObject) const
	{
		using namespace Frontend;
		FVersioningManager::Get().WaitUntilVersioningComplete(InMetaSoundObject);
	}
#endif // WITH_EDITORONLY_DATA

	void FDocumentBuilderRegistry::AddBuilderInternal(const FMetasoundFrontendClassName& InClassName, UMetaSoundBuilderBase* NewBuilder) const
	{
		FScopeLock Lock(&BuildersCriticalSection);

// #if !NO_LOGGING
//		bool bLogDuplicateEntries = CanPostEventLog(ELogEvent::DuplicateEntries, ELogVerbosity::Error);
//		if (bLogDuplicateEntries)
//		{
//			bLogDuplicateEntries = Builders.Contains(InClassName);
//		}
// #endif // !NO_LOGGING

		Builders.Add(InClassName, NewBuilder);

// #if !NO_LOGGING
// 		if (bLogDuplicateEntries)
// 		{
// 			TArray<TWeakObjectPtr<UMetaSoundBuilderBase>> Entries;
// 			Builders.MultiFind(InClassName, Entries);
// 
// 			// Don't print stale entries as during cook and some editor asset actions,
// 			// these may be removed after a new valid builder is created.  If stale
// 			// entries leak, they will show up on registry logging upon destruction.
// 			Entries.RemoveAllSwap([](const TWeakObjectPtr<UMetaSoundBuilderBase>& Builder) { return !Builder.IsValid(); });
// 
// 			if (!Entries.IsEmpty())
// 			{
// 				UE_LOG(LogMetaSound, Error, TEXT("More than one asset registered with class name '%s'. "
// 					"Look-up may return builder that is not associated with desired object! \n"
// 					"This can happen if asset was moved using revision control and original location was revived. \n"
// 					"Remove all but one of the following assets and relink a duplicate or copied replacement asset:"),
// 					*InClassName.ToString());
// 				for (const TWeakObjectPtr<UMetaSoundBuilderBase>& BuilderPtr : Entries)
// 				{
// 					UE_LOGF(LogMetaSound, Error, "- %ls", *BuilderPtr->GetConstBuilder().CastDocumentObjectChecked<UObject>().GetPathName());
// 				}
// 			}
// 		}
// #endif // !NO_LOGGING
	}

	bool FDocumentBuilderRegistry::CanPostEventLog(ELogEvent Event, ELogVerbosity::Type Verbosity) const
	{
#if NO_LOGGING
		return false;
#else // !NO_LOGGING

		if (const ELogVerbosity::Type* SetVerbosity = EventLogVerbosity.Find(Event))
		{
			return *SetVerbosity >= Verbosity;
		}

		return true;
#endif // !NO_LOGGING
	}

#if WITH_EDITORONLY_DATA
	bool FDocumentBuilderRegistry::CookPages(FName PlatformName, FMetaSoundFrontendDocumentBuilder& Builder) const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FDocumentBuilderRegistry::CookPages);

		bool bModified = false;

		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		check(Settings);
		const TArray<FGuid> PlatformTargetPageIDs = Settings->GetCookedTargetPageIDs(PlatformName);
		checkf(!PlatformTargetPageIDs.IsEmpty(), TEXT("Must have at least one targeted page ID to cook MetaSound."));

		const FString DebugName = Builder.GetDebugName();

		auto StripPageEntries = [&](
			TArray<FGuid>& PageIdsToResolve,
			TSet<FGuid>& ResolveTargetScratch,
			TFunctionRef<bool(const FGuid&)> RemovePageItem,
			const FName& ItemName,
			const FString& ItemType)
		{
			ResolveTargetScratch.Reset();
			for (const FGuid& TargetPage : PlatformTargetPageIDs)
			{
				const FGuid PageID = ResolveTargetPageIDInternal(*Settings, PageIdsToResolve, TargetPage, PlatformName);
				ResolveTargetScratch.Add(PageID);
			}

			auto IsResolvedTarget = [&ResolveTargetScratch](const FGuid& PageID) { return ResolveTargetScratch.Contains(PageID); };
			checkf(!ResolveTargetScratch.IsEmpty(), TEXT("Failed to resolve any valid target IDs, which will leave serialized page array in invalid state."));
			PageIdsToResolve.RemoveAllSwap(IsResolvedTarget, EAllowShrinking::No);

			for (const FGuid& PageID : PageIdsToResolve)
			{
				const bool bRemovedPageItem = RemovePageItem(PageID);
				if (bRemovedPageItem)
				{
					UE_LOGF(LogMetaSound, Display, "%ls: Removed %ls %ls w/PageID '%ls'",
						*DebugName,
						ItemName.IsNone() ? TEXT("paged") : *ItemName.ToString(),
						*ItemType,
						*PageID.ToString());
				}
				bModified |= bRemovedPageItem;
			}
		};

		const FMetasoundFrontendDocument& Document = Builder.GetConstDocumentChecked();

		TArray<FGuid> ResolvePageIDs;
		TSet<FGuid> ResolvedTargetScratchIDs;
		{ // Strip graphs
			auto AddPageID = [&ResolvePageIDs](const FMetasoundFrontendGraph& Graph) { ResolvePageIDs.Add(Graph.PageID); };
			Document.RootGraph.IterateGraphPages(AddPageID);

			auto RemoveGraphPage = [&Builder](const FGuid& InPageID)
			{
				return Builder.RemoveGraphPage(InPageID);
			};

			const int32 NumInitGraphs = Document.RootGraph.GetConstGraphPages().Num();
			StripPageEntries(ResolvePageIDs, ResolvedTargetScratchIDs, RemoveGraphPage, FName(), TEXT("graph"));

			const int32 NumRemainingGraphs = Document.RootGraph.GetConstGraphPages().Num();

			UE_CLOGFMT(NumRemainingGraphs <= 0, LogMetaSound, Warning,
				"Document in MetaSound asset '{AssetPath}' had all default values "
					"cooked away leaving it in an invalid state. "
					"Graph must always have at least one implementation.",
				Builder.GetDebugName());

			if (NumInitGraphs > NumRemainingGraphs)
			{
				UE_LOGF(LogMetaSound, Display, "Cook removed %i graph page(s) from '%ls'", NumInitGraphs - NumRemainingGraphs, *Builder.GetDebugName());
			}
		}

		{ // Strip default input values
			for (const FMetasoundFrontendClassInput& GraphInput : Document.RootGraph.GetDefaultInterface().Inputs)
			{
				ResolvePageIDs.Reset();
				auto AddPageID = [&ResolvePageIDs](const FGuid& PageID, const FMetasoundFrontendLiteral&) { ResolvePageIDs.Add(PageID); };
				GraphInput.IterateDefaults(AddPageID);

				auto RemoveDefault = [&Builder, &GraphInput](const FGuid& InPageID)
				{
					return Builder.StripGraphInputDefault(GraphInput.Name, InPageID);
				};

				const int32 NumInitDefaults = GraphInput.GetDefaults().Num();
				StripPageEntries(ResolvePageIDs, ResolvedTargetScratchIDs, RemoveDefault, GraphInput.Name, TEXT("input default"));

				const int32 NumRemainingDefaults = GraphInput.GetDefaults().Num();

				checkf(NumRemainingDefaults > 0,
					TEXT("Input '%s' had all default values stripped leaving it in an invalid state. "
					"Input must always have at least one default value"),
					*GraphInput.Name.ToString());

				if (NumInitDefaults > NumRemainingDefaults)
				{
					UE_LOGF(LogMetaSound, Display, "Cook removed %i default input page value(s) from input '%ls'", NumInitDefaults - NumRemainingDefaults, *GraphInput.Name.ToString());
				}
			}
		}

		return bModified;
	}

	FMetaSoundFrontendDocumentBuilder& FDocumentBuilderRegistry::FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface> MetaSound)
	{
		UObject* Object = MetaSound.GetObject();
		check(Object);

		return FindOrBeginBuilding(*Object).GetBuilder();
	}
#endif // WITH_EDITORONLY_DATA

	FMetaSoundFrontendDocumentBuilder* FDocumentBuilderRegistry::FindBuilder(TScriptInterface<IMetaSoundDocumentInterface> MetaSound) const
	{
		if (UMetaSoundBuilderBase* Builder = FindBuilderObject(MetaSound))
		{
			return &Builder->GetBuilder();
		}

		return nullptr;
	}

	FMetaSoundFrontendDocumentBuilder* FDocumentBuilderRegistry::FindBuilder(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath) const
	{
		if (UMetaSoundBuilderBase* Builder = FindBuilderObject(InClassName, AssetPath))
		{
			return &Builder->GetBuilder();
		}

		return nullptr;
	}

	TArray<FMetaSoundFrontendDocumentBuilder*> FDocumentBuilderRegistry::FindBuilders(const FMetasoundFrontendClassName& ClassName) const
	{
		TArray<TWeakObjectPtr<UMetaSoundBuilderBase>> Entries;

		{
			FScopeLock Lock(&BuildersCriticalSection);
			Builders.MultiFind(ClassName, Entries);
		}

		TArray<FMetaSoundFrontendDocumentBuilder*> ToReturn;
		Algo::TransformIf(Entries, ToReturn,
			[](const TWeakObjectPtr<UMetaSoundBuilderBase>& WeakBuilder)
			{
				return WeakBuilder.IsValid();
			},
			[](const TWeakObjectPtr<UMetaSoundBuilderBase>& WeakBuilder)
			{
				return &WeakBuilder.Get()->GetBuilder();
			});

		return ToReturn;
	}

	UMetaSoundBuilderBase* FDocumentBuilderRegistry::FindBuilderObject(TScriptInterface<const IMetaSoundDocumentInterface> MetaSound) const
	{
		UMetaSoundBuilderBase* FoundEntry = nullptr;
		if (const UObject* MetaSoundObject = MetaSound.GetObject())
		{
			const FMetasoundFrontendDocument& Document = MetaSound->GetConstDocument();
			const FMetasoundFrontendClassName& ClassName = Document.RootGraph.Metadata.GetClassName();
			TArray<TWeakObjectPtr<UMetaSoundBuilderBase>> Entries;

			{
				FScopeLock Lock(&BuildersCriticalSection);
				Builders.MultiFind(ClassName, Entries);
			}

			for (const TWeakObjectPtr<UMetaSoundBuilderBase>& BuilderPtr : Entries)
			{
				if (UMetaSoundBuilderBase* Builder = BuilderPtr.Get())
				{
					// Can be invalid if look-up is called during asset removal/destruction or the entry was
					// prematurely "finished". Only return invalid entry if builder asset path cannot be
					// matched as this is likely the destroyed entry associated with the provided AssetPath.
					const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();
					if (DocBuilder.IsValid())
					{
						UObject& TestMetaSound = BuilderPtr->GetConstBuilder().CastDocumentObjectChecked<UObject>();
						if (&TestMetaSound == MetaSoundObject)
						{
							FoundEntry = Builder;
							break;
						}
					}
					else
					{
						FoundEntry = Builder;
					}
				}
			}
		}

		return FoundEntry;
	}

	UMetaSoundBuilderBase* FDocumentBuilderRegistry::FindBuilderObject(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath) const
	{
		TArray<TWeakObjectPtr<UMetaSoundBuilderBase>> Entries;
		{
			FScopeLock Lock(&BuildersCriticalSection);
			Builders.MultiFind(InClassName, Entries);
		}

		UMetaSoundBuilderBase* FoundEntry = nullptr;
		for (const TWeakObjectPtr<UMetaSoundBuilderBase>& BuilderPtr : Entries)
		{
			if (UMetaSoundBuilderBase* Builder = BuilderPtr.Get())
			{
				const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder->GetConstBuilder();

				// Can be invalid if look-up is called during asset removal/destruction or the entry was
				// prematurely "finished". Only return invalid entry if builder asset path cannot be
				// matched as this is likely the destroyed entry associated with the provided AssetPath.
				if (DocBuilder.IsValid())
				{
					const UObject& DocObject = DocBuilder.CastDocumentObjectChecked<UObject>();
					FTopLevelAssetPath ObjectPath;
					if (ObjectPath.TrySetPath(&DocObject))
					{
						if (AssetPath.IsNull() || AssetPath == ObjectPath)
						{
							FoundEntry = Builder;
							break;
						}
					}
					else
					{
						FoundEntry = Builder;
					}
				}
				else
				{
					FoundEntry = Builder;
				}
			}
		}

		return FoundEntry;
	}

	TArray<UMetaSoundBuilderBase*> FDocumentBuilderRegistry::FindBuilderObjects(const FMetasoundFrontendClassName& InClassName) const
	{
		TArray<UMetaSoundBuilderBase*> FoundBuilders;
		TArray<TWeakObjectPtr<UMetaSoundBuilderBase>> Entries;

		{
			FScopeLock Lock(&BuildersCriticalSection);
			Builders.MultiFind(InClassName, Entries);
		}

		if (!Entries.IsEmpty())
		{
			Algo::TransformIf(Entries, FoundBuilders,
				[](const TWeakObjectPtr<UMetaSoundBuilderBase>& BuilderPtr) { return BuilderPtr.IsValid(); },
				[](const TWeakObjectPtr<UMetaSoundBuilderBase>& BuilderPtr) { return BuilderPtr.Get(); }
			);
		}

		return FoundBuilders;
	}

	FMetaSoundFrontendDocumentBuilder* FDocumentBuilderRegistry::FindOutermostBuilder(const UObject& InSubObject) const
	{
		using namespace Metasound::Frontend;
		TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface = InSubObject.GetOutermostObject();
		check(DocumentInterface.GetObject());
		return FindBuilder(DocumentInterface);
	}

	bool FDocumentBuilderRegistry::FinishBuilding(const FMetasoundFrontendClassName& InClassName, bool bForceUnregisterNodeClass) const
	{
		using namespace Metasound;
		using namespace Metasound::Engine;

		TArray<UMetaSoundBuilderBase*> FoundBuilders = FindBuilderObjects(InClassName);
		for (UMetaSoundBuilderBase* Builder : FoundBuilders)
		{
			FinishBuildingInternal(*Builder, bForceUnregisterNodeClass);

		}

		FScopeLock Lock(&BuildersCriticalSection);
		return Builders.Remove(InClassName) > 0;
	}

	bool FDocumentBuilderRegistry::FinishBuilding(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath, bool bForceUnregisterNodeClass) const
	{
		using namespace Metasound;
		using namespace Metasound::Engine;

		TWeakObjectPtr<UMetaSoundBuilderBase> BuilderPtr;
		if (UMetaSoundBuilderBase* Builder = FindBuilderObject(InClassName, AssetPath))
		{
			FinishBuildingInternal(*Builder, bForceUnregisterNodeClass);
			BuilderPtr = TWeakObjectPtr<UMetaSoundBuilderBase>(Builder);
		}

		FScopeLock Lock(&BuildersCriticalSection);
		return Builders.RemoveSingle(InClassName, BuilderPtr) > 0;
	}

	void FDocumentBuilderRegistry::FinishBuildingInternal(UMetaSoundBuilderBase& Builder, bool bForceUnregisterNodeClass) const
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		// If the builder has applied transactions to its document object that are not mirrored in the frontend registry,
		// unregister version in registry. This will ensure that future requests for the builder's associated asset will
		// register a fresh version from the object as the transaction history is intrinsically lost once this builder
		// is destroyed. It is also possible that the DocBuilder's underlying object can be invalid if object was force
		// deleted, so validity check is necessary.
		FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetBuilder();
		if (DocBuilder.IsValid())
		{
			if (Metasound::CanEverExecuteGraph())
			{
				const int32 TransactionCount = DocBuilder.GetTransactionCount();
				const int32 LastTransactionRegistered = Builder.GetLastTransactionRegistered();
				if (bForceUnregisterNodeClass || LastTransactionRegistered != TransactionCount)
				{
					UObject& MetaSound = DocBuilder.CastDocumentObjectChecked<UObject>();
					if (FMetasoundAssetBase* MetaSoundAsset = IMetasoundUObjectRegistry::Get().GetObjectAsAssetBase(&MetaSound))
					{
						MetaSoundAsset->UnregisterGraphWithFrontend();
					}
				}
			}
			DocBuilder.FinishBuilding();
		}
	}

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnResolveEditorPage& FDocumentBuilderRegistry::GetOnResolveAuditionPageDelegate()
	{
		return OnResolveAuditionPage;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnResolvePage& FDocumentBuilderRegistry::GetOnResolveProjectPageOverrideDelegate()
	{
		return OnResolveProjectPage;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FDocumentBuilderRegistry::ReloadBuilder(const FMetasoundFrontendClassName& InClassName) const
	{
		bool bReloaded = false;
		TArray<UMetaSoundBuilderBase*> ClassBuilders = FindBuilderObjects(InClassName);
		for (UMetaSoundBuilderBase* Builder : ClassBuilders)
		{
			Builder->Reload();
			bReloaded = true;
		}

		return bReloaded;
	}

	FGuid FDocumentBuilderRegistry::ResolveTargetPageID(const FMetasoundFrontendGraphClass& InGraphClass) const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FDocumentBuilderRegistry::ResolveTargetPageID_GraphClass);

#if !WITH_EDITORONLY_DATA
		// No resolution required if only one item. This is the typical scenario and saves from tying up a lock
		// and calling resolution delegate(s). For larger graphs, this can add up, and most implementation
		// only has a single page value. With editor-only data, go ahead and take the perf hit in favor of
		// resolution reporting (for example if page data is invalid and needs to be fixed up).
		const TArray<FMetasoundFrontendGraph>& GraphPages = InGraphClass.GetConstGraphPages();
		if (GraphPages.Num() == 1)
		{
			return GraphPages.Last().PageID;
		}
#endif // !WITH_EDITORONLY_DATA

		FScopeLock Lock(&TargetPageResolveScratchCritSec);
		TargetPageResolveScratch.Reset();
		InGraphClass.IterateGraphPages([this](const FMetasoundFrontendGraph& PageGraph)
		{
			TargetPageResolveScratch.Add(PageGraph.PageID);
		});

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ResolveTargetPageID(TargetPageResolveScratch);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FGuid FDocumentBuilderRegistry::ResolveTargetPageID(const FMetasoundFrontendClassInput& InClassInput) const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FDocumentBuilderRegistry::ResolveTargetPageID_ClassInput);

#if !WITH_EDITORONLY_DATA
		// No resolution required if only one item. This is the typical scenario and saves from tying up a lock
		// and calling resolution delegate(s). For larger graphs, this can add up, and most implementation
		// only has a single page value. With editor-only data, go ahead and take the perf hit in favor of
		// resolution reporting (for example if page data is invalid and needs to be fixed up).
		const TArray<FMetasoundFrontendClassInputDefault>& ClassDefaults = InClassInput.GetDefaults();
		if (ClassDefaults.Num() == 1)
		{
			return ClassDefaults.Last().PageID;
		}
#endif // !WITH_EDITORONLY_DATA

		FScopeLock Lock(&TargetPageResolveScratchCritSec);
		TargetPageResolveScratch.Reset();
		InClassInput.IterateDefaults([this](const FGuid& PageID, const FMetasoundFrontendLiteral&)
		{
			TargetPageResolveScratch.Add(PageID);
		});

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ResolveTargetPageID(TargetPageResolveScratch);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FGuid FDocumentBuilderRegistry::ResolveTargetPageID(const TArray<FMetasoundFrontendClassInputDefault>& InClassDefaults) const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FDocumentBuilderRegistry::ResolveTargetPageID_ClassDefaults);

#if !WITH_EDITORONLY_DATA
		// No resolution required if only one item. This is the typical scenario and saves from tying up a lock
		// and calling resolution delegate(s). For larger graphs, this can add up, and most implementation
		// only has a single page value. With editor-only data, go ahead and take the perf hit in favor of
		// resolution reporting (for example if page data is invalid and needs to be fixed up).
		if (InClassDefaults.Num() == 1)
		{
			return InClassDefaults.Last().PageID;
		}
#endif // !WITH_EDITORONLY_DATA

		FScopeLock Lock(&TargetPageResolveScratchCritSec);
		TargetPageResolveScratch.Reset();
		Algo::Transform(InClassDefaults, TargetPageResolveScratch, [](const FMetasoundFrontendClassInputDefault& ClassDefault) { return ClassDefault.PageID; });
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ResolveTargetPageID(TargetPageResolveScratch);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FGuid FDocumentBuilderRegistry::ResolveTargetPageID(const TArray<FGuid>& InPageIDsToResolve) const
	{
		FName PlatformName = FPlatformProperties::IniPlatformName();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (OnResolveAuditionPage.IsBound())
		{
			FPageResolutionEditorResults PreviewInfo = OnResolveAuditionPage.Execute(InPageIDsToResolve);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (PreviewInfo.PageID.IsSet())
			{
				return PreviewInfo.PageID.GetValue();
			}

			PlatformName = PreviewInfo.PlatformName;
		}
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (OnResolveProjectPage.IsBound())
		{
			const FGuid ResolvedPageID = OnResolveProjectPage.Execute(InPageIDsToResolve);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			check (InPageIDsToResolve.Contains(ResolvedPageID));
			return ResolvedPageID;
		}

		if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
		{
			const FGuid& TargetPageID = Settings->GetTargetPageSettings().UniqueId;
			return ResolveTargetPageIDInternal(*Settings, InPageIDsToResolve, TargetPageID, PlatformName);
		}

		return Frontend::DefaultPageID;
	}

	/** These methods exist to support deprecated funtions. It allows MetaSound Settings data to be
	 * accessed in the MetasoundFrontend module via a singleton IDocumentBuilderRegistry::Get(). Once
	 * the deprecated callsites are removed, these methods can be removed as well. */
	TArrayView<const FGuid> FDocumentBuilderRegistry::GetPageOrder() const
	{
		return UMetaSoundSettings::GetPageOrder();
	}

#if WITH_EDITORONLY_DATA
	TArray<FGuid> FDocumentBuilderRegistry::GetCookedTargetPages(FName InPlatformName) const
	{
		if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>(); ensure(Settings))
		{
			return Settings->GetCookedTargetPageIDs(InPlatformName);
		}
		return TArray<FGuid>{Frontend::DefaultPageID};
	}

	TArray<FGuid> FDocumentBuilderRegistry::GetCookedPageOrder(FName InPlatformName) const
	{
		if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>(); ensure(Settings))
		{
			return Settings->GetCookedPageOrder(InPlatformName);
		}
		return TArray<FGuid>{Frontend::DefaultPageID};
	}
#endif // WITH_EDITORONLY_DATA

	FGuid FDocumentBuilderRegistry::ResolveTargetPageIDInternal(const UMetaSoundSettings& Settings, const TArray<FGuid>& InPageIDsToResolve, const FGuid& TargetPageID, FName PlatformName) const
	{
		bool bResolved = false;
		FGuid ResolvedPageID = Frontend::DefaultPageID;
		constexpr bool bReverse = true;
		bool bFoundTarget = false;
		Settings.IteratePageSettings([&](const FMetaSoundPageSettings& PageSettings)
		{
			bFoundTarget |= PageSettings.UniqueId == TargetPageID;
			if (bFoundTarget && !bResolved)
			{
				const bool bAssetImplementsPage = InPageIDsToResolve.Contains(PageSettings.UniqueId);
				if (bAssetImplementsPage)
				{
#if WITH_EDITOR
					const bool bIsCooked = !PageSettings.GetExcludeFromCook(PlatformName);
					if (bIsCooked)
					{
						bResolved = true;
						ResolvedPageID = PageSettings.UniqueId;
					}
#else // !WITH_EDITOR
					bResolved = true;
					ResolvedPageID = PageSettings.UniqueId;
#endif // !WITH_EDITOR
				}
			}
		}, bReverse);

		if (!bResolved)
		{
			const FGuid& AnyPageID = InPageIDsToResolve.Last();
#if !NO_LOGGING
			auto GetDisplayPageString = [&Settings](const FGuid& InPageID)
			{
				if (const FMetaSoundPageSettings* DisplayPage = Settings.FindPageSettings(InPageID))
				{
					return DisplayPage->Name.ToString();
				}
				return InPageID.ToString();
			};
			UE_LOGF(LogMetaSound, Error,
				"Failed to resolve PageID for Target '%ls': Setting to arbitrary Page '%ls' (Target likely overridden by page not set as 'CanTarget/Targetable' for the current platform)",
				*GetDisplayPageString(TargetPageID),
				*GetDisplayPageString(AnyPageID));
#endif // !NO_LOGGING
			ResolvedPageID = AnyPageID;
		}

		return ResolvedPageID;
	}

	void FDocumentBuilderRegistry::SetEventLogVerbosity(ELogEvent Event, ELogVerbosity::Type Verbosity)
	{
		EventLogVerbosity.FindOrAdd(Event) = Verbosity;
	}
} // namespace Metasound::Engine
