// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "Metasound.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEngineModule.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundGlobals.h"
#include "MetasoundSettings.h"
#include "MetasoundUObjectRegistry.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Serialization/Archive.h"
#include "StructUtils/StructView.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITORONLY_DATA
#include "Algo/Transform.h"
#include "Interfaces/ITargetPlatform.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendRegistryContainer.h"
#include "Misc/DataValidation.h"
#include "Serialization/JsonWriter.h"
#include "UObject/GarbageCollection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtrTemplates.h"
#endif // WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "MetasoundEngine"


namespace Metasound::Engine
{
	/** MetaSound Engine Asset helper provides routines for UObject based MetaSound assets. 
	 * Any UObject deriving from FMetaSoundAssetBase should use these helper functions
	 * in their UObject overrides. 
	 */
	struct FAssetHelper
	{
#if WITH_EDITOR
		static void PreDuplicate(TScriptInterface<IMetaSoundDocumentInterface> MetaSound, FObjectDuplicationParameters& DupParams)
		{
			FDocumentBuilderRegistry::GetChecked().SetEventLogVerbosity(FDocumentBuilderRegistry::ELogEvent::DuplicateEntries, ELogVerbosity::NoLogging);
		}

		static void PostDuplicate(TScriptInterface<IMetaSoundDocumentInterface> MetaSound, EDuplicateMode::Type InDuplicateMode)
		{
			using namespace Engine;
			using namespace Frontend;

			if (InDuplicateMode == EDuplicateMode::Normal)
			{
				UObject* MetaSoundObject = MetaSound.GetObject();
				check(MetaSoundObject);

				FDocumentBuilderRegistry& BuilderRegistry = FDocumentBuilderRegistry::GetChecked();
				UMetaSoundBuilderBase& DuplicateBuilder = BuilderRegistry.FindOrBeginBuilding(*MetaSoundObject);

				FMetaSoundFrontendDocumentBuilder& DocBuilder = DuplicateBuilder.GetBuilder();
				const FMetasoundFrontendClassName DuplicateName = DocBuilder.GetConstDocumentChecked().RootGraph.Metadata.GetClassName();
				DocBuilder.GenerateNewClassName();

				constexpr bool bForceUnregisterNodeClass = true;
				BuilderRegistry.FinishBuilding(DuplicateName, MetaSound->GetAssetPathChecked(), bForceUnregisterNodeClass);
				BuilderRegistry.SetEventLogVerbosity(FDocumentBuilderRegistry::ELogEvent::DuplicateEntries, ELogVerbosity::All);
			}
		}

		static void PostEditUndo(TScriptInterface<IMetaSoundDocumentInterface>& InDocInterface)
		{
			using namespace Frontend;

			InDocInterface->GetConstDocument().Metadata.ModifyContext.SetForceRefreshViews();

			const FMetasoundFrontendClassName& ClassName = InDocInterface->GetConstDocument().RootGraph.Metadata.GetClassName();
			IDocumentBuilderRegistry& BuilderRegistry = IDocumentBuilderRegistry::GetChecked();
			
			BuilderRegistry.ReloadBuilder(ClassName);

			FMetaSoundFrontendDocumentBuilder& Builder = BuilderRegistry.FindOrBeginBuilding(InDocInterface);

#if WITH_EDITORONLY_DATA
			Builder.GetDocumentDelegates().OnDocumentTemplateChanged.Broadcast({ EDocumentTemplateChangeType::UndoRedo });
			Builder.ConfigureDocument();
#endif // WITH_EDITORONLY_DATA

			if (UMetasoundEditorGraphBase* Graph = Cast<UMetasoundEditorGraphBase>(Builder.GetMetasoundAsset().GetGraph()))
			{
				Graph->RegisterGraphWithFrontend();
			}
		}

		template<typename TMetaSoundObject>
		static void SetReferencedAssets(TMetaSoundObject& InMetaSound, TSet<Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs)
		{
			using namespace Frontend;
			
			InMetaSound.ReferencedAssetClassKeys.Reset();
			InMetaSound.ReferencedAssetClassObjects.Reset();

			for (const IMetaSoundAssetManager::FAssetRef& AssetRef : InAssetRefs)
			{
				// Has to be serialized as node class registry key string for back compat
				InMetaSound.ReferencedAssetClassKeys.Add(FNodeClassRegistryKey(AssetRef.Key).ToString());
				if (UObject* Object = FSoftObjectPath(AssetRef.Path).TryLoad())
				{
					InMetaSound.ReferencedAssetClassObjects.Add(Object);
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Failed to load referenced asset %ls from asset %ls", *AssetRef.Path.ToString(), *InMetaSound.GetPathName());
				}
			}
		}

		static EDataValidationResult IsClassNameUnique(const FMetasoundFrontendDocument& Document, FDataValidationContext& InOutContext)
		{
			using namespace Metasound::Frontend;
			using namespace Metasound::Engine;

			EDataValidationResult Result = EDataValidationResult::Valid;
			IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();
			// Validation has added assets to the asset manager
			// and we don't remove them immediately after validation to optimize possible subsequent validation
			// Set this flag to prevent log spam of active assets on shutdown
			AssetManager.SetLogActiveAssetsOnShutdown(false);

			// Add error for multiple assets with the same class name
			const FMetaSoundAssetKey Key(Document.RootGraph.Metadata);
			const TArray<FTopLevelAssetPath> AssetPaths = AssetManager.FindAssetPaths(Key);
			if (AssetPaths.Num() > 1)
			{
				Result = EDataValidationResult::Invalid;

				TArray<FText> PathStrings;
				Algo::Transform(AssetPaths, PathStrings, [](const FTopLevelAssetPath& Path) { return FText::FromString(Path.ToString()); });
				InOutContext.AddError(FText::Format(LOCTEXT("UniqueClassNameValidation",
					"Multiple assets use the same class name which may result in unintended behavior. This may happen when an asset is moved, "
					"then the move is reverted in revision control without removing the newly created asset. Please remove the offending asset "
					"or duplicate it to automatically generate a new class name." \
					"\nConflicting Asset Paths:\n{0}"), FText::Join(FText::FromString(TEXT("\n")), PathStrings)));
			}

			// Success
			return Result;
		}

		static EDataValidationResult IsDataValid(const UObject& MetaSound, const FMetasoundFrontendDocument& Document, FDataValidationContext& InOutContext)
		{
			using namespace Metasound;

			EDataValidationResult Result = EDataValidationResult::Valid;
			if (Engine::GetEditorAssetValidationEnabled())
			{
				// We cannot rely on the asset registry scan being complete during the call
				// to IsDataValid(...) while running a cook commandlet. The IMetasoundAssetManager
				// will still log errors on duplicate assets which will fail cook. 
				if (!IsRunningCookCommandlet())
				{
					Result = IsClassNameUnique(Document, InOutContext);
				}
			}

			const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
			check(Settings);

			TSet<FGuid> ValidPageIDs;
			auto ErrorIfMissing = [&](const FGuid& PageID, const FText& DataDescriptor)
			{
				if (!ValidPageIDs.Contains(PageID))
				{
					if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageID))
					{
						ValidPageIDs.Add(PageSettings->UniqueId);
					}
					else
					{
						Result = EDataValidationResult::Invalid;
						InOutContext.AddMessage(FAssetData(&MetaSound), EMessageSeverity::Error, FText::Format(
							LOCTEXT("InvalidPageDataFormat", "MetaSound contains invalid {0} with page ID '{1}': page not found in Project 'MetaSound' Settings. Remove page data or migrate to existing page identifier."),
							DataDescriptor,
							FText::FromString(PageID.ToString())));
					}
				}
			};

			const TArray<FMetasoundFrontendGraph>& Graphs = Document.RootGraph.GetConstGraphPages();
			for (const FMetasoundFrontendGraph& Graph : Graphs)
			{
				ErrorIfMissing(Graph.PageID, LOCTEXT("GraphPageDescriptor", "graph"));
			}

			for (const FMetasoundFrontendClassInput& ClassInput : Document.RootGraph.GetDefaultInterface().Inputs)
			{
				ClassInput.IterateDefaults([&](const FGuid& PageID, const FMetasoundFrontendLiteral&)
				{
					ErrorIfMissing(PageID, FText::Format(LOCTEXT("InputPageDefaultDescriptorFormat", "input '{0}' default value"), FText::FromName(ClassInput.Name)));
				});
			}

			if (const FMetaSoundFrontendDocumentTemplate* DocTemplate = Document.Template.GetPtr())
			{
				FDocumentBuilderRegistry& BuilderRegistry = FDocumentBuilderRegistry::GetChecked();
				UObject* MetaSoundObject = FAssetData(&MetaSound).GetAsset();
				check(MetaSoundObject);
				const UMetaSoundBuilderBase& Builder = BuilderRegistry.FindOrBeginBuilding(*MetaSoundObject);
				CombineDataValidationResults(Result, DocTemplate->IsDataValid(Builder.GetConstBuilder(), InOutContext));
			}
			return Result;
		}

#endif // WITH_EDITOR

		static void GetAssetRegistryTags(TScriptInterface<const IMetaSoundDocumentInterface> DocInterface, FAssetRegistryTagsContext& Context)
		{
			using namespace Frontend;

			const UObject* MetaSound = DocInterface.GetObject();
			check(MetaSound);

#if WITH_EDITORONLY_DATA
			FVersioningManager::Get().WaitUntilVersioningComplete(*MetaSound);
#endif // WITH_EDITORONLY_DATA

			if (MetaSound->GetFlags() & (RF_Transient | RF_ClassDefaultObject))
			{
				return;
			}

			const FMetaSoundAssetClassInfo ClassInfo(*DocInterface);
			ClassInfo.ExportToContext(Context);
		}

		template <typename TMetaSoundObject>
		static FTopLevelAssetPath GetAssetPathChecked(TMetaSoundObject& InMetaSound)
		{
			FTopLevelAssetPath Path;
			ensureAlwaysMsgf(Path.TrySetPath(&InMetaSound), TEXT("Failed to set TopLevelAssetPath from MetaSound '%s'. MetaSound must be highest level object in package."), *InMetaSound.GetPathName());
			ensureAlwaysMsgf(Path.IsValid(), TEXT("Failed to set TopLevelAssetPath from MetaSound '%s'. This may be caused by calling this function when the asset is being destroyed."), *InMetaSound.GetPathName());
			return Path;
		}

		template <typename TMetaSoundObject>
		static TArray<FMetasoundAssetBase*> GetReferencedAssets(const TMetaSoundObject& InMetaSound)
		{
			TArray<FMetasoundAssetBase*> ReferencedAssets;

			IMetasoundUObjectRegistry& UObjectRegistry = IMetasoundUObjectRegistry::Get();

			for (const TObjectPtr<UObject>& Object : InMetaSound.ReferencedAssetClassObjects)
			{
				if (Object.Get() != nullptr)
				{
					if (FMetasoundAssetBase* Asset = UObjectRegistry.GetObjectAsAssetBase(Object))
					{
						ReferencedAssets.Add(Asset);
						continue;
					}

					UE_LOGF(LogMetaSound, Error, "Referenced asset \"%ls\", referenced from \"%ls\", is not convertible to FMetasoundAssetBase", *Object->GetPathName(), *InMetaSound.GetPathName());
				}
			}

			return ReferencedAssets;
		}

		static void PreSaveAsset(FMetasoundAssetBase& InMetaSound, FObjectPreSaveContext InSaveContext)
		{
#if WITH_EDITORONLY_DATA
			using namespace Frontend;

			const bool bIsCooking = InSaveContext.IsCooking();
			const bool bCanEverExecute = Metasound::CanEverExecuteGraph(bIsCooking);
			if (!bCanEverExecute)
			{
				FName PlatformName;
				if (const ITargetPlatform* TargetPlatform = InSaveContext.GetTargetPlatform())
				{
					PlatformName = *TargetPlatform->IniPlatformName();
				}

				const bool bIsDeterministic = bIsCooking || IsRunningCookCommandlet();
				FDocumentIDGenerator::FScopeDeterminism DeterminismScope(bIsDeterministic);

				FMetaSoundAssetCookOptions CookOptions;
				if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>(); ensure(Settings))
				{
					CookOptions.bStripUnusedPages = true;
					CookOptions.PagesToTarget = Settings->GetCookedTargetPageIDs(PlatformName);
					CookOptions.PageOrder = Settings->GetCookedPageOrder(PlatformName);
				}
				
				InMetaSound.UpdateAndRegisterForSerialization(CookOptions);
			}
 			else if (FApp::CanEverRenderAudio())
			{
				if (UMetasoundEditorGraphBase* MetaSoundGraph = Cast<UMetasoundEditorGraphBase>(InMetaSound.GetGraph()))
				{
					// Uses graph flavor of register with frontend to update editor systems/asset editors in case editor is enabled.
					// Ignore if live auditioning to avoid stopping playback when saving in background.
					{
						FMetaSoundAssetRegistrationOptions RegOptions
						{
#if WITH_EDITOR
							.bForceViewSynchronization = false,
							.bIgnoreIfLiveAuditioning = true,
#endif // WITH_EDITOR
							.PageOrder = UMetaSoundSettings::GetPageOrder()
						};

						MetaSoundGraph->RegisterGraphWithFrontend(&RegOptions);
					}

					InMetaSound.GetModifyContext().SetForceRefreshViews();
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "PreSaveAsset for MetaSound: (%ls) is doing nothing because InSaveContext.IsCooking, IsRunningCommandlet, and FApp::CanEverRenderAudio were all false"
					, *InMetaSound.GetOwningAssetName());
			}
#endif // WITH_EDITORONLY_DATA
		}

		static void SerializeToArchive(FMetasoundAssetBase& InMetaSound, FArchive& InArchive)
		{
		}

#if WITH_EDITORONLY_DATA
		static void PostLoad(FMetasoundAssetBase& InMetaSound, TSet<FSoftObjectPath>& ReferenceAssetClassCache)
		{
			using namespace Frontend;

			UObject* OwningObject = InMetaSound.GetOwningAsset();
			check(OwningObject);

			// Do not call asset manager on CDO objects which may be loaded before asset 
			// manager is set.
			const bool bIsCDO = OwningObject->HasAnyFlags(RF_ClassDefaultObject);
			if (!bIsCDO)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (ReferenceAssetClassCache.IsEmpty())
				{
					const bool bIsDeterministic = IsRunningCookCommandlet();
					FVersioningManager::Get().VersionAssetAsync(*OwningObject, bIsDeterministic);
				}
				else
				{
					// Versioning asset is applied after async load of soft object
					// references to ensure document properties are ready for versioning.
					UE_LOGF(LogMetaSound, Display, "Delaying asset versioning due to need to async load soft references for MetaSound %ls", *InMetaSound.GetOwningAssetName());
					IMetaSoundAssetManager::GetChecked().RequestAsyncLoadReferencedAssets(InMetaSound, &ReferenceAssetClassCache); 
				}

				// There is no way to determine if a commandlet is going to read directly from a MetaSound object
				// immediately after post load to perform tasks such as serialization diffing, reference scrubbing,
				// etc. outside of the MetaSound Builder/Asset Manager API. Therefore, its best to just immediately
				// wait until versioning is complete in this context.
				if (IsRunningCommandlet())
				{
					using namespace Frontend;
					FVersioningManager::Get().WaitUntilVersioningComplete(*OwningObject);
				}
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		template<typename TMetaSoundObject>
		static void OnAsyncReferencedAssetsLoaded(TMetaSoundObject& InMetaSound, const TArray<FMetasoundAssetBase*>& InAsyncReferences)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const bool bVersioningDeferred = !InMetaSound.ReferenceAssetClassCache.IsEmpty();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			for (FMetasoundAssetBase* AssetBase : InAsyncReferences)
			{
				if (AssetBase)
				{
					if (UObject* OwningAsset = AssetBase->GetOwningAsset())
					{
						InMetaSound.ReferencedAssetClassObjects.Add(OwningAsset);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
						InMetaSound.ReferenceAssetClassCache.Remove(FSoftObjectPath(OwningAsset));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}
			}

			if (bVersioningDeferred)
			{
				const bool bIsDeterministic = IsRunningCookCommandlet();
				Frontend::FVersioningManager::Get().VersionAssetAsync(InMetaSound, bIsDeterministic);
			}
		}
#endif // WITH_EDITORONLY_DATA
	};
} // namespace Metasound::Engine
#undef LOCTEXT_NAMESPACE // MetasoundEngine
