// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEngineEnvironment.h"

#include "AudioDeviceManager.h"
#include "MetasoundEnvironment.h"

//DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(FAudioDeviceHandle);

