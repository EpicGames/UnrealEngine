// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEngineModule.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Metasound.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundAudioBus.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundGlobals.h"
#include "MetasoundLog.h"
#include "MetasoundOutputSubsystem.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundWave.h"
#include "MetasoundWaveTable.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerAudioBuffer.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerForwardValue.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerDensity.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerToTime.h"
#include "Analysis/MetasoundVertexAnalyzerAudioBusWriter.h"
#include "Interfaces/MetasoundDeprecatedInterfaces.h"
#include "Interfaces/MetasoundInterface.h"
#include "Interfaces/MetasoundInterfaceBindingsPrivate.h"
#include "Modules/ModuleManager.h"
#include "Sound/AudioSettings.h"

namespace Metasound
{
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<FWaveAsset>
	{
		static constexpr bool Value = true;
	};
}

REGISTER_METASOUND_DATATYPE(Metasound::FAudioBusAsset, "AudioBusAsset", Metasound::ELiteralType::UObjectProxy, UAudioBus);
REGISTER_METASOUND_DATATYPE(Metasound::FWaveAsset, "WaveAsset", Metasound::ELiteralType::UObjectProxy, USoundWave);
REGISTER_METASOUND_DATATYPE(WaveTable::FWaveTable, "WaveTable", Metasound::ELiteralType::FloatArray)
REGISTER_METASOUND_DATATYPE(Metasound::FWaveTableBankAsset, "WaveTableBankAsset", Metasound::ELiteralType::UObjectProxy, UWaveTableBank);

namespace Metasound::Engine
{
#if WITH_EDITOR
	namespace ModulePrivate
	{
		int32 EnableMetaSoundEditorAssetValidation = 1;
		int32 EnableMetaSoundEditorAssetAutoLoadAndRegister = 0;

		FAutoConsoleVariableRef CVarEnableMetaSoundEditorAssetValidation(
			TEXT("au.MetaSound.Editor.EnableAssetValidation"),
			EnableMetaSoundEditorAssetValidation,
			TEXT("Enables MetaSound specific asset validation.\n")
			TEXT("Default: 1 (Enabled)"),
			ECVF_Default);

		FAutoConsoleVariableRef CVarEnableMetaSoundAutoLoadingAndRegisteringAssets(
			TEXT("au.MetaSound.Editor.EnableAutoLoadAndRegisterOnAssetScan"),
			EnableMetaSoundEditorAssetAutoLoadAndRegister,
			TEXT("Enables auto-loading and registration of assets. Not recommended as it is slow, but can be useful for debugging load issues with serialized MetaSound assets. \n")
			TEXT("Default: 0 (Disabled)"),
			ECVF_Default);
	} // namespace ModulePrivate

	bool GetEditorAssetValidationEnabled()
	{
		return ModulePrivate::EnableMetaSoundEditorAssetValidation != 0;
	}
#endif // WITH_EDITOR

	void FModule::FModule::StartupModule() 
	{
		using namespace Frontend;

		METASOUND_LLM_SCOPE;
		FModuleManager::Get().LoadModuleChecked("MetasoundGraphCore");
		FModuleManager::Get().LoadModuleChecked("MetasoundFrontend");
		FModuleManager::Get().LoadModuleChecked("MetasoundStandardNodes");
		FModuleManager::Get().LoadModuleChecked("MetasoundGenerator");
		FModuleManager::Get().LoadModuleChecked("WaveTable");
		
		InitializeAssetManager();
		IDocumentBuilderRegistry::Initialize(MakeUnique<FDocumentBuilderRegistry>());

		// Set GCObject referencer for metasound frontend node registry. The MetaSound
		// frontend does not have access to Engine GC tools and must have them 
		// supplied externally.
		FMetasoundFrontendRegistryContainer::Get()->SetObjectReferencer(MakeUnique<FObjectReferencer>());

		// Register objects statically associated with this module.
		METASOUND_REGISTER_ITEMS_IN_MODULE

		// Register engine-level parameter interfaces if not done already.
		// (Potentially not already called if plugin is loaded while cooking.)
		UAudioSettings* AudioSettings = GetMutableDefault<UAudioSettings>();
		check(AudioSettings);
		AudioSettings->RegisterParameterInterfaces();

		IMetasoundUObjectRegistry::Get().RegisterUClass(MakeUnique<TMetasoundUObjectRegistryEntry<UMetaSoundBuilderDocument>>());
		IMetasoundUObjectRegistry::Get().RegisterUClass(MakeUnique<TMetasoundUObjectRegistryEntry<UMetaSoundPatch>>());
		IMetasoundUObjectRegistry::Get().RegisterUClass(MakeUnique<TMetasoundUObjectRegistryEntry<UMetaSoundSource>>());

		RegisterDeprecatedInterfaces();
		RegisterInterfaces();
		RegisterInternalInterfaceBindings();

		// Register Analyzers
		// TODO: Determine if we can move this registration to Frontend where it likely belongs
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerAudioBuffer)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerEnvelopeFollower)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerForwardBool)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerForwardFloat)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerForwardInt)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerForwardTime)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerForwardString)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerTriggerDensity)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Frontend::FVertexAnalyzerTriggerToTime)
		METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(Engine::FVertexAnalyzerAudioBusWriter)

		// Register passthrough output analyzers
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<float>(),
			Frontend::FVertexAnalyzerForwardFloat::GetAnalyzerName(),
			Frontend::FVertexAnalyzerForwardFloat::FOutputs::GetValue().Name);
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<int32>(),
			Frontend::FVertexAnalyzerForwardInt::GetAnalyzerName(),
			Frontend::FVertexAnalyzerForwardInt::FOutputs::GetValue().Name);
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<bool>(),
			Frontend::FVertexAnalyzerForwardBool::GetAnalyzerName(),
			Frontend::FVertexAnalyzerForwardBool::FOutputs::GetValue().Name);
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<FString>(),
			Frontend::FVertexAnalyzerForwardString::GetAnalyzerName(),
			Frontend::FVertexAnalyzerForwardString::FOutputs::GetValue().Name);
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<FTime>(),
			Frontend::FVertexAnalyzerForwardTime::GetAnalyzerName(),
			Frontend::FVertexAnalyzerForwardTime::FOutputs::GetValue().Name);
		UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
			GetMetasoundDataTypeName<FTrigger>(),
			Frontend::FVertexAnalyzerTriggerToTime::GetAnalyzerName(),
			Frontend::FVertexAnalyzerTriggerToTime::FOutputs::GetValue().Name);
#if WITH_EDITOR
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FModule::OnAssetScanFinished);
		}
		else
		{
		}
#endif // WITH_EDITOR

		UE_LOGF(LogMetaSound, Log, "MetaSound Engine Initialized");
	}

	void FModule::FModule::ShutdownModule()
	{
		METASOUND_UNREGISTER_ITEMS_IN_MODULE
#if WITH_EDITOR
		ShutdownAssetClassRegistry();
#endif // WITH_EDITOR
		DeinitializeAssetManager();
		Frontend::IDocumentBuilderRegistry::Deinitialize();
	}

#if WITH_EDITOR
	void FModule::AddClassRegistryAsset(const FAssetData& InAssetData)
	{
		using namespace Frontend;

		// Don't add temporary assets used for diffing
		if (InAssetData.HasAnyPackageFlags(PKG_ForDiffing))
		{
			return;
		}

		// If an object's class could not be found, ignore this asset.  This can hit for non-MetaSound assets
		// and it is up to the system in charge of interacting with that asset or the loading behavior to
		// report the failed load of the class.
		const UClass* AssetClass = InAssetData.GetClass();
		if (!AssetClass || !IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass))
		{
			return;
		}

		// Ignore requests to prime on assets with AccessFlag 'Referenceable' unset, deffering
		// registration to initial execution if existing reference from another asset requires it.
		const FMetaSoundAssetClassInfo ClassInfo(InAssetData);
		if (!EnumHasAnyFlags(ClassInfo.AccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
		{
			return;
		}

		if (ModulePrivate::EnableMetaSoundEditorAssetAutoLoadAndRegister)
		{
			IMetaSoundAssetManager::GetChecked().AddOrLoadAndUpdateFromObjectAsync(InAssetData,
			[](FMetaSoundAssetKey, UObject& AssetObject)
			{
				FModule& ThisModule = FModuleManager::GetModuleChecked<Metasound::Engine::FModule>("MetaSoundEngine");
				ThisModule.GetOnGraphRegisteredDelegate().ExecuteIfBound(AssetObject, ERegistrationAssetContext::None);
			});
		}
		else
		{
			IMetaSoundAssetManager::GetChecked().AddOrUpdateFromAssetData(InAssetData);
		}
	}
	
	void FModule::UpdateClassRegistryAsset(const FAssetData& InAssetData)
	{
		using namespace Frontend;

		// If an object's class could not be found, ignore this asset.  This can hit for non-MetaSound assets
		// and it is up to the system in charge of interacting with that asset or the loading behavior to
		// report the failed load of the class.
		if (const UClass* AssetClass = InAssetData.GetClass())
		{
			const bool bIsRegisteredClass = IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass);
			if (bIsRegisteredClass)
			{
				if (ModulePrivate::EnableMetaSoundEditorAssetAutoLoadAndRegister)
				{
					IMetaSoundAssetManager::GetChecked().AddOrLoadAndUpdateFromObjectAsync(InAssetData,
					[](FMetaSoundAssetKey AssetKey, UObject& AssetObject)
					{
						// Have to re-register to avoid registry desync.
						const bool bIsRegistered = FMetasoundFrontendRegistryContainer::Get()->IsNodeRegistered(FNodeRegistryKey(AssetKey));
						if (bIsRegistered || ModulePrivate::EnableMetaSoundEditorAssetAutoLoadAndRegister)
						{
							FModule& ThisModule = FModuleManager::GetModuleChecked<Metasound::Engine::FModule>("MetaSoundEngine");
							ThisModule.GetOnGraphRegisteredDelegate().ExecuteIfBound(AssetObject, ERegistrationAssetContext::None);
						}
					});
				}
				else
				{
					IMetaSoundAssetManager::GetChecked().AddOrUpdateFromAssetData(InAssetData);
				}
			}
		}
	}

	void FModule::OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		if (!InPackageReloadedEvent)
		{
			return;
		}

		if (InPackageReloadPhase != EPackageReloadPhase::OnPackageFixup)
		{
			return;
		}

		auto IsAssetMetaSound = [](const UObject* Obj)
		{
			check(Obj);
			if (const UClass* AssetClass = Obj->GetClass())
			{
				return IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass);
			}

			return false;
		};

		for (const TPair<UObject*, UObject*>& Pair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (UObject* Obj = Pair.Key)
			{
				if (IsAssetMetaSound(Obj))
				{
					OnGraphUnregister.ExecuteIfBound(*Obj, ERegistrationAssetContext::Reloading);
					IMetaSoundAssetManager::GetChecked().RemoveAsset(*Pair.Key);
				}
			}

			if (UObject* Obj = Pair.Value)
			{
				if (IsAssetMetaSound(Obj))
				{
					IMetaSoundAssetManager::GetChecked().AddOrUpdateFromObject(*Pair.Value);
					OnGraphRegister.ExecuteIfBound(*Obj, ERegistrationAssetContext::Reloading);
				}
			}
		}
	}

	void FModule::PrimeAssetManager()
	{
		if (!FMetaSoundAssetManager::GetChecked().IsInitialAssetScanComplete())
		{
			AssetTagPrimeStatus = EAssetTagPrimeRequestStatus::Requested;
			return;
		}

		if (AssetTagPrimeStatus < EAssetTagPrimeRequestStatus::Complete)
		{
			PrimeAssetManagerInternal();
		}
	}

	void FModule::PrimeAssetManagerInternal()
	{
		TArray<FTopLevelAssetPath> ClassNames;
		IMetasoundUObjectRegistry::Get().IterateRegisteredUClasses([&ClassNames](UClass& InClass)
		{
			ClassNames.Add(InClass.GetClassPathName());
		});

		FARFilter Filter;
		Filter.ClassPaths = ClassNames;

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().EnumerateAssets(Filter, [this](const FAssetData& AssetData)
		{
			AddClassRegistryAsset(AssetData);
			return true;
		});

		AssetTagPrimeStatus = EAssetTagPrimeRequestStatus::Complete;
		FMetaSoundAssetManager::GetChecked().SetCanNotifyAssetTagScanComplete();
	}

	void FModule::OnAssetScanFinished()
	{
		if (IsRunningCookCommandlet())
		{
			return;
		}

		if (AssetTagPrimeStatus == EAssetTagPrimeRequestStatus::Requested)
		{
			PrimeAssetManagerInternal();
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnAssetAdded().AddRaw(this, &FModule::AddClassRegistryAsset);
		AssetRegistryModule.Get().OnAssetUpdated().AddRaw(this, &FModule::UpdateClassRegistryAsset);
		AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FModule::RemoveAssetFromClassRegistry);
		AssetRegistryModule.Get().OnAssetRenamed().AddRaw(this, &FModule::RenameAssetInClassRegistry);

		AssetRegistryModule.Get().OnFilesLoaded().RemoveAll(this);
		
		FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FModule::OnPackageReloaded);
	}

	void FModule::RemoveAssetFromClassRegistry(const FAssetData& InAssetData)
	{
		using namespace Frontend;
		if (const UClass* AssetClass = InAssetData.GetClass())
		{
			const bool bIsRegisteredClass = IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass);
			if (bIsRegisteredClass)
			{
				// Use the editor version of UnregisterWithFrontend so it refreshes any open MetaSound editors
				// Doesn't use AssetData::GetAsset() as this can result in attempting to reload the object.
				// If this call is hit after the asset is removed, the assumption is unregistration already
				// occurred on object destroy.
				if (UObject* AssetObject = InAssetData.GetSoftObjectPath().ResolveObject())
				{
					OnGraphUnregister.ExecuteIfBound(*AssetObject, ERegistrationAssetContext::Removing);
				}

				IMetaSoundAssetManager::GetChecked().RemoveAsset(InAssetData);
			}
		}
	}

	void FModule::RenameAssetInClassRegistry(const FAssetData& InAssetData, const FString& InOldObjectPath)
	{
		using namespace Frontend;

		if (const UClass* AssetClass = InAssetData.GetClass())
		{
			const bool bIsRegisteredClass = IMetasoundUObjectRegistry::Get().IsRegisteredClass(*AssetClass);
			if (bIsRegisteredClass)
			{
				IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();

				// Unregister using the new asset data even though the old object was last to be registered
				// as the old asset is no longer accessible by the time rename is called. The asset at this
				// point is identical however to its prior counterpart.
				UObject* AssetObject = InAssetData.GetAsset();
				check(AssetObject);

				FMetasoundAssetBase* AssetBase = AssetManager.GetAsAsset(*AssetObject);
				check(AssetBase);
				bool bIsRegistered = AssetBase->IsRegistered();
				if (bIsRegistered)
				{
					OnGraphUnregister.ExecuteIfBound(*AssetObject, ERegistrationAssetContext::Renaming);
				}

				IMetaSoundAssetManager::GetChecked().RenameAsset(InAssetData, InOldObjectPath);

				if (bIsRegistered)
				{
					OnGraphRegister.ExecuteIfBound(*AssetObject, ERegistrationAssetContext::Renaming);
				}
			}
		}
	}

	void FModule::ShutdownAssetClassRegistry()
	{
		if (FAssetRegistryModule* AssetRegistryModule = static_cast<FAssetRegistryModule*>(FModuleManager::Get().GetModule("AssetRegistry")))
		{
			AssetRegistryModule->Get().OnAssetAdded().RemoveAll(this);
			AssetRegistryModule->Get().OnAssetUpdated().RemoveAll(this);
			AssetRegistryModule->Get().OnAssetRemoved().RemoveAll(this);
			AssetRegistryModule->Get().OnAssetRenamed().RemoveAll(this);
			AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(this);

			FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
		}
	}

	FOnMetasoundGraphRegister& FModule::GetOnGraphRegisteredDelegate()
	{
		return OnGraphRegister;
	}

	FOnMetasoundGraphUnregister& FModule::GetOnGraphUnregisteredDelegate()
	{
		return OnGraphUnregister;
	}
#endif // WITH_EDITOR
} // namespace Metasound::Engine

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(Metasound::Engine::FModule, MetasoundEngine);
