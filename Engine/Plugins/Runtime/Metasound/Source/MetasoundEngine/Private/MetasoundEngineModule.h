// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMetasoundEngineModule.h"
#include "MetasoundFrontendRegistryContainer.h"
#include "UObject/GCObject.h"

#if WITH_EDITOR
#include "HAL/IConsoleManager.h"
#endif // WITH_EDITOR

namespace Metasound::Engine
{
	enum class EAssetTagPrimeRequestStatus : uint8
	{
		NotRequested = 0,
		Requested = 1,
		Complete = 2,
	};

#if WITH_EDITOR
	bool GetEditorAssetValidationEnabled();
#endif // WITH_EDITOR

	class FModule : public IMetasoundEngineModule
	{
		// Supplies GC referencing in the MetaSound Frontend node registry for doing
		// async work on UObjets
		class FObjectReferencer
			: public FMetasoundFrontendRegistryContainer::IObjectReferencer
			, public FGCObject
		{
		public:
			virtual void AddObject(UObject* InObject) override
			{
				FScopeLock LockObjectArray(&ObjectArrayCriticalSection);
				ObjectArray.Add(InObject);
			}

			virtual void RemoveObject(UObject* InObject) override
			{
				FScopeLock LockObjectArray(&ObjectArrayCriticalSection);
				ObjectArray.Remove(InObject);
			}

			virtual void AddReferencedObjects(FReferenceCollector& Collector) override
			{
				FScopeLock LockObjectArray(&ObjectArrayCriticalSection);
				Collector.AddReferencedObjects(ObjectArray);
			}

			virtual FString GetReferencerName() const override
			{
				return TEXT("FMetasoundEngineModule::FObjectReferencer");
			}

		private:
			mutable FCriticalSection ObjectArrayCriticalSection;
			TArray<TObjectPtr<UObject>> ObjectArray;
		};

	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

#if WITH_EDITOR
		virtual void PrimeAssetManager() override;
		virtual FOnMetasoundGraphRegister& GetOnGraphRegisteredDelegate() override;
		virtual FOnMetasoundGraphUnregister& GetOnGraphUnregisteredDelegate() override;
#endif // WITH_EDITOR

	private:
#if WITH_EDITOR
		void AddClassRegistryAsset(const FAssetData& InAssetData);
		void UpdateClassRegistryAsset(const FAssetData& InAssetData);
		void RemoveAssetFromClassRegistry(const FAssetData& InAssetData);
		void RenameAssetInClassRegistry(const FAssetData& InAssetData, const FString& InOldObjectPath);
		void ShutdownAssetClassRegistry();

		void OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);
		void OnAssetScanFinished();

		void PrimeAssetManagerInternal();

		// Asset registry delegates for calling MetaSound editor module 
		FOnMetasoundGraphRegister OnGraphRegister;
		FOnMetasoundGraphUnregister OnGraphUnregister;

		// Prime tag registry state
		EAssetTagPrimeRequestStatus AssetTagPrimeStatus = EAssetTagPrimeRequestStatus::NotRequested;
#endif // WITH_EDITOR
	};
} // namespace Metasound::Engine
