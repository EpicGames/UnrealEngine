// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEngineNodesNames.h"

namespace Metasound
{
	namespace EngineNodes
	{
		const FName Namespace = "UE";

		const FName MonoVariant = "Mono";
		const FName StereoVariant = "Stereo";
		const FName QuadVariant = "Quad";
		const FName FiveDotOneVariant = "5dot1";
		const FName SevenDotOneVariant = "7dot1";
	}
}
