// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendLiteralBlueprintAccess.h"

#include "Logging/StructuredLog.h"
#include "MetasoundDataReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendLiteralBlueprintAccess)

namespace MetasoundFrontendLiteralBlueprintAccessPrivate
{
	template <typename TLiteralType>
	FMetasoundFrontendLiteral CreatePODMetaSoundLiteral(const TLiteralType& Value)
	{
		FMetasoundFrontendLiteral Literal;
		Literal.Set(Value);
		return Literal;
	}

	template <typename TLiteralType>
	TLiteralType GetPODValueFromMetaSoundLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
	{
		TLiteralType Value;
		OutResult = Literal.TryGet(Value) ? EMetaSoundBuilderResult::Succeeded : EMetaSoundBuilderResult::Failed;
		return Value;
	}

	template<typename TLiteralType>
	TLiteralType GetPODValueFromMetaSoundLiteral(const FMetasoundFrontendLiteral& Literal, const EMetasoundFrontendLiteralType ExpectedType, const TLiteralType& DefaultValue)
	{
		if (Literal.GetType() != ExpectedType)
		{
			UE_LOGFMT(LogMetaSound, Warning, "Tried to get literal as type {expected}, but literal was of type {actual}."
				, *UEnum::GetValueAsString(ExpectedType)
				, *UEnum::GetValueAsString(Literal.GetType()));
			return DefaultValue;
		}

		EMetaSoundBuilderResult Result;
		TLiteralType Value = GetPODValueFromMetaSoundLiteral<TLiteralType>(Literal, Result);

		if (Result != EMetaSoundBuilderResult::Succeeded)
		{
			UE_LOGFMT(LogMetaSound, Warning, "Failed to get {ExpectedType} value from InLiteral.", *UEnum::GetValueAsString(ExpectedType));
			return DefaultValue;
		}

		return Value;
	}
}

FString UMetasoundFrontendLiteralBlueprintAccess::Conv_MetaSoundLiteralToString(const FMetasoundFrontendLiteral& Literal)
{
	return Literal.ToString();
}

EMetasoundFrontendLiteralType UMetasoundFrontendLiteralBlueprintAccess::GetType(const FMetasoundFrontendLiteral& Literal)
{
	return Literal.GetType();
}

bool UMetasoundFrontendLiteralBlueprintAccess::EqualEqual_MetaSoundLiteral(const FMetasoundFrontendLiteral& LiteralA, const FMetasoundFrontendLiteral& LiteralB)
{
	return LiteralA.IsEqual(LiteralB);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateBoolMetaSoundLiteral(bool Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateBoolArrayMetaSoundLiteral(
	const TArray<bool>& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateFloatMetaSoundLiteral(float Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateFloatArrayMetaSoundLiteral(
	const TArray<float>& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateIntMetaSoundLiteral(int32 Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateIntArrayMetaSoundLiteral(
	const TArray<int32>& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateObjectMetaSoundLiteral(UObject* Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateObjectArrayMetaSoundLiteral(
	const TArray<UObject*>& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateStringMetaSoundLiteral(const FString& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateStringArrayMetaSoundLiteral(
	const TArray<FString>& Value)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(Value);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromParam(
	const FAudioParameter& Param)
{
	return FMetasoundFrontendLiteral{ Param };
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromBoolean(const bool InBoolean)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InBoolean);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromBooleanArray(const TArray<bool>& InBooleanArray)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InBooleanArray);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromFloat(const float InFloat)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InFloat);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromFloatArray(const TArray<float>& InFloatArray)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InFloatArray);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromInteger(const int32 InInteger)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InInteger);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromIntegerArray(const TArray<int32>& InIntegerArray)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InIntegerArray);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromObject(UObject* InObject)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InObject);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromObjectArray(const TArray<UObject*>& InObjectArray)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InObjectArray);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromString(const FString& InString)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InString);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromStringArray(const TArray<FString>& InStringArray)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::CreatePODMetaSoundLiteral(InStringArray);
}

FMetasoundFrontendLiteral UMetasoundFrontendLiteralBlueprintAccess::CreateMetaSoundLiteralFromAudioParameter(const FAudioParameter& InAudioParameter)
{
	return FMetasoundFrontendLiteral{ InAudioParameter };
}

bool UMetasoundFrontendLiteralBlueprintAccess::GetBoolValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<bool>(Literal, OutResult);
}

TArray<bool> UMetasoundFrontendLiteralBlueprintAccess::GetBoolArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<bool>>(Literal, OutResult);
}

float UMetasoundFrontendLiteralBlueprintAccess::GetFloatValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<float>(Literal, OutResult);
}

TArray<float> UMetasoundFrontendLiteralBlueprintAccess::GetFloatArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<float>>(Literal, OutResult);
}

int32 UMetasoundFrontendLiteralBlueprintAccess::GetIntValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<int32>(Literal, OutResult);
}

TArray<int32> UMetasoundFrontendLiteralBlueprintAccess::GetIntArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<int32>>(Literal, OutResult);
}

UObject* UMetasoundFrontendLiteralBlueprintAccess::GetObjectValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<UObject*>(Literal, OutResult);
}

TArray<UObject*> UMetasoundFrontendLiteralBlueprintAccess::GetObjectArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<UObject*>>(Literal, OutResult);
}

FString UMetasoundFrontendLiteralBlueprintAccess::GetStringValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<FString>(Literal, OutResult);
}

TArray<FString> UMetasoundFrontendLiteralBlueprintAccess::GetStringArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<FString>>(Literal, OutResult);
}

bool UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsBool(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<bool>(InLiteral, EMetasoundFrontendLiteralType::Boolean, false);
}

TArray<bool> UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsBoolArray(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<bool>>(InLiteral, EMetasoundFrontendLiteralType::BooleanArray, TArray<bool>());
}

float UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsFloat(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<float>(InLiteral, EMetasoundFrontendLiteralType::Float, 0.f);
}

TArray<float> UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsFloatArray(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<float>>(InLiteral, EMetasoundFrontendLiteralType::FloatArray, TArray<float>());
}

int32 UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsInteger(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<int32>(InLiteral, EMetasoundFrontendLiteralType::Integer, 0);
}

TArray<int32> UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsIntegerArray(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<int32>>(InLiteral, EMetasoundFrontendLiteralType::IntegerArray, TArray<int32>());
}

UObject* UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsObject(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<UObject*>(InLiteral, EMetasoundFrontendLiteralType::UObject, nullptr);
}

TArray<UObject*> UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsObjectArray(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<UObject*>>(InLiteral, EMetasoundFrontendLiteralType::UObjectArray, TArray<UObject*>());
}

FString UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsString(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<FString>(InLiteral, EMetasoundFrontendLiteralType::String, FString());
}

TArray<FString> UMetasoundFrontendLiteralBlueprintAccess::GetMetaSoundLiteralAsStringArray(const FMetasoundFrontendLiteral& InLiteral)
{
	return MetasoundFrontendLiteralBlueprintAccessPrivate::GetPODValueFromMetaSoundLiteral<TArray<FString>>(InLiteral, EMetasoundFrontendLiteralType::StringArray, TArray<FString>());
}
