// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGeneratorHandle.h"

#include "MetasoundGenerator.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"

#include "Async/Async.h"

#include "Components/AudioComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundGeneratorHandle)

namespace Metasound
{
	TMap<FName, FMetasoundGeneratorHandle::FPassthroughAnalyzerInfo> FMetasoundGeneratorHandle::PassthroughAnalyzers{};
	
	FMetasoundGeneratorHandle::FMetasoundGeneratorHandle(FPrivateToken,
	TWeakObjectPtr<UAudioComponent>&& InAudioComponent)
		: AudioComponent(MoveTemp(InAudioComponent))
		, AudioComponentId(AudioComponent.IsValid() ? AudioComponent->GetAudioComponentID() : INDEX_NONE)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::FMetasoundGeneratorHandle);
		
		if (!AudioComponent.IsValid())
		{
			UE_LOGF(LogMetaSound, Error, "Created a FMetaSoundGeneratorHandle with an invalid UAudioComponent.");
			return;
		}
		
		if (AudioComponent->bCanPlayMultipleInstances)
		{
			UE_LOGF(
				LogMetaSound,
				Warning,
				"Created a FMetaSoundGeneratorHandle for a UAudioComponent that is allowed to play multiple instances. This may not work as expected.")
		}
	}
	
	TSharedPtr<FMetasoundGeneratorHandle> FMetasoundGeneratorHandle::Create(
		TWeakObjectPtr<UAudioComponent>&& InAudioComponent)
	{
		TSharedRef<FMetasoundGeneratorHandle> Handle = MakeShared<FMetasoundGeneratorHandle>(
			FPrivateToken{},
			MoveTemp(InAudioComponent));

		if (Handle->IsValid())
		{
			const TWeakObjectPtr<UMetaSoundSource> Source = Handle->GetMetaSoundSource();

			if (!Source.IsValid())
			{
				UE_LOGF(LogMetaSound, Error, "FMetaSoundGeneratorHandle missing source: %ls.", *Handle->ToString());
				return nullptr;
			}
			
			const uint64 AudioComponentId = Handle->GetAudioComponentId();
			TWeakPtr<FMetasoundGenerator> GeneratorForComponent = Source->GetGeneratorForAudioComponent(AudioComponentId);

			// If we have a generator already, set it.
			if (GeneratorForComponent.IsValid())
			{
				Handle->SetGenerator(MoveTemp(GeneratorForComponent));
			}
			
			// Listen for the source creating a new generator
			Handle->GeneratorCreatedDelegateHandle = Source->OnGeneratorInstanceInfoCreated.AddSP(
				Handle,
				&FMetasoundGeneratorHandle::HandleGeneratorCreated);

			// Listen for the generator being destroyed
			Handle->GeneratorDestroyedDelegateHandle = Source->OnGeneratorInstanceInfoDestroyed.AddSP(
				Handle,
				&FMetasoundGeneratorHandle::HandleGeneratorDestroyed);

			return Handle;
		}

		return nullptr;
	}

	// Remove these PRAGMAs when cleaning up the deprecated OnGeneratorIOUpdated. 
	// The curly brace line was throwing errors for usage of the deprecated member, presumably it was doing something to tear it down and complaining about it.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMetasoundGeneratorHandle::~FMetasoundGeneratorHandle()
	{
		check(IsInGameThread());
		
		// unsubscribe from source events
		{
			const TWeakObjectPtr<UMetaSoundSource> Source = GetMetaSoundSource();
			
			if (Source.IsValid())
			{
				Source->OnGeneratorInstanceCreated.Remove(GeneratorCreatedDelegateHandle);
				Source->OnGeneratorInstanceDestroyed.Remove(GeneratorDestroyedDelegateHandle);
			}
		}

		// unset the generator and clean up
		SetGenerator(nullptr);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FMetasoundGeneratorHandle::IsValid() const
	{
		return AudioComponent.IsValid();
	}

	uint64 FMetasoundGeneratorHandle::GetAudioComponentId() const
	{
		return AudioComponentId;
	}

	TSharedPtr<FMetasoundGenerator> FMetasoundGeneratorHandle::GetGenerator() const
	{
		return Generator.Pin();
	}

	void FMetasoundGeneratorHandle::UpdateParameters(const UMetasoundParameterPack& ParameterPack)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::UpdateParameters);

		// Update the latest state
		LatestParameterState = ParameterPack.GetCopyOfParameterStorage();

		// Try to send to the generator
		SendParametersToGenerator();
	}

	bool FMetasoundGeneratorHandle::WatchOutput(
		const FName OutputName,
		const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
		const FName AnalyzerName,
		const FName AnalyzerOutputName)
	{
		return WatchOutputInternal(OutputName, FWatchOutputUnifiedDelegate(OnOutputValueChanged), AnalyzerName, AnalyzerOutputName);
	}

	bool FMetasoundGeneratorHandle::WatchOutput(
		const FName OutputName,
		const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
		const FName AnalyzerName,
		const FName AnalyzerOutputName)
	{
		return WatchOutputInternal(OutputName, FWatchOutputUnifiedDelegate(OnOutputValueChanged), AnalyzerName, AnalyzerOutputName);
	}

	bool FMetasoundGeneratorHandle::UnwatchOutput(
		FName OutputName,
		const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
		FName AnalyzerName,
		FName AnalyzerOutputName)
	{
		return UnwatchOutputInternal(OutputName, FWatchOutputUnifiedDelegate(OnOutputValueChanged), AnalyzerName, AnalyzerOutputName);
	}

	bool FMetasoundGeneratorHandle::UnwatchOutput(
		FName OutputName,
		const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
		FName AnalyzerName,
		FName AnalyzerOutputName)
	{
		return UnwatchOutputInternal(OutputName, FWatchOutputUnifiedDelegate(OnOutputValueChanged), AnalyzerName, AnalyzerOutputName);
	}

	bool FMetasoundGeneratorHandle::UnwatchOutput(
		FName OutputName,
		const FDelegateHandle& OnOutputValueChanged,
		FName AnalyzerName,
		FName AnalyzerOutputName)
	{
		return UnwatchOutputInternal(OutputName, FWatchOutputUnifiedDelegate(OnOutputValueChanged), AnalyzerName, AnalyzerOutputName);
	}

	void FMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
		const FName TypeName,
		const FName AnalyzerName,
		const FName OutputName)
	{
		check(!PassthroughAnalyzers.Contains(TypeName));
		PassthroughAnalyzers.Add(TypeName, { AnalyzerName, OutputName });
	}

	void FMetasoundGeneratorHandle::EnableRuntimeRenderTiming(bool Enable)
	{
		bRuntimeRenderTimingShouldBeEnabled = Enable;

		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			PinnedGenerator->EnableRuntimeRenderTiming(bRuntimeRenderTimingShouldBeEnabled);
		}
	}

	double FMetasoundGeneratorHandle::GetCPUCoreUtilization() const
	{
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			return PinnedGenerator->GetCPUCoreUtilization();
		}

		return 0;
	}

	FString FMetasoundGeneratorHandle::ToString() const
	{
		if (!IsValid())
		{
			return FString::Printf(TEXT("Invalid Handle"));
		}

		check(AudioComponent.IsValid());
		return FString::Printf(TEXT("%s [Id:%lld] with owner %s"), *GetNameSafe(AudioComponent.Get()), AudioComponentId, *GetNameSafe(AudioComponent->GetOwner()));
	}

	void FMetasoundGeneratorHandle::SetGenerator(TWeakPtr<FMetasoundGenerator>&& InGenerator)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::SetGenerator);
		
		check(IsInGameThread());

		// early out if the incoming generator is null and the current generator is already invalid...
		if (!Generator.IsValid() && !InGenerator.IsValid())
		{
			// If the generator is invalid but WAS, at some point, pointing at a generator,
			// we need to execute the OnGeneratorSet delegates so they know the generator 
			// went away!
			if (Generator.GetWeakPtrTypeHash() != 0 && OnGeneratorSet.IsBound())
			{
				OnGeneratorSet.Execute(TWeakPtr<FMetasoundGenerator>(InGenerator));
			}

			// Now reset the generator so it is back to nullptr...
			Generator.Reset();
			return;
		}
		
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			// skip the below logic if we are setting the same generator
			if (InGenerator.HasSameObject(PinnedGenerator.Get()))
			{
				return;
			}

			// clean up if we had another generator
			UnregisterGeneratorEvents();
		}

		// set the cached generator
		Generator = MoveTemp(InGenerator);
		
		// Notify the generator has changed
		if (OnGeneratorSet.IsBound())
		{
			OnGeneratorSet.Execute(TWeakPtr<FMetasoundGenerator>(Generator));
		}

		// We're setting a new generator, so do the setup stuff
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			// Subscribe to generator events
			RegisterGeneratorEvents();
			
			// Update params on the generator
			SendParametersToGenerator();

			// Attach any output watchers we might have
			FixUpOutputWatchers();

			// Enable render timing if appropriate
			PinnedGenerator->EnableRuntimeRenderTiming(bRuntimeRenderTimingShouldBeEnabled);
		}
	}

	void FMetasoundGeneratorHandle::RegisterGeneratorEvents()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::RegisterGeneratorEvents);
		
		check(IsInGameThread());
		
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			// Output watchers
			GeneratorOutputChangedDelegateHandle = PinnedGenerator->OnOutputChanged.AddSP(
				AsShared(),
				&FMetasoundGeneratorHandle::HandleOutputChanged);

			// Graph updated
			{
				FOnSetGraph::FDelegate GraphSetDelegate;
				GraphSetDelegate.BindSP(AsShared(), &FMetasoundGeneratorHandle::HandleGeneratorGraphSet);
				GeneratorGraphSetDelegateHandle = PinnedGenerator->AddGraphSetCallback(MoveTemp(GraphSetDelegate));
			}

			// Vertex interface updated (Live Update support)
			GeneratorVertexInterfaceChangedDelegateHandle = PinnedGenerator->OnVertexInterfaceDataUpdatedWithChanges.AddSP(
				AsShared(),
				&FMetasoundGeneratorHandle::HandleGeneratorVertexInterfaceChanged);
		}
	}

	void FMetasoundGeneratorHandle::UnregisterGeneratorEvents() const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::UnregisterGeneratorEvents);
		
		check(IsInGameThread());
		
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			PinnedGenerator->OnOutputChanged.Remove(GeneratorOutputChangedDelegateHandle);
			PinnedGenerator->RemoveGraphSetCallback(GeneratorGraphSetDelegateHandle);
			PinnedGenerator->OnVertexInterfaceDataUpdatedWithChanges.Remove(GeneratorVertexInterfaceChangedDelegateHandle);
		}
	}

	TWeakObjectPtr<UMetaSoundSource> FMetasoundGeneratorHandle::GetMetaSoundSource() const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::GetMetaSoundSource);

		check(IsInGameThread()); // UAudioComponent::GetSound() isn't thread-safe.

		if (!IsValid())
		{
			return nullptr;
		}

		return Cast<UMetaSoundSource>(AudioComponent->GetSound());
	}

	void FMetasoundGeneratorHandle::SendParametersToGenerator() const
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::SendParametersToGenerator);

		if (!LatestParameterState.IsValid())
		{
			return;
		}
		
		// If we have a generator, enqueue the updated parameter state
		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			PinnedGenerator->QueueParameterPack(LatestParameterState);
		}
	}

	bool FMetasoundGeneratorHandle::WatchOutputInternal(
		const FName OutputName,
		const FWatchOutputUnifiedDelegate& OnOutputValueChanged,
		const FName AnalyzerName,
		const FName AnalyzerOutputName)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::WatchOutputInternal);

		check(IsInGameThread());

		if (!IsValid())
		{
			return false;
		}

		Frontend::FAnalyzerAddress AnalyzerAddress;
		if (!TryCreateAnalyzerAddress(OutputName, AnalyzerName, AnalyzerOutputName, AnalyzerAddress))
		{
			return false;
		}

		// Create the watcher
		CreateOutputWatcher(AnalyzerAddress, OnOutputValueChanged);

		// Update the generator's analyzers if necessary
		FixUpOutputWatchers();

		return true;
	}

	bool FMetasoundGeneratorHandle::UnwatchOutputInternal(
		const FName OutputName,
		const FWatchOutputUnifiedDelegate& OnOutputValueChanged,
		const FName AnalyzerName,
		const FName AnalyzerOutputName)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::UnwatchOutputInternal);

		check(IsInGameThread());

		if (!IsValid())
		{
			return false;
		}

		Frontend::FAnalyzerAddress AnalyzerAddress;
		if (!TryCreateAnalyzerAddress(OutputName, AnalyzerName, AnalyzerOutputName, AnalyzerAddress))
		{
			return false;
		}

		// Remove the watcher
		RemoveOutputWatcher(AnalyzerAddress, OnOutputValueChanged);

		// Update the generator's analyzers if necessary
		FixUpOutputWatchers();

		return true;
	}

	void FMetasoundGeneratorHandle::UpdateOutputWatchersInternal()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::UpdateOutputWatchersInternal);

		check(IsInGameThread());

		// Clear the flag *before* we drain the queue, so we don't leave any output updates behind.
		OutputWatcherUpdateScheduled.clear();

		int32 NumDequeued = 0;

		while (TOptional<FOutputPayload> ChangedOutput = ChangedOutputs.Dequeue())
		{
			const FOutputWatcherKey WatcherKey
			{
				ChangedOutput->OutputName,
				ChangedOutput->AnalyzerName,
				ChangedOutput->OutputValue.Name
			};

			if (const FOutputWatcher* Watcher = OutputWatchers.Find(WatcherKey))
			{
				Watcher->OnOutputValueChanged.Broadcast(ChangedOutput->OutputName, ChangedOutput->OutputValue);
			}

			++NumDequeued;
		}

		ChangedOutputsQueueCount.store(FMath::Max(0, ChangedOutputsQueueCount.load() - NumDequeued));
	}

	bool FMetasoundGeneratorHandle::TryCreateAnalyzerAddress(
		const FName OutputName,
		const FName AnalyzerName,
		const FName AnalyzerOutputName,
		Frontend::FAnalyzerAddress& OutAnalyzerAddress)
	{
		// Make the analyzer address.
		Frontend::FAnalyzerAddress AnalyzerAddress;
		AnalyzerAddress.InstanceID = GetAudioComponentId();
		AnalyzerAddress.OutputName = OutputName;
		AnalyzerAddress.AnalyzerName = AnalyzerName;
		AnalyzerAddress.AnalyzerMemberName = AnalyzerOutputName;
		AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();

		// Find the output node and get the data type/node id from that
		{
			const TWeakObjectPtr<UMetaSoundSource> Source = GetMetaSoundSource();

			if (!Source.IsValid())
			{
				UE_LOGF(LogMetaSound, Warning, "Couldn't find the MetaSound Source");
				return false;
			}

			// Find the node id and type name
			const FMetasoundFrontendClassOutput* OutputPtr =
				Source->GetConstDocument().RootGraph.GetDefaultInterface().Outputs.FindByPredicate(
					[&AnalyzerAddress](const FMetasoundFrontendClassOutput& Output)
					{
						return Output.Name == AnalyzerAddress.OutputName;
					});

			if (nullptr == OutputPtr)
			{
				return false;
			}

			AnalyzerAddress.NodeID = OutputPtr->NodeID;
			AnalyzerAddress.DataType = OutputPtr->TypeName;
		}

		// If no analyzer name was provided, try to find a passthrough analyzer
		if (AnalyzerAddress.AnalyzerName.IsNone())
		{
			if (!PassthroughAnalyzers.Contains(AnalyzerAddress.DataType))
			{
				return false;
			}

			AnalyzerAddress.AnalyzerName = PassthroughAnalyzers[AnalyzerAddress.DataType].AnalyzerName;
			AnalyzerAddress.AnalyzerMemberName = PassthroughAnalyzers[AnalyzerAddress.DataType].OutputName;
		}

		// Check to see if the analyzer exists
		{
			using namespace Metasound::Frontend;
			const IVertexAnalyzerFactory* Factory =
				IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(AnalyzerAddress.AnalyzerName);

			if (nullptr == Factory)
			{
				return false;
			}
		}

		OutAnalyzerAddress = AnalyzerAddress;
		return true;
	}

	void FMetasoundGeneratorHandle::FixUpOutputWatchers()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::FixUpOutputWatchers);

		check(IsInGameThread());

		if (!IsValid())
		{
			return;
		}

		if (const TSharedPtr<FMetasoundGenerator> PinnedGenerator = Generator.Pin())
		{
			// For each watcher, make sure the generator has a corresponding analyzer
			// (will fail gracefully on duplicates or non-existent outputs)
			// we can also remove any analyzer that has no further bindings
			for (const auto& Watcher : OutputWatchers)
			{
				if (Watcher.Value.OnOutputValueChanged.IsBound())
				{
					PinnedGenerator->AddOutputVertexAnalyzer(Watcher.Value.AnalyzerAddress);
				}
				else
				{
					PinnedGenerator->RemoveOutputVertexAnalyzer(Watcher.Value.AnalyzerAddress);
				}
			}
		}
	}

	void FMetasoundGeneratorHandle::CreateOutputWatcher(
		const Frontend::FAnalyzerAddress& AnalyzerAddress,
		const FWatchOutputUnifiedDelegate& OnOutputValueChanged)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::CreateOutputWatcher);

		check(IsInGameThread()); // modifying watchers isn't thread-safe
		
		// If we already have a watcher for this output, just add the delegate to that one
		const FOutputWatcherKey WatcherKey
		{
			AnalyzerAddress.OutputName,
			AnalyzerAddress.AnalyzerName,
			AnalyzerAddress.AnalyzerMemberName
		};

		if (FOutputWatcher* Watcher = OutputWatchers.Find(WatcherKey))
		{
			Watcher->OnOutputValueChanged.Add(OnOutputValueChanged);
		}
		// Otherwise add a new watcher
		else
		{
			OutputWatchers.Emplace(WatcherKey, { AnalyzerAddress, OnOutputValueChanged });
		}
	}

	void FMetasoundGeneratorHandle::RemoveOutputWatcher(
		const Frontend::FAnalyzerAddress& AnalyzerAddress,
		const FWatchOutputUnifiedDelegate& OnOutputValueChanged)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::RemoveOutputWatcher);

		check(IsInGameThread()); // modifying watchers isn't thread-safe

		// Watcher must exist in order to be removed
		const FOutputWatcherKey WatcherKey
		{
			AnalyzerAddress.OutputName,
			AnalyzerAddress.AnalyzerName,
			AnalyzerAddress.AnalyzerMemberName
		};

		if (FOutputWatcher* Watcher = OutputWatchers.Find(WatcherKey))
		{
			Watcher->OnOutputValueChanged.Remove(OnOutputValueChanged);
		}		
	}

	void FMetasoundGeneratorHandle::HandleGeneratorCreated(const FGeneratorInstanceInfo& GeneratorInfo)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::HandleGeneratorCreated);
					
		if (GeneratorInfo.AudioComponentID == GetAudioComponentId())
		{
			CurrentGeneratorInstanceID = GeneratorInfo.InstanceID;

			// Set the generator on the game thread. We grab a weak pointer in case this gets destroyed while we wait.
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = AsWeak(), WeakGenerator = GeneratorInfo.Generator]()
			{
				if (const TSharedPtr<FMetasoundGeneratorHandle> PinnedThis = WeakThis.Pin())
				{
					PinnedThis->SetGenerator(TWeakPtr<FMetasoundGenerator>(WeakGenerator));
				}
			});
		}
	}

	void FMetasoundGeneratorHandle::HandleGeneratorDestroyed(const FGeneratorInstanceInfo& GeneratorInfo)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::HandleGeneratorDestroyed);

		if (GeneratorInfo.AudioComponentID == GetAudioComponentId() && GeneratorInfo.InstanceID == CurrentGeneratorInstanceID)
		{
			// Unset the generator on the game thread. We grab a weak pointer in case this gets destroyed while we wait.
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = AsWeak()]()
			{
				if (const TSharedPtr<FMetasoundGeneratorHandle> PinnedThis = WeakThis.Pin())
				{
					PinnedThis->SetGenerator(nullptr);
				}
			});
		}
	}

	void FMetasoundGeneratorHandle::HandleGeneratorGraphSet()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::HandleGeneratorGraphSet);

		// Defer to the game thread. We grab a weak pointer in case this gets destroyed while we wait.
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = AsWeak()]()
		{
			if (const TSharedPtr<FMetasoundGeneratorHandle> PinnedThis = WeakThis.Pin())
			{
				PinnedThis->SendParametersToGenerator();
				PinnedThis->FixUpOutputWatchers();

				if (PinnedThis->OnGraphUpdated.IsBound())
				{
					PinnedThis->OnGraphUpdated.Execute();
				}
			}
		});
	}

	void FMetasoundGeneratorHandle::HandleGeneratorVertexInterfaceChanged(const TArray<FVertexInterfaceChange>& VertexInterfaceChanges)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::HandleGeneratorVertexInterfaceChanged);

		// Defer to the game thread. We grab a weak pointer in case this gets destroyed while we wait.
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = AsWeak(), VertexInterfaceChanges]()
		{
			if (const TSharedPtr<FMetasoundGeneratorHandle> PinnedThis = WeakThis.Pin())
			{
				PinnedThis->SendParametersToGenerator();
				PinnedThis->FixUpOutputWatchers();

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (PinnedThis->OnGeneratorIOUpdated.IsBound())
				{
					PinnedThis->OnGeneratorIOUpdated.Execute();
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (PinnedThis->OnGeneratorIOUpdatedWithChanges.IsBound())
				{
					PinnedThis->OnGeneratorIOUpdatedWithChanges.Execute(VertexInterfaceChanges);
				}
			}
		});
	}

	void FMetasoundGeneratorHandle::HandleOutputChanged(
		FName AnalyzerName,
		FName OutputName,
		FName AnalyzerOutputName,
		TSharedPtr<IOutputStorage> OutputData)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundGeneratorHandle::HandleOutputChanged);

		if (ChangedOutputsQueueCount >= ChangedOutputsQueueMax)
		{
			// Log only once per handle
			if (ChangedOutputsQueueShouldLogIfFull.load())
			{
				UE_LOGF(LogMetaSound, Warning, "UMetasoundGeneratorHandle output queue is full.");
				ChangedOutputsQueueShouldLogIfFull.store(false);
			}
			
			return;
		}
		
		ChangedOutputs.Enqueue(AnalyzerName, OutputName, AnalyzerOutputName, OutputData);
		ChangedOutputsQueueCount.fetch_add(1);

		// Drain the queue on the game thread, but don't bother if it's already been scheduled
		if (!OutputWatcherUpdateScheduled.test_and_set())
		{
			// Defer to the game thread. We grab a weak pointer in case this gets destroyed while we wait.
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = AsWeak()]()
			{
				if (const TSharedPtr<FMetasoundGeneratorHandle> PinnedThis = WeakThis.Pin())
				{
					PinnedThis->UpdateOutputWatchersInternal();
				}
			});
		}
	}
}

UMetasoundGeneratorHandle* UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle(UAudioComponent* OnComponent)
{
	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundGeneratorHandle::CreateMetaSoundGeneratorHandle);
	
	if (!OnComponent)
	{
		return nullptr;
	}

	UMetasoundGeneratorHandle* Handle = NewObject<UMetasoundGeneratorHandle>();
	return Handle->InitGeneratorHandle(OnComponent) ? Handle : nullptr;
}

void UMetasoundGeneratorHandle::BeginDestroy()
{
	Super::BeginDestroy();

	GeneratorHandle.Reset();
}

bool UMetasoundGeneratorHandle::IsValid() const
{
	return GeneratorHandle.IsValid() && GeneratorHandle->IsValid();
}

uint64 UMetasoundGeneratorHandle::GetAudioComponentId() const
{
	return IsValid() ? GeneratorHandle->GetAudioComponentId() : INDEX_NONE;
}

bool UMetasoundGeneratorHandle::ApplyParameterPack(UMetasoundParameterPack* Pack)
{
	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundGeneratorHandle::ApplyParameterPack);
	
	if (nullptr == Pack)
	{
		return false;
	}

	if (IsValid())
	{
		GeneratorHandle->UpdateParameters(*Pack);
		return true;
	}

	return false;
}

TSharedPtr<Metasound::FMetasoundGenerator> UMetasoundGeneratorHandle::GetGenerator() const
{
	return IsValid() ? GeneratorHandle->GetGenerator() : nullptr;
}

FDelegateHandle UMetasoundGeneratorHandle::AddGraphSetCallback(FOnSetGraph::FDelegate&& Delegate)
{
	return OnGeneratorsGraphChanged.Add(MoveTemp(Delegate));
}

bool UMetasoundGeneratorHandle::RemoveGraphSetCallback(const FDelegateHandle& Handle)
{
	return OnGeneratorsGraphChanged.Remove(Handle);
}

bool UMetasoundGeneratorHandle::TryCreateAnalyzerAddress(const FName OutputName, const FName AnalyzerName, const FName AnalyzerOutputName, Metasound::Frontend::FAnalyzerAddress& OutAnalyzerAddress)
{
	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundGeneratorHandle::WatchOutput);

	if (!IsValid())
	{
		return false;
	}

	return GeneratorHandle->TryCreateAnalyzerAddress(OutputName, AnalyzerName, AnalyzerOutputName, OutAnalyzerAddress);
}

bool UMetasoundGeneratorHandle::WatchOutput(
	const FName OutputName,
	const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundGeneratorHandle::WatchOutput);

	if (!IsValid())
	{
		return false;
	}

	return GeneratorHandle->WatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

bool UMetasoundGeneratorHandle::WatchOutput(
	const FName OutputName,
	const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundGeneratorHandle::WatchOutput);

	if (!IsValid())
	{
		return false;
	}

	return GeneratorHandle->WatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

void UMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(
	const FName TypeName,
	const FName AnalyzerName,
	const FName OutputName)
{
	Metasound::FMetasoundGeneratorHandle::RegisterPassthroughAnalyzerForType(TypeName, AnalyzerName, OutputName);
}

void UMetasoundGeneratorHandle::UpdateWatchers() const
{
	// Do nothing. No longer necessary.
}

void UMetasoundGeneratorHandle::EnableRuntimeRenderTiming(const bool Enable) const
{
	if (IsValid())
	{
		GeneratorHandle->EnableRuntimeRenderTiming(Enable);
	}
}

double UMetasoundGeneratorHandle::GetCPUCoreUtilization() const
{
	if (IsValid())
	{
		return GeneratorHandle->GetCPUCoreUtilization();
	}

	return 0;
}

bool UMetasoundGeneratorHandle::InitGeneratorHandle(TWeakObjectPtr<UAudioComponent>&& AudioComponent)
{
	GeneratorHandle = Metasound::FMetasoundGeneratorHandle::Create(MoveTemp(AudioComponent));

	if (!GeneratorHandle.IsValid())
	{
		return false;
	}

	// Attach delegates
	// NB: FMetasoundGeneratorHandle already executes these on the game thread,
	// and its lifetime is tied to UMetasoundGeneratorHandle's lifetime,
	// so we can guarantee the this pointer is valid when these get called.
	GeneratorHandle->OnGeneratorSet.BindLambda([this](TWeakPtr<Metasound::FMetasoundGenerator>&& Generator)
	{
		if (Generator.IsValid())
		{
			OnGeneratorHandleAttached.Broadcast();
		}
		else
		{
			OnGeneratorHandleDetached.Broadcast();
		}
	});

	GeneratorHandle->OnGraphUpdated.BindLambda([this]()
	{
		OnGeneratorsGraphChanged.Broadcast();
	});

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GeneratorHandle->OnGeneratorIOUpdated.BindLambda([this]()
	{
		OnIOUpdated.Broadcast();
	});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GeneratorHandle->OnGeneratorIOUpdatedWithChanges.BindLambda([this](const TArray<Metasound::FVertexInterfaceChange>& VertexInterfaceChanges)
	{
		OnIOUpdatedWithChanges.Broadcast(VertexInterfaceChanges);
	});

	return true;
}
