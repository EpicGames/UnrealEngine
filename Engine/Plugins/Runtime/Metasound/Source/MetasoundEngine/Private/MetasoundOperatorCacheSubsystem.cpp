// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundOperatorCacheSubsystem.h"

#include "AudioMixerDevice.h"
#include "MetasoundAssetManager.h"
#include "MetasoundGenerator.h"
#include "MetasoundGeneratorModule.h"
#include "MetasoundOperatorCache.h"
#include "Modules/ModuleManager.h"
#include "Misc/Optional.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundOperatorCacheSubsystem)

namespace Metasound::OperatorCachePrivate
{
	static bool bOperatorPrecacheEnabled = true;
	static FAutoConsoleVariableRef CVarOperatorPrecacheEnabled(
		TEXT("au.MetaSound.OperatorCache.EnablePrecache"),
		bOperatorPrecacheEnabled,
		TEXT("If precaching metasound operators via the UMetaSoundCacheSubsystem is enabled.")
	);

	TOptional<FGeneratorInitParams> CreateInitParams(UMetaSoundSource& InMetaSound, const FSoundGeneratorInitParams& InParams)
	{
		using namespace Metasound::Frontend;
		using namespace Metasound::Engine;
		using namespace Metasound::SourcePrivate;

		// Dynamic MetaSounds cannot be precached
		if (InMetaSound.IsDynamic())
		{
			return { };
		}

		FOperatorSettings InSettings = InMetaSound.GetOperatorSettings(static_cast<FSampleRate>(InParams.SampleRate));
		FMetasoundEnvironment Environment = InMetaSound.CreateEnvironment(InParams);

		FOperatorBuilderSettings BuilderSettings = FOperatorBuilderSettings::GetDefaultSettings();
		// Graph analyzer currently only enabled for preview sounds (but can theoretically be supported for all sounds)
		BuilderSettings.bPopulateInternalDataReferences = InParams.bIsPreviewSound;
	
		// Resolve format and channel count via the centralized method on UMetaSoundSource.
		// Empty default parameters — precaching has no runtime parameter overrides.
		const UMetaSoundSource::FResolvedSourceFormat ResolvedFormat = InMetaSound.ResolveSourceFormatAndNumChannels(InParams, {}, Environment);
		if (ResolvedFormat.NumChannels == INDEX_NONE)
		{
			return {};
		}

		return FGeneratorInitParams
		{
			.OperatorSettings = InSettings,
			.BuilderSettings = MoveTemp(BuilderSettings),
			.Graph = { }, // Will be retrieved from the FrontEnd Registry in FOperatorPool::BuildAndAddOperator()
			.Environment = Environment,
			.AudioOutputNames = InMetaSound.GetOutputAudioChannelOrder(),
			.bBuildSynchronous = true,
			.AssetPath = FTopLevelAssetPath(&InMetaSound),
			.SourceFormatName = ResolvedFormat.SourceFormatName,
		};
	}
} // namespace Metasound::OperatorCachePrivate

bool UMetaSoundCacheSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return !IsRunningDedicatedServer();
}

void UMetaSoundCacheSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	using namespace Audio;
	using namespace Metasound;

	const FMixerDevice* MixerDevice = GetMixerDevice();

	if (ensure(MixerDevice))
	{
		BuildParams.AudioDeviceID = GetAudioDeviceHandle().GetDeviceID();
		BuildParams.SampleRate = MixerDevice->GetSampleRate();
		BuildParams.AudioMixerNumOutputFrames = MixerDevice->GetNumOutputFrames();
		BuildParams.NumChannels = MixerDevice->GetNumDeviceChannels();
		BuildParams.NumFramesPerCallback = 0;
		BuildParams.InstanceID = 0;
	}
}

void UMetaSoundCacheSubsystem::Update()
{
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
	using namespace Metasound;
	if (TSharedPtr<FOperatorPool> OperatorPool = FModuleManager::GetModuleChecked<IMetasoundGeneratorModule>("MetasoundGenerator").GetOperatorPool())
	{
		OperatorPool->UpdateHitRateTracker();
	}
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED
}

void UMetaSoundCacheSubsystem::PrecacheMetaSoundInternal(UMetaSoundSource* InMetaSound, int32 InNumInstances, bool bTouchExisting)
{
	using namespace Audio;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	if (!OperatorCachePrivate::bOperatorPrecacheEnabled)
	{
		UE_LOGF(LogMetaSound, Log, "Ignoring PrecacheMetaSound request since au.MetaSound.OperatorCache.EnablePrecache is false.");
		return;
	}

	IMetasoundGeneratorModule* Module = FModuleManager::GetModulePtr<IMetasoundGeneratorModule>("MetasoundGenerator");
	if (!ensure(Module))
	{
		return;
	}

	TSharedPtr<FOperatorPool> OperatorPool = Module->GetOperatorPool();
	if (!ensure(OperatorPool))
	{
		return;
	}

	if (!InMetaSound)
	{
		UE_LOGF(LogMetaSound, Error, "PrecacheMetaSound called without being provided a MetaSound, ignoring request");
		return;
	}

	if (InNumInstances < 1)
	{
		UE_LOGF(LogMetaSound, Error, "PrecacheMetaSound called with invalid NumInstances %i, ignoring request", InNumInstances);
		return;
	}

	InMetaSound->InitResources();

	BuildParams.GraphName = InMetaSound->GetOwningAssetName();
	TOptional<FGeneratorInitParams> InitParams = OperatorCachePrivate::CreateInitParams(*InMetaSound, BuildParams);
	if (!InitParams.IsSet())
	{
		return;
	}

	// Graph inflation may interact with cache. Need to find the same graph registry key
	// that is found when a MetaSound generator is created. 
	const UMetaSoundSource&	NoninflatableSource = InMetaSound->FindFirstNoninflatableSource(InitParams->Environment, [](const UMetaSoundSource&){});

	FGuid AssetID;
	const FMetasoundFrontendClassName& ClassName = InMetaSound->GetConstDocument().RootGraph.Metadata.GetClassName();
	if (!IMetaSoundAssetManager::GetChecked().TryGetAssetIDFromClassName(ClassName, AssetID))
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to retrieve MetaSoundClassName when precaching operator for MetaSound '%ls'", *InMetaSound->GetPathName());
		return;
	}

	TUniquePtr<FOperatorBuildData> Data = MakeUnique<FOperatorBuildData>(
		  MoveTemp(InitParams.GetValue())
		, NoninflatableSource.GetGraphRegistryKey()
		, AssetID
		, InNumInstances
		, bTouchExisting
	);

	OperatorPool->BuildAndAddOperator(MoveTemp(Data));
}

void UMetaSoundCacheSubsystem::PrecacheMetaSound(UMetaSoundSource* InMetaSound, int32 InNumInstances)
{
	constexpr bool bTouchExisting = false;
	PrecacheMetaSoundInternal(InMetaSound, InNumInstances, bTouchExisting);
}

void UMetaSoundCacheSubsystem::TouchOrPrecacheMetaSound(UMetaSoundSource* InMetaSound, int32 InNumInstances)
{
	constexpr bool bTouchExisting = true;
	PrecacheMetaSoundInternal(InMetaSound, InNumInstances, bTouchExisting);
}

void UMetaSoundCacheSubsystem::RemoveCachedOperatorsForMetaSound(UMetaSoundSource* InMetaSound)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	// Note: we're not checking the bOperatorPrecacheEnabled cvar here in case it was disable after some sounds had already been cached.
	// If nothing is cached this will do very little.

	if (!InMetaSound)
	{
		UE_LOGF(LogMetaSound, Warning, "Remove Cached Operators called without being provided a MetaSound, ignoring request");
		return;
	}

	IMetasoundGeneratorModule* Module = FModuleManager::GetModulePtr<IMetasoundGeneratorModule>("MetasoundGenerator");
	if (!ensure(Module))
	{
		return;
	}

	if (TSharedPtr<FOperatorPool> OperatorPool = Module->GetOperatorPool())
	{
		FGuid AssetID;
		const FMetasoundFrontendClassName& ClassName = InMetaSound->GetConstDocument().RootGraph.Metadata.GetClassName();
		if (IMetaSoundAssetManager::GetChecked().TryGetAssetIDFromClassName(ClassName, AssetID))
		{
			OperatorPool->RemoveOperatorsWithAssetClassID(AssetID);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to retrieve MetaSoundClassName when removing cached operator for MetaSound '%ls'", *InMetaSound->GetPathName());
		}
	}
}
