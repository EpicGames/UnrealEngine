// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOutput.h"

#include "MetasoundPrimitives.h"
#include "MetasoundTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundOutput)

FMetaSoundOutput::FMetaSoundOutput(const FMetaSoundOutput& Other)
{
	Name = Other.Name;

	if (Other.IsValid())
	{
		Data = Other.Data->Clone();
	}
}

FMetaSoundOutput& FMetaSoundOutput::operator=(const FMetaSoundOutput& Other)
{
	if (this != &Other)
	{
		this->Name = Other.Name;
		
		if (Other.IsValid())
		{
			Data = Other.Data->Clone();
		}
	}

	return *this;
}

FMetaSoundOutput::FMetaSoundOutput(FMetaSoundOutput&& Other) noexcept
{
	Name = MoveTemp(Other.Name);
	Data = MoveTemp(Other.Data);
}

FMetaSoundOutput& FMetaSoundOutput::operator=(FMetaSoundOutput&& Other) noexcept
{
	if (this != &Other)
	{
		Name = MoveTemp(Other.Name);
		Data = MoveTemp(Other.Data);
	}

	return *this;
}

FMetaSoundOutput::FMetaSoundOutput(FName InName, const TSharedPtr<Metasound::IOutputStorage>& InData)
{
	Name = MoveTemp(InName);
	if (InData.IsValid())
	{
		Data = InData->Clone();
	}
}

bool FMetaSoundOutput::IsValid() const
{
	return nullptr != Data;
}

FName FMetaSoundOutput::GetDataTypeName() const
{
	return IsValid() ? Data->GetDataTypeName() : FName();
}

bool UMetasoundOutputBlueprintAccess::IsFloat(const FMetaSoundOutput& Output)
{
	return Output.IsType<float>();
}

float UMetasoundOutputBlueprintAccess::GetFloat(const FMetaSoundOutput& Output, bool& Success)
{
	float Value = 0;
	Success = Output.Get(Value);
	return Value;
}

bool UMetasoundOutputBlueprintAccess::IsInt32(const FMetaSoundOutput& Output)
{
	return Output.IsType<int32>();
}

int32 UMetasoundOutputBlueprintAccess::GetInt32(const FMetaSoundOutput& Output, bool& Success)
{
	int32 Value = 0;
	Success = Output.Get(Value);
	return Value;
}

bool UMetasoundOutputBlueprintAccess::IsBool(const FMetaSoundOutput& Output)
{
	return Output.IsType<bool>();
}

bool UMetasoundOutputBlueprintAccess::GetBool(const FMetaSoundOutput& Output, bool& Success)
{
	bool Value = false;
	Success = Output.Get(Value);
	return Value;
}

bool UMetasoundOutputBlueprintAccess::IsString(const FMetaSoundOutput& Output)
{
	return Output.IsType<FString>();
}

FString UMetasoundOutputBlueprintAccess::GetString(const FMetaSoundOutput& Output, bool& Success)
{
	FString Value;
	Success = Output.Get(Value);
	return Value;
}

bool UMetasoundOutputBlueprintAccess::IsTime(const FMetaSoundOutput& Output)
{
	return Output.IsType<Metasound::FTime>();
}

double UMetasoundOutputBlueprintAccess::GetTimeSeconds(const FMetaSoundOutput& Output, bool& Success)
{
	Metasound::FTime TimeValue;
	Success = Output.Get(TimeValue);
	return TimeValue.GetSeconds();
}
