// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOutputSubsystem.h"

#include "MetasoundTrace.h"
#include "Components/AudioComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundOutputSubsystem)

bool HandleIsValid(const TSharedPtr<Metasound::FMetasoundGeneratorHandle>& Handle)
{
	return Handle.IsValid() && Handle->IsValid();
}

bool UMetaSoundOutputSubsystem::WatchOutput(
	UAudioComponent* AudioComponent,
	const FName OutputName,
	const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::WatchOutput_Dynamic);

	const TSharedPtr<Metasound::FMetasoundGeneratorHandle> Handle = GetOrCreateGeneratorHandle(AudioComponent);

	if (!HandleIsValid(Handle))
	{
		return false;
	}

	return Handle->WatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

bool UMetaSoundOutputSubsystem::WatchOutput(
	UAudioComponent* AudioComponent,
	const FName OutputName,
	const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::WatchOutput_Native);

	const TSharedPtr<Metasound::FMetasoundGeneratorHandle> Handle = GetOrCreateGeneratorHandle(AudioComponent);

	if (!HandleIsValid(Handle))
	{
		return false;
	}

	return Handle->WatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

bool UMetaSoundOutputSubsystem::UnwatchOutput(
	UAudioComponent* AudioComponent,
	const FName OutputName,
	const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::UnwatchOutput_Dynamic);

	const TSharedPtr<Metasound::FMetasoundGeneratorHandle> Handle = GetOrCreateGeneratorHandle(AudioComponent);

	if (!HandleIsValid(Handle))
	{
		return false;
	}

	return Handle->UnwatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

bool UMetaSoundOutputSubsystem::UnwatchOutput(
	UAudioComponent* AudioComponent,
	const FName OutputName,
	const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
	const FName AnalyzerName,
	const FName AnalyzerOutputName)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::UnwatchOutput_Native);

	const TSharedPtr<Metasound::FMetasoundGeneratorHandle> Handle = GetOrCreateGeneratorHandle(AudioComponent);

	if (!HandleIsValid(Handle))
	{
		return false;
	}

	return Handle->UnwatchOutput(OutputName, OnOutputValueChanged, AnalyzerName, AnalyzerOutputName);
}

TSharedPtr<Metasound::FMetasoundGeneratorHandle> UMetaSoundOutputSubsystem::GetOrCreateGeneratorHandle(UAudioComponent* AudioComponent)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::GetOrCreateGeneratorHandle);
	
	check(IsInGameThread());
	
	CleanUpInvalidGeneratorHandles();
	
	if (nullptr == AudioComponent)
	{
		return nullptr;
	}

	// Try to find an existing handle
	const uint64 AudioComponentId = AudioComponent->GetAudioComponentID();
	
	if (const TSharedPtr<Metasound::FMetasoundGeneratorHandle>* FoundHandle = TrackedGenerators.FindByPredicate(
		[AudioComponentId](const TSharedPtr<Metasound::FMetasoundGeneratorHandle>& ExistingHandle)
		{
			return HandleIsValid(ExistingHandle) && ExistingHandle->GetAudioComponentId() == AudioComponentId;
		}))
	{
		return *FoundHandle;
	}
	
	// Create a new one
	{
		const TSharedPtr<Metasound::FMetasoundGeneratorHandle> Handle = Metasound::FMetasoundGeneratorHandle::Create(AudioComponent);

		if (HandleIsValid(Handle))
		{
			TrackedGenerators.Add(Handle);
			return Handle;
		}
	}

	return nullptr;
}

void UMetaSoundOutputSubsystem::CleanUpInvalidGeneratorHandles()
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetasoundOutputSubsystem::CleanUpInvalidGeneratorHandles);

	TrackedGenerators.RemoveAll([](const TSharedPtr<Metasound::FMetasoundGeneratorHandle>& Handle)
	{
		return !HandleIsValid(Handle);
	});
}
