// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundSettings.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "HAL/IConsoleManager.h"
#include "MetasoundFrontendDocument.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundSettings)

#define LOCTEXT_NAMESPACE "MetaSound"


namespace Metasound::SettingsPrivate
{
#if WITH_EDITOR
	template<typename SettingsStructType>
	TSet<FName> GetStructNames(const TArray<SettingsStructType>& InSettings, int32 IgnoreIndex = INDEX_NONE)
	{
		TSet<FName> Names;
		for (int32 Index = 0; Index < InSettings.Num(); ++Index)
		{
			if (Index != IgnoreIndex)
			{
				Names.Add(InSettings[Index].Name);
			}
		}
		return Names;
	}

	/** Generate new name for the item. **/
	static FName GenerateUniqueName(const TSet<FName>& Names, const TCHAR* InBaseName)
	{
		FString NewName = InBaseName;
		for (int32 Postfix = 1; Names.Contains(*NewName); ++Postfix)
		{
			NewName = FString::Format(TEXT("{0}_{1}"), { InBaseName, Postfix });
		}
		return FName(*NewName);
	}

	template<typename SettingsStructType>
	void OnCreateNewSettingsStruct(const TArray<SettingsStructType>& InSettings, const FString& InBaseName, SettingsStructType& OutNewItem)
	{
		const TSet<FName> Names = GetStructNames(InSettings);
		OutNewItem.Name = GenerateUniqueName(Names, *InBaseName);
		OutNewItem.UniqueId = FGuid::NewGuid();
	}

	template<typename SettingsStructType>
	void OnRenameSettingsStruct(const TArray<SettingsStructType>& InSettings, int32 Index, const FString& InBaseName, SettingsStructType& OutRenamed)
	{
		if (OutRenamed.Name.IsNone())
		{
			const TSet<FName> Names = GetStructNames(InSettings);
			OutRenamed.Name = GenerateUniqueName(Names, *InBaseName);
		}
		else
		{
			const TSet<FName> Names = GetStructNames(InSettings, Index);
			if (Names.Contains(OutRenamed.Name))
			{
				OutRenamed.Name = GenerateUniqueName(Names, *OutRenamed.Name.ToString());
			}
		}
	}
#endif // WITH_EDITOR

	template<typename SettingsStructType>
	const SettingsStructType* FindSettingsStruct(const TArray<SettingsStructType>& Settings, const FGuid& InUniqueID)
	{
		auto MatchesIDPredicate = [&InUniqueID](const SettingsStructType& Struct) { return Struct.UniqueId == InUniqueID; };
		return Settings.FindByPredicate(MatchesIDPredicate);
	}

	template<typename SettingsStructType>
	const SettingsStructType* FindSettingsStruct(const TArray<SettingsStructType>& Settings, FName Name)
	{
		auto MatchesNamePredicate = [Name](const SettingsStructType& Struct) { return Struct.Name == Name; };
		return Settings.FindByPredicate(MatchesNamePredicate);
	}

#if WITH_EDITOR
	template<typename SettingsStructType>
	void PostEditChainChangedStructMember(FPropertyChangedChainEvent& PostEditChangeChainProperty, TArray<SettingsStructType>& StructSettings, FName PropertyName, const FString& NewItemName)
	{
		const int32 ItemIndex = PostEditChangeChainProperty.GetArrayIndex(PropertyName.ToString());

		if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* HeadNode = PostEditChangeChainProperty.PropertyChain.GetHead())
		{
			const FProperty* Prop = HeadNode->GetValue();
			if (Prop->GetName() != PropertyName)
			{
				return;
			}
		}

		// Item changed..
		if (ItemIndex != INDEX_NONE && StructSettings.IsValidIndex(ItemIndex))
		{
			SettingsStructType& Item = StructSettings[ItemIndex];
			if (PostEditChangeChainProperty.GetPropertyName() == "Name")
			{
				OnRenameSettingsStruct<SettingsStructType>(StructSettings, ItemIndex, NewItemName, Item);
			}
			else if (PostEditChangeChainProperty.GetPropertyName() == PropertyName)
			{
				// Array change add or duplicate
				if (PostEditChangeChainProperty.ChangeType == EPropertyChangeType::ArrayAdd
					|| PostEditChangeChainProperty.ChangeType == EPropertyChangeType::Duplicate)
				{
					OnCreateNewSettingsStruct<SettingsStructType>(StructSettings, NewItemName, Item);
				}
			}
		}

		// Handle pasting separately as we might not have a valid index in the case of pasting when array is empty.
		if (PostEditChangeChainProperty.GetPropertyName() == PropertyName)
		{
			// Paste...
			if (PostEditChangeChainProperty.ChangeType == EPropertyChangeType::ValueSet)
			{
				const int32 IndexOfPastedItem = ItemIndex != INDEX_NONE ? ItemIndex : 0;
				if (StructSettings.IsValidIndex(IndexOfPastedItem))
				{
					SettingsStructType& Item = StructSettings[IndexOfPastedItem];
					OnCreateNewSettingsStruct<SettingsStructType>(StructSettings, NewItemName, Item);
				}
			}
		}
	}
#endif // WITH_EDITOR
} // namespace Metasound::SettingsPrivate

#if WITH_EDITOR
bool FMetaSoundPageSettings::GetExcludeFromCook(FName PlatformName) const
{
	if (PlatformCanTargetPage(PlatformName))
	{
		return false;
	}

	return ExcludeFromCook.GetValueForPlatform(PlatformName);
}

TArray<FName> FMetaSoundPageSettings::GetTargetPlatforms() const
{
	TArray<FName> PlatformNames;
	Algo::TransformIf(CanTarget.PerPlatform, PlatformNames,
		[](const TPair<FName, bool>& Pair) { return Pair.Value; },
		[](const TPair<FName, bool>& Pair) { return Pair.Key; });
	return PlatformNames;
}

bool FMetaSoundPageSettings::PlatformCanTargetPage(FName PlatformName) const
{
	const bool bIsTargeted = CanTarget.GetValueForPlatform(PlatformName);
	return bIsTargeted;
}

void UMetaSoundSettings::ConformPageSettings(bool bNotifyDefaultRenamed)
{
	using namespace Metasound;

	DefaultPageSettings.UniqueId = Metasound::Frontend::DefaultPageID;
	DefaultPageSettings.Name = Metasound::Frontend::DefaultPageName;
	DefaultPageSettings.bIsDefaultPage = true;
	DefaultPageSettings.ExcludeFromCook = false;

	bool bInvalidDefaultRenamed = false;
	TMap<FName, bool> PlatformHasTarget;
	auto GatherPlatformTargets = [&PlatformHasTarget](const FMetaSoundPageSettings& Page)
	{
		PlatformHasTarget.FindOrAdd({ }) |= Page.CanTarget.Default;
		for (const TPair<FName, bool>& Pair : Page.CanTarget.PerPlatform)
		{
			PlatformHasTarget.FindOrAdd(Pair.Key) |= Pair.Value;
		}
	};

	GatherPlatformTargets(DefaultPageSettings);
	for (FMetaSoundPageSettings& Page : PageSettings)
	{
		const bool bIsDefaultName = Page.Name == Frontend::DefaultPageName;
		if (bIsDefaultName)
		{
			const TSet<FName> PageNames(GetPageNames());
			Page.Name = SettingsPrivate::GenerateUniqueName(PageNames, *Page.Name.ToString());
			bInvalidDefaultRenamed = true;
		}

		GatherPlatformTargets(Page);

		Page.bIsDefaultPage = false;
	}

	// Forces each platform to target at least one page setting.
 	for (const TPair<FName, bool>& Pair : PlatformHasTarget)
 	{
 		if (!Pair.Value)
 		{
 			if (Pair.Key.IsNone())
 			{
 				DefaultPageSettings.CanTarget.Default = true;
 			}
 			else
 			{
 				DefaultPageSettings.CanTarget.PerPlatform.FindOrAdd(Pair.Key) = true;
 			}
 		}
 	}

#if WITH_EDITORONLY_DATA
	{
		FScopeLock Lock(&CookPlatformTargetCritSec);
		CookPlatformTargetPageIDs.Reset();
		CookPlatformTargetPage = { };
	}
#endif // WITH_EDITORONLY_DATA

	TargetPageNameOverride.Reset();

	for (int32 Index = PageSettings.Num() - 1; Index >= 0; --Index)
	{
		FMetaSoundPageSettings& PageSetting = PageSettings[Index];
		if (PageSetting.UniqueId == Metasound::Frontend::DefaultPageID || PageSetting.Name == Metasound::Frontend::DefaultPageName)
		{
			PageSettings.RemoveAt(Index);
		}
	}

	if (bNotifyDefaultRenamed && bInvalidDefaultRenamed)
	{
		OnDefaultRenamed.Broadcast();
	}
}
#endif // WITH_EDITOR


const FMetaSoundPageSettings* UMetaSoundSettings::FindPageSettings(FName Name) const
{
	if (Name == Metasound::Frontend::DefaultPageName)
	{
		return &GetDefaultPageSettings();
	}
	return Metasound::SettingsPrivate::FindSettingsStruct(PageSettings, Name);
}

const FMetaSoundPageSettings* UMetaSoundSettings::FindPageSettings(const FGuid& InPageID) const
{
	if (InPageID == Metasound::Frontend::DefaultPageID)
	{
		return &GetDefaultPageSettings();
	}
	return Metasound::SettingsPrivate::FindSettingsStruct(PageSettings, InPageID);
}

const FMetaSoundQualitySettings* UMetaSoundSettings::FindQualitySettings(FName Name) const
{
	return Metasound::SettingsPrivate::FindSettingsStruct(QualitySettings, Name);
}

const FMetaSoundQualitySettings* UMetaSoundSettings::FindQualitySettings(const FGuid& InQualityID) const
{
	return Metasound::SettingsPrivate::FindSettingsStruct(QualitySettings, InQualityID);
}

const FMetaSoundPageSettings& UMetaSoundSettings::GetDefaultPageSettings() const
{
	return DefaultPageSettings;
}

#if WITH_EDITOR
TArray<FName> UMetaSoundSettings::GetAllPlatformNamesImplementingTargets() const
{
	TSet<FName> PlatformNames;
	IteratePageSettings([&](const FMetaSoundPageSettings& InPageSetting)
	{
		TArray<FName> PagePlatforms = InPageSetting.GetTargetPlatforms();
		PlatformNames.Append(MoveTemp(PagePlatforms));
	});
	return PlatformNames.Array();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
TArray<FGuid> UMetaSoundSettings::GetCookedTargetPageIDs(FName PlatformName) const
{
	FScopeLock Lock(&CookPlatformTargetCritSec);
	CacheCookedPageIDs(PlatformName);
	return CookPlatformTargetPageIDs;
}

TArray<FGuid> UMetaSoundSettings::GetCookedPageOrder(FName PlatformName) const
{
	FScopeLock Lock(&CookPlatformTargetCritSec);
	CacheCookedPageIDs(PlatformName);
	return CookPlatformPageOrder;
}

void UMetaSoundSettings::CacheCookedPageIDs(FName PlatformName) const
{
	/* Cache the CookPlatformTargetPageIDs and CookPlatformPageOrder to 
	 * avoid rebuilding these for every asset being cooked. */

	if ((PlatformName != CookPlatformTargetPage) || (PlatformName.IsNone() && CookPlatformTargetPageIDs.IsEmpty() && CookPlatformPageOrder.IsEmpty()))
	{
		CookPlatformTargetPage = PlatformName;
		CookPlatformTargetPageIDs.Reset();
		CookPlatformPageOrder.Reset();

		/* Cache CookPlatformTargetPageIDs by finding all target pages for the 
		 * given platform. */
		auto CanTargetPage = [&PlatformName](const FMetaSoundPageSettings& PageSetting)
		{
#if WITH_EDITOR
			return PageSetting.PlatformCanTargetPage(PlatformName);
#else // !WITH_EDITOR
			return true;
#endif // !WITH_EDITOR
		};

		auto GetID = [](const FMetaSoundPageSettings& PageSetting)
		{
			return PageSetting.UniqueId;
		};

		if (CanTargetPage(DefaultPageSettings))
		{
			CookPlatformTargetPageIDs.Add(DefaultPageSettings.UniqueId);
		}

		Algo::TransformIf(PageSettings, CookPlatformTargetPageIDs, CanTargetPage, GetID);


		if (CookPlatformTargetPageIDs.IsEmpty())
		{
#if WITH_EDITOR
			// Allow default page to be a target if no platform name is provided.
			// This can occur when generating reflection code for a MetaSound
			// in a cloud cooker. 
			const bool bCanTargetDefault = PlatformName.IsNone() || DefaultPageSettings.CanTarget.GetValueForPlatform(PlatformName);
#else // !WITH_EDITOR
			const bool bCanTargetDefault = DefaultPageSettings.CanTarget.GetValue();
#endif // !WITH_EDITOR

			if (!PageSettings.IsEmpty() && !bCanTargetDefault)
			{
				UE_LOGF(LogMetaSound, Warning, "No pages set to be targeted for platform '%ls', forcing 'Default' page as target", *PlatformName.ToString());
			}

			CookPlatformTargetPageIDs.Add(Metasound::Frontend::DefaultPageID);
		}


		/* Cache cook target page order by filtering out all pages that are
		 * excluded from the platform and filtering out all pages that come before
		 * any of the cook target page IDs. */
		constexpr bool bReverse = true;
		bool bFoundAnyTargetPage = false;
		IteratePageSettings([&](const FMetaSoundPageSettings& InPageSettings) 
			{
#if WITH_EDITOR
				// Skip this page if it explicitly excluded from cooking.
				if (InPageSettings.GetExcludeFromCook(PlatformName))
				{
					return;
				}
#endif // !WITH_EDITOR
				
				// Skip any pages that occur before the any of the target pages.
				if (!bFoundAnyTargetPage)
				{
					// Sets flag to true if any of the CookPlatformTargetPageIDs equal PageSettings.PageID
					bFoundAnyTargetPage = Algo::AnyOf(CookPlatformTargetPageIDs, [&InPageSettings](const FGuid& InTargetPageID) { return InTargetPageID == InPageSettings.UniqueId; });
				}

				if (bFoundAnyTargetPage)
				{
					CookPlatformPageOrder.Add(InPageSettings.UniqueId);
				}

			}, bReverse);

		if (CookPlatformPageOrder.IsEmpty())
		{
#if WITH_EDITOR
			// Allow default page to be in page order if no platform name is provided.
			// This can occur when generating reflection code for a MetaSound
			// in a cloud cooker. 
			const bool bCanCookDefault = PlatformName.IsNone() || !DefaultPageSettings.ExcludeFromCook.GetValueForPlatform(PlatformName);
#else // !WITH_EDITOR
			const bool bCanCookDefault = !DefaultPageSettings.ExcludeFromCook.GetValue();
#endif // !WITH_EDITOR

			if (!PageSettings.IsEmpty() && !bCanCookDefault)
			{
				UE_LOGF(LogMetaSound, Warning, "No pages set to be cook for platform '%ls', forcing 'Default' page as only cooked page", *PlatformName.ToString());
			}

			CookPlatformPageOrder.Add(Metasound::Frontend::DefaultPageID);
		}
	}
}

void UMetaSoundSettings::IterateCookedTargetPageIDs(FName PlatformName, TFunctionRef<void(const FGuid&)> Iter) const
{
	FScopeLock Lock(&CookPlatformTargetCritSec);
	CacheCookedPageIDs(PlatformName);

	for (const FGuid& CookPlatform : CookPlatformTargetPageIDs)
	{
		Iter(CookPlatform);
	}
}
#endif // WITH_EDITORONLY_DATA

TArray<FGuid> UMetaSoundSettings::GeneratePageOrder() const
{
#if WITH_EDITOR
	if (PageOverrideData.IsSet())
	{
		return GeneratePageOrderFromOverride();
	}
#endif // WITH_EDITOR

	FName PlatformName = FPlatformProperties::IniPlatformName();

	return GeneratePageOrderInternal(PlatformName, GetTargetPageSettings().UniqueId);
};

const FGuid* UMetaSoundSettings::GetPageID(FName PageName)
{
	if (!PageName.IsNone())
	{
		const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
		check(Settings);

		if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageName))
		{
			return &PageSettings->UniqueId;
		}
	}
	return nullptr;
}

FName UMetaSoundSettings::GetPageName(FGuid PageID)
{
	FName PageName = Metasound::Frontend::DefaultPageName;
	const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>();
	check(Settings);
	if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(PageID))
	{
		PageName = PageSettings->Name;
	}
	return PageName;
}

TArrayView<const FGuid> UMetaSoundSettings::GetPageOrder()
{
	// Thread access for the CachedPageOrder is a bit hairy due to the existence
	// of the AudioThread. 
	//
	// For non-editor builds, the page order is cached during PostInitProperties
	// and then never altered. For these builds it is safe to access as long as
	// it has been initialized. 
	//
	// For editor builds, access to the cached paged order is allowed from the 
	// game thread or the audio thread. The audio thread runs in a handful of 
	// editor builds, primarily when running StandaloneGame. In these builds, 
	// there is a codepath where the CachedPageOrder could be altered on the 
	// game thread and read on the audio thread, but this is generally disallowed
	// as StandaloneGame does not expose changes to the UMetaSoundSettings.
#if WITH_EDITOR
	check(IsInGameThread() || IsInAudioThread());
#endif
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		checkf(Settings->bIsPageOrderCached, TEXT("Attempt to get PageOrder before UMetaSoundSettings have finished initializing"));
		return Settings->CachedPageOrder;
	}
	return {};
}

const FMetaSoundPageSettings& UMetaSoundSettings::GetTargetPageSettings() const
{
	auto WarnIfUninitialized = [this](const FMetaSoundPageSettings& SettingsSet)
	{
#if !NO_LOGGING
		if (bWarnAccessBeforeInit)
		{
			UE_LOGF(LogMetaSound, Display, "Target Page Settings accessed prior to 'PostInitProperties' being called.  Uninitialized PageSettings '%ls' being returned.", *SettingsSet.Name.ToString());
			bWarnAccessBeforeInit = false;
		}
#endif // !NO_LOGGING
	};

	const FName TargetPage = TargetPageNameOverride.IsSet() ? *TargetPageNameOverride : TargetPageName;

#if WITH_EDITOR
	if (const FMetaSoundPageSettings* TargetSettings = GetTargetPageSettingsFromOverride(TargetPage))
	{
		WarnIfUninitialized(*TargetSettings);
		return *TargetSettings;
	}
#endif // WITH_EDITOR

	if (const FMetaSoundPageSettings* TargetSettings = FindPageSettings(TargetPage))
	{
		WarnIfUninitialized(*TargetSettings);
		return *TargetSettings;
	}

	// Shouldn't hit this, but if for some reason the target page is in a bad state,
	// try and return any page setting set as a valid target.
	if (PageSettings.Num() > 0)
	{
		WarnIfUninitialized(PageSettings[0]);
		return PageSettings[0];
	}

	WarnIfUninitialized(DefaultPageSettings);
	return DefaultPageSettings;
}

#if WITH_EDITOR
void UMetaSoundSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PostEditChangeChainProperty)
{
	using namespace Metasound::SettingsPrivate;

	PostEditChainChangedStructMember(PostEditChangeChainProperty, PageSettings, GetPageSettingPropertyName(), TEXT("New Page"));
	PostEditChainChangedStructMember(PostEditChangeChainProperty, QualitySettings, GetQualitySettingPropertyName(), TEXT("New Quality"));

	constexpr bool bNotifyDefaultRenamed = true;
	ConformPageSettings(bNotifyDefaultRenamed);

	Super::PostEditChangeChainProperty(PostEditChangeChainProperty);
}

void UMetaSoundSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{	
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	constexpr bool bNotifyDefaultRenamed = true;
	ConformPageSettings(bNotifyDefaultRenamed);

	if (PropertyChangedEvent.MemberProperty->GetName() == GetPageSettingPropertyName())
	{
		OnPageSettingsUpdated.Broadcast();
	}
	UpdateCachedPageOrder();
}

const FMetaSoundPageSettings* UMetaSoundSettings::GetTargetPageSettingsFromOverride(const FName InTargetPage) const
{
	const FMetaSoundPageSettings* TargetSettings = nullptr;

	if (PageOverrideData.IsSet())
	{
		// The override can optionally reference a specific PageID. In that case,
		// the specific page settings are returned. 
		if (PageOverrideData->PageID.IsSet())
		{
			TargetSettings = FindPageSettings(*PageOverrideData->PageID);
			if (TargetSettings)
			{
				return TargetSettings;
			}
		}

		// If the PageID is not explicitly overridden, then search the page settings
		// to find a page that is supported by the platform. 
		constexpr bool bReverse = true;
		bool bFoundTargetPage = false;
		bool bFoundPageSupportedByPlatform = false;

		IteratePageSettings([&](const FMetaSoundPageSettings& InPageSettings)
		{
			bFoundTargetPage |= InPageSettings.Name == InTargetPage;
			if (bFoundTargetPage && !bFoundPageSupportedByPlatform)
			{
				const bool bIsCooked = !InPageSettings.GetExcludeFromCook(PageOverrideData->PlatformName);
				if (bIsCooked)
				{
					bFoundPageSupportedByPlatform = true;
					TargetSettings = &InPageSettings;
				}
			}
		}, bReverse);
	}

	return TargetSettings;
}

TArray<FGuid> UMetaSoundSettings::GeneratePageOrderFromOverride() const
{
	check(PageOverrideData.IsSet());
	FGuid TargetPageID;

	if (PageOverrideData->PageID.IsSet())
	{
		TargetPageID = *PageOverrideData->PageID;
	}
	else
	{
		TargetPageID = GetTargetPageSettings().UniqueId;
	}

	return GeneratePageOrderInternal(PageOverrideData->PlatformName, TargetPageID);
}
#endif // WITH_EDITOR

void UMetaSoundSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	constexpr bool bNotifyDefaultRenamed = false;
	ConformPageSettings(bNotifyDefaultRenamed);
#endif // WITH_EDITOR

#if !NO_LOGGING
	bWarnAccessBeforeInit = false;
#endif // !NO_LOGGING

	if (const FMetaSoundPageSettings* Page = FindPageSettings(TargetPageName))
	{
		UE_LOGF(LogMetaSound, Display, "MetaSound Page Target Initialized to '%ls'", *GetTargetPageSettings().Name.ToString());
	}
	else
	{
		UE_LOGF(LogMetaSound, Warning, "TargetPageName '%ls' at time of 'UMetaSoundSettings::PostInitProperties' did not correspond to a valid page.", *TargetPageName.ToString());
		if (PageSettings.IsEmpty())
		{
			UE_LOGF(LogMetaSound, Warning, "Setting target to '%ls' page settings.", *DefaultPageSettings.Name.ToString());
			TargetPageName = DefaultPageSettings.Name;
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Setting target to highest project page settings '%ls'.", *PageSettings.Last().Name.ToString());
			TargetPageName = PageSettings.Last().Name;
		}
	}

	// Initialize the cached page order
	CachedPageOrder = GeneratePageOrder();
	bIsPageOrderCached = true;
}

bool UMetaSoundSettings::SetTargetPage(FName PageName)
{
#if WITH_EDITOR
	if (const FMetaSoundPageSettings* PageSetting = FindPageSettings(PageName))
	{
		const FName TargetPage = TargetPageNameOverride.IsSet() ? *TargetPageNameOverride : TargetPageName;
		if (TargetPage != PageSetting->Name)
		{
			TargetPageNameOverride = PageSetting->Name;
			UE_LOGF(LogMetaSound, Display, "Target page override set to '%ls'.", *PageSetting->Name.ToString());
			UpdateCachedPageOrder();
			return true;
		}
	}
#else // !WITH_EDITOR
	UE_LOGF(LogMetaSound, Display, "Target page override cannot be set to '%ls'. (Target ID not mutable in runtime builds)", *PageName.ToString());
#endif // !WITH_EDITOR

	return false;
}

#if WITH_EDITORONLY_DATA
Metasound::Engine::FOnSettingsDefaultConformed& UMetaSoundSettings::GetOnDefaultRenamedDelegate()
{
	return OnDefaultRenamed;
}

Metasound::Engine::FOnPageSettingsUpdated& UMetaSoundSettings::GetOnPageSettingsUpdatedDelegate()
{
	return OnPageSettingsUpdated;
}

FName UMetaSoundSettings::GetPageSettingPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UMetaSoundSettings, PageSettings);
}

FName UMetaSoundSettings::GetQualitySettingPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UMetaSoundSettings, QualitySettings);
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

void UMetaSoundSettings::OverrideTargetPageSettings(TOptional<Metasound::Engine::FTargetPageOverride> InOverride)
{
	PageOverrideData = MoveTemp(InOverride);
	UpdateCachedPageOrder();
}

TArray<FName> UMetaSoundSettings::GetPageNames()
{
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		TArray<FName> Names;
		Settings->IteratePageSettings([&Names](const FMetaSoundPageSettings& InPageSetting)
		{
			Names.Add(InPageSetting.Name);
		});
		return Names;
	}

	return { };
}

TArray<FName> UMetaSoundSettings::GetQualityNames()
{
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		TArray<FName> Names;
		auto GetName = [](const FMetaSoundQualitySettings& Quality) { return Quality.Name; };
		Algo::Transform(Settings->GetQualitySettings(), Names, GetName);
		return Names;
	}

	return { };
}

#endif // WITH_EDITOR

void UMetaSoundSettings::UpdateCachedPageOrder()
{
	// CachedPageOrder should only be updated on the GameThread.
	check(IsInGameThread());
	CachedPageOrder = GeneratePageOrder();
}

FName UMetaSoundSettings::GetCategoryName() const
{
	return FName("Plugins");
}

TArray<FGuid> UMetaSoundSettings::GeneratePageOrderInternal(const FName& InPlatformName, const FGuid& InTargetPageID) const
{
	// Reverse iterate over page settings to build the priority stack. Every page
	// that exists before the TargetPageID is discarded. Ever page after the TargetPageID
	// is a fallback in case the TargetPageID does not exist for a given element. 
	TArray<FGuid> PageOrder;

	constexpr bool bReverse = true;
	bool bFoundTargetPage = false;

	IteratePageSettings([&](const FMetaSoundPageSettings& InPageSettings)
	{
		bFoundTargetPage |= InPageSettings.UniqueId == InTargetPageID;
		if (bFoundTargetPage)
		{
#if WITH_EDITOR
			const bool bIsCooked = !InPageSettings.GetExcludeFromCook(InPlatformName);
			if (bIsCooked)
			{
				PageOrder.Add(InPageSettings.UniqueId);
			}
#else
			PageOrder.Add(InPageSettings.UniqueId);
#endif // WITH_EDITOR
		}
	}, bReverse);

	if (PageOrder.Num() < 1)
	{
		PageOrder.Add(Metasound::Frontend::DefaultPageID);
	}

	// rvo
	return PageOrder;
}

void UMetaSoundSettings::IteratePageSettings(TFunctionRef<void(const FMetaSoundPageSettings&)> Iter, bool bReverse) const
{
	if (bReverse)
	{
		for (int32 Index = PageSettings.Num() - 1; Index >= 0; --Index)
		{
			Iter(PageSettings[Index]);
		}
		Iter(GetDefaultPageSettings());
	}
	else
	{
		Iter(GetDefaultPageSettings());
		for (const FMetaSoundPageSettings& Setting : PageSettings)
		{
			Iter(Setting);
		}
	}
}
#undef LOCTEXT_NAMESPACE // MetaSound
