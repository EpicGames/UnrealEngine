// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundSource.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioMixerBlueprintLibrary.h"
#include "Containers/Ticker.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Internationalization/Text.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAudioFormats.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundEngineAsset.h"
#include "MetasoundEngineEnvironment.h"
#include "MetasoundEnvironment.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundGenerator.h"
#include "MetasoundGlobals.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorBuilderSettings.h"
#include "MetasoundOperatorCacheSubsystem.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundPrimitives.h"
#include "MetasoundReceiveNode.h"
#include "MetasoundSettings.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundUObjectRegistry.h"
#include "MetasoundWave.h"
#include "MetasoundChannelAgnosticType.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "DSP/WavFormatUtils.h"
#include "Sound/ISoundWaveContainer.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ScriptInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundSource)


#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif // WITH_EDITOR


#if WITH_EDITORONLY_DATA
#include "EdGraph/EdGraph.h"
#endif // WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "MetaSound"

// Clamp a CAT format's channel count to the nearest mixer-compatible value (1, 2, 4, 6, 8).
// The AudioMixer only supports these counts for buffer allocation and interleaving.
// The actual CAT format identity is carried separately via SourceFormatName for transcoding.
static int32 ClampToMixerChannelCount(int32 InChannels)
{
	if (InChannels <= 1) return 1;
	if (InChannels <= 2) return 2;
	if (InChannels <= 4) return 4;
	if (InChannels <= 6) return 6;
	return 8;
}

namespace Metasound
{
	namespace ConsoleVariables
	{
		bool bEnableExperimentalRuntimePresetGraphInflation = false;
	}

	namespace SourcePrivate
	{
		static constexpr float DefaultBlockRateConstant = 100.f;
		static constexpr float DefaultSampleRateConstant = 48000.f;

		static bool IsCookedForEditor(const FArchive& InArchive, const UObject* InObj)
		{
#if WITH_EDITORONLY_DATA
			return ((InArchive.GetPortFlags() & PPF_Duplicate) == 0) && InObj->GetPackage()->HasAnyPackageFlags(PKG_Cooked);
#else //WITH_EDITORONLY_DATA
			return false;
#endif //WITH_EDITORONLY_DATA
		}
		
		static const FLazyName TriggerName = "Trigger";

		// Holds onto a global static TSet for tracking which error/warning logs have been
		// trigger in order to avoid log spam.
		bool HasNotBeenLoggedForThisObject(const UMetaSoundSource& InMetaSound, uint32 InLogLineNumber)
		{
			using FObjectAddressAndLineNum = TTuple<const void*, uint32>;

			static TSet<FObjectAddressAndLineNum> LoggedSet;

			bool bIsAlreadyInSet = false;
			LoggedSet.Add(FObjectAddressAndLineNum(&InMetaSound, InLogLineNumber), &bIsAlreadyInSet);

			return !bIsAlreadyInSet;
		}

		Frontend::FMetaSoundAssetRegistrationOptions GetInitRegistrationOptions()
		{
			Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
			RegOptions.bForceReregister = false;
#if !WITH_EDITOR 
			// When without editor, don't AutoUpdate or ResolveDocument at runtime. This only happens at cook or save.
			// When with editor, those are needed because sounds are not necessarily saved before previewing.
			RegOptions.bAutoUpdate = false;
#endif // !WITH_EDITOR
			if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
			{
				RegOptions.bAutoUpdateLogWarningOnDroppedConnection = Settings->bAutoUpdateLogWarningOnDroppedConnection;
				RegOptions.PageOrder = Settings->GetPageOrder();
			}

			return RegOptions;
		}

		class FParameterRouter
		{
			struct FQueueState
			{
				TWeakPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> DataChannel;
				bool bWriterAvailable = true;
			};

		public:

			using FAudioDeviceIDAndInstanceID = TTuple<Audio::DeviceID, uint64>;

			TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> FindOrCreateDataChannelForReader(Audio::DeviceID InDeviceID, uint64 InstanceID)
			{
				constexpr bool bIsForWriter = false;
				return FindOrCreateDataChannel(InDeviceID, InstanceID, bIsForWriter);
			}

			TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> FindOrCreateDataChannelForWriter(Audio::DeviceID InDeviceID, uint64 InstanceID)
			{
				constexpr bool bIsForWriter = true;
				return FindOrCreateDataChannel(InDeviceID, InstanceID, bIsForWriter);
			}

		private:

			TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> FindOrCreateDataChannel(Audio::DeviceID InDeviceID, uint64 InstanceID, bool bIsForWriter)
			{
				FScopeLock Lock(&DataChannelMapCS);

				FAudioDeviceIDAndInstanceID Key = {InDeviceID, InstanceID};
				const bool bIsForReader = !bIsForWriter;

				if (FQueueState* State = DataChannels.Find(Key))
				{
					// Allow multiple readers to be returned because FMetaSoundGenerators are recreated when they come out of virtualization.
					// Only allow a single writer to be returned because FMetaSoundParameterTransmitters are only created once
					const bool bIsAvailable = bIsForReader || (State->bWriterAvailable && bIsForWriter);
					if (bIsAvailable)
					{
						TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> Channel = State->DataChannel.Pin();
						if (Channel.IsValid())
						{
							if (bIsForWriter)
							{
								State->bWriterAvailable = false;
							}
							return Channel;
						}
					}
				}

				TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> NewChannel = MakeShared<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>>();

				FQueueState NewState;
				NewState.DataChannel = NewChannel;
				if (bIsForWriter)
				{
					NewState.bWriterAvailable = false;
				}

				DataChannels.Add(Key, NewState);
				return NewChannel;
			}

			FCriticalSection DataChannelMapCS;
			TSortedMap<FAudioDeviceIDAndInstanceID, FQueueState> DataChannels;
		};

		void CreateUObjectProxies(const Frontend::IDataTypeRegistry& InRegistry, FName InVertexTypeName, bool bClearUObjectPointers, FAudioParameter& InOutParamToInit)
		{
			using namespace Metasound;

			switch (InOutParamToInit.ParamType)
			{
				case EAudioParameterType::Object:
				{
					TSharedPtr<Audio::IProxyData> ProxyPtr = InRegistry.CreateProxyFromUObject(InVertexTypeName, InOutParamToInit.ObjectParam);
					InOutParamToInit.ObjectProxies.Emplace(MoveTemp(ProxyPtr));

					if (bClearUObjectPointers)
					{
						InOutParamToInit.ObjectParam = nullptr;
					}
				}
				break;

				case EAudioParameterType::ObjectArray:
				{
					const FName ElementTypeName = CreateElementTypeNameFromArrayTypeName(InVertexTypeName);
					for (TObjectPtr<UObject>& Object : InOutParamToInit.ArrayObjectParam)
					{
						TSharedPtr<Audio::IProxyData> ProxyPtr = InRegistry.CreateProxyFromUObject(ElementTypeName, Object);
						InOutParamToInit.ObjectProxies.Emplace(MoveTemp(ProxyPtr));
					}

					if (bClearUObjectPointers)
					{
						InOutParamToInit.ArrayObjectParam.Reset();
					}
				}
				break;

				default:
					break;
			}
		}

		FAudioParameter MakeAudioParameter(const Frontend::IDataTypeRegistry& InRegistry, FName InParamName, FName InTypeName, const FMetasoundFrontendLiteral& InLiteral, bool bCreateUObjectProxies) 
		{
			constexpr bool bClearUObjectPointers = false;

			FAudioParameter Params;
			Params.ParamName = InParamName;
			Params.TypeName = InTypeName;
			const EMetasoundFrontendLiteralType InLiteralType = InLiteral.GetType(); 

			if (Params.TypeName == TriggerName)
			{
				Params.ParamType = EAudioParameterType::Trigger;
				check(InLiteralType == EMetasoundFrontendLiteralType::Boolean
					|| InLiteralType == EMetasoundFrontendLiteralType::None);
			}

			switch (InLiteralType)
			{
				case EMetasoundFrontendLiteralType::Boolean:
				{
					if (Params.TypeName != TriggerName)
					{
						Params.ParamType = EAudioParameterType::Boolean;
					}
					
					ensure(InLiteral.TryGet(Params.BoolParam));	
				}
				break;

				case EMetasoundFrontendLiteralType::BooleanArray:
				{
					Params.ParamType = EAudioParameterType::BooleanArray;
					ensure(InLiteral.TryGet(Params.ArrayBoolParam));
				}
				break;

				case EMetasoundFrontendLiteralType::Integer:
				{
					Params.ParamType = EAudioParameterType::Integer;
					ensure(InLiteral.TryGet(Params.IntParam));
				}
				break;

				case EMetasoundFrontendLiteralType::IntegerArray:
				{
					Params.ParamType = EAudioParameterType::IntegerArray;
					ensure(InLiteral.TryGet(Params.ArrayIntParam));
				}
				break;

				case EMetasoundFrontendLiteralType::Float:
				{
					Params.ParamType = EAudioParameterType::Float;
					ensure(InLiteral.TryGet(Params.FloatParam));
				}
				break;

				case EMetasoundFrontendLiteralType::FloatArray:
				{
					Params.ParamType = EAudioParameterType::FloatArray;
					ensure(InLiteral.TryGet(Params.ArrayFloatParam));
				}
				break;

				case EMetasoundFrontendLiteralType::String:
				{
					Params.ParamType = EAudioParameterType::String;
					ensure(InLiteral.TryGet(Params.StringParam));
				}
				break;

				case EMetasoundFrontendLiteralType::StringArray:
				{
					Params.ParamType = EAudioParameterType::StringArray;
					ensure(InLiteral.TryGet(Params.ArrayStringParam));
				}
				break;

				case EMetasoundFrontendLiteralType::UObject:
				{
					Params.ParamType = EAudioParameterType::Object;
					UObject* Object = nullptr;
					ensure(InLiteral.TryGet(Object));
					Params.ObjectParam = Object;
					if (bCreateUObjectProxies)
					{
						CreateUObjectProxies(InRegistry, InTypeName, bClearUObjectPointers, Params);
					}
				}
				break;

				case EMetasoundFrontendLiteralType::UObjectArray:
				{
					Params.ParamType = EAudioParameterType::ObjectArray;
					ensure(InLiteral.TryGet(MutableView(Params.ArrayObjectParam)));
					if (bCreateUObjectProxies)
					{
						CreateUObjectProxies(InRegistry, InTypeName, bClearUObjectPointers, Params);
					}
				}
				break;

				default:
				break;
			}

			return Params;
		}

	} // namespace SourcePrivate

	FGeneratorInstanceInfo::FGeneratorInstanceInfo(uint64 InAudioComponentID, uint64 InInstanceID, TWeakPtr<FMetasoundGenerator> InGenerator) :
	AudioComponentID(InAudioComponentID), InstanceID(InInstanceID), Generator(InGenerator)
	{
	}
} // namespace Metasound

FAutoConsoleVariableRef CVarMetaSoundEnableExperimentalRUntimePresetGraphInflation(
	TEXT("au.MetaSound.Experimental.EnableRuntimePresetGraphInflation"),
	Metasound::ConsoleVariables::bEnableExperimentalRuntimePresetGraphInflation,
	TEXT("Enables experimental feature of MetaSounds which reduces overhead of preset graphs\n")
	TEXT("Default: false"),
	ECVF_Default);

// FAudioParameterCollector is used during preset graph inflation to collect audio parameters
// overriden by preset graphs. This collection process is needed because preset graph inflation
// circumvents the normal storage location of default inputs (They are stored on the registered
// IGraph). 
class UMetaSoundSource::FAudioParameterCollector
{
	// Utilitiy for sorting parameters by ParamName
	struct FSortPredicate
	{
		FORCEINLINE bool operator()(const FAudioParameter& InLHS, const FAudioParameter& InRHS) const
		{
			return InLHS.ParamName.FastLess(InRHS.ParamName);
		}
	};

	// Utility for accessing the ParamName from a audio parameter
	struct FProjection
	{
		FORCEINLINE FName operator()(const FAudioParameter& InParam) const
		{
			return InParam.ParamName;
		}
	};

public:
	
	// FAudioParameterCollection merges new parameter into the provided array. The array is manipulated
	// in-place and so care must be taken that the `FAudioParameterCollector` does not attempt to access
	// invalid parameter arrays. 
	FAudioParameterCollector(TArray<FAudioParameter>& InOutParameters)
	: Parameters(InOutParameters)
	{
		Algo::Sort(Parameters, FSortPredicate());
	}
	
	// Do not allow the FAudioParameterCollector to escape scope in order to prevent accidental
	// invalid access to Parameter array reference.
	FAudioParameterCollector(const FAudioParameterCollector&) = delete;
	FAudioParameterCollector(FAudioParameterCollector&&) = delete;
	FAudioParameterCollector& operator=(const FAudioParameterCollector&) = delete;
	FAudioParameterCollector& operator=(FAudioParameterCollector&&) = delete;

	// Merge in parameters from this preset.
	void CollectPresetOverrides(const FMetasoundFrontendDocument& Document, const Metasound::TSortedVertexNameMap<FRuntimeInput>& InInputMap)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		const FPresetTemplate* PresetTemplate = Document.Template.GetPtr<FPresetTemplate>();
		if (!PresetTemplate)
		{
			return;
		}

		if (InInputMap.IsEmpty())
		{
			return;
		}

		TArray<const FMetasoundFrontendClassInput*> ClassInputs;
		Algo::Transform(Document.RootGraph.GetDefaultInterface().Inputs, ClassInputs, [](const FMetasoundFrontendClassInput& Input) { return &Input; });

		for (const TPair<FVertexName, FRuntimeInput>& Pair : InInputMap)
		{
			// Avoid adding parameters which already exist in the ParameterArray
			int32 InsertPos = Algo::LowerBoundBy(Parameters, Pair.Key, FProjection(), FSortPredicate());

			const bool bDoesNotAlreadyExist = (InsertPos >= Parameters.Num()) || (Pair.Key != Parameters[InsertPos].ParamName);
			if (bDoesNotAlreadyExist)
			{
				auto IsInput = [&Pair](const FMetasoundFrontendClassInput* Input) { return Input->Name == Pair.Key; };
				const int32 ClassIndex = ClassInputs.IndexOfByPredicate(IsInput);
				if (ClassIndex != INDEX_NONE)
				{
					// Avoid adding parameters which are NOT overriding a default.
					const FGuid& NodeID = ClassInputs[ClassIndex]->NodeID;
					if (const FPresetVertexMetadata* VertexMetadata = PresetTemplate->FindConstVertexMetadata<FPresetVertexMetadata>(NodeID))
					{
						if (VertexMetadata->bOverrideInheritedDefault)
						{
							Parameters.Insert(Pair.Value.DefaultParameter, InsertPos);
						}
					}
					ClassInputs.RemoveAtSwap(ClassIndex, EAllowShrinking::No);
				}
			}
		}
	}

private:

	TArray<FAudioParameter>& Parameters;
};

UMetaSoundSource::UMetaSoundSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FMetasoundAssetBase()
{
	// Default Root Graph uses static ID to distinguish between a default constructed document
	// (invalid ID) and CDO.  A MetaSoundSource asset should only be constructed using the Document
	// Builder API to avoid ID collisions, but underlying UObjects must always be deterministically
	// generated using NewObject for serialization (and for CDOs).
	RootMetasoundDocument.RootGraph.ID = FGuid(0x4D657461, 0x536F756E, 0x64536F75, 0x72636500);

	bProcedural = true;
	bRequiresStopFade = true;
	NumChannels = 1;

	// Note: Can't call FDiscreteChannelAgnosticType::GetDefaultCatFormat() here as the CDO
	// constructor runs during module load before the channel registry is initialized.
	static const FName DefaultDiscreteFormat(TEXT("Cat:Mono1Dot0"));
	ChannelAgnosticCustomFormat = DefaultDiscreteFormat;
}

const UClass& UMetaSoundSource::GetBaseMetaSoundUClass() const
{
	return *UMetaSoundSource::StaticClass();
}

const UClass& UMetaSoundSource::GetBuilderUClass() const
{
	return *UMetaSoundSourceBuilder::StaticClass();
}

EMetasoundFrontendClassAccessFlags UMetaSoundSource::GetDefaultAccessFlags() const
{
	return EMetasoundFrontendClassAccessFlags::None;
}

const FMetasoundFrontendDocument& UMetaSoundSource::GetConstDocument() const
{
	return RootMetasoundDocument;
}

#if WITH_EDITOR
void UMetaSoundSource::PostEditUndo()
{
	Super::PostEditUndo();

	TScriptInterface<IMetaSoundDocumentInterface> DocInterface(this);
	Metasound::Engine::FAssetHelper::PostEditUndo(DocInterface);
}

void UMetaSoundSource::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);
	Metasound::Engine::FAssetHelper::PreDuplicate(this, DupParams);
}

void UMetaSoundSource::PostDuplicate(EDuplicateMode::Type InDuplicateMode)
{
	Super::PostDuplicate(InDuplicateMode);
	Metasound::Engine::FAssetHelper::PostDuplicate(this, InDuplicateMode);
}

void UMetaSoundSource::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, OutputFormat))
	{
		PostEditChangeOutputFormat();
	}
	// When the CAT custom format or format chooser changes, resolve NumChannels immediately
	// so the mixer allocates the correct buffer size on the next play. (UE-370811)
	// NumChannels must be mixer-compatible (1, 2, 4, 6, 8) — the CAT format identity is
	// carried separately via SourceFormatName, and transcoding handles the conversion.
	if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, ChannelAgnosticCustomFormat) ||
		InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, ChannelAgnosticFormatChooser))
	{
		if (OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic &&
			ChannelAgnosticFormatChooser == EMetasoundChannelAgnosticSourceFormatChooser::Custom &&
			!ChannelAgnosticCustomFormat.IsNone())
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			if (const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindChannel(ChannelAgnosticCustomFormat))
			{
				NumChannels = ClampToMixerChannelCount(Found->NumChannels());
			}
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}
	}
	if (InEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, SampleRateOverride) ||
		InEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, BlockRateOverride) ||
		InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, QualitySetting) )
	{
		PostEditChangeQualitySettings();
	}
}

bool UMetaSoundSource::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	// Allow changes to quality if we don't have any overrides.
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMetaSoundSource, QualitySetting))
	{
		const TArray<FName> Platforms = FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType::AllPlatformInfos);
		const int32 DefaultBlockRate = BlockRateOverride.GetDefault();
		const float DefaultSampleRate = SampleRateOverride.GetDefault();

		if (DefaultBlockRate > 0 && DefaultSampleRate > 0)
		{
			return false;
		}

		for (const FName Platform : Platforms)
		{
			if (BlockRateOverride.GetValueForPlatform(Platform) != DefaultBlockRate )
			{
				return false;
			}
			if (!FMath::IsNearlyEqual(SampleRateOverride.GetValueForPlatform(Platform), DefaultSampleRate))
			{
				return false;
			}
		}
	}
		
	return true;
}

EDataValidationResult UMetaSoundSource::IsDataValid(FDataValidationContext& Context) const
{
	const EDataValidationResult Result = Metasound::Engine::FAssetHelper::IsDataValid(*this, RootMetasoundDocument, Context);
	return CombineDataValidationResults(Result, Super::IsDataValid(Context));
}

void UMetaSoundSource::PostEditChangeOutputFormat()
{
	using namespace Metasound::Engine;

	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	UMetaSoundSourceBuilder& SourceBuilder = FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding<UMetaSoundSourceBuilder>(*this);
	SourceBuilder.SetFormat(OutputFormat, Result);

	if (Result == EMetaSoundBuilderResult::Succeeded)
	{
		// Update the data in this UMetaSoundSource to reflect what is in the metasound document.
		ConformObjectToDocument();

		// Use the editor form of register to ensure other editors'
		// MetaSounds are auto-updated if they are referencing this graph.
		if (EditorGraph)
		{
			EditorGraph->RegisterGraphWithFrontend();
		}
		MarkPackageDirty();
	}
}

void UMetaSoundSource::PostEditChangeQualitySettings()
{
	// Refresh the SampleRate (which is what the engine sees from the operator settings).
	SampleRate = GetOperatorSettings(CachedAudioDeviceSampleRate).GetSampleRate();

	// Always refresh the GUID with the selection.
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())	
	{
		auto FindByName = [&Name = QualitySetting](const FMetaSoundQualitySettings& Q) -> bool { return Q.Name == Name; };
		if (const FMetaSoundQualitySettings* Found = Settings->GetQualitySettings().FindByPredicate(FindByName))
		{
			QualitySettingGuid = Found->UniqueId;
		}
	}
}
#endif // WITH_EDITOR


bool UMetaSoundSource::ConformObjectToDocument()
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	bool bDidAlterObjectData = false;

	// Update the OutputFormat and NumChannels to match the audio format interface
	// on the root document.
	const FOutputAudioFormatInfoMap& FormatInfo = GetOutputAudioFormatInfo();
	for (const FOutputAudioFormatInfoPair& Pair : FormatInfo)
	{
		if (RootMetasoundDocument.Interfaces.Contains(Pair.Value.InterfaceVersion))
		{
			// Channel Agnostic (CAT) output: resolve NumChannels from the custom format
			// if one is selected, otherwise default to 1 (mono). NumChannels here is the
			// mixer-facing channel count, clamped to supported values (1,2,4,6,8).
			if (Pair.Key == EMetaSoundOutputAudioFormat::ChannelAgnostic)
			{
				// Resolve NumChannels now if a custom format is selected and the registry is available.
				// Previously this was hardcoded to 1 as a placeholder, causing a two-save sync bug (UE-370811).
				int32 ResolvedChannels = 1;
				if (ChannelAgnosticFormatChooser == EMetasoundChannelAgnosticSourceFormatChooser::Custom && !ChannelAgnosticCustomFormat.IsNone())
				{
					PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
					if (const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindChannel(ChannelAgnosticCustomFormat))
					{
						ResolvedChannels = ClampToMixerChannelCount(Found->NumChannels());
					}
					PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
				}

				// Only mark altered if something actually changed (avoids unnecessary asset dirtying).
				if (OutputFormat != Pair.Key || NumChannels != ResolvedChannels)
				{
					OutputFormat = Pair.Key;
					NumChannels = ResolvedChannels;
					bDidAlterObjectData = true;
				}
			}
			else if ((OutputFormat != Pair.Key) || (NumChannels != Pair.Value.OutputVertexChannelOrder.Num()))
			{
				OutputFormat = Pair.Key;
				NumChannels = Pair.Value.OutputVertexChannelOrder.Num();
				bDidAlterObjectData = true;
			}

			break;
		}
	}

	return bDidAlterObjectData;
}

FTopLevelAssetPath UMetaSoundSource::GetAssetPathChecked() const
{
	return Metasound::Engine::FAssetHelper::GetAssetPathChecked(*this);
}

void UMetaSoundSource::BeginDestroy()
{
	OnNotifyBeginDestroy();
	Super::BeginDestroy();
}

void UMetaSoundSource::PreSave(FObjectPreSaveContext InSaveContext)
{
	Super::PreSave(InSaveContext);
	Metasound::Engine::FAssetHelper::PreSaveAsset(*this, InSaveContext);
}

void UMetaSoundSource::Serialize(FArchive& InArchive)
{
	using namespace Metasound::SourcePrivate;

	Super::Serialize(InArchive);
	Metasound::Engine::FAssetHelper::SerializeToArchive(*this, InArchive);

	// Load/Save cooked data.
	if (InArchive.IsCooking() || (FPlatformProperties::RequiresCookedData() && InArchive.IsLoading()) || IsCookedForEditor(InArchive, this))
	{
		const FName PlatformName = InArchive.CookingTarget() ? *InArchive.CookingTarget()->IniPlatformName() : FName(FPlatformProperties::IniPlatformName());
		SerializeCookedQualitySettings(PlatformName, InArchive);
	}
}

bool UMetaSoundSource::GetQualitySettings(
	const FName InPlatformName, Metasound::SourcePrivate::FCookedQualitySettings& OutQualitySettings) const
{
#if WITH_EDITORONLY_DATA
	
	// Query Project settings. 
	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		if (const FMetaSoundQualitySettings* Found = Settings->GetQualitySettings().FindByPredicate([&QT = QualitySetting](const FMetaSoundQualitySettings& Q) -> bool { return Q.Name == QT; }))
		{
			// Allow partial applications of settings, if some are non-zero.
			if (const float Value = Found->BlockRate.GetValueForPlatform(InPlatformName); Value > 0.f)
			{
				UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] using Quality '%ls', BlockRate=%3.3f", *GetName(), *QualitySetting.ToString(), Value);
				OutQualitySettings.BlockRate = Value;
			}
			if (const int32 Value = Found->SampleRate.GetValueForPlatform(InPlatformName); Value > 0)
			{
				UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] using Quality '%ls', SampleRate=%d", *GetName(), *QualitySetting.ToString(), Value);
				OutQualitySettings.SampleRate = Value;
			}
		}
	}

	// Query overrides defined on this asset.
	if (const float SerializedBlockRate = BlockRateOverride.GetValueForPlatform(InPlatformName); SerializedBlockRate > 0.0f)
	{
		UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] BlockRate Override: %3.3f", *GetName(), SerializedBlockRate);
		OutQualitySettings.BlockRate = SerializedBlockRate;
	}
	if (const int32 SerializedSampleRate = SampleRateOverride.GetValueForPlatform(InPlatformName); SerializedSampleRate > 0)
	{
		UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] SampleRate Override: %d", *GetName(), SerializedSampleRate);
		OutQualitySettings.SampleRate = SerializedSampleRate;
	}
	
	// Success.
	UE_LOGF(LogMetaSound, Verbose, "Metasound [%ls] using SampleRate=%d, BlockRate=%2.3f (not-cooked)",
	 	*GetName(), OutQualitySettings.SampleRate.GetValue(), OutQualitySettings.BlockRate.GetValue()) ;
	
	return true;

#else //WITH_EDITORONLY_DATA	

	// If we've been cooked, this should contain the quality settings.
	if (CookedQualitySettings)
	{
		if (const float SerializedBlockRate = CookedQualitySettings->BlockRate.GetValue(); SerializedBlockRate > 0.0f)
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] BlockRate: %3.3f", *GetName(), SerializedBlockRate);
			OutQualitySettings.BlockRate = SerializedBlockRate;
		}
		if (const int32 SerializedSampleRate = CookedQualitySettings->SampleRate.GetValue(); SerializedSampleRate > 0)
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "Metasound [%ls] SampleRate: %d", *GetName(), SerializedSampleRate);
			OutQualitySettings.SampleRate = SerializedSampleRate;
		}
	
		UE_LOGF(LogMetaSound, Verbose, "Metasound [%ls] using SampleRate=%d, BlockRate=%2.3f (cooked)",
			*GetName(), OutQualitySettings.SampleRate.GetValue(), OutQualitySettings.BlockRate.GetValue()) ;
		return true;
	}
	
	// Fail.
	return false;
	
#endif //WITH_EDITORONLY_DATA
}
void UMetaSoundSource::SerializeCookedQualitySettings(const FName PlatformName, FArchive& Ar)
{
	Metasound::SourcePrivate::FCookedQualitySettings Settings;
	if (Ar.IsSaving())
	{
		GetQualitySettings(PlatformName, Settings);
	}
	// Use Struct Serializer.
	FMetaSoundQualitySettings::StaticStruct()->SerializeItem(Ar,&Settings,nullptr);
		
	if (Ar.IsLoading())
	{
		CookedQualitySettings = MakePimpl<Metasound::SourcePrivate::FCookedQualitySettings>(Settings);
	}
}

static TSharedPtr<const Audio::FDiscreteChannelTypeFamily> NumChannelsToFormat(const int32 NumChannels)
{
	return Audio::FChannelAgnosticUtils::FindDiscreteFormatFromNumChannels(NumChannels);
}

TSharedPtr<const Audio::FChannelTypeFamily> UMetaSoundSource::CalculateAutoSourceFormat(const TArray<FAudioParameter>& InParams) const
{
	using namespace Audio;

	// This only applies if we're set to Channel Agnostic Output.
	if (OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic)
	{
		// Default behavior for automatic channel format determination is to find
		// the greatest channel count. In the future there may be additional controls
		// to influence how this channel count is determined.
		int32 GreatestNumChannels = 1;

		checkf(RuntimeInputData.bIsValid.load(), TEXT("RuntimeInputData must be cached before calling CalculateAutoSourceFormat"));

		// Iterate through all inputs 
		for (const auto& Pair : RuntimeInputData.InputMap)
		{
			const FRuntimeInput& RuntimeInput = Pair.Value;
			const FAudioParameter* Parameter = &RuntimeInput.DefaultParameter;
			// Check for overrides from input parameters
			if (const FAudioParameter* Found = InParams.FindByPredicate([&RuntimeInput](const FAudioParameter& Param) -> bool { return Param.ParamName == RuntimeInput.Name; }))
			{
				Parameter = Found;
			}

			// Update greatest channel count
			if ((EAudioParameterType::Object == Parameter->ParamType) || (EAudioParameterType::ObjectArray == Parameter->ParamType))
			{
				for (const TSharedPtr<IProxyData>& ObjProxy : Parameter->ObjectProxies)
				{
					if (const ISoundWaveContainer* Container = QueryInterface<ISoundWaveContainer>(ObjProxy.Get()))
					{
						for (const FSoundWaveProxyPtr& SoundProxyPtr : Container->GetContainedWaveProxies())
						{
							if (SoundProxyPtr.IsValid())
							{
								GreatestNumChannels = FMath::Max(GreatestNumChannels, (int32)SoundProxyPtr->GetSoundWaveDataRef()->GetNumChannels());

							}
						}
					}
					else if (const FSoundWaveProxy* SoundProxy = QueryInterface<FSoundWaveProxy>(ObjProxy.Get()))
					{
						GreatestNumChannels = FMath::Max(GreatestNumChannels, (int32)SoundProxy->GetSoundWaveDataRef()->GetNumChannels());
					}
				}
				
			}
		}

		
		// Convert to Format Name.
		if (const TSharedPtr<const Audio::FDiscreteChannelTypeFamily> SP = NumChannelsToFormat(GreatestNumChannels))
		{
			return SP;	
		}
	}
	return nullptr;
}

#if WITH_EDITOR
void UMetaSoundSource::SetReferencedAssets(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs)
{
	Metasound::Engine::FAssetHelper::SetReferencedAssets(*this, MoveTemp(InAssetRefs));
}
#endif // WITH_EDITOR

TArray<FMetasoundAssetBase*> UMetaSoundSource::GetReferencedAssets() const
{
	return Metasound::Engine::FAssetHelper::GetReferencedAssets(*this);
}

#if WITH_EDITORONLY_DATA
void UMetaSoundSource::OnAsyncReferencedAssetsLoaded(const TArray<FMetasoundAssetBase*>& InAsyncReferences)
{
	Metasound::Engine::FAssetHelper::OnAsyncReferencedAssetsLoaded(*this, InAsyncReferences);
}

void UMetaSoundSource::MigrateEditorGraph(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Graph)
	{
		Graph->MigrateEditorDocumentData(OutBuilder);
		Graph = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UEdGraph* UMetaSoundSource::GetGraph() const
{
	return EditorGraph;
}

UEdGraph& UMetaSoundSource::GetGraphChecked() const
{
	check(EditorGraph);
	return *EditorGraph;
}

FText UMetaSoundSource::GetDisplayName() const
{
	FString TypeName = UMetaSoundSource::StaticClass()->GetName();
	return FMetasoundAssetBase::GetDisplayName(MoveTemp(TypeName));
}
#endif // WITH_EDITORONLY_DATA

void UMetaSoundSource::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Metasound::Engine::FAssetHelper::PostLoad(*this, ReferenceAssetClassCache);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	Duration = GetDuration();
	bLooping = IsLooping();

	PostLoadQualitySettings();
}

void UMetaSoundSource::PostLoadQualitySettings()
{
#if WITH_EDITORONLY_DATA
	
	// Ensure that our Quality settings resolve. 
	if (UMetaSoundSettings* Settings = GetMutableDefault<UMetaSoundSettings>())
	{
		ResolveQualitySettings(Settings);
		
		// Register for any changes to the settings while we're open in the editor.
		Settings->OnSettingChanged().AddWeakLambda(this, [WeakSource = MakeWeakObjectPtr(this)](UObject* InObj, struct FPropertyChangedEvent& InEvent)
		{
			if (
				WeakSource.IsValid() &&
				InEvent.GetMemberPropertyName() == UMetaSoundSettings::GetQualitySettingPropertyName()
			)
			{
				WeakSource->ResolveQualitySettings(CastChecked<UMetaSoundSettings>(InObj));
			}
		});

		// Register for changes from the CVars that control overrides.
		// We cache the OperatorSettings, so reset when these change.
		auto ResetOperatorSettings = [WeakSource = MakeWeakObjectPtr(this)](IConsoleVariable* Var)
		{
			if (WeakSource.IsValid())
			{
				WeakSource->ResolveQualitySettings(GetMutableDefault<UMetaSoundSettings>());
			
				// Override SampleRate with the Operator settings version which uses our Quality settings.
				WeakSource->SampleRate = WeakSource->GetOperatorSettings(WeakSource->CachedAudioDeviceSampleRate).GetSampleRate();
			}
		};
		Metasound::Frontend::GetBlockRateOverrideChangedDelegate().AddWeakLambda(this, ResetOperatorSettings);
		Metasound::Frontend::GetSampleRateOverrideChangedDelegate().AddWeakLambda(this, ResetOperatorSettings);
	}
#endif //WITH_EDITORONLY_DATA
}

void UMetaSoundSource::ResolveQualitySettings(const UMetaSoundSettings* Settings)
{
#if WITH_EDITORONLY_DATA

	const FMetaSoundQualitySettings* Resolved = nullptr;

	// 1. Try and resolve by name. (most should resolve unless its been renamed, deleted).
	auto FindByName = [&Name = QualitySetting](const FMetaSoundQualitySettings& Q) -> bool { return Q.Name == Name; };
	Resolved = Settings->GetQualitySettings().FindByPredicate(FindByName);

	// 2. If that failed, try by guid (if its been renamed in the settings, we can still find it).
	if (!Resolved && QualitySettingGuid.IsValid())
	{
		auto FindByGuid = [&Guid = QualitySettingGuid](const FMetaSoundQualitySettings& Q) -> bool { return Q.UniqueId == Guid; };
		Resolved = Settings->GetQualitySettings().FindByPredicate(FindByGuid);
	}

	// 3. If still failed to resolve, use defaults and warn.
	if (!Resolved)
	{
		// Disable the warning for now.
		//UE_LOGF(LogMetaSound, Warning, "Failed to resolve Quality '%ls', resetting to the default.", *QualitySetting.ToString());

		// Reset to defaults. (and make sure they are sane)
		QualitySetting = GetDefault<UMetaSoundSource>()->QualitySetting;
		QualitySettingGuid = GetDefault<UMetaSoundSource>()->QualitySettingGuid;
		if (!Settings->GetQualitySettings().FindByPredicate(FindByName) && !Settings->GetQualitySettings().IsEmpty())
		{
			// Default doesn't point to anything, use first one in the list.
			QualitySetting = Settings->GetQualitySettings()[0].Name;
			QualitySettingGuid = Settings->GetQualitySettings()[0].UniqueId;
		}
	}

	// Refresh the guid/name now we've resolved to correctly reflect.
	if (Resolved)
	{
		QualitySetting = Resolved->Name;
		QualitySettingGuid = Resolved->UniqueId;
	}
#endif //WITH_EDITORONLY_DATA
}

void UMetaSoundSource::InitParameters(TArray<FAudioParameter>& ParametersToInit, FName InFeatureName)
{
	using namespace Metasound::SourcePrivate;

	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSource::InitParameters);


	if (bIsBuilderActive)	
	{
		// Do not create UObject proxies in the runtime input map because they proxies 
		// stored there will not be used. The necessary proxies in the ParametersToInit 
		// will be created and used instead. 
		constexpr bool bCreateUObjectProxiesInRuntimeInputMap = false; 
		InitParametersInternal(CreateRuntimeInputMap(bCreateUObjectProxiesInRuntimeInputMap, UMetaSoundSettings::GetPageOrder()), ParametersToInit, InFeatureName);
	}
	else
	{
		const bool bIsRuntimeInputDataValid = RuntimeInputData.bIsValid.load();
		if (bIsRuntimeInputDataValid)
		{
			InitParametersInternal(RuntimeInputData.InputMap, ParametersToInit, InFeatureName);
		}
		else
		{
			// The runtime input data should have been cached, but is not so we use
			// a fallback method. If this is occurring, then callers need to ensure
			// that InitResources has been called before this method executes or else
			// suffer the consequences of incurring significant performance losses 
			// each time a parameter is set on the MetaSound. 
			UE_CLOGF(HasNotBeenLoggedForThisObject(*this, __LINE__), LogMetaSound, Warning, "Initializing parameters on uninitialized UMetaSoundSource %ls will result in slower performance. UMetaSoundSource::InitResources should finish executing on the game thread before attempting to call UMetaSoundSource::InitParameters(...)", *GetOwningAssetName());

			// Do not create UObject proxies in the runtime input map because they proxies 
			// stored there will not be used. The necessary proxies in the ParametersToInit 
			// will be created and used instead. 
			constexpr bool bCreateUObjectProxiesInRuntimeInputMap = false; 
			InitParametersInternal(CreateRuntimeInputMap(bCreateUObjectProxiesInRuntimeInputMap, UMetaSoundSettings::GetPageOrder()), ParametersToInit, InFeatureName);
		}
	}
}

void UMetaSoundSource::InitResources()
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;
	using namespace Metasound::SourcePrivate;

	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSource::InitResources); 
	ensureMsgf(Metasound::CanEverExecuteGraph(), TEXT("UMetaSoundSource::InitResources can only be called in applications where MetaSounds can execute."));

	if (IsInGameThread())
	{
		UpdateAndRegisterForExecution(GetInitRegistrationOptions());
	}
	else
	{
		const bool bIsInGCSafeThread = IsInAudioThread() || IsInAsyncLoadingThread(); // Audio Thread is safe from GC, so we can safely construct the TWeakObjectPtr<> to this.
		if (!bIsInGCSafeThread)
		{
			UE_LOGF(LogMetaSound, Warning, "Attempt to call UMetaSoundSource::InitResources() on %ls from thread which may not provide garbage collection safety of the UMetaSoundSource", *GetOwningAssetName());
		}

		ExecuteOnGameThread(
			UE_SOURCE_LOCATION, 
			[MetaSoundSourcePtr=TWeakObjectPtr<UMetaSoundSource>(this)]()
			{
				if (UMetaSoundSource* Source = MetaSoundSourcePtr.Get())
				{
					Source->InitResources();
				}
			}
		);
	}
}

TSharedPtr<Audio::IProxyData> UMetaSoundSource::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!IsRegistered())
	{
		// Make sure the metasound is registered and ready to go before making a proxy.
		InitResources();
	}

	if (IsRegistered())
	{
		return FMetasoundAssetBase::CreateProxyData(InitParams);
	}
	else
	{
		return nullptr;
	}
}

void UMetaSoundSource::UpdateAndRegisterForExecution(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions& InRegistrationOptions)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	check(IsInGameThread());

	FMetasoundAssetBase::UpdateAndRegisterForExecution(InRegistrationOptions);
	const bool bIsRuntimeInputDataValid = RuntimeInputData.bIsValid.load();
	// Runtime data does not need to and should not be created in builds that don't execute graphs
	if (!bIsRuntimeInputDataValid && Metasound::CanEverExecuteGraph())
	{
		CacheRuntimeInputData(InRegistrationOptions.PageOrder);
	}
}

bool UMetaSoundSource::IsPlayable() const
{
	return true;
}

void UMetaSoundSource::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	Metasound::Engine::FAssetHelper::GetAssetRegistryTags(this, Context);
}

float UMetaSoundSource::GetDuration() const
{
	// This is an unfortunate function required by logic in determining what sounds can be potentially
	// culled (in this case prematurally). MetaSound OneShots are stopped either by internally logic that
	// triggers OnFinished, or if an external system requests the sound to be stopped. Setting the duration
	// as a "close to" maximum length without being considered looping avoids the MetaSound from being
	// culled inappropriately.
	return IsOneShot() ? INDEFINITELY_LOOPING_DURATION - 1.0f : INDEFINITELY_LOOPING_DURATION;
}

Metasound::Frontend::FDocumentAccessPtr UMetaSoundSource::GetDocumentAccessPtr(bool bReloadBuilder)
{
	using namespace Metasound::Frontend;

	// Mutation of a document via the soft deprecated access ptr/controller system is not tracked by
	// the builder registry, so the document cache is invalidated here.
	if (bReloadBuilder)
	{
		if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
		{
			BuilderRegistry->ReloadBuilder(RootMetasoundDocument.RootGraph.Metadata.GetClassName());
		}
	}

	// Return document using FAccessPoint to inform the TAccessPtr when the 
	// object is no longer valid.
	return MakeAccessPtr<FDocumentAccessPtr>(RootMetasoundDocument.AccessPoint, RootMetasoundDocument);
}

Metasound::Frontend::FConstDocumentAccessPtr UMetaSoundSource::GetDocumentConstAccessPtr() const
{
	using namespace Metasound::Frontend;

	// Return document using FAccessPoint to inform the TAccessPtr when the 
	// object is no longer valid.
	return MakeAccessPtr<FConstDocumentAccessPtr>(RootMetasoundDocument.AccessPoint, RootMetasoundDocument);
}

bool UMetaSoundSource::ImplementsParameterInterface(Audio::FParameterInterfacePtr InInterface) const
{
	const FMetasoundFrontendVersion Version { InInterface->GetName(), { InInterface->GetVersion().Major, InInterface->GetVersion().Minor } };
	return GetConstDocument().Interfaces.Contains(Version);
}

UMetaSoundSource::FResolvedSourceFormat UMetaSoundSource::ResolveSourceFormatAndNumChannels(const FSoundGeneratorInitParams& InParams, const TArray<FAudioParameter>& InDefaultParameters, Metasound::FMetasoundEnvironment& InOutEnvironment)
{
	using namespace Metasound::Frontend;

	FResolvedSourceFormat Result;
	if (OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic)
	{
		// Priority: 1) user-specified custom format, 2) auto-detect from wave proxy assets,
		// 3) return error condition for undetermined type
		if (ChannelAgnosticFormatChooser == EMetasoundChannelAgnosticSourceFormatChooser::Custom && !ChannelAgnosticCustomFormat.IsNone())
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			if (const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindChannel(ChannelAgnosticCustomFormat))
			{
				Result.SourceFormatName = Found->GetName();
				InOutEnvironment.SetValue<FName>(SourceInterface::Environment::SourceFormatName, Result.SourceFormatName);
				Result.NumChannels = ClampToMixerChannelCount(Found->NumChannels());
			}
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Custom format '%ls' not found in channel registry", *ChannelAgnosticCustomFormat.ToString());
				Result.NumChannels = ClampToMixerChannelCount(InParams.NumChannels);
			}
		}
		else if (TSharedPtr<const Audio::FChannelTypeFamily> SP = CalculateAutoSourceFormat(InDefaultParameters); SP.IsValid())
		{
			Result.SourceFormatName = SP->GetName();
			InOutEnvironment.SetValue<FName>(SourceInterface::Environment::SourceFormatName, Result.SourceFormatName);
			Result.NumChannels = ClampToMixerChannelCount(SP->NumChannels());
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Failed to determine concrete format for ChannelAgnotistic output on source \"%ls\"", *GetName()) ;
		}
	}
	else
	{
		Result.NumChannels = GetOutputAudioChannelOrder().Num();
	}

	return Result;
}

ISoundGeneratorPtr UMetaSoundSource::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams, TArray<FAudioParameter>&& InDefaultParameters)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace Metasound::Engine;
	using namespace Metasound::SourcePrivate;

	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSource::CreateSoundGenerator);

	FOperatorSettings InSettings = GetOperatorSettings(static_cast<FSampleRate>(InParams.SampleRate));

	SampleRate = InSettings.GetSampleRate();

	FMetasoundEnvironment Environment = CreateEnvironment(InParams);
	const UMetaSoundSource::FResolvedSourceFormat ResolvedFormat = ResolveSourceFormatAndNumChannels(InParams, InDefaultParameters, Environment);
	if (ResolvedFormat.NumChannels == INDEX_NONE)
	{
		// Failed to resolve number of channels. This should not happen, but if it does,
		// do not attempt to render audio.
		return nullptr;
	}
	NumChannels = ResolvedFormat.NumChannels;

	FParameterRouter& Router = UMetaSoundSource::GetParameterRouter();
	TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> DataChannel = Router.FindOrCreateDataChannelForReader(InParams.AudioDeviceID, InParams.InstanceID);

	if (InParams.StartTime > 0.0f)
	{
		if (IsInterfaceDeclared(SourceStartTimeInterface::GetVersion()))
		{
			InDefaultParameters.Add({ SourceStartTimeInterface::Inputs::StartTime, InParams.StartTime });
		}
		else if(!bHasWarnedAboutStartTime)
		{
			const FString AssetName = GetName();
			UE_LOGF(LogMetaSound, Warning, "A non-zero StartTime (%.2f) was used in asset '%ls' that doesn't implement the Start Time interface. StartTime will be ignored.", InParams.StartTime, *AssetName);
			bHasWarnedAboutStartTime = true;
		}
	}

	FOperatorBuilderSettings BuilderSettings = FOperatorBuilderSettings::GetDefaultSettings();
	// Graph analyzer currently only enabled for preview sounds (but can theoretically be supported for all sounds)
	BuilderSettings.bPopulateInternalDataReferences = InParams.bIsPreviewSound;

	constexpr bool bBuildSynchronous = false;

	const bool bIsDynamic = DynamicTransactor.IsValid();
	TSharedPtr<FMetasoundGenerator> Generator;

	if (bIsDynamic)
	{
		// In order to ensure synchronization and avoid race conditions the current state
		// of the graph is copied and transform queue created here. This ensures that:
		//
		// 1. Modifications to the underlying FGraph in the FDynamicOperatorTransactor can continue 
		// while the generator is being constructed on an async task.  If this were not ensured, 
		// a race condition would be introduced wherein the FGraph could be manipulated while the
		// graph is being read while building the generator.
		//
		// 2. The state of the FGraph and TransformQueue are synchronized so that any additional
		// changes applied to the FDynamicOperatorTransactor will be placed in the TransformQueue.
		// The dynamic operator & generator will then consume these transforms after it has finished 
		// being built.

		BuilderSettings.bEnableOperatorRebind = true;

		TSharedRef<FGraphRenderCost> GraphRenderCost = FGraphRenderCost::MakeGraphRenderCost();
		FMetasoundDynamicGraphGeneratorInitParams InitParams
		{
			FGeneratorInitParams
			{
				.OperatorSettings = InSettings,
				.BuilderSettings = MoveTemp(BuilderSettings),
				.Graph = MakeShared<FGraph>(DynamicTransactor->GetGraph()), // Make a copy of the graph.
				.Environment = Environment,
				.AudioOutputNames = GetOutputAudioChannelOrder(),
				.DefaultParameters = MoveTemp(InDefaultParameters),
				.bBuildSynchronous = bBuildSynchronous,
				.DataChannel = DataChannel,
				.GraphRenderCost = GraphRenderCost,
				.ClassName = GetClass()->GetFName(),
				.AssetPath = GetAssetPathChecked(),
				.SourceFormatName = ResolvedFormat.SourceFormatName
			},
			DynamicTransactor->CreateTransformQueue(InSettings, Environment, GraphRenderCost) // Create transaction queue
		};
		TSharedPtr<FMetasoundDynamicGraphGenerator> DynamicGenerator = MakeShared<FMetasoundDynamicGraphGenerator>(InSettings);
		DynamicGenerator->Init(MoveTemp(InitParams));

		Generator = MoveTemp(DynamicGenerator);
	}
	else 
	{
		// By default, the sound generator for a metasound preset uses a graph specifically
		// associated with the UMetaSoundSource_Preset. The overridden defaults for that
		// preset are baked into the IGraph. Unfortunately, this makes the MetaSound
		// operator pool less efficient because it associates the operator with the IGraph.
		// The way the presets use the IGraph mean that there is less sharing of cached
		// operators.
		//
		// To improve the efficiency of the operator pool, we have presets use their
		// base IGraphs so that more MetaSounds utilize the same IGraph. This requires
		// us to retrieve that specific graph. We also supply the parameters that were overridden
		// in the preset to the FMetaSoundGenerator, because they are not backed into
		// the base IGraph. 
		FAudioParameterCollector AudioParameterCollector(InDefaultParameters);
		TSharedPtr<const IGraph> MetasoundGraph = FindFirstNoninflatableGraph(AudioParameterCollector, Environment);

		if (MetasoundGraph.IsValid())
		{
			// Create generator.
			FGeneratorInitParams InitParams
			{
				.OperatorSettings = InSettings,
				.BuilderSettings = MoveTemp(BuilderSettings),
				.Graph = MetasoundGraph,
				.Environment = Environment,
				.AudioOutputNames = GetOutputAudioChannelOrder(),
				.DefaultParameters = MoveTemp(InDefaultParameters),
				.bBuildSynchronous = bBuildSynchronous,
				.DataChannel = DataChannel,
				.ClassName = GetClass()->GetFName(),
				.SourceFormatName = ResolvedFormat.SourceFormatName
			};
			InitParams.AssetPath = GetAssetPathChecked();
			Generator = MakeShared<FMetasoundConstGraphGenerator>(MoveTemp(InitParams));
		}
	}

	if (Generator.IsValid())
	{
		FGeneratorInstanceInfo Info = FGeneratorInstanceInfo(InParams.AudioComponentId, InParams.InstanceID, Generator);
		TrackGenerator(MoveTemp(Info));
	}

	return ISoundGeneratorPtr(Generator);
}

void UMetaSoundSource::OnEndGenerate(ISoundGeneratorPtr Generator)
{
	using namespace Metasound;
	ForgetGenerator(Generator);
}

TArray<UMetaSoundSource::FRuntimeInput> UMetaSoundSource::GatherVerseParameters() const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace Metasound::Engine;

	constexpr bool bCreateUObjectProxies = false;
	TSortedVertexNameMap<FRuntimeInput> InputMap = CreateRuntimeInputMap(bCreateUObjectProxies, UMetaSoundSettings::GetPageOrder());

	TArray<FRuntimeInput> OutParameters;
	OutParameters.Reserve(InputMap.Num());
	for(const TPair<FVertexName, FRuntimeInput>& Pair : InputMap)
	{
		OutParameters.Add(Pair.Value);
	}

	return OutParameters;
}

bool UMetaSoundSource::GetAllDefaultParameters(TArray<FAudioParameter>& OutParameters) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;
	using namespace Metasound::Engine;

	if (Metasound::CanEverExecuteGraph())
	{
		// If we are in an executable context, we expect runtime input data to be
		// cached as default parameters may be requested many times. 
		if(!RuntimeInputData.bIsValid.load())
		{
			UE_CLOGF(SourcePrivate::HasNotBeenLoggedForThisObject(*this, __LINE__), LogMetaSound, Warning, "Default parameters will be ommitted. Accessing invalid runtime data on MetaSound %ls. Ensure that UMetaSoundSource::InitResources() is executed on the game thread before calling UMetaSoundSource::GetAllDefaultParameters(...)", *GetOwningAssetName());
			return false;
		}

		for(const TPair<FVertexName, FRuntimeInput>& Pair : RuntimeInputData.InputMap)
		{
			OutParameters.Add(Pair.Value.DefaultParameter);
		}
		return true;
	}
	else
	{
		// Non executable contexts do not have cached runtime data and need their 
		// parameters created each time they are requested. 
		constexpr bool bCreateUObjectProxies = false;
		TSortedVertexNameMap<FRuntimeInput> InputMap = CreateRuntimeInputMap(bCreateUObjectProxies, UMetaSoundSettings::GetPageOrder());

		for(const TPair<FVertexName, FRuntimeInput>& Pair : InputMap)
		{
			OutParameters.Add(Pair.Value.DefaultParameter);
		}

		return true;
	}
}

void UMetaSoundSource::InitParametersInternal(const Metasound::TSortedVertexNameMap<FRuntimeInput>& InInputMap, TArray<FAudioParameter>& ParametersToInit, FName InFeatureName) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	METASOUND_LLM_SCOPE;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSource::InitParametersInternal);

	checkf(IsInGameThread() || IsInAudioThread(), TEXT("Parameter initialization must happen on the GameThread or AudioThread to allow for safe creation of UObject proxies"));

	IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

	// Removes values that are not explicitly defined by the ParamType
	auto Sanitize = [](FAudioParameter& Parameter)
	{
		switch (Parameter.ParamType)
		{
			case EAudioParameterType::Trigger:
			{
				Parameter = FAudioParameter(Parameter.ParamName, EAudioParameterType::Trigger);
			}
			break;
			
			case EAudioParameterType::Boolean:
			{
				Parameter = FAudioParameter(Parameter.ParamName, Parameter.BoolParam);
			}
			break;

			case EAudioParameterType::BooleanArray:
			{
				TArray<bool> TempArray = Parameter.ArrayBoolParam;
				Parameter = FAudioParameter(Parameter.ParamName, MoveTemp(TempArray));
			}
			break;

			case EAudioParameterType::Float:
			{
				Parameter = FAudioParameter(Parameter.ParamName, Parameter.FloatParam);
			}
			break;

			case EAudioParameterType::FloatArray:
			{
				TArray<float> TempArray = Parameter.ArrayFloatParam;
				Parameter = FAudioParameter(Parameter.ParamName, MoveTemp(TempArray));
			}
			break;

			case EAudioParameterType::Integer:
			{
				Parameter = FAudioParameter(Parameter.ParamName, Parameter.IntParam);
			}
			break;

			case EAudioParameterType::IntegerArray:
			{
				TArray<int32> TempArray = Parameter.ArrayIntParam;
				Parameter = FAudioParameter(Parameter.ParamName, MoveTemp(TempArray));
			}
			break;

			case EAudioParameterType::Object:
			{
				Parameter = FAudioParameter(Parameter.ParamName, Parameter.ObjectParam);
			}
			break;

			case EAudioParameterType::ObjectArray:
			{
				TArray<UObject*> TempArray = Parameter.ArrayObjectParam;
				Parameter = FAudioParameter(Parameter.ParamName, MoveTemp(TempArray));
			}
			break;


			case EAudioParameterType::String:
			{
				Parameter = FAudioParameter(Parameter.ParamName, Parameter.StringParam);
			}
			break;

			case EAudioParameterType::StringArray:
			{
				TArray<FString> TempArray = Parameter.ArrayStringParam;
				Parameter = FAudioParameter(Parameter.ParamName, MoveTemp(TempArray));
			}
			break;

			case EAudioParameterType::None:
			default:
			break;
		}
	};

	for (int32 i = ParametersToInit.Num() - 1; i >= 0; --i)
	{
		bool bIsParameterValid = false;

		FAudioParameter& Parameter = ParametersToInit[i];
		if (const FRuntimeInput* Input = InInputMap.Find(Parameter.ParamName))
		{	
			if (IsParameterValidInternal(Parameter, Input->TypeName, DataTypeRegistry))
			{
				bIsParameterValid = true;
				
				// note: this was originally called 2x, the 1st call would create the proxy
				// and clear out the ObjectParam.  The 2nd call would clear out the proxy,
				// and fail to re-create it because we cleared out the ObjectParam. 
				
				// if there is already a proxy, don't attempt to recreate them.
				if (Parameter.ObjectProxies.IsEmpty())
				{
					Sanitize(Parameter);
					constexpr bool bClearUObjectPointers = true; // protect against leaking UObject ptrs to the audio thread 
					SourcePrivate::CreateUObjectProxies(DataTypeRegistry, Input->TypeName, bClearUObjectPointers, Parameter);
				}
			}
		}

		if (!bIsParameterValid)
		{
			ParametersToInit.RemoveAtSwap(i, EAllowShrinking::No);

#if !NO_LOGGING
			if (::Metasound::MetaSoundParameterEnableWarningOnIgnoredParameterCVar)
			{
				const FString AssetName = GetName();
				UE_LOGF(LogMetaSound, Warning, "Failed to set parameter '%ls' in asset '%ls': No name specified, no transmittable input found, or type mismatch.", *Parameter.ParamName.ToString(), *AssetName);
			}
#endif // !NO_LOGGING
		}
	}
}

bool UMetaSoundSource::IsParameterValid(const FAudioParameter& InParameter) const
{
	const TArray<FMetasoundFrontendClassInput>& Inputs = GetConstDocument().RootGraph.GetDefaultInterface().Inputs;
	const FMetasoundFrontendVertex* Vertex = Algo::FindByPredicate(Inputs, [&InParameter] (const FMetasoundFrontendClassInput& Input)
	{
		return Input.Name == InParameter.ParamName;
	});
	
	if (Vertex)
	{
		return IsParameterValidInternal(InParameter, Vertex->TypeName, Metasound::Frontend::IDataTypeRegistry::Get());
	}
	else
	{
		return false;
	}
}

bool UMetaSoundSource::IsParameterValidInternal(const FAudioParameter& InParameter, const FName& InTypeName, Metasound::Frontend::IDataTypeRegistry& InDataTypeRegistry) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	if (InParameter.ParamName.IsNone())
	{
		// Invalid parameter name
		return false;
	}

	if (!InParameter.TypeName.IsNone() && InParameter.TypeName != InTypeName)
	{
		// Mismatched parameter type and vertex data type
		return false;
	}

	// Special handling for UObject proxies
	if (InParameter.ParamType == EAudioParameterType::Object)
	{
		return InDataTypeRegistry.IsValidUObjectForDataType(InTypeName, InParameter.ObjectParam);
	}
	else if (InParameter.ParamType == EAudioParameterType::ObjectArray)
	{
		bool bIsValid = true;

		const FName ElementTypeName = CreateElementTypeNameFromArrayTypeName(InTypeName);
		for (const UObject* Object : InParameter.ArrayObjectParam)
		{
			bIsValid = InDataTypeRegistry.IsValidUObjectForDataType(ElementTypeName, Object);
			if (!bIsValid)
			{
				break;
			}
		}
		return bIsValid;
	}

	const IDataTypeRegistryEntry* RegistryEntry = InDataTypeRegistry.FindDataTypeRegistryEntry(InTypeName);
	if (!RegistryEntry)
	{
		// Unregistered MetaSound data type
		return false;
	}

	switch (InParameter.ParamType)
	{
		case EAudioParameterType::Trigger:
		case EAudioParameterType::Boolean:
		{
			return RegistryEntry->GetDataTypeInfo().bIsBoolParsable;
		}

		case EAudioParameterType::BooleanArray:
		{
			return RegistryEntry->GetDataTypeInfo().bIsBoolArrayParsable;
		}

		case EAudioParameterType::Float:
		{
			return RegistryEntry->GetDataTypeInfo().bIsFloatParsable;
		}

		case EAudioParameterType::FloatArray:
		{
			return RegistryEntry->GetDataTypeInfo().bIsFloatArrayParsable;
		}

		case EAudioParameterType::Integer:
		{
			return RegistryEntry->GetDataTypeInfo().bIsIntParsable;
		}

		case EAudioParameterType::IntegerArray:
		{
			return RegistryEntry->GetDataTypeInfo().bIsIntArrayParsable;
		}

		case EAudioParameterType::String:
		{
			return RegistryEntry->GetDataTypeInfo().bIsStringParsable;
		}

		case EAudioParameterType::StringArray:
		{
			return RegistryEntry->GetDataTypeInfo().bIsStringArrayParsable;
		}

		case EAudioParameterType::NoneArray:
		{
			return RegistryEntry->GetDataTypeInfo().bIsDefaultArrayParsable;
		}

		case EAudioParameterType::None:
		{
			return RegistryEntry->GetDataTypeInfo().bIsDefaultParsable;
		}

		default:
		{
			// All parameter types should be covered.
			static_assert(static_cast<uint8>(EAudioParameterType::COUNT) == 13, "Possible unhandled EAudioParameterType");
			checkNoEntry();
			// Unhandled parameter type
			return false;
		}
	}

}

bool UMetaSoundSource::IsLooping() const
{
	return !IsOneShot();
}

bool UMetaSoundSource::IsOneShot() const
{
	using namespace Metasound::Frontend;

	// If the metasound source implements the one-shot interface, then it's a one-shot metasound
	return IsInterfaceDeclared(SourceOneShotInterface::GetVersion());
}

TSharedPtr<Audio::IParameterTransmitter> UMetaSoundSource::CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const
{
	using namespace Metasound;
	using namespace Metasound::SourcePrivate;

	METASOUND_LLM_SCOPE;

	auto CreateParameterTransmitterInternal = [this](const TSortedVertexNameMap<FRuntimeInput>& InInputMap, Audio::FParameterTransmitterInitParams& InParams)
	{
		// Build list of parameters that can be set at runtime.
		TArray<FName> ValidParameters;
		for (const TPair<FVertexName, FRuntimeInput>& Pair : InInputMap)
		{
			if (Pair.Value.bIsTransmittable && (Pair.Value.AccessType == EMetasoundFrontendVertexAccessType::Reference))
			{
				ValidParameters.Add(Pair.Value.Name);
			}
		}

		FParameterRouter& Router = UMetaSoundSource::GetParameterRouter();
		TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> DataChannel = Router.FindOrCreateDataChannelForWriter(InParams.AudioDeviceID, InParams.InstanceID);

		Metasound::FMetaSoundParameterTransmitter::FInitParams InitParams
		(
			GetOperatorSettings(InParams.SampleRate), 
			InParams.InstanceID, 
			MoveTemp(InParams.DefaultParams), 
			MoveTemp(ValidParameters), 
			DataChannel
		);

		InitParams.DebugMetaSoundName = this->GetFName();

		return MakeShared<Metasound::FMetaSoundParameterTransmitter>(MoveTemp(InitParams));
	};

	const bool bIsRuntimeInputDataValid = RuntimeInputData.bIsValid.load();
	const bool bCreateInputMapOnTheFly = bIsBuilderActive || !bIsRuntimeInputDataValid;

	if (bCreateInputMapOnTheFly)
	{
		if (!bIsBuilderActive)
		{
			// If we're not using a builder, that means the metasound cannot change and that the runtime input data should have been cached. 
			UE_LOGF(LogMetaSound, Warning, "Creating a Parameter Transmiiter on uninitialized UMetaSoundSource %ls will result in slower performance. UMetaSoundSource::InitResources should finish executing on the game thread before attempting to call UMetaSoundSource::CreateParameterTransmitter(...)", *GetOwningAssetName());
		}

		// Do not create UObject proxies in the runtime input map because they proxies 
		// stored there will not be used. The necessary proxies in the ParametersToInit 
		// will be created and used instead. 
		constexpr bool bCreateUObjectProxiesInRuntimeInputMap = false; 
		return CreateParameterTransmitterInternal(CreateRuntimeInputMap(bCreateUObjectProxiesInRuntimeInputMap, UMetaSoundSettings::GetPageOrder()), InParams);
	}
	else
	{
		return CreateParameterTransmitterInternal(RuntimeInputData.InputMap, InParams);
	}
}

Metasound::FOperatorSettings UMetaSoundSource::GetOperatorSettings(Metasound::FSampleRate InDeviceSampleRate) const
{	
	using namespace Metasound;
	using namespace Metasound::SourcePrivate;
	
	// Default sensibly.
	FCookedQualitySettings Settings;
	Settings.BlockRate = DefaultBlockRateConstant;
	Settings.SampleRate = InDeviceSampleRate > 0 ? InDeviceSampleRate : DefaultSampleRateConstant;

	// Fetch our quality settings.
	// If we are cooked these are baked, if we are editor these are queried from the project settings and this assets overrides.
	const bool bFoundQualitySettings = GetQualitySettings(FPlatformProperties::IniPlatformName(), Settings);
	UE_CLOGF(!bFoundQualitySettings && IsAsset(), LogMetaSound, Error, "Could not retrieve quality settings for asset %ls", *GetOwningAssetName());
	
	// Query CVars. (Override with CVars if they are > 0)
	using namespace Metasound::Frontend;
	const float BlockRateCVar = GetBlockRateOverride();
	const int32 SampleRateCvar = GetSampleRateOverride();

	if (SampleRateCvar > 0)
	{
		Settings.SampleRate = SampleRateCvar;
	}
	if (BlockRateCVar > 0)
	{
		Settings.BlockRate = BlockRateCVar;
	}

	// Sanity clamps.
	const TRange<float> BlockRange = GetBlockRateClampRange();
	const TRange<int32> RateRange = GetSampleRateClampRange();
	Settings.BlockRate = FMath::Clamp(Settings.BlockRate.GetValue(), BlockRange.GetLowerBoundValue(), BlockRange.GetUpperBoundValue());
	Settings.SampleRate = FMath::Clamp(Settings.SampleRate.GetValue(), RateRange.GetLowerBoundValue(), RateRange.GetUpperBoundValue());

	UE_LOGF(LogMetaSound, Verbose, "Metasound [%ls] GetOperatorSettings: SampleRate: %d, BlockRate: %3.3f",
		*GetName(), Settings.SampleRate.GetValue(), Settings.BlockRate.GetValue());
	
	return Metasound::FOperatorSettings(
		/* SampleRate */ Settings.SampleRate.GetValue(),
		/* BlockRate */ Settings.BlockRate.GetValue());
}

Metasound::FMetasoundEnvironment UMetaSoundSource::CreateEnvironment() const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FMetasoundEnvironment Environment;
	Environment.SetValue<uint32>(SourceInterface::Environment::SoundUniqueID, GetUniqueID());

	return Environment;
}

Metasound::FMetasoundEnvironment UMetaSoundSource::CreateEnvironment(const FSoundGeneratorInitParams& InParams) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	FMetasoundEnvironment Environment = CreateEnvironment();
	Environment.SetValue<bool>(SourceInterface::Environment::IsPreview, InParams.bIsPreviewSound);
	Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, InParams.InstanceID);
	Environment.SetValue<uint64>(SourceInterface::Environment::TransmitterID, InParams.InstanceID);
	Environment.SetValue<Audio::FDeviceId>(SourceInterface::Environment::DeviceID, InParams.AudioDeviceID);
	Environment.SetValue<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames, InParams.AudioMixerNumOutputFrames);
	Environment.SetValue<float>(SourceInterface::Environment::AudioMixerSampleRate, InParams.SampleRate);

#if WITH_METASOUND_DEBUG_ENVIRONMENT
	Environment.SetValue<FString>(SourceInterface::Environment::GraphName, GetFullName());
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

	return Environment;
}

Metasound::FMetasoundEnvironment UMetaSoundSource::CreateEnvironment(const Audio::FParameterTransmitterInitParams& InParams) const
{
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	FMetasoundEnvironment Environment = CreateEnvironment();
	Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, InParams.InstanceID);

	return Environment;
}

const TArray<Metasound::FVertexName>& UMetaSoundSource::GetOutputAudioChannelOrder() const
{
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	if (const FOutputAudioFormatInfo* FormatInfo = GetOutputAudioFormatInfo().Find(OutputFormat))
	{
		return FormatInfo->OutputVertexChannelOrder;
	}
	else
	{
		// Unhandled audio format. Need to update audio output format vertex key map.
		checkNoEntry();
		static const TArray<Metasound::FVertexName> Empty;
		return Empty;
	}
}

void UMetaSoundSource::TrackGenerator(Metasound::FGeneratorInstanceInfo&& GeneratorInfo)
{
	FScopeLock GeneratorMapLock(&GeneratorMapCriticalSection);
	Generators.Add(MoveTemp(GeneratorInfo));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnGeneratorInstanceCreated.Broadcast(Generators.Last().AudioComponentID, Generators.Last().Generator.Pin());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OnGeneratorInstanceInfoCreated.Broadcast(Generators.Last());
}

void UMetaSoundSource::ForgetGenerator(ISoundGeneratorPtr Generator)
{
	using namespace Metasound;
	FMetasoundGenerator* AsMetasoundGenerator = static_cast<FMetasoundGenerator*>(Generator.Get());
	FScopeLock GeneratorMapLock(&GeneratorMapCriticalSection);
	const uint32 Index = Generators.IndexOfByPredicate([AsMetasoundGenerator](const FGeneratorInstanceInfo& GeneratorInfo)
	{
		return GeneratorInfo.Generator.HasSameObject(AsMetasoundGenerator);
	});
	if (Index != INDEX_NONE)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OnGeneratorInstanceDestroyed.Broadcast(Generators[Index].AudioComponentID, Generators[Index].Generator.Pin());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OnGeneratorInstanceInfoDestroyed.Broadcast(Generators[Index]);
		Generators.RemoveAtSwap(Index);
	}
}

TWeakPtr<Metasound::FMetasoundGenerator> UMetaSoundSource::GetGeneratorForAudioComponent(uint64 ComponentId) const
{
	using namespace Metasound;
	check(ComponentId != INDEX_NONE)
	FScopeLock GeneratorMapLock(&GeneratorMapCriticalSection);
	const FGeneratorInstanceInfo* GeneratorInfo = Generators.FindByPredicate([ComponentId](const FGeneratorInstanceInfo& Info)
	{
		return Info.AudioComponentID == ComponentId;
	});
	if (GeneratorInfo != nullptr)
	{
		return GeneratorInfo->Generator;
	}
	return TWeakPtr<FMetasoundGenerator>(nullptr);
}

TWeakPtr<Metasound::FMetasoundGenerator> UMetaSoundSource::GetGeneratorForInstanceID(uint64 InstanceId) const
{
	using namespace Metasound;
	FScopeLock GeneratorMapLock(&GeneratorMapCriticalSection);
	const FGeneratorInstanceInfo* GeneratorInfo = Generators.FindByPredicate([InstanceId](const FGeneratorInstanceInfo& Info)
	{
		return Info.InstanceID == InstanceId;
	});
	if (GeneratorInfo != nullptr)
	{
		return GeneratorInfo->Generator;
	}
	return TWeakPtr<FMetasoundGenerator>(nullptr);
}

bool UMetaSoundSource::IsDynamic() const
{
	return DynamicTransactor.IsValid();
}

Metasound::SourcePrivate::FParameterRouter& UMetaSoundSource::GetParameterRouter()
{
	using namespace Metasound::SourcePrivate;

	static FParameterRouter Router;
	return Router;
}

bool UMetaSoundSource::IsActivelyBuilding() const
{
	return bIsBuilderActive;
}

void UMetaSoundSource::OnBeginActiveBuilder()
{
	using namespace Metasound::Frontend;

	// If a builder is activating, make sure any in-flight registration
	// tasks have completed. Async registration tasks use the FMetasoundFrontendDocument
	// that lives on this object. We need to make sure that registration task
	// completes so that the FMetasoundFrontendDocument does not get modified
	// by a builder while it is also being read by async registration.
	const FGraphRegistryKey GraphKey = GetGraphRegistryKey();
	if (GraphKey.IsValid())
	{
		FMetasoundFrontendRegistryContainer::Get()->WaitForAsyncGraphRegistration(GraphKey);
	}

	if (bIsBuilderActive)
	{
		UE_LOGF(LogMetaSound, Error, "OnBeginActiveBuilder() call while prior builder is still active. This may indicate that multiple builders are attempting to modify the MetaSound %ls concurrently.", *GetOwningAssetName())
	}

	bIsBuilderActive = true;

	// Currently we do not have information on whether inputs were added or removed
	// from the document. We invalidate the cached runtime inputs just in case.
	// MetaSounds which have an active builder should not be using cached runtime
	// input data until the builder is no longer active. 
	InvalidateCachedRuntimeInputData();
}

void UMetaSoundSource::OnFinishActiveBuilder()
{
	bIsBuilderActive = false;
}

TSharedPtr<Metasound::DynamicGraph::FDynamicOperatorTransactor> UMetaSoundSource::SetDynamicGeneratorEnabled(bool bInIsEnabled)
{
	using namespace Metasound;
	using namespace Metasound::DynamicGraph;

	if (bInIsEnabled)
	{
		if (!DynamicTransactor.IsValid())
		{
			// If a FGraph exists for this UMetaSoundSource, then we need to initialize
			// the DynamicTransactor with the existing FGraph so it has the correct
			// initial state. 
			//
			// Currently, any existing FGraph will be stored in the node registry,
			// hence we check if the graph is registered and retrieve the current 
			// graph to see if any FGraph already exists. 
			if (IsRegistered())
			{
				TSharedPtr<const FGraph> CurrentGraph = FMetasoundFrontendRegistryContainer::Get()->GetGraph(GetGraphRegistryKey());

				if (CurrentGraph.IsValid())
				{
					DynamicTransactor = MakeShared<FDynamicOperatorTransactor>(*CurrentGraph);
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to get existing graph for dynamic metasound %ls. Initializing to empty graph.", *GetOwningAssetName());
					DynamicTransactor = MakeShared<FDynamicOperatorTransactor>();
				}
			}
			else
			{
				DynamicTransactor = MakeShared<FDynamicOperatorTransactor>();
			}
		}
	}
	else
	{
		DynamicTransactor.Reset();
	}

	return DynamicTransactor;
}

TSharedPtr<Metasound::DynamicGraph::FDynamicOperatorTransactor> UMetaSoundSource::GetDynamicGeneratorTransactor() const
{
	return DynamicTransactor;
}

UMetaSoundSource::FRuntimeInput UMetaSoundSource::CreateRuntimeInput(const Metasound::Frontend::IDataTypeRegistry& Registry, const FMetasoundFrontendClassInput& Input, bool bCreateUObjectProxies, TArrayView<const FGuid> InPageOrder)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const TArray<FMetasoundFrontendClassInputDefault>& InputDefualts = Input.GetDefaults();
	const bool bIsPaged = InputDefualts.Num() != 1 || InputDefualts.Last().PageID != Frontend::DefaultPageID;

	const IDataTypeRegistryEntry* RegistryEntry = Registry.FindDataTypeRegistryEntry(Input.TypeName);
	const FMetasoundFrontendClassInputDefault* Default = FindPreferredPage(InputDefualts, InPageOrder);

	UE_CLOGF(!RegistryEntry, LogMetaSound, Warning, "Failed to find data type '%ls' in registry. Assuming data type is not transmittable", *Input.TypeName.ToString());
	UE_CLOGF(!Default, LogMetaSound, Error, "Failed to find default literal for parameter %ls", *Input.Name.ToString());

	const bool bIsTransmittable = RegistryEntry? RegistryEntry->GetDataTypeInfo().bIsTransmittable : false;

	const FAudioParameter DefaultParameter = SourcePrivate::MakeAudioParameter
	(
		  Registry
		, Input.Name
		, Input.TypeName
		, (Default)? Default->Literal : FMetasoundFrontendLiteral(Registry.CreateDefaultLiteral(Input.TypeName))
		, bCreateUObjectProxies
	);

	return FRuntimeInput
	{
		  Input.Name
		, Input.TypeName
		, Input.AccessType
		, DefaultParameter
		, bIsTransmittable
#if WITH_EDITORONLY_DATA
		, bIsPaged
#endif // #if WITH_EDITORONLY_DATA
	};
}

Metasound::TSortedVertexNameMap<UMetaSoundSource::FRuntimeInput> UMetaSoundSource::CreateRuntimeInputMap(bool bCreateUObjectProxies, TArrayView<const FGuid> InPageOrder) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(UMetaSoundSource::CreateRuntimeInputMap);

	auto GetInputName = [](const FMetasoundFrontendClassInput& InInput) { return InInput.Name; };

	const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();
	const IInterfaceRegistry& InterfaceRegistry = IInterfaceRegistry::Get();

	const FMetasoundFrontendDocument& Doc = GetConstDocument();

	// Inputs which are controlled by an interface are private unless
	// their router name is `Audio::IParameterTransmitter::RouterName`
	TSet<FVertexName> PrivateInputs;
	for (const FMetasoundFrontendVersion& InterfaceVersion : Doc.Interfaces)
	{
		FName RouterName;
		if (IInterfaceRegistry::Get().FindInterfaceRouter(InterfaceVersion, RouterName))
		{
			if (RouterName != Audio::IParameterTransmitter::RouterName)
			{
				FMetasoundFrontendInterface Interface;
				InterfaceRegistry.FindInterface(InterfaceVersion, Interface);
				Algo::Transform(Interface.Inputs, PrivateInputs, GetInputName);
			}
		}
	}

	// Cache all inputs which are not private inputs.
	TSortedVertexNameMap<FRuntimeInput> PublicInputs;
	for (const FMetasoundFrontendClassInput& Input : Doc.RootGraph.GetDefaultInterface().Inputs)
	{
		if (!PrivateInputs.Contains(Input.Name))
		{
			PublicInputs.Add(Input.Name, CreateRuntimeInput(DataTypeRegistry, Input, bCreateUObjectProxies, InPageOrder));
		}
	}

	// Add the parameter pack input that ALL MetaSounds have
	FMetasoundFrontendClassInput ParameterPackInput = UMetasoundParameterPack::GetClassInput();
	const FMetasoundFrontendLiteral& DefaultLiteral = ParameterPackInput.FindConstDefaultChecked(Frontend::DefaultPageID);
	FAudioParameter ParameterPackDefaultParameter = SourcePrivate::MakeAudioParameter(DataTypeRegistry, ParameterPackInput.Name, ParameterPackInput.TypeName, DefaultLiteral, bCreateUObjectProxies) ;
	PublicInputs.Add(ParameterPackInput.Name, FRuntimeInput{ParameterPackInput.Name, ParameterPackInput.TypeName, ParameterPackInput.AccessType, ParameterPackDefaultParameter, true /* bIsTransmittable */});
	
	return PublicInputs;
}

void UMetaSoundSource::CacheRuntimeInputData(TArrayView<const FGuid> InPageOrder)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	constexpr bool bCreateUObjectProxies = true;
	RuntimeInputData.InputMap = CreateRuntimeInputMap(bCreateUObjectProxies, InPageOrder);

	// Determine if preset graph inflation is possible 
	//
	// Constructor inputs conflict with `Preset Graph Inflation` and `Operator Caching`. 
	// This logic protects against attempting to use preset graph inflation when the preset 
	// graph has overridden constructor pins. 
	// 
	// Operator caching of base preset graphs fail when there are constructor inputs because
	// constructor inputs set on the preset cannot be updated after the base operator is 
	// cached.

	bIsPresetGraphInflationSupported = false;
	if (const FPresetTemplate* PresetTemplate = RootMetasoundDocument.Template.GetPtr<FPresetTemplate>())
	{
		auto IsOverriddenConstructorInput = [this, &PresetTemplate](const FMetasoundFrontendClassInput& Input)
		{
			if (const FRuntimeInput* RuntimeInput = RuntimeInputData.InputMap.Find(Input.Name))
			{
				if (const FPresetVertexMetadata* VertexMetadata = PresetTemplate->FindConstVertexMetadata<FPresetVertexMetadata>(Input.NodeID))
				{
					return VertexMetadata->bOverrideInheritedDefault && RuntimeInput->AccessType == EMetasoundFrontendVertexAccessType::Value;
				}
			}
			return false;
		};

		const TArray<FMetasoundFrontendClassInput>& Inputs = RootMetasoundDocument.RootGraph.GetDefaultInterface().Inputs;
		bIsPresetGraphInflationSupported = !Algo::AnyOf(Inputs, IsOverriddenConstructorInput);
	}

	RuntimeInputData.bIsValid.store(true);
}

void UMetaSoundSource::InvalidateCachedRuntimeInputData()
{
	bIsPresetGraphInflationSupported = false; 
	RuntimeInputData.bIsValid.store(false);
}

const UMetaSoundSource& UMetaSoundSource::FindFirstNoninflatableSource(Metasound::FMetasoundEnvironment& InOutEnvironment, TFunctionRef<void(const UMetaSoundSource&)> OnTraversal) const
{
	TArray<FGuid> GraphHierarchy;

	const UMetaSoundSource& FoundSource = FindFirstNoninflatableSourceInternal(GraphHierarchy, OnTraversal);

	if (GraphHierarchy.Num())
	{
		// Preset graph inflation needs to emulate the graph hierarchy for nodes which depend upon accurate graph hierarchies. 
		InOutEnvironment.SetValue<TArray<FGuid>>(Metasound::CoreInterface::Environment::GraphHierarchy, MoveTemp(GraphHierarchy));
	}

	return FoundSource;
}

const UMetaSoundSource& UMetaSoundSource::FindFirstNoninflatableSourceInternal(TArray<FGuid>& OutHierarchy, TFunctionRef<void(const UMetaSoundSource&)> OnTraversal) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const bool bIsDynamic = DynamicTransactor.IsValid();
	if (!bIsDynamic && ConsoleVariables::bEnableExperimentalRuntimePresetGraphInflation && bIsPresetGraphInflationSupported)
	{
		if (ReferencedAssetClassObjects.Num() == 1)
		{
			// Get first element from TSet<>
			TObjectPtr<const UObject> BaseGraph = *ReferencedAssetClassObjects.CreateConstIterator();

			// Get the reference graph as a UMetaSoundSource
			TObjectPtr<const UMetaSoundSource> BaseMetaSoundSource = Cast<const UMetaSoundSource>(BaseGraph);
			if (ensure(BaseMetaSoundSource)) // SourcePresets assume they are referencing a UMetaSoundSource
			{
				// Preset graph inflation needs to emulate the graph hierarchy for nodes which depend upon accurate graph hierarchies.
				FGuid ClassID;
				ensureAlways(IMetaSoundAssetManager::GetChecked().TryGetAssetIDFromClassName(RootMetasoundDocument.RootGraph.Metadata.GetClassName(), ClassID));
				OutHierarchy.Add(ClassID);

				OnTraversal(*this);

				// If the base metasound is also a preset that can be inflate, recurse into it. 
				return BaseMetaSoundSource->FindFirstNoninflatableSourceInternal(OutHierarchy, OnTraversal);
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Attempt to reference parent of metasound preset failed due to unexpected number of reference asses (%d) from MetaSound Preset %ls", ReferencedAssetClassObjects.Num(), *GetOwningAssetName());
		}
	}

	return *this;
}

TSharedPtr<const Metasound::IGraph> UMetaSoundSource::FindFirstNoninflatableGraph(UMetaSoundSource::FAudioParameterCollector& InOutParameterCollector, Metasound::FMetasoundEnvironment& InOutEnvironment) const
{
	using namespace Metasound;

	auto OnGraphInflation = [&InOutParameterCollector](const UMetaSoundSource& InInflatedSource)
	{
		// Any preset overrides on this object need to be baked in to the parameters
		InOutParameterCollector.CollectPresetOverrides(InInflatedSource.RootMetasoundDocument, InInflatedSource.RuntimeInputData.InputMap);
	};

	const UMetaSoundSource& NoninflatableSource = FindFirstNoninflatableSource(InOutEnvironment, OnGraphInflation);

	return FMetasoundFrontendRegistryContainer::Get()->GetGraph(NoninflatableSource.GetGraphRegistryKey());
}

#undef LOCTEXT_NAMESPACE // MetaSound
