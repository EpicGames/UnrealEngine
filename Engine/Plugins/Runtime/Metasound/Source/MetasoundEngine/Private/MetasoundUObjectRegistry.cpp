// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundUObjectRegistry.h"

#include "Algo/Copy.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "Engine/AssetManager.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "MetasoundTrace.h"
#include "UObject/Object.h"


namespace Metasound
{
	class FMetasoundUObjectRegistry : public IMetasoundUObjectRegistry
	{
		public:
			FMetasoundUObjectRegistry() = default;

			void RegisterUClass(TUniquePtr<IMetasoundUObjectRegistryEntry>&& InEntry) override
			{
				METASOUND_LLM_SCOPE;
				Entries.Add(InEntry.Get());
				Storage.Add(MoveTemp(InEntry));
			}

			bool IsRegisteredClass(UObject* InObject) const override
			{
				return (nullptr != GetEntryByUObject(InObject));
			}

			bool IsRegisteredClass(const UClass& InClass) const override
			{
				auto IsChildClassOfRegisteredClass = [&](const IMetasoundUObjectRegistryEntry* Entry)
				{
					if (Entry)
					{
						return Entry->IsChildClass(&InClass);
					}

					return false;
				};

				return FindEntryByPredicate(IsChildClassOfRegisteredClass) != nullptr;
			}

			FMetasoundAssetBase* GetObjectAsAssetBase(UObject* InObject) const override
			{
				if (const IMetasoundUObjectRegistryEntry* Entry = GetEntryByUObject(InObject))
				{
					return Entry->Cast(InObject);
				}
				return nullptr;
			}

			const FMetasoundAssetBase* GetObjectAsAssetBase(const UObject* InObject) const override
			{
				if (const IMetasoundUObjectRegistryEntry* Entry = GetEntryByUObject(InObject))
				{
					return Entry->Cast(InObject);
				}
				return nullptr;
			}

			void IterateRegisteredUClasses(TFunctionRef<void(UClass&)> InFunc, bool bAssetTypesOnly) const override
			{
				for (const IMetasoundUObjectRegistryEntry* Entry : Entries)
				{
					if (Entry)
					{
						if (!bAssetTypesOnly || Entry->IsAssetType())
						{
							if (UClass* Class = Entry->GetUClass())
							{
								InFunc(*Class);
							}
						}
					}
				}
			}

		private:
			const IMetasoundUObjectRegistryEntry* FindEntryByPredicate(TFunctionRef<bool (const IMetasoundUObjectRegistryEntry*)> InPredicate) const
			{
				const IMetasoundUObjectRegistryEntry* const* Entry = Entries.FindByPredicate(InPredicate);

				if (nullptr == Entry)
				{
					return nullptr;
				}

				return *Entry;
			}

			TArray<const IMetasoundUObjectRegistryEntry*> FindEntriesByPredicate(TFunctionRef<bool (const IMetasoundUObjectRegistryEntry*)> InPredicate) const
			{
				TArray<const IMetasoundUObjectRegistryEntry*> FoundEntries;

				Algo::CopyIf(Entries, FoundEntries, InPredicate);

				return FoundEntries;
			}

			const IMetasoundUObjectRegistryEntry* GetEntryByUObject(const UObject* InObject) const
			{
				auto IsChildClassOfRegisteredClass = [&](const IMetasoundUObjectRegistryEntry* Entry)
				{
					if (nullptr == Entry)
					{
						return false;
					}

					return Entry->IsChildClass(InObject);
				};

				return FindEntryByPredicate(IsChildClassOfRegisteredClass);
			}

			TArray<TUniquePtr<IMetasoundUObjectRegistryEntry>> Storage;
			TArray<const IMetasoundUObjectRegistryEntry*> Entries;
	};

	IMetasoundUObjectRegistry& IMetasoundUObjectRegistry::Get()
	{
		static FMetasoundUObjectRegistry Registry;
		return Registry;
	}
} // namespace Metasound
