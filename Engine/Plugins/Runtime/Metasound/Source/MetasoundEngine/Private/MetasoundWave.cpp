// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundWave.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "ContentStreaming.h"
#include "DSP/ParamInterpolator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrace.h"
#include "Sound/SoundWave.h"


static int32 DisableMetasoundWaveAssetCachePriming = 0;
FAutoConsoleVariableRef CVarDisableMetasoundWaveAssetCachePriming(
	TEXT("au.MetaSound.DisableWaveCachePriming"),
	DisableMetasoundWaveAssetCachePriming,
	TEXT("Disables MetaSound Wave Cache Priming.\n")
	TEXT("0 (default): Enabled, 1: Disabled"),
	ECVF_Default);

namespace Metasound
{
	FWaveAsset::FWaveAsset(const TSharedPtr<const Audio::IProxyData>& InInitData)
	{
		if (InInitData.IsValid())
		{
			if (InInitData->CheckTypeCast<FSoundWaveProxy>())
			{
				// todo: remove const cast once deprecated getters can be removed and SoundWaveProxy is const.
				SoundWaveProxy = ConstCastSharedPtr<FSoundWaveProxy>(InInitData->GetAs<const FSoundWaveProxy>().AsShared().ToSharedPtr());

				if (ensureAlways(SoundWaveProxy.IsValid()))
				{
					TSharedRef<const FSoundWaveData> SoundWaveData = SoundWaveProxy->GetSoundWaveDataRef();
					
					// TODO HACK: Prime the sound for playback.
					//
					// Preferably playback latency would be controlled externally.
					// With the current decoder and waveplayer implementation, the 
					// wave player does not know whether samples were actually decoded
					// or if the decoder is still waiting on the stream cache. Generally
					// this is not an issue except for looping. Looping requires counting
					// of decoded samples to get exact loop points. When the decoder 
					// returns zeroed audio (because the stream cache has not loaded
					// the requested chunk) the sample counting gets off. Currently
					// there is not route to expose that information to the wave 
					// player to correct the sample counting logic. 
					//
					// In hopes of mitigating the issue, the stream cache
					// is primed here in the hopes that the chunk is ready by the
					// time that the decoder attempts to decode audio.
					if (0 == DisableMetasoundWaveAssetCachePriming)
					{
						if (SoundWaveData->IsStreaming())
						{
							if (SoundWaveData->GetNumChunks() > 1)
							{
								if (FStreamingManagerCollection* StreamingMgr = IStreamingManager::Get_Concurrent())
								{
									IAudioStreamingManager& AudioStreamingMgr = StreamingMgr->GetAudioStreamingManager();
									AudioStreamingMgr.RequestChunk(SoundWaveData->AsShared(), 1, [](EAudioChunkLoadResult) {});
								}
							}
						}
					}
				}
			}
		}
	}

	bool FWaveAsset::IsSoundWaveValid() const
	{
		return SoundWaveProxy.IsValid();
	}
}
