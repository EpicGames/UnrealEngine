// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Internationalization/Text.h"

#include "MetasoundWave.h"
#include "MetasoundFacade.h"
#include "MetasoundPrimitives.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundWaveInfo"

namespace Metasound
{
	// forward declarations
	// ...

	namespace WaveInfoNodeParameterNames
	{
		// inputs
		METASOUND_PARAM(ParamWaveAsset, "Wave", "Input Wave Asset");

		// outputs
		METASOUND_PARAM(ParamDurationSeconds, "Duration", "Duration of the wave asset in seconds");
		METASOUND_PARAM(OutParamAssetName, "Name", "Name of the wave asset");
		METASOUND_PARAM(OutParamAssetPath, "Path", "Full path of the wave asset");

	} // namespace WaveInfoNodeParameterNames	

	class FWaveInfoNodeOperator : public TExecutableOperator < FWaveInfoNodeOperator >
	{
	public:
		// ctor
		FWaveInfoNodeOperator(const FOperatorSettings& InSettings, const FWaveAssetReadRef& InWaveAsset);

		// node interface
		static const FNodeClassMetadata& GetNodeInfo();
		static FVertexInterface DeclareVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private: // members
		// input pins
		FWaveAssetReadRef WaveAsset;

		// output pins
		FTimeWriteRef DurationSeconds;
		FStringWriteRef NameOutput;
		FStringWriteRef PathOutput;

		// other
		TSharedPtr<const FSoundWaveData> SoundWaveData;

	}; // class FWaveInfoNodeOperator

	// ctor
	FWaveInfoNodeOperator::FWaveInfoNodeOperator(const FOperatorSettings& InSettings, const FWaveAssetReadRef& InWaveAsset)
		: WaveAsset(InWaveAsset)
		, DurationSeconds(FTimeWriteRef::CreateNew(0.0f))
		, NameOutput(FStringWriteRef::CreateNew(TEXT("")))
		, PathOutput(FStringWriteRef::CreateNew(TEXT("")))
	{
		Execute();
	}

	const FNodeClassMetadata& FWaveInfoNodeOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::EngineNodes::Namespace, "Get Wave Duration", " " };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("MetasoundGetWaveInfo_ClassNodeDisplayName", "Get Wave Info");
			Info.Description = METASOUND_LOCTEXT("GetWaveInfo_NodeDescription", "Returns the Info from the Wave Asset"),
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Debug);
			Info.Keywords = { METASOUND_LOCTEXT("WaveInfoNode_DurationKeyword", "Duration"), METASOUND_LOCTEXT("WaveInfoNode_NameKeyword", "Name"), METASOUND_LOCTEXT("WaveInfoNode_PathKeyword", "Path") };
			
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FWaveInfoNodeOperator::DeclareVertexInterface()
	{
		using namespace WaveInfoNodeParameterNames;
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FWaveAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamWaveAsset))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamDurationSeconds)),
				TOutputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamAssetName)),
				TOutputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamAssetPath))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FWaveInfoNodeOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace WaveInfoNodeParameterNames;
		
		const FInputVertexInterfaceData& InputData = InParams.InputData;
		// inputs
		FWaveAssetReadRef WaveAssetIn = InputData.GetOrCreateDefaultDataReadReference<FWaveAsset>(METASOUND_GET_PARAM_NAME(ParamWaveAsset), InParams.OperatorSettings);

		return MakeUnique < FWaveInfoNodeOperator >(InParams.OperatorSettings, WaveAssetIn);
	}

	void FWaveInfoNodeOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace WaveInfoNodeParameterNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamWaveAsset), WaveAsset);	
	}

	void FWaveInfoNodeOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		// expose read access to our output buffer for other processors in the graph
		using namespace WaveInfoNodeParameterNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamDurationSeconds), DurationSeconds);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamAssetName), NameOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamAssetPath), PathOutput);
	}

	void FWaveInfoNodeOperator::Execute()
	{
		TSharedPtr<const FSoundWaveProxy> ProxyPtr = (*WaveAsset).GetWaveProxy();
		TSharedPtr<const FSoundWaveData> DataPtr;
		if (ProxyPtr)
		{
			DataPtr = ProxyPtr->GetSoundWaveDataRef();
		}

		if (SoundWaveData != DataPtr)
		{
			SoundWaveData = DataPtr;

			if (SoundWaveData.IsValid() && (*WaveAsset).IsSoundWaveValid())
			{
				*DurationSeconds = FTime::FromSeconds(SoundWaveData->GetDuration());

				FName AssetName = SoundWaveData->GetFName();
				*NameOutput = AssetName.ToString();

				FString FullPath;
				SoundWaveData->GetPackageName().AppendString(FullPath);
				if (!AssetName.IsNone())
				{
					FullPath += TEXT(".");
					AssetName.AppendString(FullPath);
				}
				*PathOutput = FullPath;
			}
			else
			{
				*DurationSeconds = FTime::FromSeconds(0.0f);
				*NameOutput = TEXT("");
				*PathOutput = TEXT("");
			}
		}
	}

	void FWaveInfoNodeOperator::Reset(const IOperator::FResetParams& InParams)
	{
		Execute();
	}

	using FWaveInfoNode = TNodeFacade<FWaveInfoNodeOperator>;
	METASOUND_REGISTER_NODE(FWaveInfoNode);

} // namespace Metasound

#undef LOCTEXT_NAMESPACE //MetasoundWaveInfo
