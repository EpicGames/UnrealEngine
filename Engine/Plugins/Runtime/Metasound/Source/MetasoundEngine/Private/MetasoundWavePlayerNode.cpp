// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BufferVectorOperations.h"
#include "DSP/MultichannelBuffer.h"
#include "MetasoundBuildError.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundLog.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRenderCost.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "MetasoundWave.h"
#include "Sound/SoundWaveProxyPlayer.h"

#define LOCTEXT_NAMESPACE "MetasoundWaveNode"

namespace Metasound
{
	namespace WavePlayerVertexNames
	{
		METASOUND_PARAM(InputTriggerPlay, "Play", "Play the wave player.")
		METASOUND_PARAM(InputTriggerStop, "Stop", "Stop the wave player.")
		METASOUND_PARAM(InputWaveAsset, "Wave Asset", "The wave asset to be real-time decoded.")
		METASOUND_PARAM(InputStartTime, "Start Time", "Time into the wave asset to start (seek) the wave asset.")
		METASOUND_PARAM(InputPitchShift, "Pitch Shift", "The pitch shift to use for the wave asset in semitones.")
		METASOUND_PARAM(InputLoop, "Loop", "Whether or not to loop between the start and specified end times.")
		METASOUND_PARAM(InputLoopStart, "Loop Start", "When to start the loop.")
		METASOUND_PARAM(InputLoopDuration, "Loop Duration", "The duration of the loop when wave player is enabled for looping. A negative value will loop the whole wave asset.")
		METASOUND_PARAM(InputMaintainAudioSync, "Maintain Audio Sync", "Force audio to seek to maintain sample accurate playback even if the source audio is latent")
		
		METASOUND_PARAM(OutputTriggerOnPlay, "On Play", "Triggers when Play is triggered.")
		METASOUND_PARAM(OutputTriggerOnDone, "On Finished", "Triggers when the wave played has finished playing or Stop is triggered.")
		METASOUND_PARAM(OutputTriggerOnNearlyDone, "On Nearly Finished", "Triggers when the wave played has almost finished playing (the block before it finishes). Allows time for logic to trigger different variations to play seamlessly.")
		METASOUND_PARAM(OutputTriggerOnLooped, "On Looped", "Triggers when the wave player has looped.")
		METASOUND_PARAM(OutputTriggerOnCuePoint, "On Cue Point", "Triggers when a wave cue point was hit during playback.")
		METASOUND_PARAM(OutputCuePointID, "Cue Point ID", "The cue point ID that was triggered.")
		METASOUND_PARAM(OutputCuePointLabel, "Cue Point Label", "The cue point label that was triggered (if there was a label parsed in the imported .wav file).")
		METASOUND_PARAM(OutputLoopRatio, "Loop Percent", "Returns the current playback location as a ratio of the loop (0-1) if looping is enabled.")
		METASOUND_PARAM(OutputPlaybackLocation, "Playback Location", "Returns the absolute position of the wave playback as a ratio of wave duration (0-1).")
		METASOUND_PARAM(OutputPlaybackTime, "Playback Time", "Returns the current absolute playback time of the wave.")
		METASOUND_PARAM(OutputAudioMono, "Out Mono", "The mono channel audio output.")
		METASOUND_PARAM(OutputAudioLeft, "Out Left", "The left channel audio output.")
		METASOUND_PARAM(OutputAudioRight, "Out Right", "The right channel audio output.")
		METASOUND_PARAM(OutputAudioFrontRight, "Out Front Right", "The front right channel audio output.")
		METASOUND_PARAM(OutputAudioFrontLeft, "Out Front Left", "The front left channel audio output.")
		METASOUND_PARAM(OutputAudioFrontCenter, "Out Front Center", "The front center channel audio output.")
		METASOUND_PARAM(OutputAudioLowFrequency, "Out Low Frequency", "The low frequency channel audio output.")
		METASOUND_PARAM(OutputAudioSideRight, "Out Side Right", "The side right channel audio output.")
		METASOUND_PARAM(OutputAudioSideLeft, "Out Side Left", "The side left channel audio output.")
		METASOUND_PARAM(OutputAudioBackRight, "Out Back Right", "The back right channel audio output.")
		METASOUND_PARAM(OutputAudioBackLeft, "Out Back Left", "The back left channel audio output.")
	}

	// Maximum decode size in frames.
	static int32 MaxDecodeSizeInFrames = 1024;
	FAutoConsoleVariableRef CVarMetaSoundWavePlayerMaxDecodeSizeInFrames(
		TEXT("au.MetaSound.WavePlayer.MaxDecodeSizeInFrames"),
		MaxDecodeSizeInFrames,
		TEXT("Max size in frames used for decoding audio in the MetaSound wave player node.\n")
		TEXT("Default: 1024"),
		ECVF_Default);

	namespace WavePlayerOperatorPrivate
	{
		template<typename AudioChannelConfigurationInfoType>
		FVertexInterface GetVertexInterface()
		{
			using namespace WavePlayerVertexNames;

			//Adds uncommonly used pins to the Advanced View, to reduce the size of the node.
			FDataVertexMetadata InputLoopStartMetaData = METASOUND_GET_PARAM_METADATA(InputLoopStart);
			InputLoopStartMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata InputLoopDurationMetaData = METASOUND_GET_PARAM_METADATA(InputLoopDuration);
			InputLoopDurationMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata InputMaintainAudioSyncMetaData = METASOUND_GET_PARAM_METADATA(InputMaintainAudioSync);
			InputMaintainAudioSyncMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata OutputTriggerOnNearlyDoneMetaData = METASOUND_GET_PARAM_METADATA(OutputTriggerOnNearlyDone);
			OutputTriggerOnNearlyDoneMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata OutputTriggerOnCuePointMetaData = METASOUND_GET_PARAM_METADATA(OutputTriggerOnCuePoint);
			OutputTriggerOnCuePointMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata OutputCuePointIDMetaData = METASOUND_GET_PARAM_METADATA(OutputCuePointID);
			OutputCuePointIDMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata OutputCuePointLabelMetaData = METASOUND_GET_PARAM_METADATA(OutputCuePointLabel);
			OutputCuePointLabelMetaData.bIsAdvancedDisplay = true;

			FDataVertexMetadata OutputPlaybackLocationMetaData = METASOUND_GET_PARAM_METADATA(OutputPlaybackLocation);
			OutputPlaybackLocationMetaData.bIsAdvancedDisplay = true;

			// Workaround to override display name of OutputLoopRatio
			FDataVertexMetadata OutputLoopRatioMetadata
			{
				METASOUND_GET_PARAM_TT(OutputLoopRatio), // description 
				METASOUND_LOCTEXT("OutputLoopRatioNotPercentDisplayName", "Loop Ratio"), // display name  
				true // Is Advanced Display
			};

			FVertexInterface VertexInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerPlay)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerStop)),
					TInputDataVertex<FWaveAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputWaveAsset)),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartTime), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPitchShift), 0.0f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLoop), false),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(InputLoopStart), InputLoopStartMetaData, 0.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(InputLoopDuration), InputLoopDurationMetaData, -1.0f),
					TInputConstructorVertex<bool>(METASOUND_GET_PARAM_NAME(InputMaintainAudioSync), InputMaintainAudioSyncMetaData, false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnPlay)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnDone)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME(OutputTriggerOnNearlyDone), OutputTriggerOnNearlyDoneMetaData),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnLooped)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME(OutputTriggerOnCuePoint), OutputTriggerOnCuePointMetaData),
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME(OutputCuePointID), OutputCuePointIDMetaData),
					TOutputDataVertex<FString>(METASOUND_GET_PARAM_NAME(OutputCuePointLabel), OutputCuePointLabelMetaData),
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME(OutputLoopRatio), OutputLoopRatioMetadata),
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME(OutputPlaybackLocation), OutputPlaybackLocationMetaData),
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputPlaybackTime))
				)
			);

			// Add audio outputs dependent upon source info.
			for (const FOutputDataVertex& OutputDataVertex : AudioChannelConfigurationInfoType::GetAudioOutputs())
			{
				VertexInterface.GetOutputInterface().Add(OutputDataVertex);
			}

			return VertexInterface;
		}

		template<typename AudioChannelConfigurationInfoType>
		FNodeClassMetadata GetNodeInfo()
		{
			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::EngineNodes::Namespace, TEXT("Wave Player"), AudioChannelConfigurationInfoType::GetVariantName() };
			Info.MajorVersion = 1;
			Info.MinorVersion = 1;
			Info.DisplayName = AudioChannelConfigurationInfoType::GetNodeDisplayName();
			Info.Description = METASOUND_LOCTEXT("Metasound_WavePlayerNodeDescription", "Plays a wave asset. The wave's channel configurations will be up or down mixed to match the wave players audio channel format.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface<AudioChannelConfigurationInfoType>();
			Info.Keywords = { METASOUND_LOCTEXT("WavePlayerSoundKeyword", "Sound"),
				                METASOUND_LOCTEXT("WavePlayerCueKeyword", "Cue")
			};

			return Info;
		}
	}

	struct FWavePlayerOpArgs
	{
		FOperatorSettings Settings;
		TArray<FOutputDataVertex> OutputAudioVertices;
		FTriggerReadRef PlayTrigger;
		FTriggerReadRef StopTrigger;
		FWaveAssetReadRef WaveAsset;
		FTimeReadRef StartTime;
		FFloatReadRef PitchShift;
		FBoolReadRef bLoop;
		FTimeReadRef LoopStartTime;
		FTimeReadRef LoopDuration;
		bool bMaintainAudioSync;
		FNodeRenderCost CostReporter;
	};

	/** MetaSound operator for the wave player node. */
	class FWavePlayerOperator : public TExecutableOperator<FWavePlayerOperator>
	{	
	public:

		// Maximum absolute pitch shift in octaves. 
		static constexpr float MaxAbsPitchShiftInOctaves = 6.0f;

		FWavePlayerOperator(const FWavePlayerOpArgs& InArgs)
			: OperatorSettings(InArgs.Settings)
			, CostReporter(InArgs.CostReporter)
			, PlayTrigger(InArgs.PlayTrigger)
			, StopTrigger(InArgs.StopTrigger)
			, WaveAsset(InArgs.WaveAsset)
			, StartTime(InArgs.StartTime)
			, PitchShift(InArgs.PitchShift)
			, bLoop(InArgs.bLoop)
			, LoopStartTime(InArgs.LoopStartTime)
			, LoopDuration(InArgs.LoopDuration)
			, bMaintainAudioSync(InArgs.bMaintainAudioSync)
			, TriggerOnDone(FTriggerWriteRef::CreateNew(InArgs.Settings))
			, TriggerOnNearlyDone(FTriggerWriteRef::CreateNew(InArgs.Settings))
			, TriggerOnLooped(FTriggerWriteRef::CreateNew(InArgs.Settings))
			, TriggerOnCuePoint(FTriggerWriteRef::CreateNew(InArgs.Settings))
			, CuePointID(FInt32WriteRef::CreateNew(0))
			, CuePointLabel(FStringWriteRef::CreateNew(TEXT("")))
			, LoopPercent(FFloatWriteRef::CreateNew(0.0f))
			, PlaybackLocation(FFloatWriteRef::CreateNew(0.0f))
			, PlaybackTime(FTimeWriteRef::CreateNew(0.0))
		{
			for (const FOutputDataVertex& OutputAudioVertex : InArgs.OutputAudioVertices)
			{
				OutputAudioBufferVertexNames.Add(OutputAudioVertex.VertexName);

				FAudioBufferWriteRef AudioBuffer = FAudioBufferWriteRef::CreateNew(InArgs.Settings);
				OutputAudioBuffers.Add(AudioBuffer);

				// Hold on to a view of the output audio. Audio buffers are only writable
				// by this object and will not be reallocated.
				MultiChannelOutputView.Emplace(AudioBuffer->GetData(), AudioBuffer->Num());
			}

			const float SampleRate = InArgs.Settings.GetSampleRate();
			const int32 NumChannels = OutputAudioBuffers.Num();
			FSoundWaveProxyPlayer::FSettings PlayerSettings(SampleRate, NumChannels);
			PlayerSettings.MaxDecodeSizeFrames = FMath::IsPowerOfTwo(MaxDecodeSizeInFrames) ?
				MaxDecodeSizeInFrames : FMath::RoundUpToPowerOfTwo(MaxDecodeSizeInFrames);
			PlayerSettings.bMaintainAudioSync = bMaintainAudioSync;
			ProxyPlayer = FSoundWaveProxyPlayer::Create(PlayerSettings);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WavePlayerVertexNames;
			
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerPlay), PlayTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerStop), StopTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputWaveAsset), WaveAsset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartTime), StartTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPitchShift), PitchShift);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLoop), bLoop);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLoopStart), LoopStartTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLoopDuration), LoopDuration);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(InputMaintainAudioSync), bMaintainAudioSync);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WavePlayerVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnPlay), PlayTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnDone), TriggerOnDone);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnNearlyDone), TriggerOnNearlyDone);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnLooped), TriggerOnLooped);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnCuePoint), TriggerOnCuePoint);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputCuePointID), CuePointID);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputCuePointLabel), CuePointLabel);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputLoopRatio), LoopPercent);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputPlaybackLocation), PlaybackLocation);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputPlaybackTime), PlaybackTime);

			check(OutputAudioBuffers.Num() == OutputAudioBufferVertexNames.Num());

			for (int32 i = 0; i < OutputAudioBuffers.Num(); i++)
			{
				InOutVertexData.BindReadVertex(OutputAudioBufferVertexNames[i], OutputAudioBuffers[i]);
			}
		}

		void Execute()
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FWavePlayerOperator::Execute);

			// Advance all triggers owned by this operator. 
			TriggerOnDone->AdvanceBlock();
			TriggerOnNearlyDone->AdvanceBlock();
			TriggerOnCuePoint->AdvanceBlock();
			TriggerOnLooped->AdvanceBlock();

			// Reset flag to track render cost.
			bDidWaveRenderThisBlock = false;

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
			// Mid-playback hot-reload: if the SoundWave's data has been republished since
			// playback started, rebind the player to the new snapshot and seek back to where
			// we were. CurrentSoundWaveData is the snapshot the player is currently rendering.
			// Skip if the sound naturally finished — re-binding past EOF is pointless.
			if (bIsPlaying && WaveAsset->IsSoundWaveValid() && CurrentSoundWaveData.IsValid() && ProxyPlayer.IsValid() && !ProxyPlayer->IsFinished())
			{
				const TSharedPtr<const FSoundWaveProxy> Proxy = WaveAsset->GetWaveProxy();
				if (Proxy.IsValid() && !Proxy->IsLatestData(CurrentSoundWaveData.Get()))
				{
					const float ResumeTimeSeconds = ProxyPlayer->GetCurrentPlaybackTimeSeconds();
					const TSharedRef<const FSoundWaveData> NewData = Proxy->GetSoundWaveDataRef();
					// Only commit the snapshot after SetSoundWave succeeds. On failure, leave
					// CurrentSoundWaveData unchanged (so the next tick can retry) and clear
					// bIsPlaying so the operator stops trying to render against a dead player.
					if (ProxyPlayer->SetSoundWave(NewData))
					{
						CurrentSoundWaveData = NewData;
						ProxyPlayer->SeekToTime(ResumeTimeSeconds);
					}
					else
					{
						bIsPlaying = false;
					}
				}
			}
#endif // UE_USOUNDWAVE_HOT_RELOADING_ENABLED

			if (ProxyPlayer.IsValid())
			{
				ProxyPlayer->SetSpeed(CalculateSpeedFromPitchShift());
				ProxyPlayer->SetLoop(*bLoop, LoopStartTime->GetSeconds(), LoopDuration->GetSeconds());
				ExecuteSubblocks();
				*LoopPercent = ProxyPlayer->GetLoopProgress();
				*PlaybackLocation = ProxyPlayer->GetPlaybackProgress();
				*PlaybackTime = FTime::FromSeconds(ProxyPlayer->GetCurrentPlaybackTimeSeconds());
			}
			else
			{
				*LoopPercent = 0.0f;
				*PlaybackLocation = 0.0f;
				*PlaybackTime = FTime::FromSeconds(0.0f);
				ZeroFrameRange(0, OperatorSettings.GetNumFramesPerBlock());
			}

			if (bDidWaveRenderThisBlock)
			{
				CostReporter.SetRenderCost(1.f);
			}
		}

		void Reset(const IOperator::FResetParams& ResetParams)
		{
			ensure(ProxyPlayer->GetOutputSampleRate() == ResetParams.OperatorSettings.GetSampleRate());
			ensure(ProxyPlayer->GetOutputNumChannels() == OutputAudioBuffers.Num());
			OperatorSettings = ResetParams.OperatorSettings;

			TriggerOnDone->Reset();
			TriggerOnNearlyDone->Reset();
			TriggerOnLooped->Reset();
			TriggerOnCuePoint->Reset();

			*CuePointID = 0;
			*CuePointLabel = TEXT("");
			*LoopPercent = 0.0f;
			*PlaybackLocation = 0.0f;
			*PlaybackTime = FTime(0.0);
			for (const FAudioBufferWriteRef& BufferRef : OutputAudioBuffers)
			{
				BufferRef->Zero();
			}

			const float SampleRate = ResetParams.OperatorSettings.GetSampleRate();
			const int32 NumChannels = OutputAudioBuffers.Num();
			FSoundWaveProxyPlayer::FSettings PlayerSettings(SampleRate, NumChannels);
			PlayerSettings.MaxDecodeSizeFrames = FMath::IsPowerOfTwo(MaxDecodeSizeInFrames) ?
				MaxDecodeSizeInFrames : FMath::RoundUpToPowerOfTwo(MaxDecodeSizeInFrames);
			PlayerSettings.bMaintainAudioSync = bMaintainAudioSync;
			ProxyPlayer->Reset(PlayerSettings);

			CurrentWaveAsset = FWaveAsset();
			// Clear the hot-reload snapshot so the next StartPlaying captures a fresh one.
			CurrentSoundWaveData.Reset();
			bIsPlaying = false;
			bDidWaveRenderThisBlock = false;
		}

	private:

		void ExecuteSubblocks()
		{
			// Hot reload is handled at the top of Execute() via the FSoundWaveProxy broker
			// (CurrentSoundWaveData / IsLatestData / SetSoundWave), so this used to call into
			// the now-removed FSoundWaveProxy::TryGetLatest().

			// Parse triggers and render audio
			int32 PlayTrigIndex = 0;
			int32 NextPlayFrame = 0;
			const int32 NumPlayTrigs = PlayTrigger->NumTriggeredInBlock();

			int32 StopTrigIndex = 0;
			int32 NextStopFrame = 0;
			const int32 NumStopTrigs = StopTrigger->NumTriggeredInBlock();

			int32 CurrAudioFrame = 0;
			int32 NextAudioFrame = 0;
			const int32 LastAudioFrame = OperatorSettings.GetNumFramesPerBlock() - 1;
			const int32 NoTrigger = OperatorSettings.GetNumFramesPerBlock() << 1;

			while (NextAudioFrame < LastAudioFrame)
			{
				// get the next Play and Stop indices
				// (play)
				if (PlayTrigIndex < NumPlayTrigs)
				{
					NextPlayFrame = (*PlayTrigger)[PlayTrigIndex];
				}
				else
				{
					NextPlayFrame = NoTrigger;
				}

				// (stop)
				if (StopTrigIndex < NumStopTrigs)
				{
					NextStopFrame = (*StopTrigger)[StopTrigIndex];
				}
				else
				{
					NextStopFrame = NoTrigger;
				}

				// determine the next audio frame we are going to render up to
				NextAudioFrame = FMath::Min(NextPlayFrame, NextStopFrame);

				// no more triggers, rendering to the end of the block
				if (NextAudioFrame == NoTrigger)
				{
					NextAudioFrame = OperatorSettings.GetNumFramesPerBlock();
				}

				// render audio (while loop handles looping audio)
				while (CurrAudioFrame != NextAudioFrame)
				{
					if (bIsPlaying)
					{
						RenderFrameRange(CurrAudioFrame, NextAudioFrame);
					}
					else
					{
						ZeroFrameRange(CurrAudioFrame, NextAudioFrame);
					}
					CurrAudioFrame = NextAudioFrame;
				}

				// execute the next trigger
				if (CurrAudioFrame == NextPlayFrame)
				{
					if (!StartPlaying())
					{
						TriggerOnDone->TriggerFrame(CurrAudioFrame);
					}

					++PlayTrigIndex;
				}

				if (CurrAudioFrame == NextStopFrame)
				{
					bIsPlaying = false;
					TriggerOnDone->TriggerFrame(CurrAudioFrame);
					++StopTrigIndex;
				}
			}
		}

		void RenderFrameRange(int32 StartFrame, int32 EndFrame)
		{
			using namespace Audio;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FWavePlayerOperator::RenderFrameRange);

			check(ProxyPlayer.IsValid());
			check(ProxyPlayer->IsPlayerValid());
			check(bIsPlaying);
			const int32 NumFramesToGenerate = EndFrame - StartFrame;

			// Set flag that this node rendered wave audio this block.
			bDidWaveRenderThisBlock = true;

			TArray<FSoundWaveProxyPlayer::FSourceEvent> SourceEvents;
			if (NumFramesToGenerate > 0)
			{
				Audio::FMultichannelBufferView OutputSlice = Audio::SliceMultichannelBufferView(MultiChannelOutputView, StartFrame, NumFramesToGenerate);
				ProxyPlayer->RenderMultiChannelAudio(StartFrame, OutputSlice, SourceEvents);
				bIsPlaying = !ProxyPlayer->IsFinished();
			}

			for (const FSoundWaveProxyPlayer::FSourceEvent& SourceEvent : SourceEvents)
			{
				switch (SourceEvent.Type)
				{
				case FSoundWaveProxyPlayer::FSourceEvent::Loop:
					TriggerOnLooped->TriggerFrame(SourceEvent.OutputFrameIndex);
					break;
				case FSoundWaveProxyPlayer::FSourceEvent::OnNearlyFinished:
					TriggerOnNearlyDone->TriggerFrame(SourceEvent.OutputFrameIndex);
					break;
				case FSoundWaveProxyPlayer::FSourceEvent::OnFinished:
					TriggerOnDone->TriggerFrame(SourceEvent.OutputFrameIndex);
					break;
				case FSoundWaveProxyPlayer::FSourceEvent::CuePoint:
					{
						check(ProxyPlayer->GetCuePoints().IsValidIndex(SourceEvent.CuePointIndex));
						const FSoundWaveCuePoint& CuePoint = ProxyPlayer->GetCuePoints()[SourceEvent.CuePointIndex];
						*CuePointID = CuePoint.CuePointID;
						*CuePointLabel = CuePoint.Label;
						TriggerOnCuePoint->TriggerFrame(SourceEvent.OutputFrameIndex);
					}
					break;
				}
			}
		}

		void ZeroFrameRange(int32 StartFrame, int32 EndFrame)
		{
			const int32 NumFramesToZero = EndFrame - StartFrame;
			if (NumFramesToZero > 0)
			{
				Audio::FMultichannelBufferView OutputSlice = Audio::SliceMultichannelBufferView(MultiChannelOutputView, StartFrame, NumFramesToZero);
				for (TArrayView<float>& ChannelView : OutputSlice)
				{
					FMemory::Memset(ChannelView.GetData(), 0, ChannelView.Num() * ChannelView.GetTypeSize());
				}
			}
		}

		float GetPitchShiftClamped() const
		{
			return FMath::Clamp(*PitchShift, -12.0f * MaxAbsPitchShiftInOctaves, 12.0f * MaxAbsPitchShiftInOctaves);
		}

		float CalculateSpeedFromPitchShift() const
		{
			return FMath::Pow(2.0f, GetPitchShiftClamped() / 12.0f);
		}

		bool StartPlaying()
		{
			check(ProxyPlayer.IsValid());
			CurrentWaveAsset = *WaveAsset;
			// Snapshot the SoundWaveData and bind it to the player. Hold the snapshot in
			// CurrentSoundWaveData so the Execute() hot-reload check can detect a republish
			// and rebind mid-playback (see UE_USOUNDWAVE_HOT_RELOADING_ENABLED branch).
			const TSharedPtr<const FSoundWaveProxy> Proxy = CurrentWaveAsset.GetWaveProxy();
			if (!Proxy.IsValid())
			{
				bIsPlaying = false;
				return false;
			}
			const TSharedRef<const FSoundWaveData> NewSoundWaveData = Proxy->GetSoundWaveDataRef();
			CurrentSoundWaveData = NewSoundWaveData;
			if (ProxyPlayer->SetSoundWave(NewSoundWaveData))
			{
				ProxyPlayer->SetSpeed(CalculateSpeedFromPitchShift());
				ProxyPlayer->SetLoop(*bLoop, LoopStartTime->GetSeconds(), LoopDuration->GetSeconds());
				ProxyPlayer->SeekToTime(StartTime->GetSeconds());
			}

			bIsPlaying = ProxyPlayer.IsValid() && ProxyPlayer->IsPlayerValid() && !ProxyPlayer->IsFinished();
			return bIsPlaying;
		}

		FOperatorSettings OperatorSettings;
		FNodeRenderCost CostReporter;
		
		// Inputs
		FTriggerReadRef PlayTrigger;
		FTriggerReadRef StopTrigger;
		FWaveAssetReadRef WaveAsset;
		FTimeReadRef StartTime;
		FFloatReadRef PitchShift;
		FBoolReadRef bLoop;
		FTimeReadRef LoopStartTime;
		FTimeReadRef LoopDuration;
		const bool bMaintainAudioSync;

		// Outputs
		FTriggerWriteRef TriggerOnDone;
		FTriggerWriteRef TriggerOnNearlyDone;
		FTriggerWriteRef TriggerOnLooped;
		FTriggerWriteRef TriggerOnCuePoint;
		FInt32WriteRef CuePointID;
		FStringWriteRef CuePointLabel;
		FFloatWriteRef LoopPercent;
		FFloatWriteRef PlaybackLocation;
		FTimeWriteRef PlaybackTime;
		TArray<FAudioBufferWriteRef> OutputAudioBuffers;
		TArray<FName> OutputAudioBufferVertexNames;

		// Internal state
		TSharedPtr<const FSoundWaveData> CurrentSoundWaveData;
		bool bIsPlaying = false;
		bool bDidWaveRenderThisBlock = false;
		FWaveAsset CurrentWaveAsset;
		TUniquePtr<FSoundWaveProxyPlayer> ProxyPlayer;
		Audio::FMultichannelBufferView MultiChannelOutputView;
	};

	class FWavePlayerOperatorFactory : public IOperatorFactory
	{
	public:
		FWavePlayerOperatorFactory(const TArray<FOutputDataVertex>& InOutputAudioVertices)
		: OutputAudioVertices(InOutputAudioVertices)
		{
		}

		virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
		{
			using namespace WavePlayerVertexNames;

			const FInputVertexInterfaceData& Inputs = InParams.InputData;

			FWavePlayerOpArgs Args =
			{
				InParams.OperatorSettings,
				OutputAudioVertices,
				Inputs.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerPlay), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerStop), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<FWaveAsset>(METASOUND_GET_PARAM_NAME(InputWaveAsset), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputStartTime), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPitchShift), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputLoop), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputLoopStart), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputLoopDuration), InParams.OperatorSettings),
				Inputs.GetOrCreateDefaultValue<bool>(METASOUND_GET_PARAM_NAME(InputMaintainAudioSync), InParams.OperatorSettings),
				FNodeRenderCost(InParams.GraphRenderCost ? InParams.GraphRenderCost->AddNode(InParams.Node.GetInstanceID(), InParams.Environment) : FNodeRenderCost{})
			};

			return MakeUnique<FWavePlayerOperator>(Args);
		}
	private:
		TArray<FOutputDataVertex> OutputAudioVertices;
	};

	template<typename AudioChannelConfigurationInfoType>
	class TWavePlayerNode : public FBasicNode
	{

	public:
		TWavePlayerNode(const FNodeInitData& InInitData)
			: TWavePlayerNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, WavePlayerOperatorPrivate::GetVertexInterface<AudioChannelConfigurationInfoType>()), MakeShared<const FNodeClassMetadata>(CreateNodeClassMetadata()))
		{
		}

		TWavePlayerNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
			: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
			, Factory(MakeOperatorFactoryRef<FWavePlayerOperatorFactory>(AudioChannelConfigurationInfoType::GetAudioOutputs()))
		{
		}


		virtual ~TWavePlayerNode() = default;

		static FNodeClassMetadata CreateNodeClassMetadata()
		{
			return WavePlayerOperatorPrivate::GetNodeInfo<AudioChannelConfigurationInfoType>();
		}

		virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override
		{
			return Factory;
		}

	private:
		FOperatorFactorySharedRef Factory;
	};

	struct FMonoAudioChannelConfigurationInfo
	{
		static FText GetNodeDisplayName() { return METASOUND_LOCTEXT("Metasound_WavePlayerMonoNodeDisplayName", "Wave Player (1.0, Mono)"); }
		static FName GetVariantName() { return Metasound::EngineNodes::MonoVariant; }

		static TArray<FOutputDataVertex> GetAudioOutputs()
		{
			using namespace WavePlayerVertexNames;
			return {
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioMono))
			};
		}
	};
	using FMonoWavePlayerNode = TWavePlayerNode<FMonoAudioChannelConfigurationInfo>;
	METASOUND_REGISTER_NODE(FMonoWavePlayerNode);

	struct FStereoAudioChannelConfigurationInfo
	{
		static FText GetNodeDisplayName() { return METASOUND_LOCTEXT("Metasound_WavePlayerStereoNodeDisplayName", "Wave Player (2.0, Stereo)"); }
		static FName GetVariantName() { return Metasound::EngineNodes::StereoVariant; }

		static TArray<FOutputDataVertex> GetAudioOutputs()
		{
			using namespace WavePlayerVertexNames;
			return {
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioRight))
			};
		}
	};
	using FStereoWavePlayerNode = TWavePlayerNode<FStereoAudioChannelConfigurationInfo>;
	METASOUND_REGISTER_NODE(FStereoWavePlayerNode);

	struct FQuadAudioChannelConfigurationInfo
	{
		static FText GetNodeDisplayName() { return METASOUND_LOCTEXT("Metasound_WavePlayerQuadNodeDisplayName", "Wave Player (4.0, Quad)"); }
		static FName GetVariantName() { return Metasound::EngineNodes::QuadVariant; }

		static TArray<FOutputDataVertex> GetAudioOutputs()
		{
			using namespace WavePlayerVertexNames;
			return {
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontRight)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideRight))
			};
		}
	};
	using FQuadWavePlayerNode = TWavePlayerNode<FQuadAudioChannelConfigurationInfo>;
	METASOUND_REGISTER_NODE(FQuadWavePlayerNode);

	struct FFiveDotOneAudioChannelConfigurationInfo
	{
		static FText GetNodeDisplayName() { return METASOUND_LOCTEXT("Metasound_WavePlayerFiveDotOneNodeDisplayName", "Wave Player (5.1, Surround)"); }
		static FName GetVariantName() { return Metasound::EngineNodes::FiveDotOneVariant; }

		static TArray<FOutputDataVertex> GetAudioOutputs()
		{
			using namespace WavePlayerVertexNames;
			return {
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontRight)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontCenter)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLowFrequency)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideRight))
			};
		}
	};
	using FFiveDotOneWavePlayerNode = TWavePlayerNode<FFiveDotOneAudioChannelConfigurationInfo>;
	METASOUND_REGISTER_NODE(FFiveDotOneWavePlayerNode);

	struct FSevenDotOneAudioChannelConfigurationInfo
	{
		static FText GetNodeDisplayName() { return METASOUND_LOCTEXT("Metasound_WavePlayerSevenDotOneNodeDisplayName", "Wave Player (7.1, Surround)"); }
		static FName GetVariantName() { return Metasound::EngineNodes::SevenDotOneVariant; }

		static TArray<FOutputDataVertex> GetAudioOutputs()
		{
			using namespace WavePlayerVertexNames;
			return {
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontRight)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioFrontCenter)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLowFrequency)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSideRight)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioBackLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioBackRight))
			};
		}
	};
	using FSevenDotOneWavePlayerNode = TWavePlayerNode<FSevenDotOneAudioChannelConfigurationInfo>;
	METASOUND_REGISTER_NODE(FSevenDotOneWavePlayerNode);
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundWaveNode