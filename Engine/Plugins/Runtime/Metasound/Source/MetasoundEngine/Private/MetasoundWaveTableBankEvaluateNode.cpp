// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundWaveTable.h"
#include "WaveTable.h"
#include "WaveTableSampler.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	namespace WaveTableBankEvaluateNode
	{
		METASOUND_PARAM(WaveTableBankEval_BankParam, "WaveTableBank", "The WaveTableBank to evaluate");
		METASOUND_PARAM(WaveTableBankEval_InputParam, "Input", "The X input with which to evaluate the wavetable ([-1, 1] or [0, 1] depending on Bank's 'bipolar' setting)");
		METASOUND_PARAM(WaveTableBankEval_IndexParam, "Index", "The table index to interpolate and evaluate the result of (wraps over number of entries)");
		METASOUND_PARAM(WaveTableBankEval_OutParam, "Output", "The linearly mixed value of the provided WaveTableBank's applicable entries");
	} // WaveTableBankEvaluateNode
	
	class FMetasoundWaveTableBankEvaluateNodeOperator : public TExecutableOperator<FMetasoundWaveTableBankEvaluateNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;
			using namespace WaveTableBankEvaluateNode;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FWaveTableBankAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTableBankEval_BankParam)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTableBankEval_InputParam), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTableBankEval_IndexParam), 0.0f),
					TInputDataVertex<FEnumWaveTableInterpolationMode>("Interpolation", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableBankEvaluateNode_InterpDescription", "How interpolation occurs between WaveTable values."),
						LOCTEXT("MetasoundWaveTableBankEvaluateNode_Interp", "Interpolation"),
						true /* bIsAdvancedDisplay */
					}, static_cast<int32>(FWaveTableSampler::EInterpolationMode::Linear))
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTableBankEval_OutParam))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					{ EngineNodes::Namespace, "WaveTableBankEvaluate", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("MetasoundWaveTableBankEvaluateNode_Name", "Evaluate WaveTableBank"),
					LOCTEXT("MetasoundWaveTableBankEvaluateNode_Description",
						"Evaluates a WaveTableBank's given entires for a given input value, linearly interpolating inline between using the provided index. More performant "
						"than using 'WaveTableGet' and using the 'WaveTableEvaluate' nodes as no resulting WaveTable is generated any time the input float index is changed."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ NodeCategories::Envelopes, METASOUND_LOCTEXT("WaveTableBankEvaluateCurveKeyword", "Curve") },
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveTable;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableBankAssetReadRef InWaveTableReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTableBankAsset>("WaveTableBank", InParams.OperatorSettings);
			FFloatReadRef InInputReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("Input", InParams.OperatorSettings);
			FFloatReadRef InIndexReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("Index", InParams.OperatorSettings);
			FEnumWaveTableInterpModeReadRef InInterpReadRef = InputData.GetOrCreateDefaultDataReadReference<FEnumWaveTableInterpolationMode>("Interpolation", InParams.OperatorSettings);

			return MakeUnique<FMetasoundWaveTableBankEvaluateNodeOperator>(InParams, InWaveTableReadRef, InInputReadRef, InIndexReadRef, InInterpReadRef);
		}

		FMetasoundWaveTableBankEvaluateNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableBankAssetReadRef& InWaveTableBankReadRef,
			const FFloatReadRef& InInputReadRef,
			const FFloatReadRef& InIndexReadRef,
			const FEnumWaveTableInterpModeReadRef& InInterpModeReadRef)
			: WaveTableBankReadRef(InWaveTableBankReadRef)
			, InputReadRef(InInputReadRef)
			, IndexReadRef(InIndexReadRef)
			, InterpModeReadRef(InInterpModeReadRef)
			, OutWriteRef(TDataWriteReferenceFactory<float>::CreateAny(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual ~FMetasoundWaveTableBankEvaluateNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WaveTableBankEvaluateNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveTableBankEval_BankParam), WaveTableBankReadRef);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveTableBankEval_InputParam), InputReadRef);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveTableBankEval_IndexParam), IndexReadRef);
			InOutVertexData.BindReadVertex("Interpolation", InterpModeReadRef);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WaveTableBankEvaluateNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveTableBankEval_OutParam), OutWriteRef);
		}

		void Execute()
		{
			using namespace WaveTable;

			const FWaveTableBankAsset& WaveTableBankAsset = *WaveTableBankReadRef;
			FWaveTableBankAssetProxyPtr Proxy = WaveTableBankAsset.GetProxy();
			float NewIndex = 0.f;

			if (!WaveTableBankAsset.IsValid())
			{
				*OutWriteRef = 0.f; // same as Reset() state
				return;
			}

			const float Min = WaveTableBankAsset->IsBipolar() ? -1.f : 0.f;
			const float Input = FMath::Clamp(*InputReadRef, Min, 1.f);
			if (!ResolveNextComputeIndex(Proxy, Input, NewIndex))
			{
				return;
			}

			const TArray<FWaveTableData>& WaveTables = WaveTableBankAsset->GetWaveTableData();
			const int32 IndexFloor = FMath::FloorToInt32(NewIndex) % WaveTables.Num();
			const int32 IndexCeil = FMath::CeilToInt32(NewIndex) % WaveTables.Num();

			const FWaveTableData& IndexFloorTable = WaveTables[IndexFloor];

			float IndexFloorValue = 0.0f;
			constexpr FWaveTableSampler::ESingleSampleMode SampleMode =  FWaveTableSampler::ESingleSampleMode::Hold;

			Sampler.Reset();
			Sampler.SetPhase(Input);
			Sampler.Process(IndexFloorTable, IndexFloorValue, SampleMode);

			if (IndexFloor != IndexCeil)
			{
				const FWaveTableData& IndexCeilTable = WaveTables[IndexCeil];
				float IndexCeilValue = 0.f;
				Sampler.Reset();
				Sampler.SetPhase(Input);
				Sampler.Process(IndexCeilTable, IndexCeilValue, SampleMode);

				const float Fractional = FMath::Frac(NewIndex);
				CachedState.Value = (IndexFloorValue * (1.f - Fractional)) + (IndexCeilValue * Fractional);
			}
			else
			{
				CachedState.Value = IndexFloorValue;
			}

			*OutWriteRef = CachedState.Value;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace WaveTable;

			FWaveTableSampler::FSettings Settings;
			Settings.Freq = 0.0f; // Sampler phase is manually progressed via this node
			Sampler = FWaveTableSampler(MoveTemp(Settings));

			CachedState = { };
			*OutWriteRef = 0.f;
		}

	private:
		// Returns true if new computation is required, setting OutIndex to index to compute.
		// Returns false if new computation isn't required, resetting output & cached data accordingly.
		bool ResolveNextComputeIndex(const FWaveTableBankAssetProxyPtr& Proxy, float Input, float& OutIndex)
		{
			using namespace WaveTable;

			if (!Proxy.IsValid())
			{
				CachedState = { };
				*OutWriteRef = 0.f;
				return false;
			}

			const float Index = *IndexReadRef;
			const FWaveTableSampler::EInterpolationMode NewInterpolationMode = *InterpModeReadRef;
			if (CachedState.InterpolationMode == NewInterpolationMode)
			{
				if (FMath::IsNearlyEqual(Index, CachedState.Index))
				{
					if (FMath::IsNearlyEqual(Input, CachedState.Input))
					{
						*OutWriteRef = CachedState.Value;
						return false;
					}
				}
			}

			Sampler.SetInterpolationMode(NewInterpolationMode);
			OutIndex = FMath::Abs(Index); // Avoids fractional, interpolative flip at zero crossing

			CachedState.InterpolationMode = NewInterpolationMode;
			CachedState.Input = Input;
			CachedState.Index = OutIndex;
			return true;
		}

		FWaveTableBankAssetReadRef WaveTableBankReadRef;
		FFloatReadRef InputReadRef;
		FFloatReadRef IndexReadRef;
		FEnumWaveTableInterpModeReadRef InterpModeReadRef;

		WaveTable::FWaveTableSampler Sampler;

		FFloatWriteRef OutWriteRef;

		struct FCachedState
		{
			WaveTable::FWaveTableSampler::EInterpolationMode InterpolationMode = WaveTable::FWaveTableSampler::EInterpolationMode::COUNT;
			float Index = TNumericLimits<float>::Max();
			float Input = TNumericLimits<float>::Max();
			float Value = TNumericLimits<float>::Max();
		} CachedState;
	};

	using FMetasoundWaveTableBankEvaluateNode = TNodeFacade<FMetasoundWaveTableBankEvaluateNodeOperator>;

	METASOUND_REGISTER_NODE(FMetasoundWaveTableBankEvaluateNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
