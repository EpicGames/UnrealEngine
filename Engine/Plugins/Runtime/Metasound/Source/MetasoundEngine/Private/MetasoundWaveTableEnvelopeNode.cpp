// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTime.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundWaveTable.h"
#include "WaveTableSampler.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"


namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(WaveTable::FWaveTableSampler::ESingleSampleMode, FEnumWaveTableEnvelopeMode, "WaveTableEnvelopeMode")
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::ESingleSampleMode::Loop, "LoopDisplayName", "Loop", "EnvMode_LoopDescription", "Interpolates last value and first value in WaveTable, starting over interpolation of envelope on completion."),
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::ESingleSampleMode::Hold, "HoldDisplayName", "Hold", "EnvMode_HoldDescription", "Holds last value in table if elapsed beyond WaveTable length"),
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::ESingleSampleMode::Unit, "UnitDisplayName", "Unit", "EnvMode_UnitDescription", "Interpolates last value in table with unit (1.0f) if elapsed beyond WaveTable length."),
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::ESingleSampleMode::Zero, "ZeroDisplayName", "Zero", "EnvMode_ZeroDescription", "Interpolates last value in table with zero (0.0f) if elapsed beyond WaveTable length"),
	DEFINE_METASOUND_ENUM_END()

	DEFINE_METASOUND_ENUM_BEGIN(WaveTable::FWaveTableSampler::EInterpolationMode, FEnumWaveTableInterpolationMode, "WaveTableInterpolation")
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::EInterpolationMode::None, "InterpModeDisplayName", "None (Step)", "EnvMode_InterpDescriptionNone", "No interpolation between values (uses lowest)."),
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::EInterpolationMode::Linear, "InterpModeNameLinear", "Linear", "EnvMode_InterpDescriptionLinearly", "Linearly interpolates between values."),
		DEFINE_METASOUND_ENUM_ENTRY(WaveTable::FWaveTableSampler::EInterpolationMode::Cubic, "InterpModeNameCubic", "Cubic", "EnvMode_InterpDescriptionCubically", "Cubically interpolates between values.")
	DEFINE_METASOUND_ENUM_END()

	class FMetasoundWaveTableEnvelopeNodeOperator : public TExecutableOperator<FMetasoundWaveTableEnvelopeNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FWaveTable>("WaveTable", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_InputWaveTable", "WaveTable") }),
					TInputDataVertex<FTrigger>("Play", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_InputOnPlay", "Play") }),
					TInputDataVertex<FTrigger>("Stop", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_InputOnStop", "Stop") }),
					TInputDataVertex<FTrigger>("Pause", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_InputOnPause", "Pause") }),
					TInputDataVertex<FTime>("Duration", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_Duration", "Duration") }, 1.0f),
					TInputDataVertex<FEnumWaveTableEnvelopeMode>("Mode", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableEnvelopeNode_ModeDescription", "What value the envelope completes on (or whether it loops)."),
						LOCTEXT("MetasoundWaveTableEnvelopeNode_Mode", "Mode"),
						true /* bIsAdvancedDisplay */
					}, static_cast<int32>(WaveTable::FWaveTableSampler::ESingleSampleMode::Zero)),
					TInputDataVertex<FEnumWaveTableInterpolationMode>("Interpolation", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableEnvelopeNode_InterpDescription", "How the envelope interpolates between WaveTable values."),
						LOCTEXT("MetasoundWaveTableEnvelopeNode_Interp", "Interpolation"),
						true /* bIsAdvancedDisplay */
					}, static_cast<int32>(FWaveTableSampler::EInterpolationMode::Linear))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>("OnFinished", FDataVertexMetadata{ LOCTEXT("MetasoundWaveTableEnvelopeNode_OnFinished", "OnFinished") }),
					TOutputDataVertex<float>("Out", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableEnvelopeNode_Output", "Out") })
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					{ EngineNodes::Namespace, "WaveTableEnvelope", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("MetasoundWaveTableEnvelopeNode_Name", "WaveTable Envelope"),
					LOCTEXT("MetasoundWaveTableEnvelopeNode_Description", "Reads through the given WaveTable over the given duration."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ NodeCategories::Generators, METASOUND_LOCTEXT("WaveTableEnvelopeSynthesisKeyword", "Synthesis")},
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveTable;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableReadRef InWaveTableReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTable>("WaveTable", InParams.OperatorSettings);
			FTriggerReadRef InPlayReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Play", InParams.OperatorSettings);
			FTriggerReadRef InStopReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Stop", InParams.OperatorSettings);
			FTriggerReadRef InPauseReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Pause", InParams.OperatorSettings);
			FTimeReadRef InDurationReadRef = InputData.GetOrCreateDefaultDataReadReference<FTime>("Duration", InParams.OperatorSettings);
			FEnumWaveTableEnvelopeModeReadRef InModeReadRef = InputData.GetOrCreateDefaultDataReadReference<FEnumWaveTableEnvelopeMode>("Mode", InParams.OperatorSettings);
			FEnumWaveTableInterpModeReadRef InInterpReadRef = InputData.GetOrCreateDefaultDataReadReference<FEnumWaveTableInterpolationMode>("Interpolation", InParams.OperatorSettings);

			return MakeUnique<FMetasoundWaveTableEnvelopeNodeOperator>(InParams, InWaveTableReadRef, InPlayReadRef, InStopReadRef, InPauseReadRef, InDurationReadRef, InModeReadRef, InInterpReadRef);
		}

		FMetasoundWaveTableEnvelopeNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableReadRef& InWaveTableReadRef,
			const FTriggerReadRef& InPlayReadRef,
			const FTriggerReadRef& InStopReadRef,
			const FTriggerReadRef& InPauseReadRef,
			const FTimeReadRef& InDurationReadRef,
			const FEnumWaveTableEnvelopeModeReadRef& InModeReadRef,
			const FEnumWaveTableInterpModeReadRef& InInterpModeReadRef)
			: WaveTableReadRef(InWaveTableReadRef)
			, PlayReadRef(InPlayReadRef)
			, StopReadRef(InStopReadRef)
			, PauseReadRef(InPauseReadRef)
			, DurationReadRef(InDurationReadRef)
			, ModeReadRef(InModeReadRef)
			, InterpModeReadRef(InInterpModeReadRef)
			, OnFinishedWriteRef(TDataWriteReference<FTrigger>::CreateNew(InParams.OperatorSettings))
			, OutWriteRef(TDataWriteReferenceFactory<float>::CreateAny(InParams.OperatorSettings))
			, SampleRate(InParams.OperatorSettings.GetSampleRate())
		{
			Reset(InParams);
		}

		virtual ~FMetasoundWaveTableEnvelopeNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			
			InOutVertexData.BindReadVertex("WaveTable", WaveTableReadRef);
			InOutVertexData.BindReadVertex("Play", PlayReadRef);
			InOutVertexData.BindReadVertex("Stop", StopReadRef);
			InOutVertexData.BindReadVertex("Pause", PauseReadRef);
			InOutVertexData.BindReadVertex("Duration", DurationReadRef);
			InOutVertexData.BindReadVertex("Mode", ModeReadRef);
			InOutVertexData.BindReadVertex("Interpolation", InterpModeReadRef);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex("OnFinished", OnFinishedWriteRef);
			InOutVertexData.BindReadVertex("Out", OutWriteRef);
		}

		float GetStopValue(WaveTable::FWaveTableSampler::ESingleSampleMode InSampleMode) const
		{
			using namespace WaveTable;

			if (bPaused)
			{
				return *OutWriteRef;
			}

			switch (InSampleMode)
			{
				case FWaveTableSampler::ESingleSampleMode::Unit:
				return 1.0f;

				case FWaveTableSampler::ESingleSampleMode::Hold:
				return WaveTableReadRef->GetView().FinalValue;

				case FWaveTableSampler::ESingleSampleMode::Zero:
				case FWaveTableSampler::ESingleSampleMode::Loop:
				default:
				return 0.0f;
			}
		}

		void Execute()
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundWaveTableEnvelopeNodeOperator::Execute);

			// To avoid unexpected behavior of only firing a single "OnFinished" per block if set to looping,
			// limit the duration to a minimum threshold of SecondsPerBlock.
			const float Dur = FMath::Max(DurationReadRef->GetSeconds(), SecondsPerBlock);

			WaveTable::FWaveTableSampler::ESingleSampleMode Mode = *ModeReadRef;
			WaveTable::FWaveTableSampler::EInterpolationMode InterpMode = *InterpModeReadRef;

			OnFinishedWriteRef->AdvanceBlock();

			float NextValue = GetStopValue(Mode);

			auto GetLastIndex = [](const FTriggerReadRef& Trigger)
			{
				int32 LastIndex = -1;
				Trigger->ExecuteBlock([](int32, int32) {}, [&LastIndex](int32 StartFrame, int32 EndFrame)
				{
					LastIndex = FMath::Max(LastIndex, StartFrame);
				});
				return LastIndex;
			};

			const int32 LastPlayIndex = GetLastIndex(PlayReadRef);
			const int32 LastStopIndex = GetLastIndex(StopReadRef);
			const int32 LastPauseIndex = GetLastIndex(PauseReadRef);

			if (LastPlayIndex >= 0 || LastStopIndex >= 0)
			{
				if (LastPlayIndex > LastStopIndex)
				{
					Elapsed = 0.0f;
					bPaused = LastPauseIndex > LastPlayIndex;
				}
				else // Stop wins if same frame
				{
					if (Elapsed >= 0.0f)
					{
						Elapsed = -1.0f;
						OnFinishedWriteRef->TriggerFrame(LastStopIndex);
					}
					bPaused = false;
				}
			}
			else if (LastPauseIndex >= 0)
			{
				if (Elapsed >= 0.0f)
				{
					bPaused = !bPaused;
					if (bPaused)
					{
						NextValue = *OutWriteRef;
					}
				}
			}

			const WaveTable::FWaveTableView& WaveTableView = WaveTableReadRef->GetView();

			if (WaveTableView.SampleView.IsEmpty())
			{
				if (Elapsed >= 0.0f)
				{
					Elapsed = -1.0f;
					OnFinishedWriteRef->TriggerFrame(0);
					bPaused = false;
				}
			}
			else
			{
				if (Elapsed >= 0.0f && !bPaused)
				{
					if (Dur > 0.0f)
					{
						Sampler.Reset();
						Sampler.SetInterpolationMode(*InterpModeReadRef);
						Sampler.SetPhase(FMath::Clamp(Elapsed / Dur, 0.0f, 1.0f));
						Sampler.Process(WaveTableView, NextValue, Mode);
					}

					Elapsed += SecondsPerBlock;
				}
			}

			if (Elapsed > Dur)
			{
				int32 FinishSampleIndex = 0;
				if (Mode == WaveTable::FWaveTableSampler::ESingleSampleMode::Loop)
				{
					while (Elapsed >= Dur)
					{
						Elapsed -= Dur;
					}
					FinishSampleIndex = Elapsed * SampleRate;
				}
				else
				{
					FinishSampleIndex = (Elapsed - Dur) * SampleRate;
					Elapsed = -1.0f;
				}

				FinishSampleIndex %= BlockSize;
				OnFinishedWriteRef->TriggerFrame(FinishSampleIndex);
			}

			*OutWriteRef = NextValue;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			{
				using namespace WaveTable;
				FWaveTableSampler::FSettings Settings;
				Settings.Freq = 0.0f; // Sampler phase is manually progressed via this node
				Sampler = FWaveTableSampler(MoveTemp(Settings));
			}

			{
				const float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
				check(BlockRate > 0.0f && !FMath::IsNearlyZero(BlockRate));
				SecondsPerBlock = 1.0f / BlockRate;
			}

			{
				BlockSize = InParams.OperatorSettings.GetNumFramesPerBlock();
				check(BlockSize > 0);
			}

			OnFinishedWriteRef->Reset();
			*OutWriteRef = 0.f;

			Elapsed = -1.0f;
			bPaused = false;
		}

	private:
		FWaveTableReadRef WaveTableReadRef;
		FTriggerReadRef PlayReadRef;
		FTriggerReadRef StopReadRef;
		FTriggerReadRef PauseReadRef;
		FTimeReadRef DurationReadRef;
		FEnumWaveTableEnvelopeModeReadRef ModeReadRef;
		FEnumWaveTableInterpModeReadRef InterpModeReadRef;

		WaveTable::FWaveTableSampler Sampler;

		FTriggerWriteRef OnFinishedWriteRef;
		FFloatWriteRef OutWriteRef;

		int32 BlockSize = 0;

		float SecondsPerBlock = 0.0f;
		float SampleRate = 0.0f;
		float Elapsed = -1.0f;

		bool bPaused = false;
	};

	using FMetasoundWaveTableEnvelopeNode = TNodeFacade<FMetasoundWaveTableEnvelopeNodeOperator>;
	METASOUND_REGISTER_NODE(FMetasoundWaveTableEnvelopeNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
