// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundWaveTable.h"
#include "WaveTableSampler.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	namespace WaveTableEvaluateNode
	{
		METASOUND_PARAM(WaveTableParam, "WaveTable", "The Wavetable to evaluate");
		METASOUND_PARAM(InputParam, "Input", "[0, 1] the X input with which to evaluate the wavetable");
		METASOUND_PARAM(OutParam, "Output", "The value of the wavetable at the specified point");
	}
	
	class FMetasoundWaveTableEvaluateNodeOperator : public TExecutableOperator<FMetasoundWaveTableEvaluateNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;
			using namespace WaveTableEvaluateNode;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FWaveTable>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTableParam)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputParam), 0.0f),
					TInputDataVertex<FEnumWaveTableInterpolationMode>("Interpolation", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableEvaluateNode_InterpDescription", "How the Evaluate interpolates between WaveTable values."),
						LOCTEXT("MetasoundWaveTableEvaluateNode_Interp", "Interpolation"),
						true /* bIsAdvancedDisplay */
					}, static_cast<int32>(FWaveTableSampler::EInterpolationMode::Linear))
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParam))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					{ EngineNodes::Namespace, "WaveTableEvaluate", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("MetasoundWaveTableEvaluateNode_Name", "Evaluate WaveTable"),
					LOCTEXT("MetasoundWaveTableEvaluateNode_Description", "Evaluates a wavetable for a given input value."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ NodeCategories::Envelopes, METASOUND_LOCTEXT("WaveTableEvaluateCurveKeyword", "Curve")},
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveTable;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableReadRef InWaveTableReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTable>("WaveTable", InParams.OperatorSettings);
			FFloatReadRef InInputReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("Input", InParams.OperatorSettings);
			FEnumWaveTableInterpModeReadRef InInterpReadRef = InputData.GetOrCreateDefaultDataReadReference<FEnumWaveTableInterpolationMode>("Interpolation", InParams.OperatorSettings);

			return MakeUnique<FMetasoundWaveTableEvaluateNodeOperator>(InParams, InWaveTableReadRef, InInputReadRef, InInterpReadRef);
		}

		FMetasoundWaveTableEvaluateNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableReadRef& InWaveTableReadRef,
			const FFloatReadRef& InInputReadRef,
			const FEnumWaveTableInterpModeReadRef& InInterpModeReadRef)
			: WaveTableReadRef(InWaveTableReadRef)
			, InputReadRef(InInputReadRef)
			, InterpModeReadRef(InInterpModeReadRef)
			, OutWriteRef(TDataWriteReferenceFactory<float>::CreateAny(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual ~FMetasoundWaveTableEvaluateNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WaveTableEvaluateNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveTableParam), WaveTableReadRef);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputParam), InputReadRef);
			InOutVertexData.BindReadVertex("Interpolation", InterpModeReadRef);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace WaveTableEvaluateNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParam), OutWriteRef);
		}

		void Execute()
		{
			using namespace WaveTable;
			const float Input = *InputReadRef;

			float NextValue = 0.f;
			
			const FWaveTableView& WaveTableView = WaveTableReadRef->GetView();

			constexpr auto SampleMode =  FWaveTableSampler::ESingleSampleMode::Hold;
			
			Sampler.Reset();
			Sampler.SetInterpolationMode(*InterpModeReadRef);
			Sampler.SetPhase(FMath::Clamp(Input, 0.0f, 1.0f));
			Sampler.Process(WaveTableView, NextValue, SampleMode);

			*OutWriteRef = NextValue;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace WaveTable;
			FWaveTableSampler::FSettings Settings;
			Settings.Freq = 0.0f; // Sampler phase is manually progressed via this node
			Sampler = FWaveTableSampler(MoveTemp(Settings));

			*OutWriteRef = 0.f;
		}

	private:
		FWaveTableReadRef WaveTableReadRef;
		FFloatReadRef InputReadRef;
		FEnumWaveTableInterpModeReadRef InterpModeReadRef;

		WaveTable::FWaveTableSampler Sampler;

		FFloatWriteRef OutWriteRef;
	};

	using FMetasoundWaveTableEvaluateNode = TNodeFacade<FMetasoundWaveTableEvaluateNodeOperator>;
	METASOUND_REGISTER_NODE(FMetasoundWaveTableEvaluateNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
