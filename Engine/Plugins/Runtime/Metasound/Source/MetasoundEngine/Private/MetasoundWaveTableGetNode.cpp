// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FloatArrayMath.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTrace.h"
#include "MetasoundWaveTable.h"
#include "WaveTable.h"

#define LOCTEXT_NAMESPACE "MetasoundEngine"


namespace Metasound
{
	class FMetasoundWaveTableGetNodeOperator : public TExecutableOperator<FMetasoundWaveTableGetNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FWaveTableBankAsset>("WaveTableBank", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableGetNode_InputWaveTableBank", "WaveTableBank") }),
					TInputDataVertex<float>("TableIndex", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableGetNode_InputTableIndex", "Index - supports Float input to allow index interpolation") }, 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FWaveTable>("Out", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableGetNode_OutputWaveTable", "Out") })
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					FNodeClassName { EngineNodes::Namespace, "MetasoundWaveTableGet", "" },
					1, // Major Version
					1, // Minor Version
					LOCTEXT("MetasoundWaveTableGetNode_Name", "Get WaveTable From Bank"),
					LOCTEXT("MetasoundWaveTableGetNode_Description",
						"Gets or generates interpolated WaveTable from provided WaveTableBank asset based on asset sampling mode "
						"(Table Interpolation is supported only for 'FixedResolution' banks).\n"
						"v1.1: Now supports discrete selection of WaveTableBank assets set to 'Fixed"
						"Sample Rate' and/or a bit depth of 16 bit, however index interpolation"
						"is only supported for assets set to 'Fixed Resolution'"),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ },
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace Metasound;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableBankAssetReadRef WaveTableBankReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTableBankAsset>("WaveTableBank", InParams.OperatorSettings);
			FFloatReadRef TableIndexReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("TableIndex", InParams.OperatorSettings);

			return MakeUnique<FMetasoundWaveTableGetNodeOperator>(InParams, WaveTableBankReadRef, TableIndexReadRef);
		}

		FMetasoundWaveTableGetNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableBankAssetReadRef& InWaveTableBankReadRef,
			const FFloatReadRef& InTableIndexReadRef)
			: WaveTableBankReadRef(InWaveTableBankReadRef)
			, TableIndexReadRef(InTableIndexReadRef)
			, OutTable(TDataWriteReferenceFactory<WaveTable::FWaveTable>::CreateAny(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual ~FMetasoundWaveTableGetNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			InOutVertexData.BindReadVertex("WaveTableBank", WaveTableBankReadRef);
			InOutVertexData.BindReadVertex("TableIndex", TableIndexReadRef);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			InOutVertexData.BindReadVertex("Out", OutTable);
		}

	private:
		void ComputeGainIndexData(const TArray<FWaveTableData>& WaveTables, EWaveTableSamplingMode InSampleMode, ::WaveTable::FWaveTable& OutputWaveTable)
		{
			checkf(!WaveTables.IsEmpty(), TEXT("ComputGainIndexData must have WaveTable to operate on"));

			if (WaveTables.Num() == 1)
			{
				GainIndexData.LowIndex = INDEX_NONE;
				GainIndexData.HighIndex = 0;
				GainIndexData.HighIndexGain = 1.0f;
				GainIndexData.NumSamples = WaveTables[0].GetNumSamples();
				return;
			}

			switch (InSampleMode)
			{
				case EWaveTableSamplingMode::FixedSampleRate:
				{
					GainIndexData.LowIndex = INDEX_NONE;
					GainIndexData.HighIndex = FMath::FloorToInt32(GainIndexData.TableIndex);
					GainIndexData.HighIndexGain = 1.0f;
					GainIndexData.NumSamples = WaveTables[GainIndexData.HighIndex].GetNumSamples();
				}
				break;

				case EWaveTableSamplingMode::FixedResolution:
				{
					GainIndexData.LowIndex = FMath::FloorToInt32(GainIndexData.TableIndex);
					GainIndexData.HighIndex = FMath::CeilToInt32(GainIndexData.TableIndex) % WaveTables.Num();

					// If low/high are same index, give all energy to high index and invalidate
					// low index (the float index is computed to be an integer equivalent)
					if (GainIndexData.LowIndex == GainIndexData.HighIndex)
					{
						GainIndexData.LowIndex = INDEX_NONE;
						GainIndexData.HighIndexGain = 1.0f;
					}
					else
					{
						GainIndexData.HighIndexGain = GainIndexData.TableIndex - GainIndexData.LowIndex;
					}

					GainIndexData.NumSamples = WaveTables.Last().GetNumSamples();
				}
				break;

				default:
				{
					static_assert(static_cast<int32>(EWaveTableSamplingMode::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableSamplingMode'");
					checkNoEntry();
				}
			};
		}

	public:
		void Execute()
		{
			using namespace WaveTable;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundWaveTableGetNodeOperator::Execute);

			const FWaveTableBankAsset& WaveTableBankAsset = *WaveTableBankReadRef;
			FWaveTable& OutputWaveTable = *OutTable;

			FWaveTableBankAssetProxyPtr Proxy = WaveTableBankAsset.GetProxy();
			if (Proxy.IsValid() == false)
			{
				ResetInternal();
				return;
			}
			
			const TArray<FWaveTableData>& WaveTables = Proxy->GetWaveTableData();
			if (WaveTables.IsEmpty())
			{
				ResetInternal();
				return;
			}

			GainIndexData.TableIndex = *TableIndexReadRef;
			FWaveTable::WrapIndexSmooth(WaveTables.Num(), GainIndexData.TableIndex);

			const bool bIsLastIndex = FMath::IsNearlyEqual(GainIndexData.LastTableIndex, GainIndexData.TableIndex);
			const bool bIsLastProxy = LastTableId == Proxy->GetObjectId();
			if (bIsLastIndex && bIsLastProxy)
			{
				return;
			}

			LastTableId = Proxy->GetObjectId();
			GainIndexData.LastTableIndex = GainIndexData.TableIndex;

			ComputeGainIndexData(WaveTables, Proxy->GetSampleMode(), OutputWaveTable);
			if (GainIndexData.NumSamples <= 0)
			{
				LastTableId = INDEX_NONE;
				OutputWaveTable = { };
				return;
			}

			float FinalValue = 0.0f;
			OutputWaveTable.SetNum(GainIndexData.NumSamples);
			OutputWaveTable.Zero();
			for (int32 TableIndex = 0; TableIndex < WaveTables.Num(); ++TableIndex)
			{
				float Gain = 0.0f;

				if (TableIndex == GainIndexData.LowIndex)
				{
					Gain = 1.0f - GainIndexData.HighIndexGain;
				}
				else if (TableIndex == GainIndexData.HighIndex)
				{
					Gain = GainIndexData.HighIndexGain;
				}

				if (Gain > 0.0f)
				{
					const FWaveTableData& InputTableData = WaveTables[TableIndex];
					InputTableData.ArrayMixIn(OutputWaveTable.GetSamples(), Gain);
					FinalValue += InputTableData.GetFinalValue() * Gain;
				}
			}

			OutputWaveTable.SetFinalValue(FinalValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			BlockPeriod = InParams.OperatorSettings.GetActualBlockRate() / InParams.OperatorSettings.GetSampleRate();
			ResetInternal();
		}

	private:
		void ResetInternal()
		{
			GainIndexData = { };
			LastTableId = INDEX_NONE;
			*OutTable = { };
		}

		FWaveTableBankAssetReadRef WaveTableBankReadRef;
		FFloatReadRef TableIndexReadRef;

		struct FGainIndexData
		{
			int32 NumSamples = 0;

			int32 LowIndex = 0;
			int32 HighIndex = INDEX_NONE;

			float LastTableIndex = -1.0f;
			float TableIndex = 0.0f;

			float HighIndexGain = 1.0f;
		} GainIndexData;

		float BlockPeriod = 0.0f;

		uint32 LastTableId = INDEX_NONE;

		FWaveTableWriteRef OutTable;
	};

	using FMetasoundWaveTableGetNode = TNodeFacade<FMetasoundWaveTableGetNodeOperator>;
	METASOUND_REGISTER_NODE(FMetasoundWaveTableGetNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE
