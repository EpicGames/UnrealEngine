// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundWaveTable.h"
#include "WaveTableSampler.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"


namespace Metasound
{
	class FMetasoundWaveTableOscillatorNodeOperator : public TExecutableOperator<FMetasoundWaveTableOscillatorNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>("Play", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableOscillatorNode_InputPlayDesc", "Plays the oscillator (block rate)") }),
					TInputDataVertex<FTrigger>("Stop", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableOscillatorNode_InputStopDesc", "Stops the oscillator (block rate)") }),
					TInputDataVertex<FWaveTable>("WaveTable", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableOscillatorNode_InputWaveTableDesc", "WaveTable") }),
					TInputDataVertex<FTrigger>("Sync", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableOscillatorNode_InputSyncDesc", "Restarts playing the WaveTable on the trigger boundary (sample rate)"),
						LOCTEXT("MetasoundWaveTableOscillatorNode_InputSyncName", "Sync"),
						true /* bIsAdvancedDisplay */
					}),
					TInputDataVertex<float>("Freq", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableOscillatorNode_FreqDesc", "Frequency (number of times to sample one period of wavetable per second) [-20000Hz, 20000Hz]") }, 440.0f),
					TInputDataVertex<FAudioBuffer>("PhaseMod", FDataVertexMetadata
					{
						LOCTEXT("MetasoundWaveTableOscillatorNode_PhaseModDescription", "Modulation audio source for modulating oscillation phase of provided table. A value of 0 is no phase modulation and 1 a full table length (360 degrees) of phase shift."),
						LOCTEXT("MetasoundWaveTableOscillatorNode_PhaseMod", "Phase Modulator"),
						true /* bIsAdvancedDisplay */
					})
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>("Out", FDataVertexMetadata { LOCTEXT("MetasoundWaveTableOscillatorNode_Output", "Out") })
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					{ EngineNodes::Namespace, "WaveTableOscillator", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("MetasoundWaveTableOscillatorNode_Name", "WaveTable Oscillator"),
					LOCTEXT("MetasoundWaveTableOscillatorNode_Description", "Reads through the given WaveTable at the provided frequency."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ METASOUND_LOCTEXT("WaveTableOscillatorSynthesisKeyword", "Synthesis")},
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveTable;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableReadRef InWaveTableReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTable>("WaveTable", InParams.OperatorSettings);
			FTriggerReadRef InPlayReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Play", InParams.OperatorSettings);
			FTriggerReadRef InStopReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Stop", InParams.OperatorSettings);
			FTriggerReadRef InSyncReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Sync", InParams.OperatorSettings);
			FFloatReadRef InFreqReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("Freq", InParams.OperatorSettings);

			TOptional<FAudioBufferReadRef> InPhaseModReadRef;
			if (const FAnyDataReference* DataRef = InputData.FindDataReference("PhaseMod"))
			{
				InPhaseModReadRef = DataRef->GetDataReadReference<FAudioBuffer>();
			}

			return MakeUnique<FMetasoundWaveTableOscillatorNodeOperator>(InParams, InWaveTableReadRef, InPlayReadRef, InStopReadRef, InSyncReadRef, InFreqReadRef, MoveTemp(InPhaseModReadRef));
		}

		FMetasoundWaveTableOscillatorNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableReadRef& InWaveTableReadRef,
			const FTriggerReadRef& InPlayReadRef,
			const FTriggerReadRef& InStopReadRef,
			const FTriggerReadRef& InSyncReadRef,
			const FFloatReadRef& InFreqReadRef,
			TOptional<FAudioBufferReadRef>&& InPhaseModReadRef
		)
			: WaveTableReadRef(InWaveTableReadRef)
			, PlayReadRef(InPlayReadRef)
			, StopReadRef(InStopReadRef)
			, SyncReadRef(InSyncReadRef)
			, FreqReadRef(InFreqReadRef)
			, PhaseModReadRef(MoveTemp(InPhaseModReadRef))
			, OutBufferWriteRef(TDataWriteReferenceFactory<FAudioBuffer>::CreateAny(InParams.OperatorSettings))
		{
			const float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
			if (BlockRate > 0.0f)
			{
				BlockPeriod = 1.0f / BlockRate;
			}
		}

		virtual ~FMetasoundWaveTableOscillatorNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex("WaveTable", WaveTableReadRef);
			InOutVertexData.BindReadVertex("Play", PlayReadRef);
			InOutVertexData.BindReadVertex("Stop", StopReadRef);
			InOutVertexData.BindReadVertex("Sync", SyncReadRef);
			InOutVertexData.BindReadVertex("Freq", FreqReadRef);

			if (PhaseModReadRef.IsSet())
			{
				InOutVertexData.BindReadVertex("PhaseMod", *PhaseModReadRef);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex("Out", OutBufferWriteRef);
		}

		void Execute()
		{
			using namespace WaveTable;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundWaveTableOscillatorNodeOperator::Execute);

			FAudioBuffer& OutBuffer = *OutBufferWriteRef;
			OutBuffer.Zero();

			auto GetLastIndex = [](const FTriggerReadRef& Trigger)
			{
				int32 LastIndex = -1;
				Trigger->ExecuteBlock([](int32, int32) {}, [&LastIndex](int32 StartFrame, int32 EndFrame)
				{
					LastIndex = FMath::Max(LastIndex, StartFrame);
				});
				return LastIndex;
			};

			const int32 LastPlayIndex = GetLastIndex(PlayReadRef);
			const int32 LastStopIndex = GetLastIndex(StopReadRef);
			if (LastPlayIndex >= 0 || LastStopIndex >= 0)
			{
				bPlaying = LastPlayIndex > LastStopIndex;
			}

			if (bPlaying)
			{
				const FTrigger& SyncTrigger = *SyncReadRef;
				TArrayView<float> SyncBufferView;
				if (SyncTrigger.IsTriggered())
				{
					SyncBuffer.SetNum(OutBuffer.Num());
					FMemory::Memset(SyncBuffer.GetData(), 0, sizeof(float) * SyncBuffer.Num());
					SyncTrigger.ExecuteBlock(
						[](int32 StartFrame, int32 EndFrame) {},
						[this](int32 StartFrame, int32 EndFrame)
						{
							SyncBuffer[StartFrame] = 1.0f;
						}
					);
					SyncBufferView = SyncBuffer;
				}

				TArrayView<const float> PhaseMod;
				if (PhaseModReadRef.IsSet())
				{
					const FAudioBuffer& Buffer = *(*PhaseModReadRef);
					PhaseMod = { Buffer.GetData(), Buffer.Num() };
				}

				// Limit wrap operations running off toward infinity while allowing sampler to play in reverse
				Sampler.SetFreq(FMath::Clamp(*FreqReadRef, -20000.f, 20000.f) * BlockPeriod);

				const FWaveTable& InputTable = *WaveTableReadRef;
				Sampler.Process(InputTable.GetView(), { }, PhaseMod, SyncBufferView, OutBuffer);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			const float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
			if (BlockRate > 0.0f)
			{
				BlockPeriod = 1.0f / BlockRate;
			}

			bPlaying = false;
			if (SyncBuffer.Num() > 0)
			{
				FMemory::Memset(SyncBuffer.GetData(), 0, sizeof(float) * SyncBuffer.Num());
			}
			Sampler = WaveTable::FWaveTableSampler{};
			OutBufferWriteRef->Zero();
		}

	private:
		float BlockPeriod = 0.0f;
		bool bPlaying = false;

		FWaveTableReadRef WaveTableReadRef;
		FTriggerReadRef PlayReadRef;
		FTriggerReadRef StopReadRef;
		FTriggerReadRef SyncReadRef;
		FFloatReadRef FreqReadRef;

		Audio::FAlignedFloatBuffer SyncBuffer;
		TOptional<FAudioBufferReadRef> PhaseModReadRef;

		WaveTable::FWaveTableSampler Sampler;

		TDataWriteReference<FAudioBuffer> OutBufferWriteRef;
	};

	using FMetasoundWaveTableOscillatorNode = TNodeFacade<FMetasoundWaveTableOscillatorNodeOperator>;
	METASOUND_REGISTER_NODE(FMetasoundWaveTableOscillatorNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
