// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEngineNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundWaveTable.h"
#include "WaveTableSampler.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"


namespace Metasound
{
	class FMetaSoundWaveTablePlayerNodeOperator : public TExecutableOperator<FMetaSoundWaveTablePlayerNodeOperator>
	{
	public:
		// Maximum absolute pitch shift in semitones. 
		static constexpr float MaxAbsPitchShiftInSemitones = 72.0f;

		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace WaveTable;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>("Play", FDataVertexMetadata { LOCTEXT("MetaSoundWaveTablePlayerNode_InputPlayDesc", "Plays the wavetable") }),
					TInputDataVertex<FTrigger>("Stop", FDataVertexMetadata { LOCTEXT("MetaSoundWaveTablePlayerNode_InputStopDesc", "Stops the wavetable") }),
					TInputDataVertex<FTrigger>("Sync", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputSyncDesc", "Restarts playing the WaveTable on the trigger boundary (sample rate)"),
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputSyncName", "Sync")
					}),
					TInputDataVertex<FWaveTableBankAsset>("WaveTableBank", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputBankDesc", "WaveTable Bank to playback from"),
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputBankName", "Bank")
					}),
					TInputDataVertex<int32>("Index", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputIndexDesc", "Index of WaveTable Bank entry to play"),
						LOCTEXT("MetaSoundWaveTablePlayerNode_InputIndexName", "Index")
					}),
					TInputDataVertex<float>("PitchShift", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_PitchShiftDesc", "How much to shift the pitch of the given WaveTable"),
						LOCTEXT("MetaSoundWaveTablePlayerNode_PitchShiftName", "Pitch Shift")
					}, 0.0f),
					TInputDataVertex<bool>("Loop", FDataVertexMetadata { LOCTEXT("MetaSoundWaveTablePlayerNode_LoopDesc", "Whether or not to loop the given WaveTable") }, false),
					TInputDataVertex<FAudioBuffer>("PhaseMod", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_PhaseModDescription", "Modulation audio source for modulating oscillation phase of provided table. A value of 0 is no phase modulation and 1 a full table length (360 degrees) of phase shift."),
						LOCTEXT("MetaSoundWaveTablePlayerNode_PhaseMod", "Phase Modulator"),
						true /* bIsAdvancedDisplay */
					})
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>("MonoOut", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayer_OutputMonoOutDesc", "Mono output audio from player"),
						LOCTEXT("MetaSoundWaveTablePlayer_OutputMonoOutName", "Mono Out")
					}),
					TOutputDataVertex<FTrigger>("OnFinished", FDataVertexMetadata
					{
						LOCTEXT("MetaSoundWaveTablePlayerNode_OnFinished", "Trigger executed when player is complete (if not looping) or stop is triggered"),
						LOCTEXT("MetaSoundWaveTablePlayer_OutputOnFinishedName", "On Finished")
					})
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					{ EngineNodes::Namespace, "WaveTablePlayer", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("MetaSoundWaveTablePlayerNode_Name", "WaveTable Player"),
					LOCTEXT("MetaSoundWaveTablePlayerNode_Description", "Reads through the given WaveTableBank's entry at the given index."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::WaveTables },
					{ NodeCategories::Generators, METASOUND_LOCTEXT("WaveTablePlayerSynthesisKeyword", "Synthesis")},
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveTable;

			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FWaveTableBankAssetReadRef InWaveTableBankReadRef = InputData.GetOrCreateDefaultDataReadReference<FWaveTableBankAsset>("WaveTableBank", InParams.OperatorSettings);
			FInt32ReadRef InIndexReadRef = InputData.GetOrCreateDefaultDataReadReference<int32>("Index", InParams.OperatorSettings);
			FTriggerReadRef InPlayReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Play", InParams.OperatorSettings);
			FTriggerReadRef InStopReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Stop", InParams.OperatorSettings);
			FTriggerReadRef InSyncReadRef = InputData.GetOrCreateDefaultDataReadReference<FTrigger>("Sync", InParams.OperatorSettings);
			FFloatReadRef InPitchShiftReadRef = InputData.GetOrCreateDefaultDataReadReference<float>("PitchShift", InParams.OperatorSettings);
			FBoolReadRef InLoopReadRef = InputData.GetOrCreateDefaultDataReadReference<bool>("Loop", InParams.OperatorSettings);

			TOptional<FAudioBufferReadRef> InPhaseModReadRef;
			if (const FAnyDataReference* DataRef = InputData.FindDataReference("PhaseMod"))
			{
				InPhaseModReadRef = InputData.GetDataReadReference<FAudioBuffer>("PhaseMod");
			}

			return MakeUnique<FMetaSoundWaveTablePlayerNodeOperator>(InParams, InWaveTableBankReadRef, InIndexReadRef, InPlayReadRef, InStopReadRef, InSyncReadRef, InPitchShiftReadRef, InLoopReadRef, MoveTemp(InPhaseModReadRef));
		}

		FMetaSoundWaveTablePlayerNodeOperator(
			const FBuildOperatorParams& InParams,
			const FWaveTableBankAssetReadRef& InWaveTableBankReadRef,
			const FInt32ReadRef& InIndexReadRef,
			const FTriggerReadRef& InPlayReadRef,
			const FTriggerReadRef& InStopReadRef,
			const FTriggerReadRef& InSyncReadRef,
			const FFloatReadRef& InPitchShiftReadRef,
			const FBoolReadRef& InLoopReadRef,
			TOptional<FAudioBufferReadRef>&& InPhaseModReadRef
		)
			: SampleRate(InParams.OperatorSettings.GetSampleRate())
			, WaveTableBankReadRef(InWaveTableBankReadRef)
			, IndexReadRef(InIndexReadRef)
			, PlayReadRef(InPlayReadRef)
			, StopReadRef(InStopReadRef)
			, SyncReadRef(InSyncReadRef)
			, PitchShiftReadRef(InPitchShiftReadRef)
			, LoopReadRef(InLoopReadRef)
			, PhaseModReadRef(MoveTemp(InPhaseModReadRef))
			, OutBufferWriteRef(TDataWriteReferenceFactory<FAudioBuffer>::CreateAny(InParams.OperatorSettings))
			, OutOnFinishedRef(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InParams.OperatorSettings))
		{
			const float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
			if (BlockRate > 0.0f)
			{
				BlockPeriod = 1.0f / BlockRate;
			}
		}

		virtual ~FMetaSoundWaveTablePlayerNodeOperator() = default;
		
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex("WaveTableBank", WaveTableBankReadRef);
			InOutVertexData.BindReadVertex("Index", IndexReadRef);
			InOutVertexData.BindReadVertex("Play", PlayReadRef);
			InOutVertexData.BindReadVertex("Stop", StopReadRef);
			InOutVertexData.BindReadVertex("Sync", SyncReadRef);
			InOutVertexData.BindReadVertex("PitchShift", PitchShiftReadRef);
			InOutVertexData.BindReadVertex("Loop", LoopReadRef);

			if (PhaseModReadRef.IsSet())
			{
				InOutVertexData.BindReadVertex("PhaseMod", *PhaseModReadRef);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex("MonoOut", OutBufferWriteRef);
			InOutVertexData.BindReadVertex("OnFinished", OutOnFinishedRef);
		}

		void Execute()
		{
			using namespace WaveTable;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundWaveTablePlayerNodeOperator::Execute);

			FTrigger& OnFinishedTrigger = *OutOnFinishedRef;
			OnFinishedTrigger.AdvanceBlock();

			FAudioBuffer& OutBuffer = *OutBufferWriteRef;
			OutBuffer.Zero();
			auto GetLastTriggerFrame = [](const FTriggerReadRef& Trigger)
			{
				int32 LastIndex = INDEX_NONE;
				Trigger->ExecuteBlock([](int32, int32) {}, [&LastIndex](int32 StartFrame, int32 EndFrame)
				{
					LastIndex = FMath::Max(LastIndex, StartFrame);
				});
				return LastIndex;
			};

			bool bIsStopping = false;
			const int32 LastPlayFrame = GetLastTriggerFrame(PlayReadRef);
			int32 LastCompleteFrame = GetLastTriggerFrame(StopReadRef);
			if (LastPlayFrame >= 0 || LastCompleteFrame >= 0)
			{
				const bool bWasPlaying = bPlaying;
				bPlaying = LastPlayFrame > LastCompleteFrame;
				bIsStopping = bWasPlaying && !bPlaying;
			}

			if (bPlaying || bIsStopping)
			{
				const FTrigger& SyncTrigger = *SyncReadRef;
				TArrayView<float> SyncBufferView;
				if (SyncTrigger.IsTriggered())
				{
					SyncBuffer.SetNum(OutBuffer.Num());
					FMemory::Memset(SyncBuffer.GetData(), 0, sizeof(float) * SyncBuffer.Num());
					SyncTrigger.ExecuteBlock(
						[](int32 StartFrame, int32 EndFrame) {},
						[this](int32 StartFrame, int32 EndFrame)
						{
							SyncBuffer[StartFrame] = 1.0f;
						}
					);
					SyncBufferView = SyncBuffer;
				}

				TArrayView<const float> PhaseMod;
				if (PhaseModReadRef.IsSet())
				{
					const FAudioBuffer& Buffer = *(*PhaseModReadRef);
					PhaseMod = { Buffer.GetData(), Buffer.Num() };
				}

				const FWaveTableBankAsset& BankAsset = *WaveTableBankReadRef;
				FWaveTableBankAssetProxyPtr BankProxy = BankAsset.GetProxy();
				if (BankProxy.IsValid())
				{
					const float BankSampleRate = BankProxy->GetSampleMode() == EWaveTableSamplingMode::FixedResolution ? SampleRate : (float)BankProxy->GetSampleRate();
					float FreqRatio = 1.0f;
					if (ensureAlwaysMsgf(SampleRate > 0, TEXT("SampleRate must always be non-zero and positive")))
					{
						const float PitchShift = FMath::Clamp(*PitchShiftReadRef, -MaxAbsPitchShiftInSemitones, MaxAbsPitchShiftInSemitones);
						FreqRatio = Audio::GetFrequencyMultiplier(PitchShift) * (BankSampleRate / SampleRate);
					}

					const TArray<FWaveTableData>& WaveTableData = BankProxy->GetWaveTableData();
					if (!WaveTableData.IsEmpty())
					{
						const int32 TableIndex = FMath::Abs(*IndexReadRef) % WaveTableData.Num();

						const FWaveTableData& Entry = WaveTableData[TableIndex];
						const float Duration = Entry.GetNumSamples() / SampleRate;
						const float Freq = FreqRatio / Duration;
						Sampler.SetFreq(Freq * BlockPeriod);

						const bool bIsLooping = *LoopReadRef;
						Sampler.SetOneShot(!bIsLooping);

						Sampler.Process(WaveTableData[TableIndex], {}, PhaseMod, SyncBufferView, OutBuffer);

						if (LastCompleteFrame > 0)
						{
							checkf(LastCompleteFrame < OutBuffer.Num(), TEXT("LastCompleteFrame/output buffer size mismatch"));
							FMemory::Memzero(OutBuffer.GetData() + LastCompleteFrame, sizeof(float) * (OutBuffer.Num() - LastCompleteFrame));
						}
						else
						{
							const int32 IndexFinished = Sampler.GetIndexFinished();
							if (IndexFinished >= 0)
							{
								bIsStopping = true;
								bPlaying = false;
								LastCompleteFrame = IndexFinished;
							}
						}
					}
				}
			}

			if (bIsStopping)
			{
				OnFinishedTrigger.TriggerFrame(LastCompleteFrame);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			const float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
			SampleRate = InParams.OperatorSettings.GetSampleRate();
			if (BlockRate > 0.0f)
			{
				BlockPeriod = 1.0f / BlockRate;
			}

			bPlaying = false;
			if (SyncBuffer.Num() > 0)
			{
				FMemory::Memset(SyncBuffer.GetData(), 0, sizeof(float) * SyncBuffer.Num());
			}
			Sampler = WaveTable::FWaveTableSampler{};
			OutBufferWriteRef->Zero();
			OutOnFinishedRef->Reset();
		}

	private:
		float BlockPeriod = 0.0f;
		float SampleRate = 48000.0f;
		bool bPlaying = false;

		FWaveTableBankAssetReadRef WaveTableBankReadRef;
		FInt32ReadRef IndexReadRef;
		FTriggerReadRef PlayReadRef;
		FTriggerReadRef StopReadRef;
		FTriggerReadRef SyncReadRef;
		FFloatReadRef PitchShiftReadRef;
		FBoolReadRef LoopReadRef;

		Audio::FAlignedFloatBuffer SyncBuffer;
		TOptional<FAudioBufferReadRef> PhaseModReadRef;

		WaveTable::FWaveTableSampler Sampler;

		TDataWriteReference<FAudioBuffer> OutBufferWriteRef;
		TDataWriteReference<FTrigger> OutOnFinishedRef;
	};

	using FMetaSoundWaveTablePlayerNode = TNodeFacade<FMetaSoundWaveTablePlayerNodeOperator>;
	METASOUND_REGISTER_NODE(FMetaSoundWaveTablePlayerNode)
} // namespace Metasound

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
