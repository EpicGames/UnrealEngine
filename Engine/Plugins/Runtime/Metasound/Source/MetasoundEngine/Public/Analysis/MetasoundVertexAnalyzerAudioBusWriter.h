// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "AudioDefines.h"
#include "Containers/Array.h"

#define UE_API METASOUNDENGINE_API

namespace Metasound::Engine
{
	class FVertexAnalyzerAudioBusWriter : public Frontend::FVertexAnalyzerBase
	{
	public:
		using FAnalyzerOutput = Frontend::FAnalyzerOutput;
		using FCreateAnalyzerParams = Frontend::FCreateAnalyzerParams;

		static UE_API const FName& GetAnalyzerName();
		static UE_API const FName& GetDataType();
		static UE_API FName GetAnalyzerMemberName(const Audio::FDeviceId InDeviceID, const uint32 InAudioBusID);

		class FFactory : public Frontend::TVertexAnalyzerFactory<FVertexAnalyzerAudioBusWriter>
		{
		public:
			virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override
			{
				static const TArray<FAnalyzerOutput> Outputs;
				return Outputs;
			}
		};

		UE_API FVertexAnalyzerAudioBusWriter(const FCreateAnalyzerParams& InParams);
		virtual ~FVertexAnalyzerAudioBusWriter() = default;

		UE_API virtual void Execute() override;

	private:
		struct FBusAddress
		{
			Audio::FDeviceId DeviceID = 0;
			uint32 AudioBusID = 0;

			FString ToString() const;
			static FBusAddress FromString(const FString& InAnalyzerMemberName);
		};

		Audio::FPatchInput AudioBusPatchInput;
	};
} // namespace Metasound::Engine

#undef UE_API
