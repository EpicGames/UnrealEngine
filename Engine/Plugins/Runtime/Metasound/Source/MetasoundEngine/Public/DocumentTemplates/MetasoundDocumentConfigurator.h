// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "Containers/ContainersFwd.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendGraphLayer.h"
#include "MetasoundFrontendLiteral.h"
#include "UObject/ObjectPtr.h"


#define UE_API METASOUNDENGINE_API


namespace Metasound::Engine
{
	struct FDocumentConfigurator
	{
		struct FArgs
		{
			FMetaSoundFrontendDocumentBuilder* Builder = nullptr;

#if WITH_EDITORONLY_DATA
			// Offset to use when positioning new nodes via the configuration
			FVector2D Offset = FVector2D::Zero();
#endif // WITH_EDITORONLY_DATA

			bool bResetDoc = true;
		};

#if WITH_EDITORONLY_DATA
		// Constructor exposed to enable testing configurator that wraps a
		// transient builder. Not for use with runtime or editor behavior.
		UE_INTERNAL UE_API FDocumentConfigurator(UMetaSoundBuilderBase& InBuilder);
#endif // WITH_EDITORONLY_DATA

		UE_API FDocumentConfigurator(const FArgs& Args);
		UE_API ~FDocumentConfigurator();

#if WITH_EDITORONLY_DATA
		UE_EXPERIMENTAL(5.8, "Configurator code reflection is in development and may change implementation or be removed in the future")
		UE_API static void Reflect(FMetasoundFrontendGraphLayer InLayer, FString& OutCode);
#endif // WITH_EDITORONLY_DATA

	public:
		struct FInterface
		{
			FMetasoundFrontendVersion Version;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FInterface Interface);

		struct FInput
		{
			FName Name = "FloatInput";
			FName DataType = GetMetasoundDataTypeName<float>();
			FMetasoundFrontendLiteral DefaultValue = Frontend::MakeFrontendLiteral(0.f);
			FVector2D Location = FVector2D::Zero();
			bool bIsConstructorInput = false;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FInput Input);

#if WITH_EDITORONLY_DATA
		struct FInputNode
		{
			FName NodeName = "FloatInputNode";
			FName InputName = "FloatInput";
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FInputNode InputNode);
#endif // WITH_EDITORONLY_DATA

		struct FOutput
		{
			FName Name = "FloatOutput";
			FName DataType = GetMetasoundDataTypeName<float>();
			FMetasoundFrontendLiteral DefaultValue = Frontend::MakeFrontendLiteral(0.f);
			FVector2D Location = FVector2D::Zero();
			bool bIsConstructorOutput = false;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FOutput Output);

		struct FVariable
		{
			FName Name = "FloatVariable";
			FName DataType = GetMetasoundDataTypeName<float>();
			FMetasoundFrontendLiteral DefaultValue = Frontend::MakeFrontendLiteral(0.f);
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FVariable Variable);

		struct FVariableSetNode
		{
			FName Name;
			FName VariableName;
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FVariableSetNode Node);

		struct FVariableGetNode
		{
			FName Name;
			FName VariableName;
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FVariableGetNode Node);

		struct FVariableGetDelayedNode
		{
			FName Name;
			FName VariableName;
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FVariableGetDelayedNode Node);

		struct FNode
		{
			FName Name;
			FMetasoundFrontendClassName ClassName;
			FTopLevelAssetPath AssetPath;
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FNode Node);

		struct FEdge
		{
			FName OutputNode;
			FName Output;
			FName InputNode;
			FName Input;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FEdge Edge);

		struct FEdgeFromInput
		{
			FName GraphInput;
			FName Node;
			FName NodeInput;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FEdgeFromInput Edge);

		struct FEdgeToOutput
		{
			FName Node;
			FName NodeOutput;
			FName GraphOutput;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Add(FEdgeToOutput Edge);

		template <typename TemplateType>
		FDocumentConfigurator& Add(TArray<TemplateType> Args)
		{
			for (TemplateType& TemplateArgs : Args)
			{
				Add(MoveTemp(TemplateArgs));
			}

			return *this;
		}

		template <>
		FDocumentConfigurator& Add<FInterface>(TArray<FInterface> Interfaces)
		{
			return AddInternal(MoveTemp(Interfaces));
		}

		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API UMetaSoundBuilderBase& GetBuilder();

		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& ResetDocument();

		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& SetBuildPage(FName PageName);

		struct FInputLocation
		{
			FName Name;
			FVector2D Location = FVector2D::Zero();
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Set(FInputLocation Location);

		struct FNodeInputDefault
		{
			FName Node;
			FName Input;
			FMetasoundFrontendLiteral Value;
		};
		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API FDocumentConfigurator& Set(FNodeInputDefault Input);

		template <typename TemplateType>
		FDocumentConfigurator& Set(TArray<TemplateType> Args)
		{
			for (TemplateType& TemplateArgs : Args)
			{
				Set(MoveTemp(TemplateArgs));
			}

			return *this;
		}

		UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
		UE_API bool Succeeded() const;

	private:
		FDocumentConfigurator& AddInternal(TArray<FInterface> Interfaces);

		TStrongObjectPtr<UMetaSoundBuilderBase> Builder;
		mutable EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Succeeded;

		TMap<FName, FMetaSoundNodeHandle> NodeNameToHandle;

		FVector2D Offset;
	};
} // namespace Metasound::Engine
#undef UE_API
