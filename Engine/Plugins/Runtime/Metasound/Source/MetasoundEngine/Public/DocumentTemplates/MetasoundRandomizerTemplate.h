// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "MetasoundDocumentInterface.h"
#include "Sound/SoundWave.h"
#include "UObject/ObjectPtr.h"

#include "MetasoundRandomizerTemplate.generated.h"

#define UE_API METASOUNDENGINE_API

// Forward Declarations
class UMetaSoundFrontendMemberMetadata;
enum class EMetasoundFrontendClassType : uint8;


USTRUCT(BlueprintType, meta = (Hidden, DisplayName = "Randomizer"))
struct FMetaSoundRandomizerTemplate : public FMetaSoundFrontendDocumentTemplate
{
	GENERATED_BODY()

	virtual ~FMetaSoundRandomizerTemplate() = default;

#if WITH_EDITORONLY_DATA
	UE_INTERNAL UE_API virtual bool ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder) override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual const FMetaSoundFrontendDocumentTemplate::FEditorOptions& GetEditorOptions() const override;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

	// Helper function to remove sounds that cannot be referenced by randomizer (i.e. if the sounds are not exactly of type USoundWave
	// or, optionally, if the underlying pointer is null). Returns true if sounds were removed, false if not.
	UE_API static bool RemoveInvalidSounds(const FMetaSoundFrontendDocumentBuilder& ParentBuilder, bool bRemoveNullEntries, TArray<TObjectPtr<USoundWave>>& OutSounds);
#endif // WITH_EDITOR

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (Category = "Randomizer Options"))
	bool bIsOneShot = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (Category = "Randomizer Options", DisallowedClasses = "/Script/MetasoundEngine.MetaSoundSource, /Script/Engine.SoundSourceBus"))
	TArray<TObjectPtr<USoundWave>> Sounds;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (Category = "Randomizer Options", ClampMin = "-36.0", ClampMax = "36.0", UIMin = "-12.0", UIMax = "12.0"))
	FVector2f Pitch = { -3.f, 3.f };
};
#undef UE_API