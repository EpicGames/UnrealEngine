// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "UObject/PackageReload.h"

namespace Metasound::Engine
{
#if WITH_EDITOR
	// AssetScanStatus is no longer necessary and will be deprecated once usage is removed from MetaSound plugin API
	enum class EAssetScanStatus : uint8
	{
		NotRequested = 0,
		InProgress = 2,
		Complete = 3
	};

	// NodeClassRegistryPrimeStatus is no longer necessary and will be deprecated once usage is removed from MetaSound plugin API
	enum class ENodeClassRegistryPrimeStatus : uint8
	{
		NotRequested = 0,
		Requested = 1,
		InProgress = 2,
		Complete = 3,
		Canceled = 4
	};

	enum class ERegistrationAssetContext
	{
		None, // No special asset context associated with this graph registration action
		Removing, // Graph registration during asset removal
		Renaming, // Graph registration during asset rename
		Reloading, // Graph registration during asset reload
	};

	DECLARE_DELEGATE_TwoParams(FOnMetasoundGraphRegister, UObject&, ERegistrationAssetContext)
	DECLARE_DELEGATE_TwoParams(FOnMetasoundGraphUnregister, UObject&, ERegistrationAssetContext)
#endif // WITH_EDITOR

	class IMetasoundEngineModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override = 0;
		virtual void ShutdownModule() override = 0;

#if WITH_EDITOR
		// Request enumeration of scanned assets and kick off add of MetaSound asset tag data to MetaSoundAssetManager
		virtual void PrimeAssetManager() = 0;

		// Asset registry delegates for calling MetaSound editor module register/unregister with frontend 
		virtual FOnMetasoundGraphRegister& GetOnGraphRegisteredDelegate() = 0;
		virtual FOnMetasoundGraphUnregister& GetOnGraphUnregisteredDelegate() = 0;
#endif // WITH_EDITOR
	};
} // namespace Metasound::Engine
