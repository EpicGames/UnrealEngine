// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioParameterInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "UObject/NameTypes.h"


namespace Metasound::Engine
{
	namespace InputFormatMonoInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName MonoIn;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatMonoInterface

	namespace InputFormatStereoInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName LeftIn;
			METASOUNDENGINE_API const extern FName RightIn;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatStereoInterface

	namespace InputFormatQuadInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftIn;
			METASOUNDENGINE_API const extern FName FrontRightIn;
			METASOUNDENGINE_API const extern FName SideLeftIn;
			METASOUNDENGINE_API const extern FName SideRightIn;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatQuadInterface

	namespace InputFormatFiveDotOneInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftIn;
			METASOUNDENGINE_API const extern FName FrontRightIn;
			METASOUNDENGINE_API const extern FName FrontCenterIn;
			METASOUNDENGINE_API const extern FName LowFrequencyIn;
			METASOUNDENGINE_API const extern FName SideLeftIn;
			METASOUNDENGINE_API const extern FName SideRightIn;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatFiveDotOneInterface

	namespace InputFormatSevenDotOneInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftIn;
			METASOUNDENGINE_API const extern FName FrontRightIn;
			METASOUNDENGINE_API const extern FName FrontCenterIn;
			METASOUNDENGINE_API const extern FName LowFrequencyIn;
			METASOUNDENGINE_API const extern FName SideLeftIn;
			METASOUNDENGINE_API const extern FName SideRightIn;
			METASOUNDENGINE_API const extern FName BackLeftOut;
			METASOUNDENGINE_API const extern FName BackRightOut;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatSevenDotOneInterface
	
	namespace InputFormatChannelAgnosticInterface
	{
		namespace Inputs
		{
			METASOUNDENGINE_API const extern FName CatIn;
		} // namespace Inputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
		METASOUNDENGINE_API TArray<FMetasoundFrontendInterfaceBinding> CreateBindings();
	} // namespace InputFormatChannelAgnosticInterface 
	
} // namespace Metasound::Engine
