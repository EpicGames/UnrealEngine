// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"


namespace Metasound::Engine
{
	METASOUNDENGINE_API void RegisterInterfaceBindings(const TArray<FMetasoundFrontendInterfaceBinding>& InBindings);
	METASOUNDENGINE_API bool UnregisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, const FMetasoundFrontendVersion& InOutputInterfaceVersion);
	METASOUNDENGINE_API bool UnregisterAllInterfaceBindings(const FMetasoundFrontendVersion& InInputInterfaceVersion);
} // namespace Metasound::Engine
