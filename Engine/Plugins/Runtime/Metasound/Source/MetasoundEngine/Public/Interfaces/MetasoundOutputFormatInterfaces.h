// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "UObject/NameTypes.h"

#include "MetasoundOutputFormatInterfaces.generated.h"

// Forward Declarations
class UClass;

/** Declares supported MetaSound output audio formats */
UENUM()
enum class EMetaSoundOutputAudioFormat : uint8
{
	Mono,
	Stereo,
	Quad,
	FiveDotOne UMETA(DisplayName = "5.1"),
	SevenDotOne UMETA(DisplayName = "7.1"),
	ChannelAgnostic  UMETA(DisplayName = "Channel Agnostic (Experimental)", Hidden),
	COUNT UMETA(Hidden)
};

namespace Metasound::Engine
{
	namespace OutputFormatMonoInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName MonoOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();

		UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UClass as interfaces can support multiple UClasses")
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatMonoInterface

	namespace OutputFormatStereoInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName LeftOut;
			METASOUNDENGINE_API const extern FName RightOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();

		UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UClass as interfaces can support multiple UClasses")
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatStereoInterface

	namespace OutputFormatQuadInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftOut;
			METASOUNDENGINE_API const extern FName FrontRightOut;
			METASOUNDENGINE_API const extern FName SideLeftOut;
			METASOUNDENGINE_API const extern FName SideRightOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();

		UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UClass as interfaces can support multiple UClasses")
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatQuadInterface

	namespace OutputFormatFiveDotOneInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftOut;
			METASOUNDENGINE_API const extern FName FrontRightOut;
			METASOUNDENGINE_API const extern FName FrontCenterOut;
			METASOUNDENGINE_API const extern FName LowFrequencyOut;
			METASOUNDENGINE_API const extern FName SideLeftOut;
			METASOUNDENGINE_API const extern FName SideRightOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();

		UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UClass as interfaces can support multiple UClasses")
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatFiveDotOneInterface

	namespace OutputFormatSevenDotOneInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName FrontLeftOut;
			METASOUNDENGINE_API const extern FName FrontRightOut;
			METASOUNDENGINE_API const extern FName FrontCenterOut;
			METASOUNDENGINE_API const extern FName LowFrequencyOut;
			METASOUNDENGINE_API const extern FName SideLeftOut;
			METASOUNDENGINE_API const extern FName SideRightOut;
			METASOUNDENGINE_API const extern FName BackLeftOut;
			METASOUNDENGINE_API const extern FName BackRightOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();

		UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UClass as interfaces can support multiple UClasses")
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatSevenDotOneInterface

	namespace OutputFormatChannelAgnosticInterface
	{
		namespace Outputs
		{
			METASOUNDENGINE_API const extern FName CatOut;
		} // namespace Outputs

		METASOUNDENGINE_API const FMetasoundFrontendVersion& GetVersion();
		METASOUNDENGINE_API Audio::FParameterInterfacePtr CreateInterface();
	} // namespace OutputFormatChannelAgnosticInterface
	// Contains information on output audio formats
	struct FOutputAudioFormatInfo
	{
		FMetasoundFrontendVersion InterfaceVersion;
		TArray<Metasound::FVertexName> OutputVertexChannelOrder;
	};

	using FOutputAudioFormatInfoMap = TMap<EMetaSoundOutputAudioFormat, FOutputAudioFormatInfo>;
	using FOutputAudioFormatInfoPair = TPair<EMetaSoundOutputAudioFormat, FOutputAudioFormatInfo>;

	// Return a map containing all the supported output audio formats for a MetaSound.
	METASOUNDENGINE_API const FOutputAudioFormatInfoMap& GetOutputAudioFormatInfo();
} // namespace Metasound::Engine
