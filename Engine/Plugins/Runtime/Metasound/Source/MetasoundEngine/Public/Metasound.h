// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraph/EdGraph.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundRouter.h"
#include "MetasoundUObjectRegistry.h"
#include "Serialization/Archive.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/SoftObjectPath.h"

#include "Metasound.generated.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
#if WITH_EDITOR
class FDataValidationContext;
#endif // WITH_EDITOR

namespace Metasound::Engine
{
	struct FAssetHelper;
} // namespace Metasound::Engine


UCLASS(MinimalAPI, Abstract)
class UMetasoundEditorGraphBase : public UEdGraph
{
	GENERATED_BODY()

public:
	virtual bool IsEditorOnly() const override { return true; }
	virtual bool NeedsLoadForEditorGame() const override { return false; }

	virtual void RegisterGraphWithFrontend(Metasound::Frontend::FMetaSoundAssetRegistrationOptions* RegOptions = nullptr) PURE_VIRTUAL(UMetasoundEditorGraphBase::RegisterGraphWithFrontend(), )

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "ModifyContext is to be replaced by builder API delegates providing context when items changed and it will be up to the caller to track modification deltas.")
	virtual FMetasoundFrontendDocumentModifyContext& GetModifyContext() { static FMetasoundFrontendDocumentModifyContext InvalidModifyData; return InvalidModifyData; }

	UE_DEPRECATED(5.5, "ModifyContext is to be replaced by builder API delegates providing context when items changed and it will be up to the caller to track modification deltas.")
	virtual const FMetasoundFrontendDocumentModifyContext& GetModifyContext() const { static const FMetasoundFrontendDocumentModifyContext InvalidModifyData; return InvalidModifyData; }

	UE_DEPRECATED(5.5, "Editor Graph is now transient, so versioning flag moved to AssetBase.")
	virtual void ClearVersionedOnLoad() { }

	UE_DEPRECATED(5.5, "Editor Graph is now transient, so versioning flag moved to AssetBase.")
	virtual bool GetVersionedOnLoad() const { return false; }

	UE_DEPRECATED(5.5, "Editor Graph is now transient, so versioning flag moved to AssetBase.")
	virtual void SetVersionedOnLoad() {  }

	virtual void MigrateEditorDocumentData(FMetaSoundFrontendDocumentBuilder & OutBuilder) PURE_VIRTUAL(UMetasoundEditorGraphBase::MigrateEditorDocumentData(), )
#endif // WITH_EDITORONLY_DATA

	UE_API int32 GetHighestMessageSeverity() const;
};


/**
 * This asset type is used for Metasound assets that can only be used as nodes in other Metasound graphs.
 * Because of this, they contain no required inputs or outputs.
 */
UCLASS(MinimalAPI, hidecategories = object, BlueprintType, meta = (DisplayName = "MetaSound Patch"))
class UMetaSoundPatch : public UObject, public FMetasoundAssetBase, public IMetaSoundDocumentInterface
{
	GENERATED_BODY()

	friend struct Metasound::Engine::FAssetHelper;
	friend class UMetaSoundPatchBuilder;

protected:
	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendDocument RootMetaSoundDocument;

	UPROPERTY()
	TSet<FString> ReferencedAssetClassKeys;

	UPROPERTY()
	TSet<TObjectPtr<UObject>> ReferencedAssetClassObjects;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Soft reference cache is no longer supported"))
	TSet<FSoftObjectPath> ReferenceAssetClassCache;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use EditorGraph instead as it is now transient and generated via the FrontendDocument dynamically."))
	TObjectPtr<UMetasoundEditorGraphBase> Graph;

	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UMetasoundEditorGraphBase> EditorGraph;
#endif // WITH_EDITORONLY_DATA

public:
	UE_API UMetaSoundPatch(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FGuid AssetClassID;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FString RegistryInputTypes;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FString RegistryOutputTypes;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	int32 RegistryVersionMajor = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	int32 RegistryVersionMinor = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	bool bIsPreset = false;

	// Returns document name (for editor purposes, and avoids making document public for edit
	// while allowing editor to reference directly)
	static FName GetDocumentPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UMetaSoundPatch, RootMetaSoundDocument);
	}

	// Name to display in editors
	UE_API virtual FText GetDisplayName() const override;

	// Returns the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @return Editor graph associated with UMetaSoundSource.
	UE_API virtual UEdGraph* GetGraph() const override;
	UE_API virtual UEdGraph& GetGraphChecked() const override;
	UE_API virtual void MigrateEditorGraph(FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

	// Sets the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @param Editor graph associated with UMetaSoundSource.
	virtual void SetGraph(UEdGraph* InGraph) override
	{
		EditorGraph = CastChecked<UMetasoundEditorGraphBase>(InGraph);
	}
#endif // #if WITH_EDITORONLY_DATA

	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	UE_API virtual FTopLevelAssetPath GetAssetPathChecked() const override;
	UE_API virtual const UClass& GetBaseMetaSoundUClass() const final override;
	UE_API virtual const UClass& GetBuilderUClass() const final override;
	UE_API virtual const FMetasoundFrontendDocument& GetConstDocument() const override;
	UE_API virtual EMetasoundFrontendClassAccessFlags GetDefaultAccessFlags() const final override;

#if WITH_EDITOR
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;

#endif // WITH_EDITOR

	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
	UE_API virtual void Serialize(FArchive& InArchive) override;
	UE_API virtual void PostLoad() override;

	virtual bool ConformObjectToDocument() override { return false; }

	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	virtual const TSet<FString>& GetReferencedAssetClassKeys() const override
	{
		return ReferencedAssetClassKeys;
	}
	UE_API virtual TArray<FMetasoundAssetBase*> GetReferencedAssets() const override;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "This is deprecated behavior and will become UE_INTERNAL API in a future release for back compat.")
	UE_API virtual void OnAsyncReferencedAssetsLoaded(const TArray<FMetasoundAssetBase*>& InAsyncReferences) override;
#endif // WITH_EDITORONLY_DATA

	UObject* GetOwningAsset() override
	{
		return this;
	}

	const UObject* GetOwningAsset() const override
	{
		return this;
	}

	UE_API virtual bool IsActivelyBuilding() const override;

protected:
#if WITH_EDITOR
	UE_API virtual void SetReferencedAssets(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs) override;
#endif // #if WITH_EDITOR

	UE_DEPRECATED(5.6, "AccessPtrs are actively being deprecated. Writable access outside of the builder API "
		"is particularly problematic as in so accessing, the builder's caches are reset which can cause major "
		"editor performance regressions.")
	UE_API Metasound::Frontend::FDocumentAccessPtr GetDocumentAccessPtr(bool bReloadBuilder = true) override;
	UE_API Metasound::Frontend::FConstDocumentAccessPtr GetDocumentConstAccessPtr() const override;

private:
	virtual FMetasoundFrontendDocument& GetDocument() override
	{
		return RootMetaSoundDocument;
	}

	UE_API virtual void OnBeginActiveBuilder() override;
	UE_API virtual void OnFinishActiveBuilder() override;

	bool bIsBuilderActive = false;
};

#undef UE_API
