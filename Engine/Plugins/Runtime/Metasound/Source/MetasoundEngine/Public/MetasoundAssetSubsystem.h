// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/Engine.h"
#include "Engine/StreamableManager.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetKey.h"
#include "MetasoundAssetManager.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendQuery.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Object.h"

#include "MetasoundAssetSubsystem.generated.h"

#define UE_API METASOUNDENGINE_API

// Forward Declarations
class FMetasoundAssetBase;
class UAssetManager;

struct FDirectoryPath;
struct FMetaSoundFrontendDocumentBuilder;


namespace Metasound::Engine
{
	void DeinitializeAssetManager();
	void InitializeAssetManager();


	class FMetaSoundAssetManager :
		public Frontend::FMetaSoundAssetManagerBase,
		public FGCObject
	{
	public:
		friend class UMetaSoundAssetSubsystem;

		using FAssetClassInfoMap = TMap<FMetaSoundAssetKey, TArray<Frontend::FMetaSoundAssetClassInfo>>;
		using FOnUpdatedAssetLoaded = Frontend::IMetaSoundAssetManager::FOnUpdatedAssetLoaded;

PRAGMA_DISABLE_DEPRECATION_WARNINGS 
		using FAssetInfo UE_DEPRECATED(5.6, "Use FAssetRef instead which is more compact and usage clear in context of other Info tag/query types") = Frontend::IMetaSoundAssetManager::FAssetInfo;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		using FNodeRegistryKey = Frontend::FNodeRegistryKey;

#if WITH_EDITORONLY_DATA
		using FVersionAssetResults = Frontend::IMetaSoundAssetManager::FVersionAssetResults;
#endif // WITH_EDITORONLY_DATA

		FMetaSoundAssetManager() = default;
		UE_API virtual ~FMetaSoundAssetManager();

		static FMetaSoundAssetManager* Get()
		{
			using namespace Frontend;
			return static_cast<FMetaSoundAssetManager*>(IMetaSoundAssetManager::Get());
		}

		static FMetaSoundAssetManager& GetChecked()
		{
			using namespace Frontend;
			return static_cast<FMetaSoundAssetManager&>(IMetaSoundAssetManager::GetChecked());
		}

		/**
		 * Attempts to find the most recent class query info for the asset at the given path without loading asset.
		 * 1. If asset is loaded, builds tag data from asset (to ensure version provided is "freshest" if it has been edited but not re-serialized.
		 * 2. If asset is not loaded:
		 *	a. Returns entry in Manager if sole entry found.
		 *	b. If not registered or multiple entries found (may be more than one due to errors in duplication of assets, improper plugin migration), builds directly from the asset data last serialized with the given path/asset.
		 *	c. If the asset data is out-of-date or not found, returns invalid query info.
		 **/
		UE_API Frontend::FMetaSoundAssetClassInfo FindAssetClassInfo(const FTopLevelAssetPath& InPath) const;

		// Returns all asset query info with the given asset key (may be more than one due to errors in duplication
		// of assets, improper plugin migration, etc. but should usually just be a single entry) registered
		// with the manager. Info may not be returned if the asset has yet to be registered with the MetaSoundAssetManager
		// or may be out-of-date if edited after last registration attempt via AddOrUpdate... calls.
		UE_API virtual TArray<Frontend::FMetaSoundClassInfo> FindRegisteredClassInfo(const FMetaSoundAssetKey& InKey) const override;

		UE_API FMetaSoundAssetKey GetAssetKey(const FSoftObjectPath& InObjectPath) const;

#if WITH_EDITOR
		UE_API int32 GetActiveAsyncLoadRequestCount() const;
#endif // WITH_EDITOR

		UE_API bool IsInitialAssetScanComplete() const;

		UE_DEPRECATED(5.7, "DenyList Cache no longer supported (auto-update is now required when cooking and not supported at runtime")
		UE_API void RebuildDenyListCache(const UAssetManager& InAssetManager);

		UE_API void RegisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& Directories);

#if WITH_EDITOR
		UE_API bool ReplaceReferencesInAsset(TScriptInterface<IMetaSoundDocumentInterface> InMetaSound, const Frontend::FNodeRegistryKey& OldClassKey, const Frontend::FNodeRegistryKey& NewClassKey) const;
		UE_API bool ReplaceReferencesInDirectory(const TArray<FMetaSoundAssetDirectory>& InDirectories, const FNodeRegistryKey& OldClassKey, const FNodeRegistryKey& NewClassKey) const;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.8,
			"This is a deprecated feature, and exists to port old soft object references to hard references. "
			"This will become UE_INTERNAL API in a future release.")
		UE_API virtual void RequestAsyncLoadReferencedAssets(FMetasoundAssetBase& InAssetBase, const TSet<FSoftObjectPath>* Paths = nullptr) override;
#endif // WITH_EDITORONLY_DATA

		UE_API void SearchAndIterateDirectoryAssets(const TArray<FDirectoryPath>& InDirectories, TFunctionRef<void(const FAssetData&)> InFunction) const;

#if WITH_EDITOR
		UE_API void SetCanNotifyAssetTagScanComplete();
#endif // WITH_EDITOR

		UE_API FMetasoundAssetBase* TryLoadAsset(const FSoftObjectPath& InObjectPath) const;
		UE_API void UnregisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& Directories);

		/* IMetaSoundAssetManager Implementation */
#if WITH_EDITORONLY_DATA
		UE_API virtual bool AddAssetReferences(FMetasoundAssetBase& InAssetBase) override;
#endif // WITH_EDITORONLY_DATA
		UE_API virtual FMetaSoundAssetKey AddOrUpdateFromObject(const UObject& InObject) override;
		UE_API virtual void AddOrLoadAndUpdateFromObjectAsync(const FAssetData& InAssetData, Frontend::IMetaSoundAssetManager::FOnUpdatedAssetLoaded&& OnUpdatedAssetLoaded) override;
		UE_API virtual void AddOrUpdateFromAssetData(const FAssetData& InAssetData) override;

		UE_API virtual bool ContainsKey(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual bool ContainsKey(const Frontend::FNodeRegistryKey& InRegistryKey) const override;
		UE_API virtual FMetasoundAssetBase* FindAsset(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual TScriptInterface<IMetaSoundDocumentInterface> FindAssetAsDocumentInterface(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual FTopLevelAssetPath FindAssetPath(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual TArray<FTopLevelAssetPath> FindAssetPaths(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual FMetasoundAssetBase* GetAsAsset(UObject& InObject) const override;
		UE_API virtual const FMetasoundAssetBase* GetAsAsset(const UObject& InObject) const override;
#if WITH_EDITOR
		UE_API virtual TSet<Frontend::IMetaSoundAssetManager::FAssetRef> GetReferencedAssets(const FMetasoundAssetBase& InAssetBase) const override;
		UE_API virtual bool GetReferencedPresetHierarchy(FMetasoundAssetBase& InAsset, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const override;
		UE_API virtual bool ReassignClassName(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) override;
#endif // WITH_EDITOR

		UE_API virtual bool IsAssetClass(const FMetasoundFrontendClassMetadata& ClassMetadata) const override;

#if WITH_EDITOR
		// Iterates all asset tag data (Only recommended in certain editor contexts, as slow and blocks access to reference map).
		UE_API void IterateAssetTagData(TFunctionRef<void(Frontend::FMetaSoundAssetClassInfo)> Iter, bool bIterateDuplicates = false) const;

		// Iterates all references of a given asset key entry (Only recommended in certain editor contexts, as can be slow for
		// deep reference trees and blocks access to reference map).
		UE_API void IterateReferences(const FMetaSoundAssetKey& InKey, TFunctionRef<void(const FMetaSoundAssetKey&)> VisitFunction) const;
#endif // WITH_EDITOR

		UE_API virtual void RemoveAsset(const UObject& InObject) override;
		UE_API virtual void RemoveAsset(const FAssetData& InAssetData) override;
		UE_API virtual void RenameAsset(const FAssetData& InAssetData, const FString& InOldObjectPath) override;
		UE_API virtual void ReregisterLoadedAssets() const override;
		UE_API virtual void SetLogActiveAssetsOnShutdown(bool bInLogActiveAssetsOnShutdown) override;
		UE_API virtual FMetasoundAssetBase* TryLoadAssetFromKey(const FMetaSoundAssetKey& InKey) const override;
		UE_API virtual bool TryGetAssetIDFromClassName(const FMetasoundFrontendClassName& InClassName, FGuid& OutGuid) const override;
		UE_API virtual bool TryLoadReferencedAssets(const FMetasoundAssetBase& InAssetBase, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const override;
#if WITH_EDITORONLY_DATA
		UE_API FMetaSoundAssetManager::FVersionAssetResults SetAccessFlagsOnAssetsInFolders(const TArray<FString>& FolderPaths, EMetasoundFrontendClassAccessFlags Flags, bool bRecursePaths) const override;
		UE_API virtual FVersionAssetResults VersionAssetsInFolders(const TArray<FString>& FolderPaths, bool bRecursePaths = true) const override;
		UE_API virtual void WaitUntilAsyncLoadReferencedAssetsComplete(const FMetasoundAssetBase& InAssetBase) override;
#endif // WITH_EDITORONLY_DATA

		/* FGCObject */
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FMetaSoundAssetManager"); }

		UE_INTERNAL virtual Frontend::FMetaSoundAssetRegistrationOptions GetRegistrationOptions() const override;

	private:
		FMetaSoundAssetKey AddOrUpdateFromObjectInternal(const UObject& InObject);

#if WITH_EDITORONLY_DATA
		TArray<FMetaSoundAsyncAssetDependencies> LoadingDependencies;
#endif // WITH_EDITORONLY_DATA

		static UE_API void DepthFirstVisit_AssetKey(const FMetaSoundAssetKey& InKey, TFunctionRef<TSet<FMetaSoundAssetKey>(const FMetaSoundAssetKey&)> VisitFunction);

		// Returns all asset query info with the given asset key (may be more than one due to errors in duplication
		// of assets, improper plugin migration, etc. but should usually just be a single entry) registered
		// with the manager. Info may not be returned if the asset has yet to be registered with the MetaSoundAssetManager
		// or may be out-of-date if edited after last registration attempt via AddOrUpdate... calls.
		TArray<Frontend::FMetaSoundAssetClassInfo> FindAssetClassInfoInternal(const FMetaSoundAssetKey& InKey) const;

#if WITH_EDITORONLY_DATA
		FMetaSoundAsyncAssetDependencies* FindLoadingDependencies(const UObject* InParentAsset);
		FMetaSoundAsyncAssetDependencies* FindLoadingDependencies(int32 InLoadID);
		void RemoveLoadingDependencies(int32 InLoadID);
		void OnReferencedAssetsLoaded(int32 InLoadID);

#endif // WITH_EDITORONLY_DATA

		bool GetReferencedPresetHierarchyInternal(FMetasoundAssetBase& InAsset, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const;

#if WITH_EDITOR
		bool ReplaceReferencesInAssetInternal(UObject* MetaSoundObject, const TMap<FNodeRegistryKey, FNodeRegistryKey>& InOldToNewReferenceKeys) const;
#endif // WITH_EDITOR

		struct FPackageLoadedArgs
		{
			FName PackageName;
			UPackage* Package = nullptr;
			EAsyncLoadingResult::Type Result = EAsyncLoadingResult::Failed;
			FOnUpdatedAssetLoaded OnUpdatedAssetLoaded;
		};

		static void OnPackageLoaded(const FPackageLoadedArgs& PackageLoadedArgs);

		FStreamableManager StreamableManager;
		int32 AsyncLoadIDCounter = 0;

		FAssetClassInfoMap ClassInfoMap;

		// Critical section primarily for allowing safe access of class info map during async loading of MetaSound assets.
		mutable FCriticalSection TagDataMapCriticalSection;

		bool bLogActiveAssetsOnShutdown = true;

#if WITH_EDITORONLY_DATA
		bool bNotifyTagDataScanComplete = false;
		int32 ActiveAsyncAssetLoadRequests = 0;
#endif // WITH_EDITORONLY_DATA
	};
} // namespace Metasound::Engine


USTRUCT(BlueprintType)
struct FMetaSoundAssetDirectory
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Directories, meta = (RelativePath, LongPackageName))
	FDirectoryPath Directory;
};

/** Contains info of assets which are currently async loading. */
USTRUCT()
struct FMetaSoundAsyncAssetDependencies
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// ID of the async load
	int32 LoadID = 0;

	// Parent MetaSound 
	UPROPERTY(Transient)
	TObjectPtr<UObject> MetaSound;

	// Dependencies of parent MetaSound
	TArray<FSoftObjectPath> Dependencies;

	// Handle to in-flight streaming request
	TSharedPtr<FStreamableHandle> StreamableHandle;
#endif // WITH_EDITORONLY_DATA
};


UCLASS(MinimalAPI, meta = (DisplayName = "MetaSound Asset Subsystem"))
class UMetaSoundAssetSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		using FAssetInfo UE_DEPRECATED(5.6, "No longer supported in favor of more compact and direct AssetRef struct") = Metasound::Frontend::IMetaSoundAssetManager::FAssetInfo;
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;

#if WITH_EDITOR
	// Returns asset class info for the given MetaSound asset. Will attempt to get
	// info without loading the asset if its tag data is up-to-date, or if set to force load, will load
	// otherwise (synchronously and can be slow).  Returns true if asset is found, was MetaSound, and all data
	// was retrieved successfully, false if not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Utilities", meta = (AdvancedDisplay = "3"))
	UE_API UPARAM(DisplayName = "Info Found") bool FindAssetClassInfo(
		UPARAM(DisplayName = "Path") const FTopLevelAssetPath& InPath,
		UPARAM(DisplayName = "Document Info") FMetaSoundDocumentInfo& OutDocInfo,
		UPARAM(DisplayName = "Interface Info") FMetaSoundClassInterfaceInfo& OutInterfaceInfo,
		UPARAM(DisplayName = "Force Load") bool bForceLoad = true) const;

	// Returns info for all MetaSounds assets referencing the given MetaSound as a dependency.  Only returns assets on disk
	// (i.e. not in-memory, transient MetaSounds created by the builder API).  (Optional) Only returns references that are
	// presets of the given MetaSound if bOnlyPresets is true.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Utilities", meta = (DisplayName = "Find Referencing Asset Class Info", AdvancedDisplay = "5"))
	UE_API UPARAM(DisplayName = "Reference Info Found") bool FindReferencingAssetClassInfo(
		UPARAM(DisplayName = "MetaSound") TScriptInterface<IMetaSoundDocumentInterface> MetaSound,
		UPARAM(DisplayName = "Reference Paths") TArray<FTopLevelAssetPath>& OutPaths,
		UPARAM(DisplayName = "Document Info") TArray<FMetaSoundDocumentInfo>& OutDocInfo,
		UPARAM(DisplayName = "Interface Info") TArray<FMetaSoundClassInterfaceInfo>& OutInterfaceInfo,
		UPARAM(DisplayName = "Only Presets") bool bOnlyPresets = false,
		UPARAM(DisplayName = "Force Load") bool bForceLoad = true) const;

	// Reassigns a new class name for the given MetaSound object, invalidating all references to the given MetaSound Asset.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Utilities")
	UE_API UPARAM(DisplayName = "Reassigned") bool ReassignClassName(TScriptInterface<IMetaSoundDocumentInterface> DocInterface);

	// Replaces dependencies in a MetaSound with the given class name and version with another MetaSound with the given
	// class name and version.  Can be asset or code-defined.  It is up to the caller to validate the two classes have
	// matching interfaces (Swapping with classes of unmatched interfaces can leave MetaSound in non-executable state).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Utilities", meta = (AdvancedDisplay = "3"))
	UE_API UPARAM(DisplayName = "References Replaced") bool ReplaceReferencesInDirectory(
		const TArray<FMetaSoundAssetDirectory>& InDirectories,
		const FMetasoundFrontendClassName& OldClassName,
		const FMetasoundFrontendClassName& NewClassName,
		const FMetasoundFrontendVersionNumber OldVersion = FMetasoundFrontendVersionNumber(),
		const FMetasoundFrontendVersionNumber NewVersion = FMetasoundFrontendVersionNumber());
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Registration")
	UE_API void RegisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& Directories);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Registration")
	UE_API void UnregisterAssetClassesInDirectories(const TArray<FMetaSoundAssetDirectory>& Directories);

private:
	void PostEngineInitInternal();
	void PostInitAssetScanInternal();
};

#undef UE_API
