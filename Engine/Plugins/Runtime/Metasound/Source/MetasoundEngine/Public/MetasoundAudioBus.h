// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "IAudioProxyInitializer.h"
#include "Sound/AudioBus.h"

#define UE_API METASOUNDENGINE_API

namespace Metasound
{
	// Forward declare ReadRef
	class FAudioBus;

	// Metasound data type that holds onto a weak ptr. Mostly used as a placeholder until we have a proper proxy type.
	class FAudioBusAsset
	{
		FAudioBusProxyPtr AudioBusProxy;
		
	public:

		FAudioBusAsset() = default;
		FAudioBusAsset(const FAudioBusAsset&) = default;
		FAudioBusAsset& operator=(const FAudioBusAsset& Other) = default;

		UE_API FAudioBusAsset(const TSharedPtr<Audio::IProxyData>& InInitData);

		const FAudioBusProxyPtr& GetAudioBusProxy() const
		{
			return AudioBusProxy;
		}

		friend inline uint32 GetTypeHash(const Metasound::FAudioBusAsset& InAudioBusAsset)
		{
			const FAudioBusProxyPtr& Proxy = InAudioBusAsset.GetAudioBusProxy();
			if (Proxy.IsValid())
			{
				return GetTypeHash(*Proxy);
			}
			return INDEX_NONE;
		}
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FAudioBusAsset, METASOUNDENGINE_API, FAudioBusAssetTypeInfo, FAudioBusAssetReadRef, FAudioBusAssetWriteRef)

	METASOUNDENGINE_API int32 AudioBusReaderNodeInitialNumBlocks(int32 BlockSizeFrames, int32 AudioMixerOutputFrames);
	METASOUNDENGINE_API int32 AudioBusWriterNodeInitialNumBlocks(int32 BlockSizeFrames, int32 AudioMixerOutputFrames);
}
 

#undef UE_API
