// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundEngineNodesNames.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"

namespace Metasound::AudioBusWriterNode
{
	template<uint32 NumChannels>
	const FNodeClassName& GetClassName()
	{
		static const FName OperatorName = *FString::Printf(TEXT("Audio Bus Writer (%d)"), NumChannels);
		static const FNodeClassName ClassName = { EngineNodes::Namespace, OperatorName, TEXT("") };
		return ClassName;
	}
	
	METASOUNDENGINE_API int32 GetCurrentMajorVersion();

	namespace Inputs
	{
		DECLARE_METASOUND_PARAM(METASOUNDENGINE_API, AudioBus);
		DECLARE_METASOUND_PARAM(METASOUNDENGINE_API, Audio);
	}
}
