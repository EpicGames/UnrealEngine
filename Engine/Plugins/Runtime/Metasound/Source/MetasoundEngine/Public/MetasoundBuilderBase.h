// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundSettings.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/Function.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "MetasoundBuilderBase.generated.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
struct FMetasoundFrontendClassName;
struct FMetasoundFrontendVersion;

namespace Metasound::Engine
{
	// Forward Declarations
	class FDocumentBuilderRegistry;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBuilderReload, Metasound::Frontend::FDocumentModifyDelegates& /* OutDelegates */)
} // namespace Metasound::Engine


USTRUCT(BlueprintType, meta = (DisplayName = "MetaSound Node Input Handle"))
struct FMetaSoundBuilderNodeInputHandle : public FMetasoundFrontendVertexHandle
{
	GENERATED_BODY()

public:
	FMetaSoundBuilderNodeInputHandle() = default;
	FMetaSoundBuilderNodeInputHandle(const FGuid InNodeID, const FGuid& InVertexID)
	{
		NodeID = InNodeID;
		VertexID = InVertexID;
	}
};

USTRUCT(BlueprintType, meta = (DisplayName = "MetaSound Node Output Handle"))
struct FMetaSoundBuilderNodeOutputHandle : public FMetasoundFrontendVertexHandle
{
	GENERATED_BODY()

public:
	FMetaSoundBuilderNodeOutputHandle() = default;
	FMetaSoundBuilderNodeOutputHandle(const FGuid InNodeID, const FGuid& InVertexID)
	{
		NodeID = InNodeID;
		VertexID = InVertexID;
	}
};

USTRUCT(BlueprintType, meta = (DisplayName = "MetaSound Node Handle"))
struct FMetaSoundNodeHandle
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid NodeID;

public:
	FMetaSoundNodeHandle() = default;
	FMetaSoundNodeHandle(const FGuid& InNodeID)
		: NodeID(InNodeID)
	{
	}

	// Returns whether or not the vertex handle is set (may or may not be
	// valid depending on what builder context it is referenced against)
	bool IsSet() const
	{
		return NodeID.IsValid();
	}
};

USTRUCT(BlueprintType)
struct FMetaSoundBuilderOptions
{
	GENERATED_BODY()

	// Name of generated object. If object already exists, used as the base name to ensure new object is unique.
	// If left 'None', creates unique name.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MetaSound|Builder")
	FName Name;

	// If the resulting MetaSound is building over an existing document, a unique class name will be generated,
	// invalidating any referencing MetaSounds and registering the MetaSound as a new entry in the Frontend. If
	// building a new document, option is ignored (new document always generates a unique class name).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MetaSound|Builder", meta = (AdvancedDisplay))
	bool bForceUniqueClassName = false;

	// If true, adds MetaSound to node registry, making it available
	// for reference by other dynamically created MetaSounds.
	UPROPERTY()
	bool bAddToRegistry = true;

	// If set, builder overwrites the given MetaSound's document with the builder's copy
	// (ignores the Name field above).
	UPROPERTY()
	TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound;
};

UENUM(BlueprintType, meta = (DisplayName = "MetaSound Builder Result"))
enum class EMetaSoundBuilderResult : uint8
{
	Succeeded,
	Failed
};

/** Base implementation of MetaSound builder */
UCLASS(MinimalAPI, Abstract, BlueprintType, Transient, meta = (DisplayName = "MetaSound Builder Base"))
class UMetaSoundBuilderBase : public UObject
{
	GENERATED_BODY()

public:
	// Begin UObject interface
	UE_API virtual void BeginDestroy() override;
	// End UObject interface

	// Adds a graph input node with the given name, DataType, and sets the graph input to default value.
	// Returns the new input node's output handle if it was successfully created, or an invalid handle if it failed.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "3"))
	UE_API UPARAM(DisplayName = "Output Handle") FMetaSoundBuilderNodeOutputHandle AddGraphInputNode(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult, bool bIsConstructorInput = false);

	// Adds a graph output node with the given name, DataType, and sets output node's input to default value.
	// Returns the new output node's input handle if it was successfully created, or an invalid handle if it failed.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "3"))
	UE_API UPARAM(DisplayName = "Input Handle") FMetaSoundBuilderNodeInputHandle AddGraphOutputNode(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult, bool bIsConstructorOutput = false);

#if WITH_EDITORONLY_DATA
	// Adds a graph page to the given builder's document. Fails if the page is not a valid page registered with MetaSoundSettings
	// or if the document already contains a page with the given name. No check is done here to determine cook eligibility (i.e.
	// pages can be added even if set to be stripped for the active platform).
	UE_API void AddGraphPage(FName PageName, bool bDuplicateLastGraph, bool bSetAsBuildGraph, EMetaSoundBuilderResult& OutResult);
#endif // WITH_EDITORONLY_DATA

	// Adds a graph variable node with the given name, DataType, and sets to default value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void AddGraphVariable(FName Name, FName DataType, FMetasoundFrontendLiteral DefaultValue, EMetaSoundBuilderResult& OutResult);

	// Adds a graph variable node with the given name, DataType, and sets to default value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Get Node Handle") FMetaSoundNodeHandle AddGraphVariableGetNode(FName Name, EMetaSoundBuilderResult& OutResult);

	// Adds a graph variable node with the given name, DataType, and sets to default value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Get Delayed Node Handle") FMetaSoundNodeHandle AddGraphVariableGetDelayedNode(FName Name, EMetaSoundBuilderResult& OutResult);

	// Adds a graph variable node with the given name, DataType, and sets to default value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Set Node Handle") FMetaSoundNodeHandle AddGraphVariableSetNode(FName Name, EMetaSoundBuilderResult& OutResult);

	// Adds an interface registered with the given name to the graph, adding associated input and output nodes.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void AddInterface(FName InterfaceName, EMetaSoundBuilderResult& OutResult);

	// Adds a node to the graph using the provided MetaSound asset as its defining NodeClass.
	// Returns a node handle to the created node if successful, or an invalid handle if it failed.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", DisplayName = "Add MetaSound Node From Asset Class", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle AddNode(const TScriptInterface<IMetaSoundDocumentInterface>& NodeClass, EMetaSoundBuilderResult& OutResult);

	// Adds node referencing the highest native class version of the given class name to the document.
	// Returns a node handle to the created node if successful, or an invalid handle if it failed.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", DisplayName = "Add MetaSound Node By ClassName", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "2"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle AddNodeByClassName(const FMetasoundFrontendClassName& ClassName, EMetaSoundBuilderResult& OutResult, int32 MajorVersion = 1);
	
	UE_DEPRECATED(5.4, "This version of AddNodeByClassName is deprecated. Use the one with a default MajorVersion of 1.")
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle AddNodeByClassName(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion, EMetaSoundBuilderResult& OutResult);

	// Adds transaction listener which allows objects to respond to when certain graph operations are applied from anywhere (adding or removing nodes, edges, pages, etc.)
	// Currently there is no guarantee all transactions will be represented until the Controller API is fully deprecated! (ex. if a node or edge is added or removed via a
	// controller API call, the transaction will be missed). OnBuilderReloaded is however guaranteed to be called on mutable controller creation.
	UE_API FDelegateHandle AddTransactionListener(TSharedRef<Metasound::Frontend::IDocumentBuilderTransactionListener> BuilderListener);

#if WITH_EDITOR
	UE_API bool ClearMemberMetadata(const FGuid& InMemberID);
#endif // WITH_EDITOR

	// Connects node output to a node input. Does *NOT* provide loop detection for performance reasons.  Loop detection is checked on class registration when built or played.
	// Returns succeeded if connection made, failed if connection already exists with input, the data types do not match, or the connection is not supported due to access type
	// incompatibility (ex. constructor input to non-constructor input).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConnectNodes(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult);

	// Connects two nodes using defined MetaSound Interface Bindings registered with the MetaSound Interface registry.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConnectNodesByInterfaceBindings(const FMetaSoundNodeHandle& FromNodeHandle, const FMetaSoundNodeHandle& ToNodeHandle, EMetaSoundBuilderResult& OutResult);

	// Connects a given node's outputs to all graph outputs for shared interfaces implemented on both the node's referenced class and the builder's MetaSound graph. Returns inputs of connected output nodes.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Connected Graph Output Node Inputs") TArray<FMetaSoundBuilderNodeInputHandle> ConnectNodeOutputsToMatchingGraphInterfaceOutputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult);

	// Connects a given node's inputs to all graph inputs for shared interfaces implemented on both the node's referenced class and the builder's MetaSound graph. Returns outputs of connected input nodes.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Connected Graph Input Node Outputs") TArray<FMetaSoundBuilderNodeOutputHandle> ConnectNodeInputsToMatchingGraphInterfaceInputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult);

	// Connects a given node output to the graph output with the given name.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConnectNodeOutputToGraphOutput(FName GraphOutputName, const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult);

	// Connects a given node's named output to the graph output with the given name.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	inline void ConnectNamedNodeOutputToNamedGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FName& GraphOutputName, EMetaSoundBuilderResult& OutResult)
	{
		ConnectNodeToGraphOutput(SourceNode, NodeOutputName, GraphOutputName, OutResult);
	}
	UE_API void ConnectNodeToGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FName& GraphOutputName, EMetaSoundBuilderResult& OutResult);

	// Connects a given node's named output to the graph output with the given handle.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	inline void ConnectNamedNodeOutputToGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FMetaSoundBuilderNodeInputHandle& GraphOutput, EMetaSoundBuilderResult& OutResult)
	{
		ConnectNodeToGraphOutput(SourceNode, NodeOutputName, GraphOutput, OutResult);
	}
	UE_API void ConnectNodeToGraphOutput(const FMetaSoundNodeHandle& SourceNode, const FName& NodeOutputName, const FMetaSoundBuilderNodeInputHandle& GraphOutput, EMetaSoundBuilderResult& OutResult);

	// Connects a given node input to the graph input with the given name.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConnectNodeInputToGraphInput(FName GraphInputName, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult);

	UE_API void ConnectGraphInputToNode(const FName& GraphInputName, const FMetaSoundNodeHandle& DestinationNode, const FName& InputName, EMetaSoundBuilderResult& OutResult);

	UE_API void ConnectNodes(const FMetaSoundNodeHandle& SourceNode, const FName& OutputName, const FMetaSoundNodeHandle& DestinationNode, const FName& InputName, EMetaSoundBuilderResult& OutResult);

	// Returns whether node exists.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "IsValid") bool ContainsNode(const FMetaSoundNodeHandle& Node) const;

	// Returns whether node input exists.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "IsValid") bool ContainsNodeInput(const FMetaSoundBuilderNodeInputHandle& Input) const;

	// Returns whether node output exists.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "IsValid") bool ContainsNodeOutput(const FMetaSoundBuilderNodeOutputHandle& Output) const;

	// Disconnects node output to a node input. Returns success if connection was removed, failed if not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void DisconnectNodes(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult);

	// Removes connection to a given node input. Returns success if connection was removed, failed if not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void DisconnectNodeInput(const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult);

	// Removes all connections from a given node output. Returns success if all connections were removed, failed if not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void DisconnectNodeOutput(const FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult);

	// Disconnects two nodes using defined MetaSound Interface Bindings registered with the MetaSound Interface registry. Returns success if
	// all connections were found and removed, failed if any connections were not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void DisconnectNodesByInterfaceBindings(const FMetaSoundNodeHandle& FromNodeHandle, const FMetaSoundNodeHandle& ToNodeHandle, EMetaSoundBuilderResult& OutResult);

	// Returns graph input node by the given name if it exists, or an invalid handle if not found.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindGraphInputNode(FName InputName, UPARAM(DisplayName = "Data Type") FName& OutDataType, FMetaSoundBuilderNodeOutputHandle& NodeOutputHandle, EMetaSoundBuilderResult& OutResult);

	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindGraphInputNode(FName InputName, EMetaSoundBuilderResult& OutResult);

	// Returns graph output node by the given name if it exists, or an invalid handle if not found.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindGraphOutputNode(FName OutputName, UPARAM(DisplayName = "Data Type") FName& OutDataType, FMetaSoundBuilderNodeInputHandle& NodeInputHandle, EMetaSoundBuilderResult& OutResult);
	
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindGraphOutputNode(FName OutputName, EMetaSoundBuilderResult& OutResult);

#if WITH_EDITOR
	UE_API UMetaSoundFrontendMemberMetadata* FindMemberMetadata(const FGuid& InMemberID);
#endif // WITH_EDITOR

	// Returns node input by the given name if it exists, or an invalid handle if not found.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Input Handle") FMetaSoundBuilderNodeInputHandle FindNodeInputByName(const FMetaSoundNodeHandle& NodeHandle, FName InputName, EMetaSoundBuilderResult& OutResult);

	// Returns all node inputs.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Input  Handles") TArray<FMetaSoundBuilderNodeInputHandle> FindNodeInputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult);

	// Returns node inputs by the given DataType (ex. "Audio", "Trigger", "String", "Bool", "Float", "Int32", etc.).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Input Handles") TArray<FMetaSoundBuilderNodeInputHandle> FindNodeInputsByDataType(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, FName DataType);

	// Returns node output by the given name.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Output Handle") FMetaSoundBuilderNodeOutputHandle FindNodeOutputByName(const FMetaSoundNodeHandle& NodeHandle, FName OutputName, EMetaSoundBuilderResult& OutResult);

	// Returns all node outputs.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Output Handles") TArray<FMetaSoundBuilderNodeOutputHandle> FindNodeOutputs(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult);

	// Returns node outputs by the given DataType (ex. "Audio", "Trigger", "String", "Bool", "Float", "Int32", etc.).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Output Handles") TArray<FMetaSoundBuilderNodeOutputHandle> FindNodeOutputsByDataType(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, FName DataType);

	// Returns input nodes associated with a given interface.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Input Node Handles") TArray<FMetaSoundNodeHandle> FindInterfaceInputNodes(FName InterfaceName, EMetaSoundBuilderResult& OutResult);

	// Returns output nodes associated with a given interface.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Output Node Handles") TArray<FMetaSoundNodeHandle> FindInterfaceOutputNodes(FName InterfaceName, EMetaSoundBuilderResult& OutResult);

	// Returns input's parent node if the input is valid, otherwise returns invalid node handle.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindNodeInputParent(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult);

	// Returns output's parent node if the input is valid, otherwise returns invalid node handle.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Handle") FMetaSoundNodeHandle FindNodeOutputParent(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, EMetaSoundBuilderResult& OutResult);

	// Returns output's parent node if the input is valid, otherwise returns invalid node handle.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Node Class Version") FMetasoundFrontendVersion FindNodeClassVersion(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult);

	// Returns copy of the builder's template. Returns success if the builder has a template set, fails if not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API TInstancedStruct<FMetaSoundFrontendDocumentTemplate> GetDocumentTemplate(EMetaSoundBuilderResult& OutResult) const;

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DeprecatedFunction, DeprecationMessage = "5.8 - Use GetDocumentTemplate instead", ExpandEnumAsExecs = "OutResult"))
	TInstancedStruct<FMetaSoundFrontendDocumentTemplate> GetConfiguration(EMetaSoundBuilderResult& OutResult) const { return GetDocumentTemplate(OutResult); }

	// Gets names of all graph inputs. 
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Graph Inputs") TArray<FName> GetGraphInputNames(EMetaSoundBuilderResult& OutResult) const;
			
	// Gets names of all graph outputs. 
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Graph Outputs") TArray<FName> GetGraphOutputNames(EMetaSoundBuilderResult& OutResult) const;
	
	// Gets the graph input's default literal value.
	//  @param PageName	If the page name is not specified or not found, the builder's current build page will be used (which can be set in the MetaSound editor).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "PageName"))
	UE_API UPARAM(DisplayName = "Literal Value") FMetasoundFrontendLiteral GetGraphInputDefault(FName InputName, EMetaSoundBuilderResult& OutResult, FName PageName = FName()) const;

	// Gets the graph variable's default literal value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Literal Value") FMetasoundFrontendLiteral GetGraphVariableDefault(FName VariableName, EMetaSoundBuilderResult& OutResult) const;

	// Returns the MetaSound asset's graph class name (used by the MetaSound Node Class Registry)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder|Advanced", meta = (DisplayName = "Get MetaSound Class Name"))
	UE_API UPARAM(DisplayName = "Class Name") FMetasoundFrontendClassName GetRootGraphClassName() const;

	// Returns the interface to the MetaSound the builder is acting on.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "MetaSound") TScriptInterface<IMetaSoundDocumentInterface> GetMetaSound() const;

	// Returns node input's data if valid (including things like name and datatype).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void GetNodeInputData(const FMetaSoundBuilderNodeInputHandle& InputHandle, FName& Name, FName& DataType, EMetaSoundBuilderResult& OutResult);

	// Returns node input's literal value if set on graph, otherwise fails and returns default literal.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Default") FMetasoundFrontendLiteral GetNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult);

	// Returns node input's class literal value if set, otherwise fails and returns default literal.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Default") FMetasoundFrontendLiteral GetNodeInputClassDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult);

	// Returns whether the given node input is a constructor pin
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Is Constructor Pin") bool GetNodeInputIsConstructorPin(const FMetaSoundBuilderNodeInputHandle& InputHandle) const;

	// Returns node output's data if valid (including things like name and datatype).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void GetNodeOutputData(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, FName& Name, FName& DataType, EMetaSoundBuilderResult& OutResult);
	
	// Returns whether the given node output is a constructor pin
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Is Constructor Pin") bool GetNodeOutputIsConstructorPin(const FMetaSoundBuilderNodeOutputHandle& OutputHandle) const;

	// Return the parent asset referenced by this preset builder. Returns nullptr if the builder is not a preset.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Get Preset Parent"))
	UE_API UPARAM(DisplayName = "Referenced Preset") UObject* GetReferencedPresetAsset() const;

	// Returns if a given interface is declared.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Is Declared") bool InterfaceIsDeclared(FName InterfaceName) const;

	// Returns if a given node output and node input are connected.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Connected") bool NodesAreConnected(const FMetaSoundBuilderNodeOutputHandle& OutputHandle, const FMetaSoundBuilderNodeInputHandle& InputHandle) const;

	// Returns if a given node input has connections.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Connected") bool NodeInputIsConnected(const FMetaSoundBuilderNodeInputHandle& InputHandle) const;

	// Returns if a given node output is connected.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Connected") bool NodeOutputIsConnected(const FMetaSoundBuilderNodeOutputHandle& OutputHandle) const;

	// Returns whether this is a preset.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Is Preset") bool IsPreset() const;

	// Converts this preset to a fully accessible MetaSound; sets result to succeeded if it was converted successfully and failed if it was not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConvertFromPreset(EMetaSoundBuilderResult& OutResult);

	// Convert this builder to a MetaSound preset using the given MetaSoundDocumentInterface as its parent.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ConvertToPreset(UPARAM(DisplayName = "Parent Node Class") const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedNodeClass, EMetaSoundBuilderResult& OutResult);

	// Removes input literal defaults for all pages if set, reverting the value to the class defaults
	// Returns success if values reset, false if not (i.e. wasn't set to begin with).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void ResetGraphInputDefaults(FName Name, EMetaSoundBuilderResult& OutResult);
	
#if WITH_EDITORONLY_DATA
	// Removes all graph pages except the default.  If bClearDefaultPage is true, clears the default graph page implementation.
	UE_API void ResetGraphPages(bool bClearDefaultPage);
#endif // WITH_EDITORONLY_DATA

	// Removes input from all paged graphs if it exists; sets result to succeeded if it was removed and failed if it was not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void RemoveGraphInput(FName Name, EMetaSoundBuilderResult& OutResult);

	// Removes graph input literal default if set, reverting the value to the class default
	// @param PageName If the page name is not specified or not found, the builder's current build page will be used (which can be set in the MetaSound editor).
	// Returns success if value was removed, false if not removed (i.e. wasn't set to begin with).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "PageName"))
	UE_API void RemoveGraphInputDefault(FName Name, EMetaSoundBuilderResult& OutResult, FName PageName = FName());
	
	// Removes output from all paged graphs if it exists; sets result to succeeded if it was removed and failed if it was not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void RemoveGraphOutput(FName Name, EMetaSoundBuilderResult& OutResult);

#if WITH_EDITORONLY_DATA
	// Removes a graph page with the given name, setting result to failed if the name was not found or was invalid.
	UE_API void RemoveGraphPage(FName Name, EMetaSoundBuilderResult& OutResult);
#endif // WITH_EDITORONLY_DATA

	// Removes graph variable from the current build graph if it exists; sets result to succeeded if it was removed and failed if it was not.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void RemoveGraphVariable(FName Name, EMetaSoundBuilderResult& OutResult);

	// Removes the interface with the given name from the builder's MetaSound. Removes any graph inputs
	// and outputs associated with the given interface and their respective connections (if any).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void RemoveInterface(FName InterfaceName, EMetaSoundBuilderResult& OutResult);

	// Removes node and any associated connections from the current build graph. (Advanced) Optionally, remove unused dependencies
	// from the internal dependency list on successful removal of node.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "2"))
	UE_API void RemoveNode(const FMetaSoundNodeHandle& NodeHandle, EMetaSoundBuilderResult& OutResult, bool bRemoveUnusedDependencies = true);

	// Removes node input literal default if set, reverting the value to be whatever the node class defaults the value to.
	// Returns success if value was removed, false if not removed (i.e. wasn't set to begin with).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void RemoveNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& InputHandle, EMetaSoundBuilderResult& OutResult);

	// Explicitly remove transaction listener from builder (see corresponding 'AddTransactionListener' function).
	// (If listener provided with `AddTransactionListener` is destroyed, handled automatically)
	UE_API void RemoveTransactionListener(FDelegateHandle BuilderListenerDelegateHandle);

	// Removes dependencies in document that are no longer referenced by nodes
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void RemoveUnusedDependencies();

	UE_DEPRECATED(5.5, "Use IDocumentBuilderRegistry::GenerateNewClassName instead to maintain registry mappings.")
	UE_API void RenameRootGraphClass(const FMetasoundFrontendClassName& InName);

	UE_DEPRECATED(5.5, "Moved to internal implementation and only accessible via registry to ensure delegates are properly reloaded, "
		"path keys kept aligned, and priming managed internally")
	UE_API void ReloadCache(bool bPrimeCache = true);

#if WITH_EDITOR
	// Sets the author of the MetaSound.
	UE_API void SetAuthor(const FString& InAuthor);
#endif // WITH_EDITOR

	// Sets the given builder's template and immediately configures the MetaSound.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void SetTemplate(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Configuration);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DeprecatedFunction, DeprecationMessage = "5.8 - Use SetTemplate instead"))
	void SetConfiguration(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Configuration) { SetTemplate(MoveTemp(Configuration)); }

	// Passes view of struct along for mutating properties, firing OnDocumentTemplateChanged on completion and immediately reconfigures the MetaSound.
	UE_API void SetTemplateProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertiesFunction);

	UE_DEPRECATED(5.8, "Use SetTemplateProperties() instead")
	void SetConfigurationProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertiesFunction) { SetTemplateProperties(SetPropertiesFunction); }

	template <typename TConfig>
	bool SetTemplateProperties(TFunctionRef<void(TConfig&)> SetPropertyFunction)
	{
		const bool bConfigUpdated = GetBuilder().SetTemplateProperties(SetPropertyFunction);
		if (bConfigUpdated)
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			GetBuilder().ConfigureDocument();
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}
		return bConfigUpdated;
	}

	// Sets the node's input default value (used if no connection to the given node input is present)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetNodeInputDefault(const FMetaSoundBuilderNodeInputHandle& NodeInputHandle, const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Templated convenience function for setting the node's input default value (used if no connection to the given node input is present)
	template <typename LiteralType>
	void SetNodeInputDefault(const FMetaSoundNodeHandle& NodeHandle, const FName& InputName, const LiteralType& Literal, EMetaSoundBuilderResult& OutResult)
	{
		FMetaSoundBuilderNodeInputHandle In = FindNodeInputByName(NodeHandle, InputName, OutResult);
		if (OutResult != EMetaSoundBuilderResult::Succeeded)
		{
			return;
		}
		FMetasoundFrontendLiteral ObjectLiteral;
		ObjectLiteral.Set(Literal);
		SetNodeInputDefault(In, ObjectLiteral, OutResult);
	}

	// Disconnects the given graph input's respective template nodes and sets the graph input's AccessType should it not match the current AccessType.
	// Result succeeds if the AccessType was successfully changed or if the provided AccessType is already the input's current AccessType.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphInputAccessType(FName InputName, EMetasoundFrontendVertexAccessType AccessType, EMetaSoundBuilderResult& OutResult);

	// Disconnects the given graph input's respective template nodes and sets the graph input's DataType should it not match the current DataType.
	// Result succeeds if the DataType was successfully changed or if the provided DataType is already the input's current DataType.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphInputDataType(FName InputName, FName DataType, EMetaSoundBuilderResult& OutResult);

	/* Sets the input node's default value, overriding the default provided by the referenced graph if the graph is a preset.
	*  @param PageName	If the page name is not specified or not found, the builder's current build page will be used (which can be set in the MetaSound editor).
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult", AdvancedDisplay = "PageName"))
	UE_API void SetGraphInputDefault(FName InputName, const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult, FName PageName = FName());	

	// Sets the given graph input's name to the new name. 
	// Result succeeds if the name was successfully changed or the new name is the same as the old name, and fails if the given input name doesn't exist.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphInputName(FName InputName, FName NewName, EMetaSoundBuilderResult& OutResult);

	// Disconnects the given graph output's respective template nodes and sets the graph output's AccessType should it not match the current AccessType.
	// Result succeeds if the AccessType was successfully changed or if the provided AccessType is already the output's current AccessType.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphOutputAccessType(FName OutputName, EMetasoundFrontendVertexAccessType AccessType, EMetaSoundBuilderResult& OutResult);

	// Disconnects the given graph output's respective template nodes and sets the graph output's DataType should it not match the current DataType.
	// Result succeeds if the DataType was successfully changed or if the provided DataType is already the output's current DataType.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphOutputDataType(FName OutputName, FName DataType, EMetaSoundBuilderResult& OutResult);

	// Sets the given graph output's name to the new name. 
	// Result succeeds if the name was successfully changed or the new name is the same as the old name, and fails if the given output name doesn't exist.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetGraphOutputName(FName OutputName, FName NewName, EMetaSoundBuilderResult& OutResult);

	UE_API void SetMemberMetadata(UMetaSoundFrontendMemberMetadata& NewMetadata);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (WorldContext = "Parent", DisplayName = "Build (Soft Deprecated. Parent no longer supported and field is ignored.)"))
	virtual UPARAM(DisplayName = "MetaSound") TScriptInterface<IMetaSoundDocumentInterface> Build(UObject* Parent, const FMetaSoundBuilderOptions& Options) const { return BuildNewMetaSound(Options.Name); }

#if WITH_EDITORONLY_DATA
	UE_API TScriptInterface<IMetaSoundDocumentInterface> Build(const FMetaSoundBuilderOptions& Options) const;
#endif // WITH_EDITORONLY_DATA

	// Copies a transient MetaSound with the provided builder options, copying the underlying MetaSound
	// managed by this builder and registering it with the MetaSound Node Registry as a unique name.
	// If 'Force Unique Class Name' is true, registers MetaSound as a new class in the registry, potentially
	// invalidating existing references in other MetaSounds. Not permissible to overwrite MetaSound asset,
	// only transient MetaSound (see EditorSubsystem for overwriting assets at edit time).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Build And Overwrite MetaSound", AdvancedDisplay = "1"))
	UE_API void BuildAndOverwriteMetaSound(UPARAM(DisplayName = "Existing MetaSound") TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName = false);

	// Builds a transient MetaSound with the provided builder options, copying the underlying MetaSound
	// managed by this builder and registering it with the MetaSound Node Registry as a unique class. If
	// existing MetaSound exists with the provided NameBase, will make object with unique name with the given
	// NameBase as prefix.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Build New MetaSound"))
	UE_API virtual UPARAM(DisplayName = "New MetaSound") TScriptInterface<IMetaSoundDocumentInterface> BuildNewMetaSound(FName NameBase) const PURE_VIRTUAL(UMetaSoundBuilderBase::BuildNewMetaSound, return { }; );

	UE_API virtual bool ConformObjectToDocument();

	// Returns the base class registered with the MetaSound UObject registry.
	virtual const UClass& GetBaseMetaSoundUClass() const PURE_VIRTUAL(UMetaSoundBuilderBase::GetBaseMetaSoundUClass, return *UClass::StaticClass(); );

	UE_DEPRECATED(5.5, "Renamed to 'GetBaseMetaSoundUClass' for consistency")
	virtual const UClass& GetBuilderUClass() const { return GetBaseMetaSoundUClass(); }

	UE_API Metasound::Frontend::FDocumentModifyDelegates& GetBuilderDelegates();

	// Initializes and ensures all nodes have a position (required prior to exporting to an asset if expected to be viewed in the editor).
	UE_API virtual void InitNodeLocations();

#if WITH_EDITOR
	// Injects template nodes between builder's document inputs not connected
	// to existing template inputs, copying locational data from the represented
	// input metadata. If bForceNodeCreation is false, only generates a template
	// input node if a connection between the input and other nodes exists. If true,
	// will inject template node irrespective of whether or not the input has connections.
	UE_API void InjectInputTemplateNodes(bool bForceNodeCreation, EMetaSoundBuilderResult& OutResult);

	UE_API const FMetaSoundFrontendGraphComment* FindGraphComment(const FGuid& InCommentID) const;
	UE_API FMetaSoundFrontendGraphComment* FindGraphComment(const FGuid & InCommentID);
	UE_API FMetaSoundFrontendGraphComment& FindOrAddGraphComment(const FGuid& InCommentID);
	UE_API bool RemoveGraphComment(const FGuid& InCommentID);

	UE_API void SetNodeComment(const FMetaSoundNodeHandle& InNodeHandle, const FString& InNewComment, EMetaSoundBuilderResult& OutResult);
	UE_API void SetNodeCommentVisible(const FMetaSoundNodeHandle& InNodeHandle, bool bIsVisible, EMetaSoundBuilderResult& OutResult);
	UE_API void SetNodeLocation(const FMetaSoundNodeHandle& InNodeHandle, const FVector2D& InLocation, EMetaSoundBuilderResult& OutResult);
	UE_API void SetNodeLocation(const FMetaSoundNodeHandle & InNodeHandle, const FVector2D& InLocation, const FGuid& InLocationGuid, EMetaSoundBuilderResult& OutResult);
#endif // WITH_EDITOR

	UE_API FMetaSoundFrontendDocumentBuilder& GetBuilder();
	UE_API const FMetaSoundFrontendDocumentBuilder& GetConstBuilder() const;
	UE_API int32 GetLastTransactionRegistered() const;

	// Resets FrontendBuilder instance, creating a transient MetaSound document that is managed by this UObject Builder.
	UE_API void Initialize();

	// Initializes builder given an existing document interface 
	UE_API void Initialize(TScriptInterface<IMetaSoundDocumentInterface>& DocInterface);
	
protected:
	virtual void BuildAndOverwriteMetaSoundInternal(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName) const PURE_VIRTUAL(UMetaSoundBuilderBase::BuildAndOverwriteMetaSoundInternal, ;);
	UE_API virtual void InitDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates);

	virtual void OnAssetReferenceAdded(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) PURE_VIRTUAL(UMetaSoundBuilderBase::OnAssetReferenceAdded, );
	virtual void OnRemovingAssetReference(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) PURE_VIRTUAL(UMetaSoundBuilderBase::OnRemovingAssetReference, );

	// Runs build, conforming the document and corresponding object data on a MetaSound UObject to that managed by this builder.
	template <typename UClassType>
	UClassType& BuildInternal(UObject* Parent, const FMetaSoundBuilderOptions& BuilderOptions) const
	{
		using namespace Metasound::Frontend;

		UClassType* MetaSound = nullptr;
		const FMetasoundFrontendClassName* DocClassName = nullptr;
		if (BuilderOptions.ExistingMetaSound)
		{
			MetaSound = CastChecked<UClassType>(BuilderOptions.ExistingMetaSound.GetObject());

			// Always unregister if mutating existing object. If bAddToRegistry is set to false,
			// leaving registered would result in any references to this MetaSound executing on
			// out-of-date data. If bAddToRegistry is set, then it needs to be unregistered before
			// being registered as it is below.
			if (MetaSound)
			{
				// If MetaSound already exists, preserve the class name to avoid
				// nametable bloat & preserve potentially existing references.
				const FMetasoundFrontendDocument& ExistingDoc = CastChecked<const UClassType>(MetaSound)->GetConstDocument();
				if (!BuilderOptions.bForceUniqueClassName)
				{
					DocClassName = &ExistingDoc.RootGraph.Metadata.GetClassName();
				}

				MetaSound->UnregisterGraphWithFrontend();
			}
		}
		else
		{
			FName ObjectName = BuilderOptions.Name;
			if (!ObjectName.IsNone())
			{
				ObjectName = MakeUniqueObjectName(Parent, UClassType::StaticClass(), BuilderOptions.Name);
			}

			if (!Parent)
			{
				Parent = GetTransientPackage();
			}

			MetaSound = NewObject<UClassType>(Parent, ObjectName, RF_Public | RF_Transient);
		}

		checkf(MetaSound, TEXT("Failed to build MetaSound from builder '%s'"), *GetPathName());

		TScriptInterface<IMetaSoundDocumentInterface> MetaSoundInterface = MetaSound;
		BuildInternal(MetaSoundInterface, DocClassName);

		if (BuilderOptions.bAddToRegistry)
		{
			Metasound::Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
			RegOptions.PageOrder = UMetaSoundSettings::GetPageOrder();
			MetaSound->UpdateAndRegisterForExecution(RegOptions);
		}

		UE_LOGF(LogMetaSound, VeryVerbose, "MetaSound '%ls' built from '%ls'", *BuilderOptions.Name.ToString(), *GetFullName());
		return *MetaSound;
	}

	// Only registers provided MetaSound's graph class and referenced graphs recursively if
	// it has yet to be registered or if it has an attached builder reporting outstanding
	// transactions that have yet to be registered.
	static UE_API void RegisterGraphIfOutstandingTransactions(UObject& InMetaSound);

	UPROPERTY()
	FMetaSoundFrontendDocumentBuilder Builder;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.5 - No longer used. ClassName should be queried from associated FrontendBuilder's MetaSound"))
	FMetasoundFrontendClassName ClassName;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.4 - All source builders now operate on an underlying document source document that is also used to audition."))
	bool bIsAttached = false;
#endif // WITH_EDITORONLY_DATA

private:
	UE_API void BuildInternal(TScriptInterface<IMetaSoundDocumentInterface> NewMetaSound, const FMetasoundFrontendClassName* InDocClassName) const;
	UE_API void OnDependencyAdded(int32 Index);
	UE_API void OnRemoveSwappingDependency(int32 Index, int32 LastIndex);

private:
	// Reloads the builder, freeing the internal cache and rebuilding delegate bindings. Optionally,
	// can be associated with a new MetaSound (ex. during rename. Otherwise it reuses the existing
	// document object reference) or can have its cache primed.
	UE_API void Reload(TScriptInterface<IMetaSoundDocumentInterface> NewMetaSound = { }, bool bPrimeCache = false);

	Metasound::Engine::FOnBuilderReload BuilderReloadDelegate;

	int32 LastTransactionRegistered = 0;

	friend class Metasound::Engine::FDocumentBuilderRegistry;
};

#undef UE_API
