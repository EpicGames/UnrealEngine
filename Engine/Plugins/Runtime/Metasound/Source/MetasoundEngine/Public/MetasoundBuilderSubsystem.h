// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/Engine.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentCacheInterface.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/Function.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "MetasoundBuilderSubsystem.generated.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
class FMetasoundAssetBase;
class UAudioComponent;
class UMetaSound;
class UMetaSoundPatch;
class UMetaSoundSource;

struct FMetasoundFrontendClassName;
struct FMetasoundFrontendVersion;
struct FPerPlatformFloat;
struct FPerPlatformInt;

enum class EMetaSoundOutputAudioFormat : uint8;

namespace Metasound::Engine
{
	// Forward Declarations
	struct FOutputAudioFormatInfo;
	class FMetaSoundAssetManager;
} // namespace Metasound::Engine


DECLARE_DYNAMIC_DELEGATE_OneParam(FOnCreateAuditionGeneratorHandleDelegate, UMetasoundGeneratorHandle*, GeneratorHandle);

/** Builder in charge of building a MetaSound Patch */
UCLASS(MinimalAPI, Transient, BlueprintType, meta = (DisplayName = "MetaSound Patch Builder"))
class UMetaSoundPatchBuilder : public UMetaSoundBuilderBase
{
	GENERATED_BODY()

public:
	UE_API virtual TScriptInterface<IMetaSoundDocumentInterface> BuildNewMetaSound(FName NameBase) const override;
	UE_API virtual const UClass& GetBaseMetaSoundUClass() const override;

protected:
	UE_API virtual void BuildAndOverwriteMetaSoundInternal(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName) const override;
	UE_API virtual void OnAssetReferenceAdded(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) override;
	UE_API virtual void OnRemovingAssetReference(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) override;

	friend class UMetaSoundBuilderSubsystem;
};

/** Builder in charge of building a MetaSound Source */
UCLASS(MinimalAPI, Transient, BlueprintType, meta = (DisplayName = "MetaSound Source Builder"))
class UMetaSoundSourceBuilder : public UMetaSoundBuilderBase
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (WorldContext = "Parent", AdvancedDisplay = "2"))
	UE_API void Audition(UObject* Parent, UAudioComponent* AudioComponent, FOnCreateAuditionGeneratorHandleDelegate OnCreateGenerator, bool bLiveUpdatesEnabled = false);

	// Returns whether or not live updates are both globally enabled (via cvar) and are enabled on this builder's last built sound, which may or may not still be playing.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API bool GetLiveUpdatesEnabled() const;

	// Sets the MetaSound's BlockRate override
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void SetBlockRateOverride(float BlockRate);

	// Sets the output audio format of the source
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API void SetFormat(EMetaSoundOutputAudioFormat OutputFormat, EMetaSoundBuilderResult& OutResult);

	// Sets the MetaSound's SampleRate override
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void SetSampleRateOverride(int32 SampleRate);

	UE_API const Metasound::Engine::FOutputAudioFormatInfoPair* FindOutputAudioFormatInfo() const;

	UE_API virtual TScriptInterface<IMetaSoundDocumentInterface> BuildNewMetaSound(FName NameBase) const override;
	UE_API virtual const UClass& GetBaseMetaSoundUClass() const override;

#if WITH_EDITORONLY_DATA
	// Sets the MetaSound's BlockRate override (editor only, to allow setting per-platform values)
	UE_API void SetPlatformBlockRateOverride(const FPerPlatformFloat& PlatformFloat);

	// Sets the MetaSound's BlockRate override (editor only, to allow setting per-platform values)
	UE_API void SetPlatformSampleRateOverride(const FPerPlatformInt& PlatformInt);
#endif // WITH_EDITORONLY_DATA

	// Sets the MetaSound's Quality level
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void SetQuality(FName Quality);

protected:
	UE_API virtual void BuildAndOverwriteMetaSoundInternal(TScriptInterface<IMetaSoundDocumentInterface> ExistingMetaSound, bool bForceUniqueClassName) const override;
	UE_API virtual void InitDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates) override;
	UE_API virtual void OnAssetReferenceAdded(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) override;
	UE_API virtual void OnRemovingAssetReference(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) override;

private:
	UE_API const FMetasoundFrontendGraph& GetConstTargetPageGraphChecked() const;

	UE_API UMetaSoundSource& GetMetaSoundSource() const;

	UE_API void InitTargetPageDelegates(Metasound::Frontend::FDocumentModifyDelegates& OutDocumentDelegates);

	UE_API void OnDocumentTemplateChanged(const Metasound::Frontend::FDocumentTemplateChangedArgs& Args) const;
	UE_API void OnEdgeAdded(int32 EdgeIndex) const;
	UE_API void OnInputAdded(int32 InputIndex);
	UE_API void OnInputDataTypeChanged(int32 InputIndex);
	UE_API void OnInputDefaultChanged(const Metasound::Frontend::FDocumentArrayPagedInputArgs& Args) const;
	
	UE_API void OnInputNameChanged(FName OldName, FName NewName);
	UE_API void OnLiveComponentFinished(UAudioComponent* AudioComponent);
	UE_API void OnNodeAdded(int32 NodeIndex) const;
	UE_API void OnNodeConfigurationUpdated(int32 NodeIndex) const;
	UE_API void OnNodeInputLiteralSet(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex) const;
	UE_API void OnOutputAdded(int32 OutputIndex) const;
	UE_API void OnPageAdded(const Metasound::Frontend::FDocumentMutatePageArgs& InArgs);
	UE_API void OnRemoveSwappingEdge(int32 SwapIndex, int32 LastIndex) const;
	UE_API void OnRemovingInput(int32 InputIndex);
	UE_API void OnRemoveSwappingNode(int32 SwapIndex, int32 LastIndex) const;
	UE_API void OnRemovingNodeInputLiteral(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex) const;
	UE_API void OnRemovingOutput(int32 OutputIndex) const;
	UE_API void OnRemovingPage(const Metasound::Frontend::FDocumentMutatePageArgs& InArgs);

	UE_API bool AddEdgeToTransactor(const FMetasoundFrontendEdge& NewEdge, Metasound::DynamicGraph::FDynamicOperatorTransactor& Transactor) const;
	UE_API bool AddNodeInputLiteralToTransactor(
		const FMetasoundFrontendNode& Node,
		const FMetasoundFrontendVertex& Input,
		const FMetasoundFrontendLiteral& InputDefault,
		Metasound::DynamicGraph::FDynamicOperatorTransactor& Transactor) const;

	using FAuditionableTransaction = TFunctionRef<bool(Metasound::DynamicGraph::FDynamicOperatorTransactor&)>;
	UE_API bool ExecuteAuditionableTransaction(FAuditionableTransaction Transaction) const;

	TArray<uint64> LiveComponentIDs;
	FDelegateHandle LiveComponentHandle;
	FGuid TargetPageID = Metasound::Frontend::DefaultPageID;

	friend class UMetaSoundBuilderSubsystem;
};

/** The subsystem in charge of tracking MetaSound builders */
UCLASS(MinimalAPI, meta = (DisplayName = "MetaSound Builder Subsystem"))
class UMetaSoundBuilderSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

private:
	UPROPERTY()
	TMap<FName, TObjectPtr<UMetaSoundBuilderBase>> NamedBuilders;

public:	
	static UE_API UMetaSoundBuilderSubsystem* Get();
	static UE_API UMetaSoundBuilderSubsystem& GetChecked();
	static UE_API const UMetaSoundBuilderSubsystem* GetConst();
	static UE_API const UMetaSoundBuilderSubsystem& GetConstChecked();

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder",  meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Patch Builder") UMetaSoundPatchBuilder* CreatePatchBuilder(FName BuilderName, EMetaSoundBuilderResult& OutResult);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Source Builder") UMetaSoundSourceBuilder* CreateSourceBuilder(
		FName BuilderName,
		FMetaSoundBuilderNodeOutputHandle& OnPlayNodeOutput,
		FMetaSoundBuilderNodeInputHandle& OnFinishedNodeInput,
		TArray<FMetaSoundBuilderNodeInputHandle>& AudioOutNodeInputs,
		EMetaSoundBuilderResult& OutResult,
		EMetaSoundOutputAudioFormat OutputFormat = EMetaSoundOutputAudioFormat::Mono,
		bool bIsOneShot = true);

	// Creates a builder that acts on a generated, transient preset MetaSound UObject. Fails if Referenced MetaSound is not a UMetaSoundPatch.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder",  meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Patch Preset Builder") UMetaSoundPatchBuilder* CreatePatchPresetBuilder(FName BuilderName, UPARAM(DisplayName = "Reference Patch")  const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedPatchClass, EMetaSoundBuilderResult& OutResult);

	UE_API UMetaSoundBuilderBase& CreatePresetBuilder(FName BuilderName, const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedMetaSound, EMetaSoundBuilderResult& OutResult);
	
	// Creates a builder that acts on a generated, transient preset MetaSound UObject. Fails if Referenced MetaSound is not a UMetaSoundSource.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (ExpandEnumAsExecs = "OutResult"))
	UE_API UPARAM(DisplayName = "Source Preset Builder") UMetaSoundSourceBuilder* CreateSourcePresetBuilder(FName BuilderName, UPARAM(DisplayName = "Reference Source") const TScriptInterface<IMetaSoundDocumentInterface>& ReferencedSourceClass, EMetaSoundBuilderResult& OutResult);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Bool Literal"))
	UE_API UPARAM(DisplayName = "Bool Literal") FMetasoundFrontendLiteral CreateBoolMetaSoundLiteral(bool Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Bool Array Literal"))
	UE_API UPARAM(DisplayName = "Bool Array Literal") FMetasoundFrontendLiteral CreateBoolArrayMetaSoundLiteral(const TArray<bool>& Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Float Literal"))
	UE_API UPARAM(DisplayName = "Float Literal") FMetasoundFrontendLiteral CreateFloatMetaSoundLiteral(float Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Float Array Literal"))
	UE_API UPARAM(DisplayName = "Float Array Literal") FMetasoundFrontendLiteral CreateFloatArrayMetaSoundLiteral(const TArray<float>& Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Int Literal"))
	UE_API UPARAM(DisplayName = "Int32 Literal") FMetasoundFrontendLiteral CreateIntMetaSoundLiteral(int32 Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Int Array Literal"))
	UE_API UPARAM(DisplayName = "Int32 Array Literal") FMetasoundFrontendLiteral CreateIntArrayMetaSoundLiteral(const TArray<int32>& Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Object Literal"))
	UE_API UPARAM(DisplayName = "Object Literal") FMetasoundFrontendLiteral CreateObjectMetaSoundLiteral(UObject* Value);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Object Array Literal"))
	UE_API UPARAM(DisplayName = "Object Array Literal") FMetasoundFrontendLiteral CreateObjectArrayMetaSoundLiteral(const TArray<UObject*>& Value);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound String Literal"))
	UE_API UPARAM(DisplayName = "String Literal") FMetasoundFrontendLiteral CreateStringMetaSoundLiteral(const FString& Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound String Array Literal"))
	UE_API UPARAM(DisplayName = "String Array Literal") FMetasoundFrontendLiteral CreateStringArrayMetaSoundLiteral(const TArray<FString>& Value, FName& DataType);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Create MetaSound Literal From AudioParameter"))
	UE_API UPARAM(DisplayName = "Param Literal") FMetasoundFrontendLiteral CreateMetaSoundLiteralFromParam(const FAudioParameter& Param);

	// Returns the builder manually registered with the MetaSound Builder Subsystem with the provided custom name (if previously registered)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Find Builder By Name"))
	UE_API UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* FindBuilder(FName BuilderName);

	// Returns the builder associated with the given MetaSound (if one exists, transient or asset).
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Find Builder By MetaSound"))
	UE_API UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* FindBuilderOfDocument(TScriptInterface<const IMetaSoundDocumentInterface> InMetaSound) const;

	/**
	 * Returns the builder associated with the given MetaSound preset's parent. If bFollowPresetChain is true, continues traversing up the preset chain to return the highest non-preset ancestor.
	 * Returns nullptr if the given MetaSound is not a preset, or if the parent builder is not yet registered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder", meta = (DisplayName = "Find Parent Builder of MetaSound Preset"))
	UE_API UPARAM(DisplayName = "Builder") UMetaSoundBuilderBase* FindParentBuilderOfPreset(TScriptInterface<const IMetaSoundDocumentInterface> InMetaSoundPreset, bool bFollowPresetChain = true);

	// Returns the patch builder manually registered with the MetaSound Builder Subsystem with the provided custom name (if previously registered)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Patch Builder") UMetaSoundPatchBuilder* FindPatchBuilder(FName BuilderName);

	// Returns the source builder manually registered with the MetaSound Builder Subsystem with the provided custom name (if previously registered)
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Source Builder") UMetaSoundSourceBuilder* FindSourceBuilder(FName BuilderName);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Is Registered") bool IsInterfaceRegistered(FName InInterfaceName) const;

#if WITH_EDITOR
	UE_DEPRECATED(5.5, "No longer required as reload is now just directly called on a given builder.")
	void PostBuilderAssetTransaction(const FMetasoundFrontendClassName& InClassName) { }
#endif // WITH_EDITOR

	// Adds builder to subsystem's registry to make it persistent and easily accessible by multiple systems or Blueprints
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void RegisterBuilder(FName BuilderName, UMetaSoundBuilderBase* Builder);

	// Adds builder to subsystem's registry to make it persistent and easily accessible by multiple systems or Blueprints
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void RegisterPatchBuilder(FName BuilderName, UMetaSoundPatchBuilder* Builder);

	// Adds builder to subsystem's registry to make it persistent and easily accessible by multiple systems or Blueprints
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API void RegisterSourceBuilder(FName BuilderName, UMetaSoundSourceBuilder* Builder);

	// Sets the targeted page for all MetaSound graph & input default to resolve against.
	// If target page is not implemented (or cooked in a runtime build) for the active platform,
	// uses order of cooked pages(see 'Page Settings' for order) falling back to lower index - ordered page
	// implemented in MetaSound asset. If no fallback is found, uses default graph/input default.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Pages")
	UE_API UPARAM(DisplayName = "TargetPageChanged") bool SetTargetPage(FName PageName);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Unregistered") bool UnregisterBuilder(FName BuilderName);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Unregistered") bool UnregisterPatchBuilder(FName BuilderName);

	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Builder")
	UE_API UPARAM(DisplayName = "Unregistered") bool UnregisterSourceBuilder(FName BuilderName);

private:
	friend class UMetaSoundBuilderBase;
};

#undef UE_API
