// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MetasoundChannelAgnosticType.h"

#include "MetasoundChannelAgnosticAutoTypes.h"

//#include "MetasoundChannelAgnosticAutoFormat.generated.h"

#define UE_API METASOUNDENGINE_API

namespace Metasound
{
	class FChannelAgnosticAutoFormatHelper
	{
	public:
		UE_EXPERIMENTAL(5.8, "MetaSound CAT API is in active development and subject to change")
		UE_API static int32 ComputeAutoChannelCountFromProxies(const TArray<FSoundWaveProxyPtr>& InProxies);
		UE_EXPERIMENTAL(5.8, "MetaSound CAT API is in active development and subject to change")
		UE_API static TSharedPtr<const Audio::FChannelTypeFamily> ComputeAutoFormatFromWaveContainer(const TSharedPtr<const Audio::IProxyData>& InData);
		UE_EXPERIMENTAL(5.8, "MetaSound CAT API is in active development and subject to change")
		UE_API static TSharedPtr<const Audio::FDiscreteChannelTypeFamily> NumChannelsToFormat(const int32 NumChannels);

		UE_EXPERIMENTAL(5.8, "MetaSound CAT API is in active development and subject to change")
		UE_API static FName ComputeNodeOutputFormat(
			const EMetasoundChannelAgnosticNodeFormatChooser InMethod,
			const FName InCustomFormat,
			const TSharedPtr<const Audio::IProxyData>& InContent,
			const FMetasoundEnvironment& InEnvironment);
	};
}//namespace Metasound

#undef UE_API 
