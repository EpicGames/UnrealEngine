// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Find.h"
#include "Async/Async.h"
#include "CoreGlobals.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundLog.h"
#include "MetasoundSettings.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"

#define UE_API METASOUNDENGINE_API

namespace Metasound::Engine
{
	class FDocumentBuilderRegistry : public Frontend::IDocumentBuilderRegistry
	{
		mutable TMultiMap<FMetasoundFrontendClassName, TWeakObjectPtr<UMetaSoundBuilderBase>> Builders;

		// Critical section primarily for allowing builder collection mutation during async loading of MetaSound assets.
		mutable FCriticalSection BuildersCriticalSection;

	public:
		FDocumentBuilderRegistry() = default;
		UE_API virtual ~FDocumentBuilderRegistry();

		static FDocumentBuilderRegistry& GetChecked()
		{
			return static_cast<FDocumentBuilderRegistry&>(IDocumentBuilderRegistry::GetChecked());
		}

		enum class ELogEvent : uint8
		{
			DuplicateEntries
		};

		template <typename BuilderClass>
		BuilderClass& CreateTransientBuilder(FName BuilderName = FName())
		{
			using namespace Metasound::Frontend;

			checkf(IsInGameThread(), TEXT("Transient MetaSound builder cannot be created in non - game thread as it may result in UObject creation"));

			const EObjectFlags NewObjectFlags = RF_Public | RF_Transient;
			UPackage* TransientPackage = GetTransientPackage();
			const FName ObjectName = MakeUniqueObjectName(TransientPackage, BuilderClass::StaticClass(), BuilderName);
			TObjectPtr<BuilderClass> NewBuilder = NewObject<BuilderClass>(TransientPackage, ObjectName, NewObjectFlags);
			check(NewBuilder);
			NewBuilder->Initialize();
			const FMetasoundFrontendDocument& Document = NewBuilder->GetConstBuilder().GetConstDocumentChecked();
			const FMetasoundFrontendClassName& ClassName = Document.RootGraph.Metadata.GetClassName();
			Builders.Add(ClassName, NewBuilder);
			return *NewBuilder.Get();
		}

#if WITH_EDITORONLY_DATA
		// Find or begin building a MetaSound asset.  Optionally, provide
		// whether or not the builder is being accessed during a transaction.
		// If false, enforces MetaSound being built is an asset.  If true, does
		// not enforce (transactions may result in assets being moved and becoming
		// transient wherein the builder can and should be valid to act on the
		// transient UObject in these rare cases).
		template <typename BuilderClass = UMetaSoundBuilderBase>
		BuilderClass& FindOrBeginBuilding(UObject& InMetaSoundObject, bool bIsTransacting = false) const
		{
			check(IsInGameThread());

			if (!bIsTransacting)
			{
				check(InMetaSoundObject.IsAsset());
			}

			TScriptInterface<IMetaSoundDocumentInterface> DocInterface = &InMetaSoundObject;
			check(DocInterface.GetObject());

#if WITH_EDITORONLY_DATA
			WaitUntilVersioningComplete(InMetaSoundObject);
#endif // WITH_EDITORONLY_DATA

			FScopeLock Lock(&BuildersCriticalSection);
			if (UMetaSoundBuilderBase* Builder = FindBuilderObject(&InMetaSoundObject))
			{
				return *CastChecked<BuilderClass>(Builder);
			}

			FNameBuilder BuilderName;
			BuilderName.Append(InMetaSoundObject.GetName());
			BuilderName.Append(TEXT("_Builder"));
			const UClass& BuilderUClass = DocInterface->GetBuilderUClass();
			const FName NewName = MakeUniqueObjectName(nullptr, &BuilderUClass, FName(*BuilderName));

			TObjectPtr<UMetaSoundBuilderBase> NewBuilder = 
				CastChecked<UMetaSoundBuilderBase>(NewObject<UObject>(GetTransientPackage(), &BuilderUClass, NewName, RF_Transactional | RF_Transient));

			NewBuilder->Initialize(DocInterface);
			const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
			const FMetasoundFrontendClassName& ClassName = Document.RootGraph.Metadata.GetClassName();

			checkf(ClassName.IsValid(), TEXT("Document initialization must result in a valid class name being generated"));
			AddBuilderInternal(ClassName, NewBuilder);

			return *CastChecked<BuilderClass>(NewBuilder);
		}
#endif // WITH_EDITORONLY_DATA

		// Frontend::IDocumentBuilderRegistry Implementation
#if WITH_EDITORONLY_DATA
		// Given the provided builder, removes paged data within the associated document for a cooked build.
		// This function removes graphs and input defaults which are not to ever be used by a given cook
		// platform, allowing users to optimize away data and scale the amount of memory required for
		// initial load of input UObjects and graph topology, which can also positively effect runtime
		// performance as well, etc. Returns true if builder modified the document, false if not.
		UE_DEPRECATED(5.7, "Use Metasound::Frontend::StripUnusedPages(...) instead") UE_API virtual bool CookPages(FName PlatformName, FMetaSoundFrontendDocumentBuilder& Builder) const override;
		UE_API virtual FMetaSoundFrontendDocumentBuilder& FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface> MetaSound) override;
#endif // WITH_EDITORONLY_DATA

		UE_API virtual FMetaSoundFrontendDocumentBuilder* FindBuilder(TScriptInterface<IMetaSoundDocumentInterface> MetaSound) const override;
		UE_API virtual FMetaSoundFrontendDocumentBuilder* FindBuilder(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath) const override;

		UE_API virtual TArray<FMetaSoundFrontendDocumentBuilder*> FindBuilders(const FMetasoundFrontendClassName& ClassName) const override;

		UE_API virtual FMetaSoundFrontendDocumentBuilder* FindOutermostBuilder(const UObject& InSubObject) const override;

		UE_API virtual bool FinishBuilding(const FMetasoundFrontendClassName& InClassName, bool bForceUnregisterNodeClass = false) const override;
		UE_API virtual bool FinishBuilding(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath, bool bForceUnregisterNodeClass = false) const override;

		// Returns the builder object associated with the given MetaSound asset if one is registered and active.
		UE_API UMetaSoundBuilderBase* FindBuilderObject(TScriptInterface<const IMetaSoundDocumentInterface> MetaSound) const;

		// Returns the builder object associated with the given ClassName if one is registered and active.
		// Optionally, if provided the AssetPath and there is a conflict (i.e. more than one asset is registered
		// with a given ClassName), will return the one with the provided AssetPath.  Otherwise, will arbitrarily
		// return one.
		UE_API UMetaSoundBuilderBase* FindBuilderObject(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& AssetPath) const;

		// Returns all builder objects registered and active associated with the given ClassName.
		UE_API TArray<UMetaSoundBuilderBase*> FindBuilderObjects(const FMetasoundFrontendClassName& InClassName) const;

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.7, "Use UMetaSoundSettings::GetOnResolveAuditionPageDelegate() instead.") UE_API FOnResolveEditorPage& GetOnResolveAuditionPageDelegate();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.7, "This is no longer supported") UE_API FOnResolvePage& GetOnResolveProjectPageOverrideDelegate();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UE_API bool ReloadBuilder(const FMetasoundFrontendClassName& InClassName) const override;

		// Given the provided document and its respective pages, returns the PageID to be used for runtime IGraph and proxy generation.
		UE_DEPRECATED(5.7, "This is no longer supported.") UE_API virtual FGuid ResolveTargetPageID(const FMetasoundFrontendGraphClass& InGraphClass) const override;
		UE_DEPRECATED(5.7, "This is no longer supported.") UE_API virtual FGuid ResolveTargetPageID(const FMetasoundFrontendClassInput& InClassInput) const override;
		UE_DEPRECATED(5.7, "This is no longer supported.") UE_API virtual FGuid ResolveTargetPageID(const TArray<FMetasoundFrontendClassInputDefault>& Defaults) const override;

		UE_DEPRECATED(5.7, "Retrieve page order from UMetaSoundSettings.") UE_API FGuid ResolveTargetPageID(const TArray<FGuid>& PageIdsToResolve) const;

		UE_DEPRECATED(5.7, "This method exist only to supported deprecated functionality in FrontendNodeControllers and should not be in any new code.")
		virtual TArrayView<const FGuid> GetPageOrder() const override;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.7, "This is only used to support existing deprecated functionality in FMetasoundAssetBase cooking and should not be used in any new code.")
		virtual TArray<FGuid> GetCookedTargetPages(FName InPlatformName) const override;

		UE_DEPRECATED(5.7, "This is only used to support existing deprecated functionality in FMetasoundAssetBase cooking and should not be used in any new code.")
		virtual TArray<FGuid> GetCookedPageOrder(FName InPlatformName) const override;
#endif // WITH_EDITORONLY_DATA

		UE_API void SetEventLogVerbosity(ELogEvent Event, ELogVerbosity::Type Verbosity);

	private:
#if WITH_EDITORONLY_DATA
		UE_API void WaitUntilVersioningComplete(UObject& InMetaSoundObject) const;
#endif // WITH_EDITORONLY_DATA

		UE_API void AddBuilderInternal(const FMetasoundFrontendClassName& InClassName, UMetaSoundBuilderBase* NewBuilder) const;
		UE_API bool CanPostEventLog(ELogEvent Event, ELogVerbosity::Type Verbosity) const;
		UE_API void FinishBuildingInternal(UMetaSoundBuilderBase& Builder, bool bForceUnregisterNodeClass) const;
		UE_API FGuid ResolveTargetPageIDInternal(const UMetaSoundSettings& Settings, const TArray<FGuid>& PageIdsToResolve, const FGuid& TargetPageID, FName PlatformName) const;

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FOnResolveEditorPage OnResolveAuditionPage;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FOnResolvePage OnResolveProjectPage;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Reuseable scratch array of pages to resolve, which is used to
		// optimize/reduce number of allocations required when resolving document.
		mutable TArray<FGuid> TargetPageResolveScratch;
		mutable FCriticalSection TargetPageResolveScratchCritSec;

		TSortedMap<ELogEvent, ELogVerbosity::Type> EventLogVerbosity;
	};
} // namespace Metasound::Engine

#undef UE_API
