// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceManager.h"
#include "MetasoundEnvironment.h"

// Passing around an FAudioDeviceHandle which was held on sound generators
// caused a deadlock on shutdown.  If needed, pass an audio device ID instead 
// and make sure to release the handle when it's no longer needed.
//DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDENGINE_API, FAudioDeviceHandle);
