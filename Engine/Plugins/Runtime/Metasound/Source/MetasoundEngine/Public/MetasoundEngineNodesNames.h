// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Metasound
{
	namespace EngineNodes
	{
		METASOUNDENGINE_API extern const FName Namespace;

		METASOUNDENGINE_API extern const FName MonoVariant;
		METASOUNDENGINE_API extern const FName StereoVariant;
		METASOUNDENGINE_API extern const FName QuadVariant;
		METASOUNDENGINE_API extern const FName FiveDotOneVariant;
		METASOUNDENGINE_API extern const FName SevenDotOneVariant;
	}
}
