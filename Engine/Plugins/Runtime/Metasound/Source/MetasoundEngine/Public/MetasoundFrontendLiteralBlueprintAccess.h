// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendLiteral.h"

#include "MetasoundBuilderSubsystem.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MetasoundFrontendLiteralBlueprintAccess.generated.h"

/**
 * Blueprint support for FMetasoundFrontendLiteral
 */
UCLASS()
class UMetasoundFrontendLiteralBlueprintAccess : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (MetaSound Literal)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static FString Conv_MetaSoundLiteralToString(const FMetasoundFrontendLiteral& Literal);

	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound", meta = (DisplayName = "Get MetaSound Literal Type"))
	static UPARAM(DisplayName = "Type") EMetasoundFrontendLiteralType GetType(const FMetasoundFrontendLiteral& Literal);
	
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (MetaSound Literal)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Audio|MetaSound")
	static bool EqualEqual_MetaSoundLiteral(const FMetasoundFrontendLiteral& LiteralA, const FMetasoundFrontendLiteral& LiteralB);

	// Literal creation

	// Creates a MetaSound Literal using the given boolean value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Boolean"))
	static UPARAM(DisplayName = "Bool Literal") FMetasoundFrontendLiteral CreateBoolMetaSoundLiteral(bool Value);

	// Creates a MetaSound Literal using the given boolean array.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Boolean Array"))
	static UPARAM(DisplayName = "Bool Array Literal") FMetasoundFrontendLiteral CreateBoolArrayMetaSoundLiteral(const TArray<bool>& Value);

	// Creates a MetaSound Literal using the given float value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Float"))
	static UPARAM(DisplayName = "Float Literal") FMetasoundFrontendLiteral CreateFloatMetaSoundLiteral(float Value);

	// Creates a MetaSound Literal using the given float array.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Float Array"))
	static UPARAM(DisplayName = "Float Array Literal") FMetasoundFrontendLiteral CreateFloatArrayMetaSoundLiteral(const TArray<float>& Value);

	// Creates a MetaSound Literal using the given integer value.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Integer"))
	static UPARAM(DisplayName = "Int32 Literal") FMetasoundFrontendLiteral CreateIntMetaSoundLiteral(int32 Value);

	// Creates a MetaSound Literal using the given integer array.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Integer Array"))
	static UPARAM(DisplayName = "Int32 Array Literal") FMetasoundFrontendLiteral CreateIntArrayMetaSoundLiteral(const TArray<int32>& Value);

	// Creates a MetaSound Literal using the given object.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Object"))
	static UPARAM(DisplayName = "Object Literal") FMetasoundFrontendLiteral CreateObjectMetaSoundLiteral(UObject* Value);

	// Creates a MetaSound Literal using the given object array.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Object Array"))
	static UPARAM(DisplayName = "Object Array Literal") FMetasoundFrontendLiteral CreateObjectArrayMetaSoundLiteral(const TArray<UObject*>& Value);

	// Creates a MetaSound Literal using the given string.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from String"))
	static UPARAM(DisplayName = "String Literal") FMetasoundFrontendLiteral CreateStringMetaSoundLiteral(const FString& Value);

	// Creates a MetaSound Literal using the given string array.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from String Array"))
	static UPARAM(DisplayName = "String Array Literal") FMetasoundFrontendLiteral CreateStringArrayMetaSoundLiteral(const TArray<FString>& Value);

	// Creates a MetaSound Literal using the given audio parameter.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Create Literal", meta = (DisplayName = "Create MetaSound Literal from Audio Parameter"))
	static UPARAM(DisplayName = "Param Literal") FMetasoundFrontendLiteral CreateMetaSoundLiteralFromParam(const FAudioParameter& Param);

	// Literal creation (pure)

	// Creates a MetaSound Literal using the given boolean value.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Boolean (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromBoolean(const bool InBoolean);

	// Creates a MetaSound Literal using the given boolean array.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Boolean Array (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromBooleanArray(const TArray<bool>& InBooleanArray);

	// Creates a MetaSound Literal using the given float value.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Float (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromFloat(const float InFloat);

	// Creates a MetaSound Literal using the given float array.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Float Array (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromFloatArray(const TArray<float>& InFloatArray);

	// Creates a MetaSound Literal using the given integer value.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Integer (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromInteger(const int32 InInteger);

	// Creates a MetaSound Literal using the given integer array.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Integer Array (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromIntegerArray(const TArray<int32>& InIntegerArray);

	// Creates a MetaSound Literal using the given object.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Object (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromObject(UObject* InObject);

	// Creates a MetaSound Literal using the given object array.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Object Array (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromObjectArray(const TArray<UObject*>& InObjectArray);

	// Creates a MetaSound Literal using the given string.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from String (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromString(const FString& InString);

	// Creates a MetaSound Literal using the given string array.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from String Array (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromStringArray(const TArray<FString>& InStringArray);

	// Creates a MetaSound Literal using the given audio parameter.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Create Literal", DisplayName = "Create MetaSound Literal from Audio Parameter (Pure)")
	static FMetasoundFrontendLiteral CreateMetaSoundLiteralFromAudioParameter(const FAudioParameter& InAudioParameter);

	// Value accessors

	// Returns the value of the given MetaSound Literal as a bool. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Bool", meta = (ExpandEnumAsExecs = "OutResult"))
	static bool GetBoolValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as a bool array. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Bool Array", meta = (ExpandEnumAsExecs = "OutResult"))
	static TArray<bool> GetBoolArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as a float. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Float", meta = (ExpandEnumAsExecs = "OutResult"))
	static float GetFloatValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as a float array. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Float Array", meta = (ExpandEnumAsExecs = "OutResult"))
	static TArray<float> GetFloatArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as an integer. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Integer", meta = (ExpandEnumAsExecs = "OutResult"))
	static int32 GetIntValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as an integer array. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Integer Array", meta = (ExpandEnumAsExecs = "OutResult"))
	static TArray<int32> GetIntArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as an object. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Object", meta = (ExpandEnumAsExecs = "OutResult"))
	static UObject* GetObjectValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as an object array. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Object Array", meta = (ExpandEnumAsExecs = "OutResult"))
	static TArray<UObject*> GetObjectArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as a string. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as String", meta = (ExpandEnumAsExecs = "OutResult"))
	static FString GetStringValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Returns the value of the given MetaSound Literal as a string array. Has separate execution outputs for success and failure.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as String Array", meta = (ExpandEnumAsExecs = "OutResult"))
	static TArray<FString> GetStringArrayValueFromLiteral(const FMetasoundFrontendLiteral& Literal, EMetaSoundBuilderResult& OutResult);

	// Value accessors (pure)

	// Returns the value of the given MetaSound Literal as a bool. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Bool (Pure)")
	static bool GetMetaSoundLiteralAsBool(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as a bool array. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Bool Array (Pure)")
	static TArray<bool> GetMetaSoundLiteralAsBoolArray(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as a float. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Float (Pure)")
	static float GetMetaSoundLiteralAsFloat(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as a float array. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Float Array (Pure)")
	static TArray<float> GetMetaSoundLiteralAsFloatArray(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as an integer. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Integer (Pure)")
	static int32 GetMetaSoundLiteralAsInteger(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as an integer array. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Integer Array (Pure)")
	static TArray<int32> GetMetaSoundLiteralAsIntegerArray(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as an object. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Object (Pure)")
	static UObject* GetMetaSoundLiteralAsObject(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as an object array. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as Object Array (Pure)")
	static TArray<UObject*> GetMetaSoundLiteralAsObjectArray(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as a string. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as String (Pure)")
	static FString GetMetaSoundLiteralAsString(const FMetasoundFrontendLiteral& InLiteral);

	// Returns the value of the given MetaSound Literal as a string array. Logs a warning if the value fails to be retrieved.
	UFUNCTION(BlueprintPure, Category = "Audio|MetaSound|Get Literal As", DisplayName = "Get MetaSound Literal as String Array (Pure)")
	static TArray<FString> GetMetaSoundLiteralAsStringArray(const FMetasoundFrontendLiteral& InLiteral);
};
