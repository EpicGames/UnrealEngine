// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "MetasoundOutput.h"
#include "MetasoundParameterPack.h"
#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "Containers/SpscQueue.h"
#include "Templates/SharedPointer.h"

#include "MetasoundGeneratorHandle.generated.h"

#define UE_API METASOUNDENGINE_API

struct FMetaSoundOutput;
class UAudioComponent;
class UMetaSoundSource;

namespace Metasound
{
	class FMetasoundGenerator;
	struct FVertexInterfaceChange;
	struct FGeneratorInstanceInfo;
}

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnMetasoundOutputValueChanged, FName, OutputName, const FMetaSoundOutput&, Output);
DECLARE_DELEGATE_TwoParams(FOnMetasoundOutputValueChangedNative, FName, const FMetaSoundOutput&);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMetasoundOutputValueChangedMulticast, FName, Name, const FMetaSoundOutput&, Output);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetasoundOutputValueChangedNativeMulticast, FName, const FMetaSoundOutput&);


namespace Metasound
{
	/**
	 * User-facing interface to a FMetasoundGenerator on a UAudioComponent.
	 */
	class FMetasoundGeneratorHandle : public TSharedFromThis<FMetasoundGeneratorHandle>
	{
		struct FPrivateToken { explicit FPrivateToken() = default; };
		
	public:
		/**
		 * NOTE: This constructor is effectively private, but will work with TSharedPtr/TSharedFromThis.
		 * (see TIntrusiveReferenceController for explanation)
		 * Use FMetasoundGeneratorHandle::Create instead
		 */
		UE_API explicit FMetasoundGeneratorHandle(FPrivateToken, TWeakObjectPtr<UAudioComponent>&& InAudioComponent);

		/**
		 * Create a handle to a MetaSound generator on the given audio component
		 * 
		 * @param InAudioComponent The audio component to attach to
		 * @return The generator handle, if successfully created
		 */
		static UE_API TSharedPtr<FMetasoundGeneratorHandle> Create(TWeakObjectPtr<UAudioComponent>&& InAudioComponent);
		
		UE_API ~FMetasoundGeneratorHandle();

		/**
		 * Find out if this handle is still valid.
		 * 
		 * @return Whether this handle is still valid
		 */
		UE_API bool IsValid() const;

		/**
		 * Get the id for the UAudioComponent associated with this handle.
		 * NOTE: Be sure to check IsValid() before expecting a valid return from this method.
		 *
		 * @returns The audio component's id, or INDEX_NONE if the component is no longer valid.
		 */
		UE_API uint64 GetAudioComponentId() const;

		UE_API TSharedPtr<FMetasoundGenerator> GetGenerator() const;
	
		DECLARE_DELEGATE_OneParam(FOnGeneratorSet, TWeakPtr<FMetasoundGenerator>&&);

		/**
		 * Fires on the game thread when a generator is set or unset on this handle.
		 */
		FOnGeneratorSet OnGeneratorSet;
	
		DECLARE_DELEGATE(FOnGeneratorGraphSet);

		/**
		 * Fires on the game thread when a graph is updated on the generator.
		 */
		FOnGeneratorGraphSet OnGraphUpdated;
	
		DECLARE_DELEGATE(FOnGeneratorIOUpdated);

		/**
		 * Fires on the game thread when the generator's graph inputs or outputs change.
		 */
		UE_DEPRECATED(5.5, "Use OnGeneratorIOUpdatedWithChanges.")
		FOnGeneratorIOUpdated OnGeneratorIOUpdated;

		DECLARE_DELEGATE_OneParam(FOnGeneratorIOUpdatedWithChanges, const TArray<FVertexInterfaceChange>&);

		/**
		 * Fires on the game thread when the generator's graph inputs or outputs change, and includes a list of those changes.
		 */
		FOnGeneratorIOUpdatedWithChanges OnGeneratorIOUpdatedWithChanges;

		/**
		 * Update the current parameter state on this handle and enqueue the changes on the generator.
		 * 
		 * @param ParameterPack The parameter pack to set, which will be merged with the current state.
		 * @return true if parameters were updated, false otherwise
		 */
		UE_API void UpdateParameters(const UMetasoundParameterPack& ParameterPack);
		
		/**
		 * Watch an output value.
		 *
		 * @param OutputName - The user-specified name of the output in the Metasound
		 * @param OnOutputValueChanged - The event to fire when the output's value changes
		 * @param AnalyzerName - (optional) The name of the analyzer to use on the output, defaults to a passthrough
		 * @param AnalyzerOutputName - (optional) The name of the output on the analyzer to watch, defaults to the passthrough output
		 * @returns true if the watch setup succeeded, false otherwise
		 */
		UE_API bool WatchOutput(
			FName OutputName,
			const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API bool WatchOutput(
			FName OutputName,
			const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		/**
		 * Unwatch/Stop watching an output value
		 *
		 * @param OutputName - The user-specified name of the output in the Metasound
		 * @param OnOutputValueChanged - The event or handle previously watched
		 * @param AnalyzerName - (optional) The name of the analyzer to use on the output, defaults to a passthrough
		 * @param AnalyzerOutputName - (optional) The name of the output on the analyzer to watch, defaults to the passthrough output
		 * @returns true if the unwatch setup succeeded, false otherwise
		 */
		UE_API bool UnwatchOutput(
			FName OutputName,
			const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API bool UnwatchOutput(
			FName OutputName,
			const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API bool UnwatchOutput(
			FName OutputName,
			const FDelegateHandle& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API bool TryCreateAnalyzerAddress(
			const FName OutputName,
			const FName AnalyzerName,
			const FName AnalyzerOutputName,
			Frontend::FAnalyzerAddress& OutAnalyzerAddress);

		/**
		 * Update any watched outputs
		 */
		UE_DEPRECATED(5.5, "Directly calling UpdateOutputWatchers() is no longer necessary.")
		void UpdateOutputWatchers() const {}
		
		/**
		 * Map a type name to a passthrough analyzer name to use as a default for UMetasoundOutputSubsystem::WatchOutput()
		 *
		 * @param TypeName - The type name returned from GetMetasoundDataTypeName()
		 * @param AnalyzerName - The name of the analyzer to use
		 * @param OutputName - The name of the output in the analyzer
		 */
		static UE_API void RegisterPassthroughAnalyzerForType(FName TypeName, FName AnalyzerName, FName OutputName);

		/**
		* Enable the profiling of the MetaSound render for this playing instance. You
		* must call this before calling "GetRuntimeCPUCoreUtilization" (below) or you will just 
		* get 0.0 back for core utilization.
		*/
		UE_API void EnableRuntimeRenderTiming(bool Enable);

		/**
		* Get the CPU usage as "fraction of real time" used to render this metasound. 
		* NOTE: The MetasoundSource asset MUST have had its EnableRenderTiming function called
		* before the metasound is started!
		*/
		UE_API double GetCPUCoreUtilization() const;

		/**
		* Returns a text representation of this generator handle, useful for logging
		*/
		UE_API FString ToString() const;

	private:
		UE_API void SetGenerator(TWeakPtr<FMetasoundGenerator>&& InGenerator);

		UE_API void RegisterGeneratorEvents();

		UE_API void UnregisterGeneratorEvents() const;

		UE_API TWeakObjectPtr<UMetaSoundSource> GetMetaSoundSource() const;

		UE_API void SendParametersToGenerator() const;
		
		struct FWatchOutputUnifiedDelegate
		{
			FOnMetasoundOutputValueChanged WatchDelegate;
			FOnMetasoundOutputValueChangedNative NativeWatchDelegate;
			FDelegateHandle NativeWatchDelegateHandle;

			FWatchOutputUnifiedDelegate() {}
			FWatchOutputUnifiedDelegate(const FOnMetasoundOutputValueChanged& Delegate) : WatchDelegate(Delegate) {}
			FWatchOutputUnifiedDelegate(const FOnMetasoundOutputValueChangedNative& Delegate) : NativeWatchDelegate(Delegate), NativeWatchDelegateHandle(Delegate.GetHandle()){}
			FWatchOutputUnifiedDelegate(const FDelegateHandle& DelegateHandle) : NativeWatchDelegateHandle(DelegateHandle) {}
		};

		UE_API bool WatchOutputInternal(
			FName OutputName,
			const FWatchOutputUnifiedDelegate& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API bool UnwatchOutputInternal(
			FName OutputName,
			const FWatchOutputUnifiedDelegate& OnOutputValueChanged,
			FName AnalyzerName = NAME_None,
			FName AnalyzerOutputName = NAME_None);

		UE_API void UpdateOutputWatchersInternal();
		UE_API void FixUpOutputWatchers();

		UE_API void CreateOutputWatcher(
			const Frontend::FAnalyzerAddress& AnalyzerAddress,
			const FWatchOutputUnifiedDelegate& OnOutputValueChanged);

		UE_API void RemoveOutputWatcher(
			const Frontend::FAnalyzerAddress& AnalyzerAddress,
			const FWatchOutputUnifiedDelegate& OnOutputValueChanged);

		TWeakObjectPtr<UAudioComponent> AudioComponent;
		const uint64 AudioComponentId;

		/**
		 * Delegate management
		 */
		UE_API void HandleGeneratorCreated(const FGeneratorInstanceInfo& GeneratorInfo);
		FDelegateHandle GeneratorCreatedDelegateHandle;

		UE_API void HandleGeneratorDestroyed(const FGeneratorInstanceInfo& GeneratorInfo);
		FDelegateHandle GeneratorDestroyedDelegateHandle;

		UE_API void HandleGeneratorGraphSet();
		FDelegateHandle GeneratorGraphSetDelegateHandle;

		UE_API void HandleGeneratorVertexInterfaceChanged(const TArray<FVertexInterfaceChange>& VertexInterfaceData);
		FDelegateHandle GeneratorVertexInterfaceChangedDelegateHandle;

		UE_API void HandleOutputChanged(
			FName AnalyzerName,
			FName OutputName,
			FName AnalyzerOutputName,
			TSharedPtr<IOutputStorage> OutputData);
		FDelegateHandle GeneratorOutputChangedDelegateHandle;
		
		TWeakPtr<FMetasoundGenerator> Generator;
	
		FSharedMetasoundParameterStoragePtr LatestParameterState;

		struct FPassthroughAnalyzerInfo
		{
			FName AnalyzerName;
			FName OutputName;
		};
		static UE_API TMap<FName, FPassthroughAnalyzerInfo> PassthroughAnalyzers;

		struct FOutputWatcherKey
		{
			FName OutputName;
			FName AnalyzerName;
			FName AnalyzerMemberName;

			bool operator==(const FOutputWatcherKey& Other) const
			{
				return OutputName == Other.OutputName
				&& AnalyzerName == Other.AnalyzerName
				&& AnalyzerMemberName == Other.AnalyzerMemberName;
			}

			bool operator!=(const FOutputWatcherKey& Other) const
			{
				return !(*this == Other);
			}
		};

		friend uint32 GetTypeHash(FOutputWatcherKey Key)
		{
			return HashCombineFast(
				HashCombineFast(GetTypeHash(Key.OutputName), GetTypeHash(Key.AnalyzerName)),
				GetTypeHash(Key.AnalyzerMemberName));
		}

		struct FWatchOutputUnifiedMulticastDelegate
		{
			FOnMetasoundOutputValueChangedMulticast WatchDelegates;
			FOnMetasoundOutputValueChangedNativeMulticast NativeWatchDelegates;

			void Add(const FWatchOutputUnifiedDelegate& Delegate)
			{
				if (Delegate.WatchDelegate.IsBound())
				{
					WatchDelegates.AddUnique(Delegate.WatchDelegate);
				}

				if (Delegate.NativeWatchDelegate.IsBound())
				{
					NativeWatchDelegates.Add(Delegate.NativeWatchDelegate);
				}
			}

			void Remove(const FWatchOutputUnifiedDelegate& Delegate)
			{
				if (Delegate.WatchDelegate.IsBound())
				{
					WatchDelegates.Remove(Delegate.WatchDelegate);
				}

				if (Delegate.NativeWatchDelegateHandle.IsValid())
				{
					NativeWatchDelegates.Remove(Delegate.NativeWatchDelegateHandle);
				}
			}

			bool IsBound() const
			{
				if (WatchDelegates.IsBound())
				{
					return true;
				}

				if (NativeWatchDelegates.IsBound())
				{
					return true;
				}

				return false;
			}

			void Broadcast(FName OutputName, const FMetaSoundOutput& Output) const
			{
				WatchDelegates.Broadcast(OutputName, Output);
				NativeWatchDelegates.Broadcast(OutputName, Output);
			}
		};

		/**
		 * Info about an output being watched by one or more listeners
		 */
		struct FOutputWatcher
		{
			Frontend::FAnalyzerAddress AnalyzerAddress;
			FWatchOutputUnifiedMulticastDelegate OnOutputValueChanged;

			FOutputWatcher(
				const Frontend::FAnalyzerAddress& InAnalyzerAddress,
				const FWatchOutputUnifiedDelegate& InOnOutputValueChanged)
					: AnalyzerAddress(InAnalyzerAddress)
			{
				OnOutputValueChanged.Add(InOnOutputValueChanged);
			}
		};
		
		TMap<FOutputWatcherKey, FOutputWatcher> OutputWatchers;

		struct FOutputPayload
		{
			FName AnalyzerName;
			FName OutputName;
			FMetaSoundOutput OutputValue;

			FOutputPayload(
				const FName InAnalyzerName,
				const FName InOutputName,
				const FName AnalyzerOutputName,
				TSharedPtr<IOutputStorage> OutputData)
					: AnalyzerName(InAnalyzerName)
					, OutputName(InOutputName)
					, OutputValue(AnalyzerOutputName, OutputData)
			{}
		};
	
		uint64 CurrentGeneratorInstanceID;

		TSpscQueue<FOutputPayload> ChangedOutputs;

		// Keep ChangedOutputs from growing infinitely
		static constexpr int32 ChangedOutputsQueueMax = 1024;
		std::atomic<int32> ChangedOutputsQueueCount{ 0 };
		std::atomic<bool> ChangedOutputsQueueShouldLogIfFull{ true };

		std::atomic_flag OutputWatcherUpdateScheduled = ATOMIC_FLAG_INIT;

		bool bRuntimeRenderTimingShouldBeEnabled{ false };
	};
}

/**
 * Blueprint-facing interface to a FMetasoundGenerator on a UAudioComponent.
 */
UCLASS(MinimalAPI, BlueprintType, Category = "MetaSound", meta = (DisplayName = "MetaSound Generator Handle"))
class UMetasoundGeneratorHandle : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="MetaSound")
	static UE_API UMetasoundGeneratorHandle* CreateMetaSoundGeneratorHandle(UAudioComponent* OnComponent);

	UE_API virtual void BeginDestroy() override;

	UE_API bool IsValid() const;

	/**
	 * Get the id for the UAudioComponent associated with this handle.
	 * NOTE: Be sure to check IsValid() before expecting a valid return from this method.
	 *
	 * @returns The audio component's id, or INDEX_NONE if the component is no longer valid.
	 */
	UE_API uint64 GetAudioComponentId() const;

	/**
	 * Makes a copy of the supplied parameter pack and passes it to the MetaSoundGenerator
	 * for asynchronous processing. IT ALSO caches this copy so that if the AudioComponent
	 * is virtualized the parameter pack will be sent again when/if the AudioComponent is 
	 * "unvirtualized".
	 */
	UFUNCTION(BlueprintCallable, Category="MetaSoundParameterPack")
	UE_API bool ApplyParameterPack(UMetasoundParameterPack* Pack);

	/**
	 * Get a shared pointer to the generator, if available.
	 * NOTE: while the shared pointer is safe to get from threads other than the audio render thread,
	 * not all methods on the generator are safe to use from other threads. Ensure you know what you're doing.
	 * 
	 * @return A shared pointer to the generator, if available.
	 */
	UE_API TSharedPtr<Metasound::FMetasoundGenerator> GetGenerator() const;

	// UMetasoundGeneratorHandle shields its "clients" from "cross thread" issues
	// related to callbacks coming in the audio control or rendering threads that 
	// game thread clients (e.g. blueprints) want to know about. That is why these 
	// next delegate definitions do not need to be the "TS" variants. Assignments 
	// to members of this type, and the broadcasts there to will all happen on the
	// game thread. EVEN IF the instigator of those callbacks is on the audio
	// render thread. 
	DECLARE_MULTICAST_DELEGATE(FOnAttached);
	FOnAttached OnGeneratorHandleAttached;

	DECLARE_MULTICAST_DELEGATE(FOnDetached);
	FOnDetached OnGeneratorHandleDetached;
	
	DECLARE_MULTICAST_DELEGATE(FOnSetGraph);
	UE_API FDelegateHandle AddGraphSetCallback(UMetasoundGeneratorHandle::FOnSetGraph::FDelegate&& Delegate);
	UE_API bool RemoveGraphSetCallback(const FDelegateHandle& Handle);

	DECLARE_MULTICAST_DELEGATE(FOnIOUpdated)
	UE_DEPRECATED(5.5, "Use OnIOUpdatedWithChanges.")
	FOnIOUpdated OnIOUpdated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnIOUpdatedWithChanges, const TArray<Metasound::FVertexInterfaceChange>&)
	FOnIOUpdatedWithChanges OnIOUpdatedWithChanges;

	UE_API bool TryCreateAnalyzerAddress(
		const FName OutputName,
		const FName AnalyzerName,
		const FName AnalyzerOutputName,
		Metasound::Frontend::FAnalyzerAddress& OutAnalyzerAddress);

	/**
	 * Watch an output value.
	 *
	 * @param OutputName - The user-specified name of the output in the Metasound
	 * @param OnOutputValueChanged - The event to fire when the output's value changes
	 * @param AnalyzerName - (optional) The name of the analyzer to use on the output, defaults to a passthrough
	 * @param AnalyzerOutputName - (optional) The name of the output on the analyzer to watch, defaults to the passthrough output
	 * @returns true if the watch setup succeeded, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput", meta=(AdvancedDisplay = "2"))
	UE_API bool WatchOutput(
		FName OutputName,
		const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

	UE_API bool WatchOutput(
		FName OutputName,
		const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

	/**
	 * Map a type name to a passthrough analyzer name to use as a default for UMetasoundOutputSubsystem::WatchOutput()
	 *
	 * @param TypeName - The type name returned from GetMetasoundDataTypeName()
	 * @param AnalyzerName - The name of the analyzer to use
	 * @param OutputName - The name of the output in the analyzer
	 */
	static UE_API void RegisterPassthroughAnalyzerForType(FName TypeName, FName AnalyzerName, FName OutputName);

	/**
	 * Update any watched outputs
	 */
	UE_DEPRECATED(5.5, "Directly calling UpdateWatchers() is no longer necessary.")
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	UE_API void UpdateWatchers() const;

	/**
	* Enable the profiling of the MetaSound render for this playing instance. You
	* must call this before calling "GetRuntimeCPUCoreUtilization" (below) or you will just 
	* get 0.0 back for core utilization.
	*/
	UFUNCTION(BlueprintCallable, Category="MetaSoundPerf")
	UE_API void EnableRuntimeRenderTiming(bool Enable) const;

	/**
	* Get the CPU usage as "fraction of real time" used to render this metasound. 
	* NOTE: The MetasoundSource asset MUST have had its EnableRenderTiming function called
	* before the metasound is started!
	*/
	UFUNCTION(BlueprintCallable, Category="MetaSoundPerf")
	UE_API double GetCPUCoreUtilization() const;
	
private:
	bool InitGeneratorHandle(TWeakObjectPtr<UAudioComponent>&& AudioComponent);
	
	TSharedPtr<Metasound::FMetasoundGeneratorHandle> GeneratorHandle;
	
	DECLARE_MULTICAST_DELEGATE(FOnSetGraphMulticast);
	FOnSetGraphMulticast OnGeneratorsGraphChanged;
};

#undef UE_API
