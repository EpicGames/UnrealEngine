// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "MetasoundOperatorInterface.h"
#include "Misc/Guid.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"
#include "Sound/SoundGenerator.h"
#include "Subsystems/AudioEngineSubsystem.h"

#include "MetasoundSource.h"

#include "MetasoundOperatorCacheSubsystem.generated.h"

#define UE_API METASOUNDENGINE_API


/**
* 	UMetaSoundCacheSubsystem
*/
UCLASS(MinimalAPI)
class UMetaSoundCacheSubsystem : public UAudioEngineSubsystem
{
	GENERATED_BODY()

public: 
	//~ Begin USubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Update() override;
	//~ End USubsystem interface
	 
	/* Builds the requested number of MetaSound operators (asynchronously) and puts them in the pool for playback.
	(If these operators are not yet available when the MetaSound attempts to play, one will be created Independent of this request.) */
	UFUNCTION(BlueprintCallable, Category = "MetaSound", meta = (DisplayName = "Precache MetaSound"))
	UE_API void PrecacheMetaSound(UPARAM(DisplayName = "MetaSound Source") UMetaSoundSource* InMetaSound, UPARAM(DisplayName = "Num Instances") int32 InNumInstances = 1);

	/* same as PrecacheMetaSound except cached operator that already exists in the cache will be moved to the top instead of building,
	any operators that we couldn't move to the top, will be built.
	(i.e. if 2 operators are already cached and Num Instances is 4, it will construct 2 and move the existing 2 to the top of the cache) */
	UFUNCTION(BlueprintCallable, Category = "MetaSound", meta = (DisplayName = "Touch or Precache MetaSound"))
	UE_API void TouchOrPrecacheMetaSound(UPARAM(DisplayName = "MetaSound Source") UMetaSoundSource* InMetaSound, UPARAM(DisplayName = "Num Instances") int32 InNumInstances = 1);

	/* Clear the operator pool of any operators associated with the given MetaSound */
	UFUNCTION(BlueprintCallable, Category = "MetaSound", meta = (DisplayName = "RemoveCached Operators for MetaSound"))
	UE_API void RemoveCachedOperatorsForMetaSound(UPARAM(DisplayName = "MetaSound Source") UMetaSoundSource* InMetaSound);

private:
	void PrecacheMetaSoundInternal(UMetaSoundSource* InMetaSound, int32 InNumInstances, bool bTouchExisting);

	FSoundGeneratorInitParams BuildParams;
}; //UMetaSoundCacheSubsystem

#undef UE_API
