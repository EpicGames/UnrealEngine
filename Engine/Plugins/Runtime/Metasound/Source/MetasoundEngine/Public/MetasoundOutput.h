// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundOutputStorage.h"
#include "SoundGeneratorOutput.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "MetasoundOutput.generated.h"

#define UE_API METASOUNDENGINE_API

/**
 * Exposes the value of an output on a FMetasoundGenerator.
 */
USTRUCT(BlueprintType)
struct FMetaSoundOutput : public FSoundGeneratorOutput
{
	GENERATED_BODY()

	FMetaSoundOutput() = default;

	UE_API FMetaSoundOutput(const FMetaSoundOutput& Other);

	UE_API FMetaSoundOutput& operator=(const FMetaSoundOutput& Other);

	UE_API FMetaSoundOutput(FMetaSoundOutput&& Other) noexcept;

	UE_API FMetaSoundOutput& operator=(FMetaSoundOutput&& Other) noexcept;

	UE_API FMetaSoundOutput(FName InName, const TSharedPtr<Metasound::IOutputStorage>& InData);

	/**
	 * Has this output been initialized?
	 *
	 * @returns true if the output has been initialized, false otherwise
	 */
	UE_API bool IsValid() const;

	/**
	 * Get the type name of the output
	 *
	 * @returns The type name, or none if invalid
	 */
	UE_API FName GetDataTypeName() const;
	
	/**
	 * Initialize the output with some data.
	 *
	 * @tparam DataType - The type to use for initialization
	 * @param InitialValue - The initial value to use
	 */
	template<typename DataType>
	void Init(const DataType& InitialValue)
	{
		using FOutputData = Metasound::TOutputStorage<DataType>;
		Data = MakeUnique<FOutputData>(InitialValue);
	}

	/**
	 * Check if this output is of the given type
	 *
	 * @tparam DataType - The data type to check
	 * @returns true if it's the given type, false otherwise
	 */
	template<typename DataType>
	bool IsType() const
	{
		static const FName TypeName = Metasound::GetMetasoundDataTypeName<DataType>();
		return IsValid() && TypeName == Data->GetDataTypeName();
	}
	
	/**
	 * Get the value, for copyable registered Metasound data types
	 *
	 * @tparam DataType - The expected data type of the output
	 * @param Value - The value to use
	 * @returns true if the value was retrieved, false otherwise
	 */
	template<typename DataType>
	bool Get(DataType& Value) const
	{
		if (!IsType<DataType>())
		{
			return false;
		}

		using FOutputData = Metasound::TOutputStorage<DataType>;
		Value = static_cast<FOutputData*>(Data.Get())->Get();
		return true;
	}
	
	/**
	 * Set the value, for copyable registered Metasound data types
	 *
	 * @tparam DataType - The expected data type of the output
	 * @param Value - The value to use
	 * @returns true if the value was set, false otherwise
	 */
	template<typename DataType>
	bool Set(const DataType& Value)
	{
		if (!IsType<DataType>())
		{
			return false;
		}

		using FOutputData = Metasound::TOutputStorage<DataType>;
		static_cast<FOutputData*>(Data.Get())->Set(Value);
		return true;
	}

	/**
	 * Set the value, for moveable registered Metasound data types
	 *
	 * @tparam DataType - The expected data type of the output
	 * @param Value - The value to use
	 * @returns true if the value was set, false otherwise
	 */
	template<typename DataType>
	bool Set(DataType&& Value)
	{
		if (!IsType<DataType>())
		{
			return false;
		}

		using FOutputData = Metasound::TOutputStorage<DataType>;
		static_cast<FOutputData*>(Data.Get())->Set(MoveTemp(Value));
		return true;
	}
	
private:
	TUniquePtr<Metasound::IOutputStorage> Data;
};

/**
 * Blueprint support for core types. If you want to support more core types, add them here.
 * If you want to support types introduced in other plugins, create a blueprint library in that plugin.
 */
UCLASS()
class UMetasoundOutputBlueprintAccess final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsFloat(const FMetaSoundOutput& Output);
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static float GetFloat(const FMetaSoundOutput& Output, bool& Success);
	
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsInt32(const FMetaSoundOutput& Output);
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static int32 GetInt32(const FMetaSoundOutput& Output, bool& Success);

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsBool(const FMetaSoundOutput& Output);
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool GetBool(const FMetaSoundOutput& Output, bool& Success);

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsString(const FMetaSoundOutput& Output);
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static FString GetString(const FMetaSoundOutput& Output, bool& Success);

	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static bool IsTime(const FMetaSoundOutput& Output);
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput")
	static double GetTimeSeconds(const FMetaSoundOutput& Output, bool& Success);
};

#undef UE_API
