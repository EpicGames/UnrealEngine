// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundGeneratorHandle.h"
#include "Subsystems/WorldSubsystem.h"

#include "MetasoundOutputSubsystem.generated.h"

#define UE_API METASOUNDENGINE_API

class UAudioComponent;

/**
 * Provides access to a playing Metasound generator's outputs
 */
UCLASS(MinimalAPI)
class UMetaSoundOutputSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	virtual ~UMetaSoundOutputSubsystem() override = default;
	
	/**
	 * Watch an output on a Metasound playing on a given audio component.
	 *
	 * @param AudioComponent - The audio component
	 * @param OutputName - The user-specified name of the output in the Metasound
	 * @param OnOutputValueChanged - The event to fire when the output's value changes
	 * @param AnalyzerName - (optional) The name of the analyzer to use on the output, defaults to a passthrough
	 * @param AnalyzerOutputName - (optional) The name of the output on the analyzer to watch, defaults to the passthrough output
	 * @returns true if the watch setup succeeded, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="MetaSoundOutput", meta=(AdvancedDisplay = "3"))
	UE_API bool WatchOutput(
		UAudioComponent* AudioComponent,
		FName OutputName,
		const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

	UE_API bool WatchOutput(
		UAudioComponent* AudioComponent,
		FName OutputName,
		const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundOutput", meta = (AdvancedDisplay = "3"))
	UE_API bool UnwatchOutput(
		UAudioComponent* AudioComponent,
		FName OutputName,
		const FOnMetasoundOutputValueChanged& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

	UE_API bool UnwatchOutput(
		UAudioComponent* AudioComponent,
		FName OutputName,
		const FOnMetasoundOutputValueChangedNative& OnOutputValueChanged,
		FName AnalyzerName = NAME_None,
		FName AnalyzerOutputName = NAME_None);

private:
	UE_API TSharedPtr<Metasound::FMetasoundGeneratorHandle> GetOrCreateGeneratorHandle(UAudioComponent* AudioComponent);
	UE_API void CleanUpInvalidGeneratorHandles();

	TArray<TSharedPtr<Metasound::FMetasoundGeneratorHandle>> TrackedGenerators;
};

#undef UE_API
