// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Engine/DeveloperSettings.h"
#include "MetasoundFrontendDocument.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"

#include "MetasoundSettings.generated.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
struct FMetasoundFrontendClassName;
struct FPropertyChangedChainEvent;

namespace Metasound::Engine
{
#if WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE(FOnSettingsDefaultConformed);
	DECLARE_MULTICAST_DELEGATE(FOnPageSettingsUpdated);

#if WITH_EDITOR
	struct FTargetPageOverride
	{
		FName PlatformName;
		TOptional<FGuid> PageID;
	};


	using FPageResolutionEditorResults UE_DEPRECATED(5.7, "Use FTargetPageOverride instead.") = FTargetPageOverride;

	UE_DEPRECATED(5.7, "Use FOverrideTargetPageSettings instead.")
	DECLARE_DELEGATE_RetVal_OneParam(FTargetPageOverride, FOnResolveEditorPage, const TArray<FGuid>& /* InPageIDs */);

#endif // WITH_EDITOR
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.7, "This is no longer supported.")
	DECLARE_DELEGATE_RetVal_OneParam(FGuid, FOnResolvePage, const TArray<FGuid>& /* InPageIDs */);
} // namespace Metasound::Engine

UENUM()
enum class EMetaSoundMessageLevel : uint8
{
	Error,
	Warning,
	Info
};

USTRUCT()
struct FDefaultMetaSoundAssetAutoUpdateSettings
{
	GENERATED_BODY()

	/** MetaSound to prevent from AutoUpdate. */
	UPROPERTY(EditAnywhere, Category = "AutoUpdate", meta = (AllowedClasses = "/Script/MetasoundEngine.MetaSound, /Script/MetasoundEngine.MetaSoundSource"))
	FSoftObjectPath MetaSound;
};

UCLASS(MinimalAPI, Hidden)
class UMetaSoundQualityHelper : public UObject
{
	GENERATED_BODY()

public:
	/**
	* Returns a list of quality settings to present to a combobox
	* */
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UMetaSoundSettings::GetQualityNames instead"))
	static TArray<FName> GetQualityNames() { return { }; };
};

USTRUCT()
struct FMetaSoundPageSettings
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid UniqueId = Metasound::Frontend::DefaultPageID;

	/** Name of this page's setting to be displayed in editors and used for identification from Blueprint/native API. */
	UPROPERTY(EditAnywhere, Category = "Pages", meta = (EditCondition = "!bIsDefaultPage"))
	FName Name = Metasound::Frontend::DefaultPageName;

private:
#if WITH_EDITORONLY_DATA
	// When true, page asset data (i.e. graphs and input defaults) can be targeted
	// for the most applicable platform/platform group. If associated asset data is
	// defined, will always be cooked. If false, asset page data is only cooked if it is
	// resolved to from a higher-indexed page setting and is not set to explicitly
	// "ExcludeFromCook".
	UPROPERTY(EditAnywhere, Category = "Pages", meta = (DisplayName = "Targetable"))
	FPerPlatformBool CanTarget = true;

	// Just used to inform edit condition to enable/disable exclude from cook. Maintained by ConformPageSettings on object load/mutation.
	// EditCondition meta mark-up is hack to avoid boolean being default added to name field
	UPROPERTY(EditAnywhere, Category = "Pages", meta = (EditCondition = false, EditConditionHides))
	bool bIsDefaultPage = true;

	// When true, exclude page data when cooking from the assigned platform(s)/platform group(s).
	// If false, page data may or may not be included in cook depending on whether or not the given
	// page data is required in order to ensure a value is always resolved for the cook platform target(s).
	// (Ignored if 'Targetable' is true for most applicable platform/platform group).
	UPROPERTY(EditAnywhere, Category = "Pages", meta = (EditCondition = "!bIsDefaultPage"))
	FPerPlatformBool ExcludeFromCook = false;
#endif //WITH_EDITORONLY_DATA

public:
#if WITH_EDITOR
	// Returns whether or not page should be excluded from being cooked on the given platform/platform group.
	UE_API bool GetExcludeFromCook(FName PlatformName) const;

	// Returns array of platforms/platform groups page is valid target on.
	UE_API TArray<FName> GetTargetPlatforms() const;

	// Returns whether platform/platform group provided can target the provided page.
	UE_API bool PlatformCanTargetPage(FName PlatformName) const;
#endif //WITH_EDITOR

	friend class UMetaSoundSettings;
};

USTRUCT()
struct FMetaSoundQualitySettings
{
	GENERATED_BODY()
	
	/** A hidden GUID that will be generated once when adding a new entry. This prevents orphaning of renamed entries. **/
	UPROPERTY()
	FGuid UniqueId;

	/** Name of this quality setting. This will appear in the quality dropdown list.
		The names should be unique but are not guaranteed to be (use guid for unique match) **/
	UPROPERTY(EditAnywhere, Category = "Quality")
	FName Name;

	/** Sample Rate (in Hz). NOTE: A Zero value will have no effect and use the Device Rate. **/
	UPROPERTY(EditAnywhere, config, Category = "Quality", meta = (ClampMin = "0", ClampMax="96000"))
	FPerPlatformInt SampleRate = 0;

	/** Block Rate (in Hz). NOTE: A Zero value will have no effect and use the Default (100)  **/
	UPROPERTY(EditAnywhere, config, Category = "Quality", meta = (ClampMin = "0", ClampMax="1000"))
	FPerPlatformFloat BlockRate = 0.f;
};


UCLASS(MinimalAPI, config = MetaSound, defaultconfig, meta = (DisplayName = "MetaSounds"))
class UMetaSoundSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (Deprecated, DeprecationMessage = "5.7 - For performance reasons, AutoUpdate is now always only called during cook."))
	bool bAutoUpdateEnabled = true;

	UPROPERTY(meta = (Deprecated, DeprecationMessage = "5.7 - For performance reasons, AutoUpdate is now always only called during cook."))
	TArray<FMetasoundFrontendClassName> AutoUpdateDenylist;

	UPROPERTY(meta = (Deprecated, DeprecationMessage = "For performance reasons, AutoUpdate is now always only called during cook."))
	TArray<FDefaultMetaSoundAssetAutoUpdateSettings> AutoUpdateAssetDenylist;
#endif // WITH_EDITORONLY_DATA

	/** If true, warnings will be logged if updating a node results in existing connections being discarded. */
	UPROPERTY(EditAnywhere, config, Category = AutoUpdate, meta = (DisplayName = "Log Warning on Dropped Connection", EditCondition = "bAutoUpdateEnabled"))
	bool bAutoUpdateLogWarningOnDroppedConnection = true;

	/** Directories to scan & automatically register MetaSound post initial asset scan on engine start-up.
	  * May speed up subsequent calls to playback MetaSounds post asset scan but increases application load time.
	  * See 'MetaSoundAssetSubsystem::RegisterAssetClassesInDirectories' to dynamically register or
	  * 'MetaSoundAssetSubsystem::UnregisterAssetClassesInDirectories' to unregister asset classes.
	  */
	UPROPERTY(EditAnywhere, config, Category = Registration, meta = (RelativePath, LongPackageName))
	TArray<FDirectoryPath> DirectoriesToRegister;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "DenyList no longer supported (auto-update automatically executed during cook and not supported at runtime")
		UPROPERTY(Transient)
	int32 DenyListCacheChangeID = 0;
#endif // WITH_EDITORONLY_DATA

private:
#if WITH_EDITORONLY_DATA
	Metasound::Engine::FOnSettingsDefaultConformed OnDefaultRenamed;
	Metasound::Engine::FOnPageSettingsUpdated OnPageSettingsUpdated;
#endif //WITH_EDITORONLY_DATA

	/** Page Name to target when attempting to execute MetaSound. If target page is not implemented (or cooked in a runtime build)
	  * for the active platform, uses order of cooked pages (see 'Page Settings' for order) falling back to lower index-ordered page
	  * implemented in MetaSound asset. If no fallback is found, uses default implementation.
	  */
	UPROPERTY(EditAnywhere, config, Category = "Pages (Experimental)", meta = (GetOptions = "MetasoundEngine.MetaSoundSettings.GetPageNames"))
	FName TargetPageName = Metasound::Frontend::DefaultPageName;

	/** Default page settings to be used in editor and if no other page settings are targeted or defined. */
	UPROPERTY(EditAnywhere, config, Category = "Pages (Experimental)")
	FMetaSoundPageSettings DefaultPageSettings;

	/** Array of possible page settings that can be added to a MetaSound object. Order
	  * defines default fallback logic whereby a higher index-ordered page
	  * implemented in a MetaSound asset is higher priority (see 'Target Page').
	  */
	UPROPERTY(EditAnywhere, config, Category = "Pages (Experimental)")
	TArray<FMetaSoundPageSettings> PageSettings;

	/** Array of possible quality settings for Metasounds to chose from */
	UPROPERTY(EditAnywhere, config, Category = Quality)
	TArray<FMetaSoundQualitySettings> QualitySettings;

	/** Target page override to use set from code/script (supercedes serialized 'TargetPageName' field). */
	TOptional<FName> TargetPageNameOverride;

#if WITH_EDITORONLY_DATA
	mutable FCriticalSection CookPlatformTargetCritSec;
	mutable TArray<FGuid> CookPlatformTargetPageIDs;
	mutable TArray<FGuid> CookPlatformPageOrder;
	mutable FName CookPlatformTargetPage;
#endif // WITH_EDITORONLY_DATA

	virtual FName GetCategoryName() const override;

	// Generates the page order for a given platform and desired target page ID
	TArray<FGuid> GeneratePageOrderInternal(const FName& InPlatformName, const FGuid& InTargetPageID) const;

	std::atomic<bool> bIsPageOrderCached = false;
	TArray<FGuid> CachedPageOrder;
	// Updates the cached page order with the latest settings.
	void UpdateCachedPageOrder();

public:
	// Helper function to convert page name to id. Returns nullptr if page name not found.
	static UE_API const FGuid* GetPageID(FName PageName);

	// Helper function to convert page ID into page name. If page ID is not found, returns default page.
	static UE_API FName GetPageName(FGuid PageID);
	   
	// Returns a cached global version of the PageOrder from the UMetaSoundSettings Default Object.
	static UE_API TArrayView<const FGuid> GetPageOrder();

	// Generates the page order for a current configured platform and target ID
	TArray<FGuid> GeneratePageOrder() const;

	// Returns the page settings with the provided name. If there are multiple settings
	// with the same name, selection within the duplicates is undefined.
	UE_API const FMetaSoundPageSettings* FindPageSettings(FName Name) const;

	// Returns the page settings with the unique ID given.
	UE_API const FMetaSoundPageSettings* FindPageSettings(const FGuid& InPageID) const;

	// Returns the quality settings with the provided name. If there are multiple settings
	// with the same name, selection within the duplicates is undefined.
	UE_API const FMetaSoundQualitySettings* FindQualitySettings(FName Name) const;

	// Returns the quality settings with the unique ID given.
	UE_API const FMetaSoundQualitySettings* FindQualitySettings(const FGuid& InQualityID) const;

	UE_API const FMetaSoundPageSettings& GetDefaultPageSettings() const;

#if WITH_EDITORONLY_DATA
	// Returns PageIDs to be cooked to the given platform/platform group.
	UE_API TArray<FGuid> GetCookedTargetPageIDs(FName PlatformName) const;
	UE_API TArray<FGuid> GetCookedPageOrder(FName PlatformName) const;

	// Iterates PageIDs cooked to the given platform/platform group.
	UE_API void IterateCookedTargetPageIDs(FName PlatformName, TFunctionRef<void(const FGuid&)> Iter) const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Returns superset of page platforms (and groups) that define page target(s).
	UE_API TArray<FName> GetAllPlatformNamesImplementingTargets() const;
#endif // WITH_EDITOR

	// Returns the project-specific page settings (does not include the required default settings).
	const TArray<FMetaSoundPageSettings>& GetProjectPageSettings() const { return PageSettings; }

	// Returns the currently targeted page settings.
	UE_API const FMetaSoundPageSettings& GetTargetPageSettings() const;

	const TArray<FMetaSoundQualitySettings>& GetQualitySettings() const { return QualitySettings; }

#if WITH_EDITOR
	UE_API void OverrideTargetPageSettings(TOptional<Metasound::Engine::FTargetPageOverride> InOverride);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	UE_API Metasound::Engine::FOnSettingsDefaultConformed& GetOnDefaultRenamedDelegate();
	UE_API Metasound::Engine::FOnPageSettingsUpdated& GetOnPageSettingsUpdatedDelegate();

	static UE_API FName GetPageSettingPropertyName();
	static UE_API FName GetQualitySettingPropertyName();
#endif // WITH_EDITORONLY_DATA

	// Iterates possible page settings in order (including the default page settings, which is always last). If 
	// optionally set to reverse, iterates in reverse.
	UE_API void IteratePageSettings(TFunctionRef<void(const FMetaSoundPageSettings&)> Iter, bool bReverse = false) const;

	// Sets the target page to the given name. Returns true if associated page settings were found
	// and target set, false if not found and not set.
	UE_API bool SetTargetPage(FName PageName);

#if WITH_EDITOR
public:
	/* Returns an array of page names. Can be used to present to a combobox. Ex:
	 * UPROPERTY(... meta=(GetOptions="MetasoundEngine.MetaSoundSettings.GetPageNames"))
	 * FName Page;
	*/
	UFUNCTION()
	static UE_API TArray<FName> GetPageNames();

	/* Returns an array of quality setting names. Can be used to present to a combobox. Ex:
	 * UPROPERTY(... meta=(GetOptions="MetasoundEngine.MetaSoundSettings.GetQualityNames"))
	 * FName QualitySetting;
	*/
	UFUNCTION()
	static UE_API TArray<FName> GetQualityNames();
#endif // WITH_EDITOR

private:
#if WITH_EDITORONLY_DATA
	void CacheCookedPageIDs(const FName InPlatformName) const;
	
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API void ConformPageSettings(bool bNotifyDefaultRenamed);

	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	const FMetaSoundPageSettings* GetTargetPageSettingsFromOverride(const FName InTargetPageName) const;
	TArray<FGuid> GeneratePageOrderFromOverride() const;
	
	TOptional<Metasound::Engine::FTargetPageOverride> PageOverrideData;
#endif // WITH_EDITOR

	UE_API virtual void PostInitProperties() override;

#if !NO_LOGGING
	mutable bool bWarnAccessBeforeInit = true;
#endif // !NO_LOGGING
};

#undef UE_API
