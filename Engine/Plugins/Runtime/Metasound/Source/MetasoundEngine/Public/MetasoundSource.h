// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraph/EdGraph.h"
#include "HAL/CriticalSection.h"
#include "IAudioParameterTransmitter.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundChannelAgnosticAutoFormat.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"
#include "Sound/SoundWaveProcedural.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/MetaData.h"

#include <atomic>

#include "MetasoundSource.generated.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
#if WITH_EDITOR
class FDataValidationContext;
#endif // WITH_EDITOR
class UMetaSoundSettings;

struct FMetaSoundFrontendDocumentBuilder;
struct FMetaSoundQualitySettings;

namespace Audio
{
	using DeviceID = uint32;
} // namespace Audio

namespace Metasound
{
	class FMetasoundGenerator;

	namespace DynamicGraph
	{
		class FDynamicOperatorTransactor;
	} // namespace DynamicGraph

	namespace Engine
	{
		struct FAssetHelper;
	} // namespace Engine;

	namespace Frontend
	{
		class IDataTypeRegistry;
	} // namespace Frontend

	namespace SourcePrivate
	{
		class FParameterRouter;
		using FCookedQualitySettings = FMetaSoundQualitySettings;
	} // namespace SourcePrivate

	struct FGeneratorInstanceInfo
	{
		FGeneratorInstanceInfo() = default;
		FGeneratorInstanceInfo(uint64 InAudioComponentID, uint64 InInstanceID, TWeakPtr<FMetasoundGenerator> InGenerator);
		uint64 AudioComponentID;
		uint64 InstanceID;
		TWeakPtr<FMetasoundGenerator> Generator;
	};
} // namespace Metasound


UE_DEPRECATED(5.6, "Use FGeneratorInstanceInfoDelegate instead.")
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnGeneratorInstanceCreated, uint64, TSharedPtr<Metasound::FMetasoundGenerator>);
UE_DEPRECATED(5.6, "Use FGeneratorInstanceInfoDelegate instead.")
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnGeneratorInstanceDestroyed, uint64, TSharedPtr<Metasound::FMetasoundGenerator>);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FGeneratorInstanceInfoDelegate, const Metasound::FGeneratorInstanceInfo&);

/**
 * This Metasound type can be played as an audio source.
 */
UCLASS(MinimalAPI, hidecategories = object, BlueprintType, meta = (DisplayName = "MetaSound Source"))
class UMetaSoundSource : public USoundWaveProcedural, public FMetasoundAssetBase, public IMetaSoundDocumentInterface
{
	GENERATED_BODY()

	friend struct Metasound::Engine::FAssetHelper;
	friend class UMetaSoundSourceBuilder;

	//Forward declare
	class FAudioParameterCollector;

public:
	/** Result of ResolveSourceFormatAndNumChannels. */
	struct FResolvedSourceFormat
	{
		/** The resolved source format name (NAME_None for discrete/non-CAT sources). */
		FName SourceFormatName;

		/** The resolved channel count, or INDEX_NONE on failure. */
		int32 NumChannels = INDEX_NONE;
	};

	// FRuntimeInput represents an input to a MetaSound which can be manipulated.
	struct UE_INTERNAL FRuntimeInput
	{
		// Name of input vertex
		FName Name;
		// Data type name of input vertex.
		FName TypeName;
		// Access type of input vertex.
		EMetasoundFrontendVertexAccessType AccessType;
		// Default parameter of input vertex.
		FAudioParameter DefaultParameter;
		// True if the data type is transmittable. False otherwise.
		bool bIsTransmittable;
#if WITH_EDITORONLY_DATA
		// True if the data / default value deviates from the default page (graph)
		bool bIsPaged = false;
#endif // #if WITH_EDITORONLY_DATA
	};
private:

	struct FRuntimeInputData
	{
		std::atomic<bool> bIsValid = false;
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		Metasound::TSortedVertexNameMap<FRuntimeInput> InputMap;
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	};

protected:
	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendDocument RootMetasoundDocument;

	UPROPERTY()
	TSet<FString> ReferencedAssetClassKeys;

	UPROPERTY()
	TSet<TObjectPtr<UObject>> ReferencedAssetClassObjects;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Soft reference cache is no longer supported"))
	TSet<FSoftObjectPath> ReferenceAssetClassCache;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use EditorGraph instead as it is now transient and generated via the FrontendDocument dynamically."))
	TObjectPtr<UMetasoundEditorGraphBase> Graph;

	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UMetasoundEditorGraphBase> EditorGraph;
#endif // WITH_EDITORONLY_DATA

public:
	UE_API UMetaSoundSource(const FObjectInitializer& ObjectInitializer);

	// The output audio format of the metasound source.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Metasound)
	EMetaSoundOutputAudioFormat OutputFormat;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Metasound, meta = (DisplayName = "Source Format (exp)", EditConditionHides, EditCondition = "OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic"))
	EMetasoundChannelAgnosticSourceFormatChooser ChannelAgnosticFormatChooser = EMetasoundChannelAgnosticSourceFormatChooser::Auto;

	// The custom channel-agnostic format name (e.g. "Cat:Discrete:Stereo"). Only used when ChannelAgnosticFormatChooser == Custom.
	// Restricted to discrete (mixer-compatible) formats only — soundfield/ambisonics are not supported by AudioMixer.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Metasound, meta = (DisplayName = "Custom Format Name", GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions_AudioMixerOnly", EditConditionHides, EditCondition = "OutputFormat == EMetaSoundOutputAudioFormat::ChannelAgnostic && ChannelAgnosticFormatChooser == EMetasoundChannelAgnosticSourceFormatChooser::Custom"))
	FName ChannelAgnosticCustomFormat;

#if WITH_EDITORONLY_DATA
	// The QualitySetting MetaSound will use, as defined in 'MetaSound' Settings.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AssetRegistrySearchable, meta = (GetOptions="MetasoundEngine.MetaSoundSettings.GetQualityNames"), Category = "Metasound")
	FName QualitySetting;

	// This a editor only look up for the Quality Setting above. Preventing orphaning of the original name.
	UPROPERTY()
	FGuid QualitySettingGuid;

	// Override the BlockRate for this Sound (overrides Quality). NOTE: A Zero value will have no effect and use either the Quality setting (if set), or the defaults.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Metasound, meta = (UIMin = 0, UIMax = 1000.0, DisplayAfter="OutputFormat", DisplayName = "Override Block Rate (in Hz)"))
	FPerPlatformFloat BlockRateOverride = 0.f;

	// Override the SampleRate for this Sound (overrides Quality). NOTE: A Zero value will have no effect and use either the Quality setting (if set), or the Device Rate
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Metasound, meta = (UIMin = 0, UIMax = 96000, DisplayName = "Override Sample Rate (in Hz)"))
	FPerPlatformInt SampleRateOverride = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FGuid AssetClassID;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FString RegistryInputTypes;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	FString RegistryOutputTypes;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	int32 RegistryVersionMajor = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	int32 RegistryVersionMinor = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property is now serialized directly as asset tag"))
	bool bIsPreset = false;

	// Returns document name (for editor purposes, and avoids making document public for edit
	// while allowing editor to reference directly)
	static FName GetDocumentPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UMetaSoundSource, RootMetasoundDocument);
	}

	// Name to display in editors
	UE_API virtual FText GetDisplayName() const override;

	// Returns the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @return Editor graph associated with UMetaSoundSource.
	UE_API virtual UEdGraph* GetGraph() const override;
	UE_API virtual UEdGraph& GetGraphChecked() const override;
	UE_API virtual void MigrateEditorGraph(FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

	// Sets the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @param Editor graph associated with UMetaSoundSource.
	virtual void SetGraph(UEdGraph* InGraph) override
	{
		EditorGraph = CastChecked<UMetasoundEditorGraphBase>(InGraph);
	}
#endif // #if WITH_EDITORONLY_DATA


#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;

	virtual bool GetRedrawThumbnail() const override
	{
		return false;
	}

	virtual void SetRedrawThumbnail(bool bInRedraw) override
	{
	}

	virtual bool CanVisualizeAsset() const override
	{
		return false;
	}

	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type InDuplicateMode) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;

private:
	UE_API void PostEditChangeOutputFormat();
	UE_API void PostEditChangeQualitySettings();
#endif // WITH_EDITOR

public:
	virtual const TSet<FString>& GetReferencedAssetClassKeys() const override
	{
		return ReferencedAssetClassKeys;
	}
	UE_API virtual TArray<FMetasoundAssetBase*> GetReferencedAssets() const override;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "This is deprecated behavior and will become UE_INTERNAL API in a future release for back compat.")
	UE_API virtual void OnAsyncReferencedAssetsLoaded(const TArray<FMetasoundAssetBase*>& InAsyncReferences) override;
#endif // WITH_EDITORONLY_DATA

	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;

	UE_API void PostLoadQualitySettings();

	UE_API virtual bool ConformObjectToDocument() override;

	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	UE_API virtual FTopLevelAssetPath GetAssetPathChecked() const override;

	UObject* GetOwningAsset() override
	{
		return this;
	}

	const UObject* GetOwningAsset() const override
	{
		return this;
	}
	UE_API virtual void InitParameters(TArray<FAudioParameter>& ParametersToInit, FName InFeatureName) override;

	UE_API virtual void InitResources() override;

	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	using FMetasoundAssetBase::UpdateAndRegisterForExecution;
	UE_API virtual void UpdateAndRegisterForExecution(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions& InRegistrationOptions) override;

	UE_API virtual bool IsPlayable() const override;
	UE_API virtual float GetDuration() const override;
	UE_API virtual bool ImplementsParameterInterface(Audio::FParameterInterfacePtr InInterface) const override;
	UE_API virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams, TArray<FAudioParameter>&& InDefaultParameters) override;
	UE_API virtual void OnEndGenerate(ISoundGeneratorPtr Generator) override;
	UE_API virtual TSharedPtr<Audio::IParameterTransmitter> CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const override;
	UE_API virtual bool IsParameterValid(const FAudioParameter& InParameter) const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsOneShot() const override;
	UE_DEPRECATED(5.7, "This is no longer used.") virtual bool EnableSubmixSendsOnPreview() const override { return true; }

	/**
	 * Attempt to get a generator for a given AudioComponentID.
	 * You should only pass a valid AudioComponentID.
	 * If the ID is set to INDEX_NONE, e.g. from using FAudioDevice::PlaySoundAtLocation, use GetGeneratorForInstanceID instead.
	 * @param ComponentId The ID of the AudioComponent you want the generator from.
	 * @return A generator if it exists. A null weak pointer otherwise.
	 */
	UE_API TWeakPtr<Metasound::FMetasoundGenerator> GetGeneratorForAudioComponent(uint64 ComponentId) const;

	/**
	 * Attempt to get a generator for a given InstanceID.
	 * Use Audio::GetTransmitterID to construct an appropriate InstanceID.
	 * @param InstanceId The hash of the ComponentID, WaveInstanceHash and the ActiveSound Play Order
	 * @return A generator if it exists. A null weak pointer otherwise.
	 */
	UE_API TWeakPtr<Metasound::FMetasoundGenerator> GetGeneratorForInstanceID(uint64 InstanceId) const;
	UE_API bool IsDynamic() const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use OnGeneratorInstanceInfoCreated instead.")
	FOnGeneratorInstanceCreated OnGeneratorInstanceCreated;
	UE_DEPRECATED(5.6, "Use OnGeneratorInstanceInfoDestroyed instead.")
	FOnGeneratorInstanceDestroyed OnGeneratorInstanceDestroyed;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FGeneratorInstanceInfoDelegate OnGeneratorInstanceInfoCreated;
	FGeneratorInstanceInfoDelegate OnGeneratorInstanceInfoDestroyed;
	UE_API Metasound::FOperatorSettings GetOperatorSettings(Metasound::FSampleRate InDeviceSampleRate) const;

	UE_API virtual const FMetasoundFrontendDocument& GetConstDocument() const override;
	UE_API virtual bool IsActivelyBuilding() const override;

	UE_API virtual const UClass& GetBaseMetaSoundUClass() const final override;
	UE_API virtual const UClass& GetBuilderUClass() const final override;
	UE_API virtual EMetasoundFrontendClassAccessFlags GetDefaultAccessFlags() const final override;

	PRAGMA_DISABLE_INTERNAL_WARNINGS
	UE_INTERNAL UE_API TArray<FRuntimeInput> GatherVerseParameters() const;
	PRAGMA_ENABLE_INTERNAL_WARNINGS

protected:
	UE_DEPRECATED(5.6, "AccessPtrs are actively being deprecated. Writable access outside of the builder API "
		"is particularly problematic as in so accessing, the builder's caches are reset which can cause major "
		"editor performance regressions.")
	UE_API Metasound::Frontend::FDocumentAccessPtr GetDocumentAccessPtr(bool bReloadBuilder = true) override;
	UE_API Metasound::Frontend::FConstDocumentAccessPtr GetDocumentConstAccessPtr() const override;

	/** Gets all the default parameters for this Asset.  */
	UE_API virtual bool GetAllDefaultParameters(TArray<FAudioParameter>& OutParameters) const override;

#if WITH_EDITOR
	UE_API virtual void SetReferencedAssets(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs) override;
#endif // #if WITH_EDITOR

private:
	virtual FMetasoundFrontendDocument& GetDocument() override
	{
		return RootMetasoundDocument;
	}

	UE_API virtual void OnBeginActiveBuilder() override;
	UE_API virtual void OnFinishActiveBuilder() override;

	PRAGMA_DISABLE_INTERNAL_WARNINGS
	UE_API void InitParametersInternal(const Metasound::TSortedVertexNameMap<FRuntimeInput>& InputMap, TArray<FAudioParameter>& ParametersToInit, FName InFeatureName) const;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
	UE_API bool IsParameterValidInternal(const FAudioParameter& InParameter, const FName& InTypeName, Metasound::Frontend::IDataTypeRegistry& InDataTypeRegistry) const;

	static UE_API Metasound::SourcePrivate::FParameterRouter& GetParameterRouter();

public:
	UE_API Metasound::FMetasoundEnvironment CreateEnvironment(const FSoundGeneratorInitParams& InParams) const;
	UE_API const TArray<Metasound::FVertexName>& GetOutputAudioChannelOrder() const;

	/**
	 * Determines the "Auto" Channel Agnostic Format for this Source.
	 * Consults the document layer and makes a best guess at the appropriate format
	 * from wave proxy channel counts. Can be called before graph evaluation.
	 * @param InParams - Audio parameters to inspect for wave proxies. Pass empty array to use defaults.
	 * @return Shared pointer to the resolved channel type family, or nullptr if no format could be determined.
	 */
	UE_INTERNAL
	UE_API TSharedPtr<const Audio::FChannelTypeFamily> CalculateAutoSourceFormat(const TArray<FAudioParameter>& InParams) const;

	/**
	 * Resolves SourceFormatName and NumChannels based on the current OutputFormat.
	 * For ChannelAgnostic sources, determines format via Custom > Auto 
	 * priority and populates the Environment. For discrete sources, derives
	 * NumChannels from GetOutputAudioChannelOrder().
	 *
	 * Does NOT modify USoundWave::NumChannels -- the caller is responsible for
	 * assigning the returned value where appropriate.
	 *
	 * @param InParams - Sound generator init params (used for fallback channel count).
	 * @param InDefaultParameters - Audio parameters for auto-format detection (pass empty for precache).
	 * @param InOutEnvironment - Environment to populate with SourceFormatName.
	 * @return FResolvedSourceFormat containing the format name and channel count.
	 */
	UE_INTERNAL
	UE_API FResolvedSourceFormat ResolveSourceFormatAndNumChannels(const FSoundGeneratorInitParams& InParams, const TArray<FAudioParameter>& InDefaultParameters, Metasound::FMetasoundEnvironment& InOutEnvironment);

	/** Find the Source related to this Preset.
	 * 
	 * If this MetaSound is a preset and preset graph inflation is enabled, this
	 * will traverse the MetaSound Preset hierarchy until a UMetaSoundSource is 
	 * found which is either 
	 *  	- Not a preset
	 * 		AND/OR
	 *  	- Has modified constructor pin overrides.
	 */
	UE_INTERNAL
	UE_API const UMetaSoundSource& FindFirstNoninflatableSource(Metasound::FMetasoundEnvironment& InOutEnvironment, TFunctionRef<void(const UMetaSoundSource&)> OnTraversal) const;

private:
	UE_API const UMetaSoundSource& FindFirstNoninflatableSourceInternal(TArray<FGuid>& OutHierarchy, TFunctionRef<void(const UMetaSoundSource&)> OnTraversal) const;
	UE_API TSharedPtr<const Metasound::IGraph> FindFirstNoninflatableGraph(UMetaSoundSource::FAudioParameterCollector& InOutParameterCollector, Metasound::FMetasoundEnvironment& InOutEnvironment) const;
	
	UE_API Metasound::FMetasoundEnvironment CreateEnvironment() const;
	UE_API Metasound::FMetasoundEnvironment CreateEnvironment(const Audio::FParameterTransmitterInitParams& InParams) const;

	mutable FCriticalSection GeneratorMapCriticalSection;
	TArray<Metasound::FGeneratorInstanceInfo> Generators;
	UE_API void TrackGenerator(Metasound::FGeneratorInstanceInfo&& GeneratorInfo);

	UE_API void ForgetGenerator(ISoundGeneratorPtr Generator);

	PRAGMA_DISABLE_INTERNAL_WARNINGS
	static UE_API FRuntimeInput CreateRuntimeInput(const Metasound::Frontend::IDataTypeRegistry& Registry, const FMetasoundFrontendClassInput& Input, bool bCreateUObjectProxies, TArrayView<const FGuid> InPageOrder);
	UE_API Metasound::TSortedVertexNameMap<FRuntimeInput> CreateRuntimeInputMap(bool bCreateUObjectProxies, TArrayView<const FGuid> InPageOrder) const;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
	UE_API void CacheRuntimeInputData(TArrayView<const FGuid> InPageOrder);
	UE_API void InvalidateCachedRuntimeInputData();

	FRuntimeInputData RuntimeInputData;

	/** Enable/disable dynamic generator.
	 *
	 * Once a dynamic generator is enabled, all changes to the MetaSound should be applied to the
	 * FDynamicOperatorTransactor in order to keep parity between the document and active graph.
	 *
	 * Note: Disabling the dynamic generator will sever the communication between any active generators
	 * even if the dynamic generator is re-enabled during the lifetime of the active generators
	 */
	UE_API TSharedPtr<Metasound::DynamicGraph::FDynamicOperatorTransactor> SetDynamicGeneratorEnabled(bool bInIsEnabled);

	/** Get dynamic transactor
	 *
	 * If dynamic generators are enabled, this will return a valid pointer to a dynamic transactor.
	 * Changes to this transactor will be forwarded to any active Dynamic MetaSound Generators.
	 */
	UE_API TSharedPtr<Metasound::DynamicGraph::FDynamicOperatorTransactor> GetDynamicGeneratorTransactor() const;

	TSharedPtr<Metasound::DynamicGraph::FDynamicOperatorTransactor> DynamicTransactor;

	// Cache the AudioDevice Samplerate. (so that if we have to regenerate operator settings without the device rate we can use this).
	mutable Metasound::FSampleRate CachedAudioDeviceSampleRate = 0;
	
	bool bIsBuilderActive = false;
	bool bHasWarnedAboutStartTime = false;

	// Preset graph inflation is a performance optimization intended for use with the MetaSoundOperatorPool. If multiple presets 
	// utilize the same base MetaSound, they may be able to share their operators in the operator pool. This makes for a more
	// efficient use of the operator pool.
	bool bIsPresetGraphInflationSupported = false;

	// Quality settings. 
	UE_API bool GetQualitySettings(const FName InPlatformName, Metasound::SourcePrivate::FCookedQualitySettings& OutQualitySettings) const;
	UE_API void ResolveQualitySettings(const UMetaSoundSettings* Settings);
	UE_API void SerializeCookedQualitySettings(const FName PlatformName, FArchive& Ar);
	TPimplPtr<Metasound::SourcePrivate::FCookedQualitySettings> CookedQualitySettings;
	
};

#undef UE_API
