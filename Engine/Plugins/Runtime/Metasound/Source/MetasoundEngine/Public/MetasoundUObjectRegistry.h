// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Engine/Engine.h"
#include "MetasoundAssetBase.h"
#include "MetasoundDocumentInterface.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/Object.h"
#include "Templates/Function.h"

#define UE_API METASOUNDENGINE_API


// Forward Declarations
class UAssetManager;


namespace Metasound
{
	/** Interface for an entry into the Metasound-UObject Registry. 
	 *
	 * An entry provides information linking a FMetasoundFrontendClassInterface to a UClass.
	 * It also provides methods for accessing the FMetasoundAssetBase from a UObject.
	 */
	class IMetasoundUObjectRegistryEntry
	{
	public:
		virtual ~IMetasoundUObjectRegistryEntry() = default;

		UE_DEPRECATED(5.3, "Interfaces queries are no longer restricted to a single UObjectRegistryEntry. Options set upon registering interface determines supported UClasses.")
		virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const
		{
			static const FMetasoundFrontendVersion InterfaceVersion;
			return InterfaceVersion;
		}

		/** UClass associated with this entry. */
		virtual UClass* GetUClass() const = 0;

		/** Returns true if the UObject is of a Class which is a child of this UClass associated with this entry. */
		virtual bool IsChildClass(const UObject* InObject) const = 0;

		/** Returns true if the UClass is a child of this UClass associated with this entry. */
		virtual bool IsChildClass(const UClass* InClass) const = 0;

		/** Returns whether entry's class is a serialized asset or a transient type */
		virtual bool IsAssetType() const = 0;

		/** Attempts to cast the UObject to an FMetasoundAssetBase */
		virtual FMetasoundAssetBase* Cast(UObject* InObject) const = 0;

		/** Attempts to cast the UObject to an FMetasoundAssetBase */
		virtual const FMetasoundAssetBase* Cast(const UObject* InObject) const = 0;

		/** Creates a new object of the UClass type. */
		UE_DEPRECATED(5.3, "NewObject is now done via the MetaSoundBuilder system. Attempting to create a MetaSound via this call will now always fail.")
		virtual UObject* NewObject(UPackage* InPackage, const FName& InName) const
		{
			checkNoEntry();
			return nullptr;
		}

	private:
		IMetasoundUObjectRegistryEntry() = default;

		/** Only the TMetasoundUObjectRegistryEntry can construct this class. */
		template<typename UClassType>
		friend class TMetasoundUObjectRegistryEntry;
	};

	/** An entry into the Metasound-UObject registry.
	 *
	 * @Tparam UClassType A class which derives from UObject and IMetaSoundDocumentInterface.
	 * @Tparam IsAssetType If true, type derives from FMetasoundAssetBase and is considered a serializable, playable asset.
	 */
	template<typename UClassType>
	class TMetasoundUObjectRegistryEntry : public IMetasoundUObjectRegistryEntry
	{
		// Ensure that this is a subclass of IMetaSoundDocumentInterface and UObject.
		static_assert(std::is_base_of<IMetaSoundDocumentInterface, UClassType>::value, "UClass must be derived from IMetaSoundDocumentInterface");
		static_assert(std::is_base_of<UObject, UClassType>::value, "UClass must be derived from UObject");

	public:
		TMetasoundUObjectRegistryEntry() = default;
		virtual ~TMetasoundUObjectRegistryEntry() = default;

		UClass* GetUClass() const override
		{
			return UClassType::StaticClass();
		}

		virtual bool IsChildClass(const UObject* InObject) const override
		{
			if (nullptr != InObject)
			{
				return InObject->IsA(UClassType::StaticClass());
			}
			return false;
		}

		virtual bool IsChildClass(const UClass* InClass) const override
		{
			if (nullptr != InClass)
			{
				return InClass->IsChildOf(UClassType::StaticClass());
			}
			return false;
		}

		virtual bool IsAssetType() const override
		{
			return std::is_base_of<FMetasoundAssetBase, UClassType>::value;
		}

		virtual FMetasoundAssetBase* Cast(UObject * InObject) const override
		{
			if constexpr (!std::is_base_of<FMetasoundAssetBase, UClassType>::value)
			{
				return nullptr;
			}
			else
			{
				if (InObject)
				{
					return static_cast<FMetasoundAssetBase*>(CastChecked<UClassType>(InObject));
				}

				return nullptr;
			}
		}

		virtual const FMetasoundAssetBase* Cast(const UObject* InObject) const override
		{
			if constexpr (!std::is_base_of<FMetasoundAssetBase, UClassType>::value)
			{
				return nullptr;
			}
			else
			{
				if (InObject)
				{
					return static_cast<const FMetasoundAssetBase*>(CastChecked<const UClassType>(InObject));
				}

				return nullptr;
			}
		}
	};


	/** IMetasoundUObjectRegistry contains IMetasoundUObjectRegistryEntries.
	 *
	 * Registered UObject classes can utilize the Metasound Editor. It also enables
	 * the creation of a UObject directly from a FMetasoundFrontendDocument.
	 */
	class IMetasoundUObjectRegistry
	{
		public:
			virtual ~IMetasoundUObjectRegistry() = default;

			/** Return static singleton instance of the registry. */
			static UE_API IMetasoundUObjectRegistry& Get();

			/** Adds an entry to the registry. */
			virtual void RegisterUClass(TUniquePtr<IMetasoundUObjectRegistryEntry>&& InEntry) = 0;

			UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UObject registry as interfaces now support multiple UClasses that are registered with the interface registry.")
			virtual void RegisterUClassInterface(TUniquePtr<IMetasoundUObjectRegistryEntry>&& InEntry) { }

			UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UObject registry as interfaces now support multiple UClasses that are registered with the interface registry.")
			virtual TArray<const IMetasoundUObjectRegistryEntry*> FindInterfaceEntriesByName(FName InName) const { return { }; }

			UE_DEPRECATED(5.3, "Interfaces are no longer registered with the UObject registry as interfaces now support multiple UClasses that are registered with the interface registry.")
			virtual TArray<UClass*> FindSupportedInterfaceClasses(const FMetasoundFrontendVersion& InInterfaceVersion) const { return { }; }

			/** Creates a new object from a MetaSound document.
			 *
			 * @param InClass - A registered UClass to create.
			 * @param InDocument - The FMetasoundFrontendDocument to use when creating the class.
			 * @param InInterfaceVersion - The version of the FMetasoundFrontendClassInterface to use when creating the class.
			 * @param InPath - If in editor, the created asset will be stored at this content path.
			 *
			 * @return A new object. A nullptr on error.
			 */
			 UE_DEPRECATED(5.3, "UObject registry form of NewObject is no longer used. Use UMetaSoundBuilderSubsystem to author MetaSounds instead.")
			virtual UObject* NewObject(UClass* InClass, const FMetasoundFrontendDocument& InDocument, const FString& InPath) const { return nullptr; }

			/** Iterate all registered UClasses that serve as MetaSound assets.*/
			virtual void IterateRegisteredUClasses(TFunctionRef<void(UClass&)> InFunc, bool bAssetTypesOnly = true) const = 0;

			/** Returns true if the InObject is of a class or child class which is registered. */
			virtual bool IsRegisteredClass(UObject* InObject) const = 0;

			/** Returns true if the InClass matches a class or child class which is registered. */
			virtual bool IsRegisteredClass(const UClass& InClass) const = 0;

			/** Returns casts the UObject to a FMetasoundAssetBase if the UObject is of a registered type.
			 * If the UObject's UClass is not registered, then a nullptr is returned. 
			 */
			virtual FMetasoundAssetBase* GetObjectAsAssetBase(UObject* InObject) const = 0;

			/** Returns casts the UObject to a FMetasoundAssetBase if the UObject is of a registered type.
			 * If the UObject's UClass is not registered, then a nullptr is returned. 
			 */
			virtual const FMetasoundAssetBase* GetObjectAsAssetBase(const UObject* InObject) const = 0;
	};
} // namespace Metasound

#undef UE_API
