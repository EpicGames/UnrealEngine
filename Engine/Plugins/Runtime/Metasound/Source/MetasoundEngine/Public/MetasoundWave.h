// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "IAudioProxyInitializer.h"
#include "Sound/SoundWave.h"
#include "DSP/InterpolatedLinearPitchShifter.h"

#define UE_API METASOUNDENGINE_API


namespace Metasound
{
	// Forward declare ReadRef
	class FWaveAsset;
	typedef TDataReadReference<FWaveAsset> FWaveAssetReadRef;

	// Helper utility to test if exact types are required for a datatype.
	template <>
	struct TIsExplicit<FWaveAsset>
	{
		static constexpr bool Value = true;
	};

	// Metasound data type that holds onto a shared ptr. Mostly used as a placeholder until we have a proper proxy type.
	class FWaveAsset
	{
		// todo: this can be TSharedPtr<const FSoundWaveProxy> once depprecated functions are removed
		TSharedPtr<FSoundWaveProxy, ESPMode::ThreadSafe> SoundWaveProxy;
	
	public:

		FWaveAsset() = default;
		FWaveAsset(const FWaveAsset&) = default;
		FWaveAsset& operator=(const FWaveAsset& Other) = default;

		UE_API FWaveAsset(const TSharedPtr<const Audio::IProxyData>& InInitData);

		UE_API bool IsSoundWaveValid() const;

		UE_DEPRECATED(5.8, "Use GetWaveProxy() instead, mutable proxy access is deprecated")
		const FSoundWaveProxyPtr& GetSoundWaveProxy() const
		{
			return SoundWaveProxy;
		}

		const TSharedPtr<const FSoundWaveProxy> GetWaveProxy() const 
		{
			return SoundWaveProxy;
		}

		const FSoundWaveProxy* operator->() const
		{
			return SoundWaveProxy.Get();
		}

		UE_DEPRECATED(5.8, "Use GetWaveProxy() for non-const objects instead. (mutable proxy access is deprecated)")
		FSoundWaveProxy* operator->()
		{
			return SoundWaveProxy.Get();
		}

		friend inline uint32 GetTypeHash(const Metasound::FWaveAsset& InWaveAsset)
		{
			if (InWaveAsset.IsSoundWaveValid())
			{
				const TSharedPtr<const FSoundWaveProxy> Proxy = InWaveAsset.GetWaveProxy();
				check(Proxy);
				return GetTypeHash(*Proxy);
			}
			return INDEX_NONE;
		}
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FWaveAsset, METASOUNDENGINE_API, FWaveAssetTypeInfo, FWaveAssetReadRef, FWaveAssetWriteRef)
}
 

#undef UE_API
