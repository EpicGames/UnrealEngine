// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "IAudioProxyInitializer.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "WaveTable.h"
#include "WaveTableBank.h"
#include "WaveTableSampler.h"

namespace Metasound
{
	class FWaveTableBankAsset;
	using FWaveTableBankAssetReadRef = TDataReadReference<FWaveTableBankAsset>;
	using FWaveTableReadRef = TDataReadReference<WaveTable::FWaveTable>;
	using FWaveTableWriteRef = TDataWriteReference<WaveTable::FWaveTable>;

	class FWaveTableBankAsset
	{
		FWaveTableBankAssetProxyPtr Proxy;

	public:
		FWaveTableBankAsset()
			: Proxy(nullptr)
		{
		}

		FWaveTableBankAsset(const FWaveTableBankAsset& InOther)
			: Proxy(InOther.Proxy)
		{
		}

		FWaveTableBankAsset& operator=(const FWaveTableBankAsset& InOther)
		{
			Proxy = InOther.Proxy;
			return *this;
		}

		FWaveTableBankAsset& operator=(FWaveTableBankAsset&& InOther)
		{
			Proxy = MoveTemp(InOther.Proxy);
			return *this;
		}

		FWaveTableBankAsset(const TSharedPtr<Audio::IProxyData>& InInitData)
		{
			if (InInitData.IsValid())
			{
				if (InInitData->CheckTypeCast<FWaveTableBankAssetProxy>())
				{
					Proxy = MakeShared<FWaveTableBankAssetProxy, ESPMode::ThreadSafe>(InInitData->GetAs<FWaveTableBankAssetProxy>());
				}
			}
		}

		bool IsValid() const
		{
			return Proxy.IsValid();
		}

		const FWaveTableBankAssetProxyPtr& GetProxy() const
		{
			return Proxy;
		}

		const FWaveTableBankAssetProxy* operator->() const
		{
			return Proxy.Get();
		}

		FWaveTableBankAssetProxy* operator->()
		{
			return Proxy.Get();
		}
		
		friend inline uint32 GetTypeHash(const Metasound::FWaveTableBankAsset& InWaveTableBankAsset)
		{
			if (InWaveTableBankAsset.IsValid())
			{
				return GetTypeHash(*InWaveTableBankAsset.GetProxy());
			}
			return INDEX_NONE;
		}
	};

	// Disable arrays of WaveTables (as of addition, not supported by UX). Must be defined prior to DataType declaration macro below.
	template<>
	struct TEnableAutoArrayTypeRegistration<WaveTable::FWaveTable>
	{
		static constexpr bool Value = false;
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(WaveTable::FWaveTable, METASOUNDENGINE_API, FWaveTableTypeInfo, FWaveTableReadRef, FWaveTableWriteRef)
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FWaveTableBankAsset, METASOUNDENGINE_API, FWaveTableBankAssetTypeInfo, FWaveTableBankAssetReadRef, FWaveTableBankAssetWriteRef)

	DECLARE_METASOUND_ENUM(WaveTable::FWaveTableSampler::ESingleSampleMode, WaveTable::FWaveTableSampler::ESingleSampleMode::Zero, METASOUNDENGINE_API,
	FEnumWaveTableEnvelopeMode, FEnumWaveTableEnvelopeModeTypeInfo, FEnumWaveTableEnvelopeModeReadRef, FEnumWaveTableEnvelopeModeWriteRef);

	DECLARE_METASOUND_ENUM(WaveTable::FWaveTableSampler::EInterpolationMode, WaveTable::FWaveTableSampler::EInterpolationMode::Linear, METASOUNDENGINE_API,
	FEnumWaveTableInterpolationMode, FEnumWaveTableInterpModeTypeInfo, FEnumWaveTableInterpModeReadRef, FEnumWaveTableInterpModeWriteRef);
} // namespace Metasound

