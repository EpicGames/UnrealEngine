// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundEngineTest : ModuleRules
	{
		public MetasoundEngineTest(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDefinitions.AddRange(
				new string[]
				{
					"METASOUND_PLUGIN=Metasound",
					"METASOUND_MODULE=MetasoundEngineTest"
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"CQTest",
					"Engine",
					"AudioExtensions",
					"AudioChannelAgnosticCore",
					"AudioMixer",
					"SignalProcessing",
					"Projects",
					"AutomationDriver",
					"InputCore",
					"Slate",
					"SlateCore"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange
				(
					new string[]
					{
						"MetasoundEditor",
						"PropertyEditor",
						"UnrealEd"
					}
				);
			}

			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"MetasoundGraphCore",
					"MetasoundGenerator",
					"MetasoundFrontend",
					"MetasoundEngine",
					"MetasoundStandardNodes"
				}
			);
		}
	}
}
