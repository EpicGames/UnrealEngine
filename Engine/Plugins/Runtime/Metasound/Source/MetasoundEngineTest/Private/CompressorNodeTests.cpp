// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "MetasoundStandardNodesNames.h"
#include "NodeTestGraphBuilder.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace Metasound::Test::Nodes::Compressor
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetasoundCompressorNodeBasicTest,
		"Audio.Metasound.Nodes.Compressor.Basic",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMetasoundCompressorNodeBasicTest::RunTest(const FString&)
	{
		// Make a graph with a sine oscillator going into a compressor as the audio input
		TUniquePtr<FMetasoundGenerator> Generator;
		{
			FNodeTestGraphBuilder Builder;

			const Frontend::FNodeHandle SineOscillator = Builder.AddNode({ StandardNodes::Namespace, TEXT("Sine"), StandardNodes::AudioVariant }, 1);
			const Frontend::FNodeHandle Compressor = Builder.AddNode({ StandardNodes::Namespace, TEXT("Compressor"), StandardNodes::AudioVariant }, 1);
			FNodeTestGraphBuilder::ConnectNodes(SineOscillator, "Audio", Compressor, "Audio");

			// Expose the input bypass pin 
			Builder.AddAndConnectDataReferenceInput(Compressor, "Bypass", GetMetasoundDataTypeName<bool>());
			//output audio and output envelopes pin
			Builder.AddAndConnectDataReferenceOutput(Compressor, "Gain Envelope", GetMetasoundDataTypeName<FAudioBuffer>());
			Builder.AddAndConnectDataReferenceOutput(Compressor, "Audio", GetMetasoundDataTypeName<FAudioBuffer>());

			Generator = Builder.BuildGenerator();
		}
		UTEST_NOT_NULL("Created generator", Generator.Get());

		//output refs
		TOptional<TDataReadReference<FAudioBuffer>> OutputEnvelope = Generator->GetOutputReadReference<FAudioBuffer>("Gain Envelope");
		UTEST_TRUE("Got float envelope output", OutputEnvelope.IsSet());
		TOptional<TDataReadReference<FAudioBuffer>> OutputAudio = Generator->GetOutputReadReference<FAudioBuffer>("Audio");
		UTEST_TRUE("Got audio envelope output", OutputAudio.IsSet());

		//2 buffers for comparing bypassed and compressed audio outputs
		const int32 BlockSize = Generator->OperatorSettings.GetNumFramesPerBlock();
		FAudioBuffer TestBufferBypassed{ BlockSize };
		FAudioBuffer TestBufferWithCompression{ BlockSize };

		//set a low threshold for triggering compression
		Generator->SetInputValue("Threshold dB", -10.0f);

		//bypass the compressor, expect output envelopes to be 0
		Generator->SetInputValue("Bypass", true);

		//generate a block
		Generator->OnGenerateAudio(TestBufferBypassed.GetData(), TestBufferBypassed.Num());

		//check results
		{
			UTEST_EQUAL ("Output Envelope outputs 0 if bypassed", (*OutputEnvelope)->GetData()[0], 0.0f);
			bool HadNonZeroEnvelope = false;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float Value = (*OutputEnvelope)->GetData()[i];
				if (Value != 0.0f)
				{
					HadNonZeroEnvelope = true;
					break;
				}
			}
			UTEST_FALSE("Output Envelope should output 0 if bypassed", HadNonZeroEnvelope);
		}

		//enable dynamics compression (set bypass to false), expect non-zero output envelope
		Generator->SetInputValue("Bypass", false);

		//generate a few blocks
		constexpr int32 NumBlocksToRender = 4;
		for (int32 i = 0; i < NumBlocksToRender; ++i)
		{
			Generator->OnGenerateAudio(TestBufferWithCompression.GetData(), TestBufferWithCompression.Num());
		}

		//check results
		{
			UTEST_TRUE("Output Envelope should have non-zero values if enabled", (*OutputEnvelope)->GetData()[0] != 0.0f);
			bool HadNonZeroEnvelope = true;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float Value = (*OutputEnvelope)->GetData()[i];
				if (Value == 0.0f)
				{
					HadNonZeroEnvelope = false;
					break;
				}
			}
			UTEST_TRUE("Output Envelope should have non-zero values if enabled", HadNonZeroEnvelope);
		}

		//compare bypassed and compression-enabled buffers, expect buffers to contain different values after compression
		{
			bool BuffersContainDifferentOutputs = false;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float ByPassedValue = TestBufferBypassed.GetData()[i];
				const float CompressedValue = TestBufferWithCompression.GetData()[i];
				if (ByPassedValue != CompressedValue)
				{
					BuffersContainDifferentOutputs = true;
					break;
				}

			}
			
			UTEST_TRUE("Output audio buffer with compression enabled and bypassed should produce different values", BuffersContainDifferentOutputs);
		}

		return true;
	}
}
#endif