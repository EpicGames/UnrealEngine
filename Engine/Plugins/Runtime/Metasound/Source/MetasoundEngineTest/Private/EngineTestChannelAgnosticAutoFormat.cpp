// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS

#include "MetasoundChannelAgnosticAutoFormat.h"
#include "MetasoundChannelAgnosticAutoTypes.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundEnvironment.h"

#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "TypeFamily/ChannelTypeFamily.h"

#include "Sound/ISoundWaveContainer.h"

// Skip the test when MetasoundExperimental is not loaded (concrete CAT formats
// are registered by that plugin, so AutoFormat helpers have nothing to look up).
#define SKIP_WITHOUT_EXPERIMENTAL_PLUGIN() \
	if (!Metasound::IsExperimentalPluginEnabled()) \
	{ \
		AddInfo(TEXT("Skipped: MetasoundExperimental plugin not enabled")); \
		return true; \
	}

namespace Metasound::Test::AutoFormat
{
	// Mock IProxyData that implements ISoundWaveContainer, returning an empty
	// proxy list. This avoids needing real FSoundWaveProxy objects (which
	// require a fully-initialised USoundWave asset pipeline).
	class FMockEmptyWaveContainerProxy final
		: public Audio::IProxyData
		, public Audio::ISoundWaveContainer
	{
	public:
		FMockEmptyWaveContainerProxy()
			: Audio::IProxyData(TEXT("FMockEmptyWaveContainerProxy"))
		{
		}

		void* QueryInterface(const FName InterfaceId) override
		{
			if (InterfaceId == Audio::ISoundWaveContainer::GetInterfaceId())
			{
				return static_cast<Audio::ISoundWaveContainer*>(this);
			}
			return Audio::IProxyData::QueryInterface(InterfaceId);
		}

		TArray<FSoundWaveProxyPtr> GetContainedWaveProxies() const override
		{
			return {};
		}
	};

	// Mock IProxyData that does NOT implement ISoundWaveContainer, so
	// QueryInterface<ISoundWaveContainer> returns nullptr.
	class FMockNonContainerProxy final : public Audio::IProxyData
	{
	public:
		FMockNonContainerProxy()
			: Audio::IProxyData(TEXT("FMockNonContainerProxy"))
		{
		}

		void* QueryInterface(const FName InterfaceId) override
		{
			return Audio::IProxyData::QueryInterface(InterfaceId);
		}
	};

	//
	// ComputeAutoChannelCountFromProxies
	//

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FAutoChannelCount_EmptyArray,
		"Audio.Metasound.AutoFormat.ComputeAutoChannelCount.EmptyArray",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FAutoChannelCount_EmptyArray::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const TArray<FSoundWaveProxyPtr> Empty;
		const int32 Result = FChannelAgnosticAutoFormatHelper::ComputeAutoChannelCountFromProxies(Empty);
		TestEqual(TEXT("Empty array returns INDEX_NONE"), Result, INDEX_NONE);
		return true;
	}

	// Note: Tests with real FSoundWaveProxy objects are not included here because
	// FSoundWaveProxy requires a fully initialised USoundWave (loaded asset with
	// cooked/streaming data). ComputeAutoChannelCountFromProxies is a thin
	// Algo::MaxElementBy wrapper, so coverage of its edge case (empty) is sufficient.

	//
	// ComputeAutoFormatFromWaveContainer
	//

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FAutoFormat_NullProxy,
		"Audio.Metasound.AutoFormat.ComputeAutoFormatFromWaveContainer.NullProxy",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FAutoFormat_NullProxy::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const TSharedPtr<const Audio::FChannelTypeFamily> Result =
			FChannelAgnosticAutoFormatHelper::ComputeAutoFormatFromWaveContainer(nullptr);
		TestFalse(TEXT("Null proxy returns invalid format"), Result.IsValid());
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FAutoFormat_NonContainerProxy,
		"Audio.Metasound.AutoFormat.ComputeAutoFormatFromWaveContainer.NonContainerProxy",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FAutoFormat_NonContainerProxy::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		auto Proxy = MakeShared<FMockNonContainerProxy>();
		const TSharedPtr<const Audio::FChannelTypeFamily> Result =
			FChannelAgnosticAutoFormatHelper::ComputeAutoFormatFromWaveContainer(Proxy);
		TestFalse(TEXT("Non-container proxy returns invalid format"), Result.IsValid());
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FAutoFormat_EmptyContainer,
		"Audio.Metasound.AutoFormat.ComputeAutoFormatFromWaveContainer.EmptyContainer",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FAutoFormat_EmptyContainer::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		auto Container = MakeShared<FMockEmptyWaveContainerProxy>();
		const TSharedPtr<const Audio::FChannelTypeFamily> Result =
			FChannelAgnosticAutoFormatHelper::ComputeAutoFormatFromWaveContainer(Container);
		TestFalse(TEXT("Empty container returns invalid format"), Result.IsValid());
		return true;
	}

	//
	// ComputeNodeOutputFormat
	//

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FNodeOutputFormat_Custom,
		"Audio.Metasound.AutoFormat.ComputeNodeOutputFormat.Custom",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FNodeOutputFormat_Custom::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const FName CustomFormat(TEXT("Cat:MyCustomFormat"));
		const FMetasoundEnvironment Env;
		const FName Result = FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
			EMetasoundChannelAgnosticNodeFormatChooser::Custom,
			CustomFormat, nullptr, Env);
		TestEqual(TEXT("Custom returns the custom format name"), Result, CustomFormat);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FNodeOutputFormat_SourceWithEnv,
		"Audio.Metasound.AutoFormat.ComputeNodeOutputFormat.SourceWithEnvironment",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FNodeOutputFormat_SourceWithEnv::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const FName ExpectedFormat(TEXT("Cat:Discrete:Stereo"));
		FMetasoundEnvironment Env;
		Env.SetValue<FName>(TEXT("SourceFormatName"), ExpectedFormat);
		const FName Result = FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
			EMetasoundChannelAgnosticNodeFormatChooser::Source,
			NAME_None, nullptr, Env);
		TestEqual(TEXT("Source with env returns source format"), Result, ExpectedFormat);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FNodeOutputFormat_SourceNoEnvFallsThrough,
		"Audio.Metasound.AutoFormat.ComputeNodeOutputFormat.SourceNoEnvFallsThrough",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FNodeOutputFormat_SourceNoEnvFallsThrough::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		// Source with no SourceFormatName in env falls through to Auto.
		// Auto with no content returns the default format.
		const FMetasoundEnvironment Env;
		const FName Result = FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
			EMetasoundChannelAgnosticNodeFormatChooser::Source,
			NAME_None, nullptr, Env);
		const FName DefaultFormat = FDiscreteChannelAgnosticType::GetDefaultCatFormat();
		TestEqual(TEXT("Source with no env falls through to default"), Result, DefaultFormat);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FNodeOutputFormat_AutoNoContent,
		"Audio.Metasound.AutoFormat.ComputeNodeOutputFormat.AutoNoContentReturnsDefault",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FNodeOutputFormat_AutoNoContent::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const FMetasoundEnvironment Env;
		const FName Result = FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
			EMetasoundChannelAgnosticNodeFormatChooser::Auto,
			NAME_None, nullptr, Env);
		const FName DefaultFormat = FDiscreteChannelAgnosticType::GetDefaultCatFormat();
		TestEqual(TEXT("Auto with no content returns default format"), Result, DefaultFormat);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FNodeOutputFormat_AutoEmptyContainer,
		"Audio.Metasound.AutoFormat.ComputeNodeOutputFormat.AutoEmptyContainerReturnsDefault",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FNodeOutputFormat_AutoEmptyContainer::RunTest(const FString&)
	{
		SKIP_WITHOUT_EXPERIMENTAL_PLUGIN();
		const FMetasoundEnvironment Env;
		auto Container = MakeShared<FMockEmptyWaveContainerProxy>();
		const FName Result = FChannelAgnosticAutoFormatHelper::ComputeNodeOutputFormat(
			EMetasoundChannelAgnosticNodeFormatChooser::Auto,
			NAME_None, Container, Env);
		const FName DefaultFormat = FDiscreteChannelAgnosticType::GetDefaultCatFormat();
		TestEqual(TEXT("Auto with empty container returns default format"), Result, DefaultFormat);
		return true;
	}
}

PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

#undef SKIP_WITHOUT_EXPERIMENTAL_PLUGIN

#endif // WITH_DEV_AUTOMATION_TESTS
