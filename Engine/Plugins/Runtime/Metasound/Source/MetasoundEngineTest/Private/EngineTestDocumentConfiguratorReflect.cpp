// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentTemplates/MetasoundDocumentConfigurator.h"
#include "EngineTestMetaSoundBuilder.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendGraphLayer.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundSettings.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITORONLY_DATA

namespace Metasound::Test::ConfiguratorReflect
{
	// Builds a FMetasoundFrontendGraphLayer from a builder's current document state.
	// This mirrors what the editor's CopySelectionToLayer does but without requiring editor graph objects.
	FMetasoundFrontendGraphLayer BuildLayerFromBuilder(const UMetaSoundBuilderBase& Builder)
	{
		const FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetConstBuilder();
		const FMetasoundFrontendDocument& Document = DocBuilder.GetConstDocumentChecked();
		const FGuid& PageID = DocBuilder.GetBuildPageID();

		FMetasoundFrontendGraphLayer Layer;

		const FMetasoundFrontendGraph* Graph = Document.RootGraph.FindConstGraph(PageID);
		if (!Graph)
		{
			return Layer;
		}

		Layer.Graph = *Graph;

		// Copy inputs and outputs from the root graph class interface
		Layer.Inputs = Document.RootGraph.GetDefaultInterface().Inputs;
		Layer.Outputs = Document.RootGraph.GetDefaultInterface().Outputs;

		// Gather dependencies for all nodes in the graph
		TMap<FGuid, FMetasoundFrontendClass> DependencyMap;
		for (const FMetasoundFrontendNode& Node : Layer.Graph.Nodes)
		{
			if (const FMetasoundFrontendClass* Dep = DocBuilder.FindDependency(Node.ClassID))
			{
				DependencyMap.Add(Dep->ID, *Dep);
			}
		}
		DependencyMap.GenerateValueArray(Layer.Dependencies);

		return Layer;
	}

	// Validates that reflected code contains required boilerplate and all expected fragments.
	bool ValidateReflectedCode(FAutomationTestBase& Test, const FString& Code, const TArray<FString>& ExpectedFragments, const TArray<FString>& UnexpectedFragments = { })
	{
		bool bPassed = true;

		// Boilerplate checks
		bPassed &= Test.TestTrue(TEXT("Code contains FDC alias"), Code.Contains(TEXT("using FDC = Metasound::Engine::FDocumentConfigurator")));
		bPassed &= Test.TestTrue(TEXT("Code contains Configurator construction"), Code.Contains(TEXT("FDC Configurator(FDC::FArgs")));
		bPassed &= Test.TestTrue(TEXT("Code contains .Succeeded()"), Code.Contains(TEXT(".Succeeded()")));

		for (const FString& Fragment : ExpectedFragments)
		{
			bPassed &= Test.TestTrue(*FString::Printf(TEXT("Code contains: %s"), *Fragment), Code.Contains(Fragment));
		}

		for (const FString& Fragment : UnexpectedFragments)
		{
			bPassed &= Test.TestFalse(*FString::Printf(TEXT("Code should NOT contain: %s"), *Fragment), Code.Contains(Fragment));
		}

		return bPassed;
	}

	// ============================================================================
	// Test 1: Reflect empty layer
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectEmptyLayerTest,
		"Audio.Metasound.Configurator.Reflect.EmptyLayer",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectEmptyLayerTest::RunTest(const FString&)
	{
		FMetasoundFrontendGraphLayer Layer;
		Layer.Graph.PageID = Frontend::DefaultPageID;

		FString Code;
		Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

		UTEST_FALSE("Code is not empty", Code.IsEmpty());

		ValidateReflectedCode(*this, Code, { },
		{
			TEXT(".Add<FDC::FInput>"),
			TEXT(".Add<FDC::FOutput>"),
			TEXT(".Add<FDC::FNode>"),
			TEXT(".Add<FDC::FEdge>"),
			TEXT(".Add<FDC::FVariable>"),
			TEXT(".SetBuildPage(")
		});

		return true;
	}

	// ============================================================================
	// Test 2: Reflect inputs and outputs
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectInputsOutputsTest,
		"Audio.Metasound.Configurator.Reflect.InputsAndOutputs",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectInputsOutputsTest::RunTest(const FString&)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;

		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
		UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "ReflectInputsOutputs_Test", { });

		FMetasoundFrontendLiteral GainDefault;
		GainDefault.Set(1.0f);
		Builder.AddGraphInputNode("Gain", GetMetasoundDataTypeName<float>(), GainDefault, Result);
		UTEST_EQUAL("Add input succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		FMetasoundFrontendLiteral OutputDefault;
		OutputDefault.Set(0.0f);
		Builder.AddGraphOutputNode("Result", GetMetasoundDataTypeName<float>(), OutputDefault, Result);
		UTEST_EQUAL("Add output succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		FMetasoundFrontendGraphLayer Layer = BuildLayerFromBuilder(Builder);

		FString Code;
		Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

		ValidateReflectedCode(*this, Code,
		{
			TEXT(".Add<FDC::FInput>"),
			TEXT(".Name = \"Gain\""),
			TEXT(".bIsConstructorInput = false"),
			TEXT(".Add<FDC::FOutput>"),
			TEXT(".Name = \"Result\""),
		},
		{
			TEXT(".Add<FDC::FVariable>"),
			TEXT(".SetBuildPage(") // Default page should not emit PageName
		});

		return true;
	}

	// ============================================================================
	// Test 3: Reflect nodes and edges
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectNodesEdgesTest,
		"Audio.Metasound.Configurator.Reflect.NodesAndEdges",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectNodesEdgesTest::RunTest(const FString&)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;

		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
		UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "ReflectNodesEdges_Test", { });

		// Add Sine oscillator
		FMetaSoundNodeHandle SineNode = Builder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);
		UTEST_EQUAL("Add Sine node succeeded", Result, EMetaSoundBuilderResult::Succeeded);
		UTEST_TRUE("Sine node handle valid", SineNode.IsSet());

		// Add float input connected to Frequency
		FMetasoundFrontendLiteral FreqDefault;
		FreqDefault.Set(440.0f);
		FMetaSoundBuilderNodeOutputHandle FreqOutput = Builder.AddGraphInputNode("Frequency", GetMetasoundDataTypeName<float>(), FreqDefault, Result);
		UTEST_EQUAL("Add Frequency input succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		FMetaSoundBuilderNodeInputHandle SineFreqInput = Builder.FindNodeInputByName(SineNode, "Frequency", Result);
		Builder.ConnectNodes(FreqOutput, SineFreqInput, Result);
		UTEST_EQUAL("Connect Frequency succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		FMetasoundFrontendGraphLayer Layer = BuildLayerFromBuilder(Builder);

		FString Code;
		Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

		ValidateReflectedCode(*this, Code,
		{
			TEXT(".Add<FDC::FNode>"),
			TEXT(".ClassName = { \"UE\", \"Sine\", \"Audio\" }"),
			TEXT(".Add<FDC::FInput>"),
			TEXT(".Name = \"Frequency\""),
		});

		return true;
	}

	// ============================================================================
	// Test 4: Reflect variables
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectVariablesTest,
		"Audio.Metasound.Configurator.Reflect.Variables",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectVariablesTest::RunTest(const FString&)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;

		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
		UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "ReflectVariables_Test", { });

		// Add a float variable
		FMetasoundFrontendLiteral VarDefault;
		VarDefault.Set(0.5f);
		Builder.AddGraphVariable("MyVar", GetMetasoundDataTypeName<float>(), VarDefault, Result);
		UTEST_EQUAL("Add variable succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		// Add variable nodes
		FMetaSoundNodeHandle SetNode = Builder.AddGraphVariableSetNode("MyVar", Result);
		UTEST_EQUAL("Add variable set node succeeded", Result, EMetaSoundBuilderResult::Succeeded);
		UTEST_TRUE("Set node handle valid", SetNode.IsSet());

		FMetaSoundNodeHandle GetNode = Builder.AddGraphVariableGetNode("MyVar", Result);
		UTEST_EQUAL("Add variable get node succeeded", Result, EMetaSoundBuilderResult::Succeeded);
		UTEST_TRUE("Get node handle valid", GetNode.IsSet());

		FMetaSoundNodeHandle GetDelayedNode = Builder.AddGraphVariableGetDelayedNode("MyVar", Result);
		UTEST_EQUAL("Add variable get delayed node succeeded", Result, EMetaSoundBuilderResult::Succeeded);
		UTEST_TRUE("Get delayed node handle valid", GetDelayedNode.IsSet());

		FMetasoundFrontendGraphLayer Layer = BuildLayerFromBuilder(Builder);

		FString Code;
		Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

		ValidateReflectedCode(*this, Code,
		{
			TEXT(".Add<FDC::FVariable>"),
			TEXT(".Name = \"MyVar\""),
			TEXT(".Add<FDC::FVariableSetNode>"),
			TEXT(".VariableName = \"MyVar\""),
			TEXT(".Add<FDC::FVariableGetNode>"),
			TEXT(".Add<FDC::FVariableGetDelayedNode>"),
		},
		{
			TEXT(".SetBuildPage(") // Default page should not emit PageName
		});

		return true;
	}

	// ============================================================================
	// Test 5: Reflect page name (non-default page emits PageName, default does not)
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectPageNameTest,
		"Audio.Metasound.Configurator.Reflect.PageName",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectPageNameTest::RunTest(const FString&)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;

		// Test default page: no .PageName should appear
		{
			EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
			UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "ReflectPageName_Default_Test", { });

			FMetasoundFrontendLiteral Literal;
			Literal.Set(1.0f);
			Builder.AddGraphInputNode("TestInput", GetMetasoundDataTypeName<float>(), Literal, Result);

			FMetasoundFrontendGraphLayer Layer = BuildLayerFromBuilder(Builder);
			// Ensure PageID is the default
			Layer.Graph.PageID = Frontend::DefaultPageID;

			FString Code;
			Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

			UTEST_FALSE("Default page code not empty", Code.IsEmpty());
			UTEST_FALSE("Default page code should NOT contain .SetBuildPage", Code.Contains(TEXT(".SetBuildPage(")));
		}

		// Test non-default page: .SetBuildPage should appear
		// Only run if there are non-default pages registered in settings
		{
			const TArray<FName> PageNames = UMetaSoundSettings::GetPageNames();
			FName NonDefaultPageName;
			for (const FName& Name : PageNames)
			{
				if (Name != Frontend::DefaultPageName)
				{
					NonDefaultPageName = Name;
					break;
				}
			}

			if (!NonDefaultPageName.IsNone())
			{
				const FGuid* NonDefaultPageID = UMetaSoundSettings::GetPageID(NonDefaultPageName);
				if (NonDefaultPageID)
				{
					EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
					UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "ReflectPageName_NonDefault_Test", { });

					FMetasoundFrontendLiteral Literal;
					Literal.Set(1.0f);
					Builder.AddGraphInputNode("TestInput", GetMetasoundDataTypeName<float>(), Literal, Result);

					Builder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);

					FMetasoundFrontendGraphLayer Layer = BuildLayerFromBuilder(Builder);
					// Override PageID to non-default
					Layer.Graph.PageID = *NonDefaultPageID;

					FString Code;
					Engine::FDocumentConfigurator::Reflect(MoveTemp(Layer), Code);

					const FString ExpectedPageNameFragment = FString::Printf(TEXT(".SetBuildPage(\"%s\")"), *NonDefaultPageName.ToString());
					UTEST_TRUE("Non-default page code contains .SetBuildPage", Code.Contains(ExpectedPageNameFragment));
				}
			}
			else
			{
				AddInfo(TEXT("Skipping non-default page test: no non-default pages registered in MetaSound settings"));
			}
		}

		return true;
	}

	// ============================================================================
	// Test 6: Round-trip validation
	// Creates a graph via builder, reflects it, then builds a new graph via
	// FDocumentConfigurator and compares the two documents structurally.
	// ============================================================================
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FReflectRoundTripTest,
		"Audio.Metasound.Configurator.Reflect.RoundTrip",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FReflectRoundTripTest::RunTest(const FString&)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;
		using namespace Frontend;

		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;

		// === Build source graph ===
		UMetaSoundPatchBuilder& PatchBuilder = CreatePatchBuilderChecked(*this, "ReflectRoundTrip_Source", { });

		// Add input
		FMetasoundFrontendLiteral FreqDefault;
		FreqDefault.Set(440.0f);
		FMetaSoundBuilderNodeOutputHandle FreqOutput = PatchBuilder.AddGraphInputNode("Frequency", GetMetasoundDataTypeName<float>(), FreqDefault, Result);
		UTEST_EQUAL("Source: Add input succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		// Add Sine node
		FMetaSoundNodeHandle SineNode = PatchBuilder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);
		UTEST_EQUAL("Source: Add Sine node succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		// Connect input to Sine frequency
		FMetaSoundBuilderNodeInputHandle SineFreqInput = PatchBuilder.FindNodeInputByName(SineNode, "Frequency", Result);
		PatchBuilder.ConnectNodes(FreqOutput, SineFreqInput, Result);

		// Add variable
		FMetasoundFrontendLiteral VarDefault;
		VarDefault.Set(1.0f);
		PatchBuilder.AddGraphVariable("Gain", GetMetasoundDataTypeName<float>(), VarDefault, Result);
		UTEST_EQUAL("Source: Add variable succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		FMetaSoundNodeHandle VarGetNode = PatchBuilder.AddGraphVariableGetNode("Gain", Result);
		UTEST_EQUAL("Source: Add variable get node succeeded", Result, EMetaSoundBuilderResult::Succeeded);

		// === Build layer and reflect ===
		FMetasoundFrontendGraphLayer SourceLayer = BuildLayerFromBuilder(PatchBuilder);

		// Verify source layer has expected content
		UTEST_TRUE("Source layer has nodes", SourceLayer.Graph.Nodes.Num() > 0);
		UTEST_TRUE("Source layer has inputs", SourceLayer.Inputs.Num() > 0);
		UTEST_TRUE("Source layer has variables", SourceLayer.Graph.Variables.Num() > 0);

		FString Code;
		Engine::FDocumentConfigurator::Reflect(MoveTemp(SourceLayer), Code);

		UTEST_FALSE("Reflected code not empty", Code.IsEmpty());

		// === Apply same configuration to a new builder via FDocumentConfigurator ===
		UMetaSoundPatchBuilder& DestBuilder = CreatePatchBuilderChecked(*this, "ReflectRoundTrip_Dest", { });

		FMetaSoundFrontendDocumentBuilder& DestDocBuilder = DestBuilder.GetBuilder();

		{
			Engine::FDocumentConfigurator Configurator(DestBuilder);

			// Replicate what the reflected code would do
			Configurator
				.Add(Engine::FDocumentConfigurator::FInput
				{
					.Name = "Frequency",
					.DataType = GetMetasoundDataTypeName<float>(),
					.DefaultValue = Frontend::MakeFrontendLiteral(440.0f)
				})
				.Add(Engine::FDocumentConfigurator::FNode
				{
					.Name = "Sine",
					.ClassName = { "UE", "Sine", "Audio" }
				})
				.Add(Engine::FDocumentConfigurator::FVariable
				{
					.Name = "Gain",
					.DataType = GetMetasoundDataTypeName<float>(),
					.DefaultValue = Frontend::MakeFrontendLiteral(1.0f)
				})
				.Add(Engine::FDocumentConfigurator::FVariableGetNode
				{
					.Name = "Get_Gain",
					.VariableName = "Gain"
				});

			UTEST_TRUE("Configurator succeeded", Configurator.Succeeded());
		}

		// === Compare documents ===
		const FMetaSoundFrontendDocumentBuilder& DestConstBuilder = DestBuilder.GetConstBuilder();
		const FMetasoundFrontendDocument& DestDoc = DestConstBuilder.GetConstDocumentChecked();

		// Verify destination has the same structural elements
		UTEST_NOT_NULL("Dest has Frequency input", DestConstBuilder.FindGraphInput("Frequency"));
		UTEST_NOT_NULL("Dest has Gain variable", DestConstBuilder.FindGraphVariable("Gain"));

		const FMetasoundFrontendGraph* DestGraph = DestDoc.RootGraph.FindConstGraph(DestConstBuilder.GetBuildPageID());
		UTEST_NOT_NULL("Dest graph exists", DestGraph);

		if (DestGraph)
		{
			// Count external nodes (Sine) and variable nodes
			bool bFoundSineNode = false;
			bool bFoundVariableGetNode = false;
			for (const FMetasoundFrontendNode& Node : DestGraph->Nodes)
			{
				if (const FMetasoundFrontendClass* Dep = DestConstBuilder.FindDependency(Node.ClassID))
				{
					if (Dep->Metadata.GetClassName() == FMetasoundFrontendClassName("UE", "Sine", "Audio"))
					{
						bFoundSineNode = true;
					}
					if (Dep->Metadata.GetType() == EMetasoundFrontendClassType::VariableAccessor)
					{
						bFoundVariableGetNode = true;
					}
				}
			}
			UTEST_TRUE("Dest has Sine node", bFoundSineNode);
			UTEST_TRUE("Dest has variable get node", bFoundVariableGetNode);

			UTEST_TRUE("Dest has variables", DestGraph->Variables.Num() > 0);
		}

		return true;
	}

} // namespace Metasound::Test::ConfiguratorReflect

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITORONLY_DATA
