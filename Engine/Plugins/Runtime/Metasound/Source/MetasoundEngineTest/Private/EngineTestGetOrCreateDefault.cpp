// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "MetasoundWave.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMetasoundGetOrCreateDefault_WaveAsset_Unbound, "Audio.Metasound.GetOrCreateDefault.WaveAsset.Unbound", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMetasoundGetOrCreateDefault_WaveAsset_Unbound::RunTest(const FString& Parameters)
{
	using namespace Metasound;

	const FOperatorSettings Settings(48000, 480);
	const FVertexName VertexName = TEXT("TestWaveInput");

	// Build an input vertex interface with a single FWaveAsset vertex (unbound).
	FInputVertexInterface InputInterface
	{
		TInputDataVertex<FWaveAsset>(VertexName, FDataVertexMetadata{})
	};
	FInputVertexInterfaceData InterfaceData(InputInterface);

	// Call GetOrCreateDefaultDataReadReference — should create a default FWaveAsset via factory.
	FWaveAssetReadRef ReadRef = InterfaceData.GetOrCreateDefaultDataReadReference<FWaveAsset>(VertexName, Settings);

	// A default-constructed FWaveAsset has no valid sound wave.
	TestFalse(TEXT("Default FWaveAsset should not have a valid SoundWave"), ReadRef->IsSoundWaveValid());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMetasoundGetOrCreateDefault_WaveAsset_Bound, "Audio.Metasound.GetOrCreateDefault.WaveAsset.Bound", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMetasoundGetOrCreateDefault_WaveAsset_Bound::RunTest(const FString& Parameters)
{
	using namespace Metasound;

	const FOperatorSettings Settings(48000, 480);
	const FVertexName VertexName = TEXT("TestWaveInput");

	// Build an input vertex interface with a single FWaveAsset vertex.
	FInputVertexInterface InputInterface
	{
		TInputDataVertex<FWaveAsset>(VertexName, FDataVertexMetadata{})
	};
	FInputVertexInterfaceData InterfaceData(InputInterface);

	// Pre-bind a write ref to the vertex.
	FWaveAssetWriteRef WriteRef = FWaveAssetWriteRef::CreateNew();
	InterfaceData.BindReadVertex<FWaveAsset>(VertexName, WriteRef);

	// Call GetOrCreateDefaultDataReadReference — should return the bound ref.
	FWaveAssetReadRef ReadRef = InterfaceData.GetOrCreateDefaultDataReadReference<FWaveAsset>(VertexName, Settings);

	// Verify we got back the same underlying object that was bound.
	TestEqual(TEXT("Bound ref should be returned"), static_cast<const void*>(&(*ReadRef)), static_cast<const void*>(&(*WriteRef)));

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
