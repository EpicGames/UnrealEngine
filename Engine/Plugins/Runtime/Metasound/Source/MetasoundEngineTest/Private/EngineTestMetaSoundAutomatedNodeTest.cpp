// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineTestMetaSoundAutomatedNodeTest.h"

#include "Algo/AllOf.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Math/NumericLimits.h"
#include "Misc/AutomationTest.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundPrimitives.h"
#include "MetasoundVariableNodes.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UniquePtr.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITORONLY_DATA

namespace Metasound::EngineTest
{
	// Return audio mixer device if one is available
	Audio::FMixerDevice* GetAudioMixerDevice()
	{
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = DeviceManager->GetMainAudioDeviceRaw())
			{
				return static_cast<Audio::FMixerDevice*>(AudioDevice);
			}
		}
		return nullptr;
	}

	// Create an example environment that generally exists for a UMetaSoundSoruce
	FMetasoundEnvironment GetSourceEnvironmentForTest()
	{
		using namespace Frontend;

		FMetasoundEnvironment Environment;

		Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, 0);
		Environment.SetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy, TArray<FGuid>({ FGuid() }));

		Environment.SetValue<uint32>(SourceInterface::Environment::SoundUniqueID, 0);
		Environment.SetValue<bool>(SourceInterface::Environment::IsPreview, false);
		Environment.SetValue<uint64>(SourceInterface::Environment::TransmitterID, 0);
		Environment.SetValue<FString>(SourceInterface::Environment::GraphName, TEXT("ENGINE_TEST_REGISTERED_NODES"));

		if (Audio::FMixerDevice* MixerDevice = GetAudioMixerDevice())
		{
			Environment.SetValue<Audio::FDeviceId>(SourceInterface::Environment::DeviceID, MixerDevice->DeviceID);
			Environment.SetValue<int32>(SourceInterface::Environment::AudioMixerNumOutputFrames, MixerDevice->GetNumOutputFrames());
		}

		return Environment;
	}

	FString GetPrettyName(const Frontend::FNodeClassRegistryKey& InRegistryKey)
	{
		const FTopLevelAssetPath ObjectPath = Frontend::IMetaSoundAssetManager::GetChecked().FindAssetPath(InRegistryKey);
		if (ObjectPath.IsValid())
		{
			return ObjectPath.ToString();
		}

		FMetasoundFrontendRegistryContainer* NodeRegistry = FMetasoundFrontendRegistryContainer::Get();
		if (ensure(NodeRegistry))
		{
			FMetasoundFrontendClass NodeClass;
			if (NodeRegistry->FindFrontendClassFromRegistered(InRegistryKey, NodeClass))
			{
				return FString::Printf(TEXT("%s %s"), *NodeClass.Metadata.GetClassName().ToString(), *NodeClass.Metadata.GetVersion().ToString());
			}
		}

		return TEXT("");
	}

	// TTestTypeInfo converts test types to strings
	template<typename DataType>
	struct TTestTypeInfo
	{
		static FString ToString(const DataType& InData)
		{
			return ::LexToString(InData);
		}
	};

	template<typename ElementType>
	struct TTestTypeInfo<TArray<ElementType>>
	{
		static FString ToString(TArrayView<const ElementType> InData)
		{
			return FString::Printf(TEXT("[%s]"), *FString::JoinBy(InData, TEXT(","), &TTestTypeInfo<ElementType>::ToString));
		}
	};

	template<>
	struct TTestTypeInfo<FAudioBuffer> : TTestTypeInfo<TArray<float>>
	{
	};
	

	template<>
	struct TTestTypeInfo<FTime>
	{
		static FString ToString(const FTime& InData)
		{
			return ::LexToString(InData.GetSeconds());
		}
	};

	// TTestTypeAnalysis performs analysis operator on data types.
	template<typename DataType>
	struct TTestTypeAnalysis
	{
		static bool IsValid(const DataType& InArray)
		{
			return true;
		}

		static bool IsEqual(const DataType& InLHS, const DataType& InRHS)
		{
			return InLHS == InRHS;
		}
	};

	// Specialize type analysis for TArrays
	template<typename ElementType>
	struct TTestTypeAnalysis<TArray<ElementType>>
	{
		static bool IsValid(TArrayView<const ElementType> InArray)
		{
			return Algo::AllOf(InArray, &TTestTypeAnalysis<ElementType>::IsValid);
		}

		static bool IsEqual(TArrayView<const ElementType> InLHS, TArrayView<const ElementType> InRHS)
		{
			const int32 Num = InLHS.Num();
			if (Num == InRHS.Num())
			{
				for (int32 i = 0; i < Num; i++)
				{
					if (InLHS[i] != InRHS[i])
					{
						return false;
					}
				}
				return true;
			}
			return false;
		}
	};

	// Specialize type analysis for audio buffers.
	template<>
	struct TTestTypeAnalysis<FAudioBuffer> : TTestTypeAnalysis<TArray<float>>
	{
	};

	// Specialize type analysis for float
	template<>
	struct TTestTypeAnalysis<float>
	{
		static bool IsValid(float InValue)
		{
			return !FMath::IsNaN(InValue);
		}

		static bool IsEqual(float InLHS, float InRHS)
		{
			return InLHS == InRHS;
		}
	};

	// Specialize type analysis for time
	template<>
	struct TTestTypeAnalysis<FTime>
	{
		static bool IsValid(const FTime& InValue)
		{
			return FMath::IsFinite(InValue.GetSeconds());
		}

		static bool IsEqual(const FTime& InLHS, const FTime& InRHS)
		{
			return InLHS == InRHS;
		}
	};

	// TDataReferenceAnalyzer fulfils the IDataReferenceAnalyzer interface and uses
	// various templates to implement methods.
	template<typename DataType>
	struct TDataReferenceAnalyzer : public IDataReferenceAnalyzer
	{
		virtual FAnyDataReference Copy(const FAnyDataReference& InDataRef) const override
		{
			if (const DataType* InValue = GetDataOrLogError(InDataRef))
			{
				return FAnyDataReference{TDataValueReference<DataType>::CreateNew(*InValue)};
			}
			return InDataRef;
		}

		virtual bool IsEqual(const FAnyDataReference& InLHS, const FAnyDataReference& InRHS) const override
		{
			if (const DataType* InLHSValue = GetDataOrLogError(InLHS))
			{
				if (const DataType* InRHSValue = GetDataOrLogError(InRHS))
				{
					return TTestTypeAnalysis<DataType>::IsEqual(*InLHSValue, *InRHSValue);
				}
			}
			return false;
		}

		virtual bool IsValid(const FAnyDataReference& InDataRef) const override
		{
			if (const DataType* InValue = GetDataOrLogError(InDataRef))
			{
				return TTestTypeAnalysis<DataType>::IsValid(*InValue);
			}
			return false;
		}

		virtual FString ToString(const FAnyDataReference& InDataRef) const override
		{
			if (const DataType* InValue = GetDataOrLogError(InDataRef))
			{
				return TTestTypeInfo<DataType>::ToString(*InValue);
			}
			return TEXT("");
		}
	private:
		const DataType* GetDataOrLogError(const FAnyDataReference& InDataRef) const
		{
			const DataType* Value = InDataRef.GetValue<DataType>();
			if (nullptr == Value)	
			{
				// Data references should never be null
				UE_LOGF(LogMetaSound, Error, "Failed to get data type value of type %ls", *GetMetasoundDataTypeString<DataType>());
			}
			return Value;
		}
	};

	// Register a data reference analyzer to support analyzing output of metasound nodes.
	template<typename DataType>
	void AddDataReferenceAnalyzerToMap(TMap<FName, TSharedPtr<const IDataReferenceAnalyzer>>& InMap)
	{
		InMap.Add(GetMetasoundDataTypeName<DataType>(), MakeShared<const TDataReferenceAnalyzer<DataType>>());
	}

	// A static registry of IDataReferenceAnalyzers
	const TMap<FName, TSharedPtr<const IDataReferenceAnalyzer>>& GetDataTypeAnalyzerMap()
	{
		static TMap<FName, TSharedPtr<const IDataReferenceAnalyzer>> Map;

		AddDataReferenceAnalyzerToMap<bool>(Map);
		AddDataReferenceAnalyzerToMap<int32>(Map);
		AddDataReferenceAnalyzerToMap<float>(Map);
		AddDataReferenceAnalyzerToMap<FString>(Map);
		AddDataReferenceAnalyzerToMap<FTime>(Map);
		AddDataReferenceAnalyzerToMap<FAudioBuffer>(Map);
		AddDataReferenceAnalyzerToMap<FTrigger>(Map);
		AddDataReferenceAnalyzerToMap<TArray<bool>>(Map);
		AddDataReferenceAnalyzerToMap<TArray<int32>>(Map);
		AddDataReferenceAnalyzerToMap<TArray<float>>(Map);
		AddDataReferenceAnalyzerToMap<TArray<FString>>(Map);
		AddDataReferenceAnalyzerToMap<TArray<FTime>>(Map);

		return Map;
	}

	void FOutputVertexDataTestController::FAnalyzableOutput::CaptureValue()
	{
		CapturedValue = DataReferenceAnalyzer->Copy(DataReference);
	}

	bool FOutputVertexDataTestController::FAnalyzableOutput::IsDataReferenceValid() const
	{
		return DataReferenceAnalyzer->IsValid(DataReference);
	}

	bool FOutputVertexDataTestController::FAnalyzableOutput::IsDataReferenceEqualToCapturedValue() const
	{
		return DataReferenceAnalyzer->IsEqual(CapturedValue, DataReference);
	}

	FString FOutputVertexDataTestController::FAnalyzableOutput::DataReferenceToString() const
	{
		return DataReferenceAnalyzer->ToString(DataReference);
	}

	FString FOutputVertexDataTestController::FAnalyzableOutput::CapturedValueToString() const
	{
		return DataReferenceAnalyzer->ToString(CapturedValue);
	}

	FOutputVertexDataTestController::FOutputVertexDataTestController( const FOutputVertexInterface& InOutputInterface, const FOutputVertexInterfaceData& InOutputData)
	{
		const TMap<FName, TSharedPtr<const IDataReferenceAnalyzer>>& AnalyzerMap = GetDataTypeAnalyzerMap();

		for (const FOutputDataVertex& Vertex : InOutputInterface)
		{
			if (AnalyzerMap.Contains(Vertex.DataTypeName))
			{
				if (const FAnyDataReference* Ref = InOutputData.FindDataReference(Vertex.VertexName))
				{
					AnalyzableOutputs.Add(FAnalyzableOutput{*Ref, *Ref, Vertex.VertexName, AnalyzerMap[Vertex.DataTypeName]});
				}
			}
		}
	}

	int32 FOutputVertexDataTestController::GetNumAnalyzableOutputs() const
	{
		return AnalyzableOutputs.Num();
	}

	bool FOutputVertexDataTestController::AreAllAnalyzableOutputsValid() const
	{
		bool bAllAreValid = true;
		for (const FAnalyzableOutput& AnalyzableOutput : AnalyzableOutputs)
		{
			if (!AnalyzableOutput.IsDataReferenceValid())
			{
				UE_LOGF(LogMetaSound, Warning, "Invalid output encountered %ls %ls", *AnalyzableOutput.VertexName.ToString(), *AnalyzableOutput.DataReferenceToString());
				bAllAreValid = false;
			}
		}
		return bAllAreValid;
	}

	void FOutputVertexDataTestController::CaptureCurrentOutputValues()
	{
		for (FAnalyzableOutput& AnalyzableOutput : AnalyzableOutputs)
		{
			AnalyzableOutput.CaptureValue();
		}
	}

	bool FOutputVertexDataTestController::AreAllOutputValuesEqualToCapturedValues() const
	{
		bool bAllAreEqual = true;
		for (const FAnalyzableOutput& AnalyzableOutput : AnalyzableOutputs)
		{
			if (!AnalyzableOutput.IsDataReferenceEqualToCapturedValue())
			{
				UE_LOGF(LogMetaSound, Warning, "Unequal output encountered %ls found: %ls expected: %ls", *AnalyzableOutput.VertexName.ToString(), *AnalyzableOutput.DataReferenceToString(), *AnalyzableOutput.CapturedValueToString());
				bAllAreEqual = false;
			}
		}
		return bAllAreEqual;
	}


	// TTestTypeValues should return basic bounds for tested input data types.
	// Similar to TNumericLimits<>
	template<typename DataType>
	struct TTestTypeValues
	{};

	// TArray specialization to defer to a single element array with array element's values
	template<typename ElementType>
	struct TTestTypeValues<TArray<ElementType>>
	{
		static TArray<ElementType> Min(const FOperatorSettings& InSettings) 
		{
			return TArray<ElementType>({TTestTypeValues<ElementType>::Min(InSettings)}); 
		}

		static TArray<ElementType> Max(const FOperatorSettings& InSettings) 
		{ 
			return TArray<ElementType>({TTestTypeValues<ElementType>::Max(InSettings)}); 
		}

		static TArray<ElementType> Default(const FOperatorSettings& InSettings) 
		{ 
			return TArray<ElementType>({TTestTypeValues<ElementType>::Default(InSettings)}); 
		}

		static TArray<ElementType> Random(const FOperatorSettings& InSettings)
		{ 
			return TArray<ElementType>({TTestTypeValues<ElementType>::Random(InSettings)}); 
		}
	};

	template<>
	struct TTestTypeValues<bool>
	{
		static bool Min(const FOperatorSettings& InSettings) { return false; }
		static bool Max(const FOperatorSettings& InSettings) { return true; }
		static bool Default(const FOperatorSettings& InSettings) { return true; }
		static bool Random(const FOperatorSettings& InSettings) { return FMath::RandRange(0.f, 1.f) > 0.5; }
	};

	template<>
	struct TTestTypeValues<int32>
	{
		static int32 Min(const FOperatorSettings& InSettings) { return TNumericLimits<int32>::Min(); }
		static int32 Max(const FOperatorSettings& InSettings) { return TNumericLimits<int32>::Max(); }
		static int32 Default(const FOperatorSettings& InSettings) { return 0; }
		static int32 Random(const FOperatorSettings& InSettings) { return FMath::RandRange(TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max()); }
	};

	template<>
	struct TTestTypeValues<float>
	{
		static float Min(const FOperatorSettings& InSettings) { return TNumericLimits<float>::Min(); }
		static float Max(const FOperatorSettings& InSettings) { return TNumericLimits<float>::Max(); }
		static float Default(const FOperatorSettings& InSettings) { return 0.f; }
		static float Random(const FOperatorSettings& InSettings) { return FMath::RandRange(TNumericLimits<float>::Min(), TNumericLimits<float>::Max()); }
	};

	template<>
	struct TTestTypeValues<FTime>
	{
		static FTime Min(const FOperatorSettings& InSettings) { return FTime{TNumericLimits<float>::Min()}; }
		static FTime Max(const FOperatorSettings& InSettings) { return FTime{TNumericLimits<float>::Max()}; }
		static FTime Default(const FOperatorSettings& InSettings) { return FTime{0.f}; }
		static FTime Random(const FOperatorSettings& InSettings) { return FTime{FMath::RandRange(TNumericLimits<float>::Min(), TNumericLimits<float>::Max())}; }
	};

	template<>
	struct TTestTypeValues<FTrigger>
	{
		static FTrigger Min(const FOperatorSettings& InSettings) { return FTrigger{InSettings, false}; }
		static FTrigger Max(const FOperatorSettings& InSettings) 
		{ 
			FTrigger Trigger{InSettings, false};
			for (int32 i = 0; i < InSettings.GetNumFramesPerBlock(); i++)
			{
				Trigger.TriggerFrame(i);
			}
			return Trigger;
		}

		static FTrigger Default(const FOperatorSettings& InSettings) { return FTrigger{InSettings, true}; }
		static FTrigger Random(const FOperatorSettings& InSettings) 
		{ 
			FTrigger Trigger{InSettings, false};
			int32 NumTriggers = FMath::RandRange(0, InSettings.GetNumFramesPerBlock());
			while (NumTriggers > 0)
			{
				Trigger.TriggerFrame(FMath::RandRange(0, InSettings.GetNumFramesPerBlock()));
				NumTriggers--;
			}
			return Trigger;
		}
	};

	template<>
	struct TTestTypeValues<FString>
	{
		static FString Min(const FOperatorSettings& InSettings) { return TEXT(""); }
		static FString Max(const FOperatorSettings& InSettings) { return TEXT("THIS IS SUPPOSED TO REPRESENT A MAXIMUM STRING BUT THERE IS NO SUCH THING SO?"); }
		static FString Default(const FOperatorSettings& InSettings) { return TEXT("TestString"); }
		static FString Random(const FOperatorSettings& InSettings) { return TEXT("We should probably implement a random string."); }
	};

	template<typename DataType>
	struct TDataReferenceMutator : IDataReferenceMutator
	{
		virtual void SetDefault(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const override
		{
			*InDataRef.GetDataWriteReference<DataType>() = TTestTypeValues<DataType>::Default(InSettings);
		}

		virtual void SetMax(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const override
		{
			*InDataRef.GetDataWriteReference<DataType>() = TTestTypeValues<DataType>::Max(InSettings);
		}

		virtual void SetMin(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const override
		{
			*InDataRef.GetDataWriteReference<DataType>() = TTestTypeValues<DataType>::Min(InSettings);
		}

		virtual void SetRandom(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const override
		{
			*InDataRef.GetDataWriteReference<DataType>() = TTestTypeValues<DataType>::Random(InSettings);
		}

		virtual FString ToString(const FAnyDataReference& InDataRef) const override
		{
			if (const DataType* Data = InDataRef.GetValue<DataType>())
			{
				return FString::Printf(TEXT("%s:%s"), *GetMetasoundDataTypeString<DataType>(), *TTestTypeInfo<DataType>::ToString(*Data));
			}
			else
			{
				// Data references should never be null
				UE_LOGF(LogMetaSound, Error, "Failed to get data type value of type %ls", *GetMetasoundDataTypeString<DataType>());
				return TEXT("");
			}
		}

		virtual void SetValue(const FAnyDataReference& InSrcDataRef, const FAnyDataReference& InDstDataRef) const override
		{
			*InDstDataRef.GetDataWriteReference<DataType>() = *InSrcDataRef.GetValue<DataType>();
		}

		virtual FAnyDataReference Copy(const FAnyDataReference& InDataRef) const override
		{
			if (const DataType* Data = InDataRef.GetValue<DataType>())
			{
				return FAnyDataReference{TDataValueReference<DataType>::CreateNew(*Data)};
			}
			else
			{
				// Data references should never be null
				UE_LOGF(LogMetaSound, Error, "Failed to get data type value of type %ls", *GetMetasoundDataTypeString<DataType>());
				return FAnyDataReference{TDataValueReference<int32>::CreateNew()}; // we are going to crash soon.
			}
		}
	};

	template<typename DataType>
	void AddDataReferenceMutatorEntryToMap(TMap<FName, TSharedPtr<const IDataReferenceMutator>>& InMap)
	{
		InMap.Add(GetMetasoundDataTypeName<DataType>(), MakeShared<TDataReferenceMutator<DataType>>());
	}

	// Returns map of mutable input types
	const TMap<FName, TSharedPtr<const IDataReferenceMutator>>& GetDataTypeGeneratorMap()
	{
		static TMap<FName, TSharedPtr<const IDataReferenceMutator>> Map;

		AddDataReferenceMutatorEntryToMap<bool>(Map);
		AddDataReferenceMutatorEntryToMap<int32>(Map);
		AddDataReferenceMutatorEntryToMap<float>(Map);
		AddDataReferenceMutatorEntryToMap<FString>(Map);
		AddDataReferenceMutatorEntryToMap<FTime>(Map);
		AddDataReferenceMutatorEntryToMap<FTrigger>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<bool>>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<int32>>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<float>>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<FString>>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<FTime>>(Map);
		AddDataReferenceMutatorEntryToMap<TArray<FTrigger>>(Map);

		return Map;
	}

	// Convenience class for setting node input data reference values to default, min, max or random values. 
	FInputVertexDataTestController::FInputVertexDataTestController(const FOperatorSettings& InSettings, const FInputVertexInterface& InInputInterface, const FInputVertexInterfaceData& InInputData)
	: Settings(InSettings)
	{
		const TMap<FName, TSharedPtr<const IDataReferenceMutator>>& GeneratorMap = GetDataTypeGeneratorMap();

		for (const FInputDataVertex& Vertex : InInputInterface)
		{
			if (GeneratorMap.Contains(Vertex.DataTypeName))
			{
				if (const FAnyDataReference* Ref = InInputData.FindDataReference(Vertex.VertexName))
				{
					if (EDataReferenceAccessType::Write == Ref->GetAccessType())
					{
						MutableInputs.Add(FMutableInput{*Ref, Vertex.VertexName, GeneratorMap[Vertex.DataTypeName]});
					}
				}
			}
		}
	}

	int32 FInputVertexDataTestController::GetNumMutableInputs() const
	{
		return MutableInputs.Num();
	}

	FInterfaceState FInputVertexDataTestController::GetInterfaceState() const
	{
		FInterfaceState State;
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			State.Add(MutableInput.VertexName, MutableInput.DataReferenceMutator->Copy(MutableInput.DataReference));
		}
		return State;
	}

	void FInputVertexDataTestController::SetMutableInputsToState(const FInterfaceState& InState) 
	{
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			if (const FAnyDataReference* Value = InState.Find(MutableInput.VertexName))
			{
				MutableInput.DataReferenceMutator->SetValue(*Value, MutableInput.DataReference);
			}
		}
		UE_LOGF(LogMetaSound, Verbose, "Setting operator input values:%ls%ls", LINE_TERMINATOR, *FString::Join(GetInputValueStrings(), LINE_TERMINATOR));
	}

	void FInputVertexDataTestController::SetMutableInputsToMin()
	{
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			MutableInput.DataReferenceMutator->SetMin(Settings, MutableInput.DataReference);
		}
		UE_LOGF(LogMetaSound, Verbose, "Setting operator input values:%ls%ls", LINE_TERMINATOR, *FString::Join(GetInputValueStrings(), LINE_TERMINATOR));
	}

	void FInputVertexDataTestController::SetMutableInputsToMax()
	{
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			MutableInput.DataReferenceMutator->SetMax(Settings, MutableInput.DataReference);
		}
		UE_LOGF(LogMetaSound, Verbose, "Setting operator input values:%ls%ls", LINE_TERMINATOR, *FString::Join(GetInputValueStrings(), LINE_TERMINATOR));
	}

	void FInputVertexDataTestController::SetMutableInputsToDefault()
	{
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			MutableInput.DataReferenceMutator->SetDefault(Settings, MutableInput.DataReference);
		}
		UE_LOGF(LogMetaSound, Verbose, "Setting operator input values:%ls%ls", LINE_TERMINATOR, *FString::Join(GetInputValueStrings(), LINE_TERMINATOR));
	}

	void FInputVertexDataTestController::SetMutableInputsToRandom()
	{
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			MutableInput.DataReferenceMutator->SetRandom(Settings, MutableInput.DataReference);
		}
		UE_LOGF(LogMetaSound, Verbose, "Setting operator input values:%ls%ls", LINE_TERMINATOR, *FString::Join(GetInputValueStrings(), LINE_TERMINATOR));
	}

	TArray<FString> FInputVertexDataTestController::GetInputValueStrings() const
	{
		TArray<FString> ValueStrings;
		for (const FMutableInput& MutableInput : MutableInputs)
		{
			ValueStrings.Add(FString::Printf(TEXT("%s %s"), *MutableInput.VertexName.ToString(), *MutableInput.DataReferenceMutator->ToString(MutableInput.DataReference)));
		}
		return ValueStrings;
	}


	static const FLazyName TestNodeName{"TEST_NODE"};
	static const FLazyName TestVertexName{"TEXT_VERTEX"};
	static const FGuid TestNodeID{0xA5A5A5A5, 0xA5A5A5A5, 0xA5A5A5A5, 0xA5A5A5A5};

	// Create a node from a node registry key
	TUniquePtr<INode> CreateNodeFromRegistry(const Frontend::FNodeRegistryKey& InNodeRegistryKey)
	{
		using namespace Frontend;

		TUniquePtr<INode> Node;

		FMetasoundFrontendRegistryContainer* NodeRegistry = FMetasoundFrontendRegistryContainer::Get();
		check(nullptr != NodeRegistry);
		IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

		// Lookup node class metadata to determine how to create this node.
		FMetasoundFrontendClass NodeClass;
		if (!NodeRegistry->FindFrontendClassFromRegistered(InNodeRegistryKey, NodeClass))
		{
			UE_LOGF(LogMetaSound, Error, "Failed to find registered class with registry key %ls", *InNodeRegistryKey.ToString());
			return MoveTemp(Node);
		}

		// Build node differently dependent upon node type
		switch (NodeClass.Metadata.GetType())
		{
			case EMetasoundFrontendClassType::VariableDeferredAccessor:
			case EMetasoundFrontendClassType::VariableAccessor:
			case EMetasoundFrontendClassType::VariableMutator:
			case EMetasoundFrontendClassType::External:
			case EMetasoundFrontendClassType::Graph:
			{
				FVertexInterface VertexInterface;
				NodeRegistry->FindDefaultVertexInterface(InNodeRegistryKey, VertexInterface);
				Node = NodeRegistry->CreateNode(InNodeRegistryKey, FNodeData{ TestNodeName, TestNodeID , VertexInterface });
			}
			break;

			case EMetasoundFrontendClassType::Input:
			{
				FName DataTypeName = NodeClass.Metadata.GetClassName().Name;
				TSharedPtr<const FNodeClassMetadata> InputClassMetadata = DataTypeRegistry.GetInputClassMetadata(DataTypeName);
				checkf(InputClassMetadata, TEXT("Input class metadata could not be found from data type registry"));

				FVertexInterface InputNodeInterface = InputClassMetadata->DefaultInterface;
				// Vertex names must be set for input nodes
				InputNodeInterface.GetInputInterface().At(0).VertexName = TestVertexName;
				InputNodeInterface.GetOutputInterface().At(0).VertexName = TestVertexName;
				InputNodeInterface.GetInputInterface().At(0).SetDefaultLiteral(NodeClass.GetDefaultInterface().Inputs[0].FindConstDefaultChecked(Frontend::DefaultPageID).ToLiteral(DataTypeName));

				Node = DataTypeRegistry.CreateInputNode(DataTypeName, FNodeData{ TestNodeName, TestNodeID, InputNodeInterface });
			}
			break;

			case EMetasoundFrontendClassType::Variable:
			{
				FName DataTypeName = NodeClass.Metadata.GetClassName().Name;
				TSharedPtr<const FNodeClassMetadata> VariableMetadata = DataTypeRegistry.GetVariableClassMetadata(DataTypeName);
				checkf(VariableMetadata, TEXT("Variable class metadata could not be found from data type registry"));
				Node = DataTypeRegistry.CreateVariableNode(DataTypeName, DataTypeRegistry.CreateDefaultLiteral(DataTypeName),
					FNodeData{ TestNodeName, TestNodeID, VariableMetadata->DefaultInterface });
			}
			break;

			case EMetasoundFrontendClassType::Literal:
			{
				FName DataTypeName = NodeClass.Metadata.GetClassName().Name;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FDefaultLiteralNodeConstructorParams NodeInitData{TestNodeName, TestNodeID, DataTypeRegistry.CreateDefaultLiteral(DataTypeName)};
				Node = DataTypeRegistry.CreateLiteralNode(DataTypeName, MoveTemp(NodeInitData));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			break;

			case EMetasoundFrontendClassType::Output:
			{
				FName DataTypeName = NodeClass.Metadata.GetClassName().Name;
				TSharedPtr<const FNodeClassMetadata> OutputClassMetadata = DataTypeRegistry.GetOutputClassMetadata(DataTypeName);
				checkf(OutputClassMetadata, TEXT("Output class metadata could not be found from data type registry"));

				FVertexInterface OutputNodeInterface = OutputClassMetadata->DefaultInterface;
				// Vertex names must be set for output nodes
				OutputNodeInterface.GetInputInterface().At(0).VertexName = TestVertexName;
				OutputNodeInterface.GetOutputInterface().At(0).VertexName = TestVertexName;
				Node = DataTypeRegistry.CreateOutputNode(DataTypeName, FNodeData{ TestNodeName, TestNodeID, OutputNodeInterface });
			}
			break;

			case EMetasoundFrontendClassType::Template:
			default:
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missed EMetasoundFrontendClassType case coverage");
		}

		return MoveTemp(Node);
	}


	void GetAllRegisteredNodes(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands, bool bInIncludeDeprecatedNodes)
	{
		using namespace Frontend;

		// Get all the classes that have been registered
		constexpr bool bIncludeUnloadedAssets = false;
		ISearchEngine& NodeSearchEngine = ISearchEngine::Get();
		TArray<FMetaSoundClassInfo> AllClasses = NodeSearchEngine.FindAllClasses(ISearchEngine::EResultVersion::All, bIncludeUnloadedAssets);

		for (const FMetaSoundClassInfo& NodeClass : AllClasses)
		{
			// Exclude template classes because they cannot be created directly from the node registry
			if (NodeClass.ClassType == EMetasoundFrontendClassType::Template)
			{
				continue;
			}
			else if (!bInIncludeDeprecatedNodes && EnumHasAnyFlags(NodeClass.AccessFlags, EMetasoundFrontendClassAccessFlags::Deprecated))
			{
				// Optionally skip deprecated classes. 
				continue;
			}

			OutBeautifiedNames.Add(FString::Printf(TEXT("%s %s"), *NodeClass.ClassName.ToString(), *NodeClass.Version.ToString()));

			// Test commands are node registry keys
			OutTestCommands.Add(NodeClass.ToRegistryKey().ToString());
		}
	}

	void GetAllRegisteredNativeNodes(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands, bool bInIncludeDeprecatedNodes)
	{
		using namespace Frontend;

		// Get all the classes that have been registered
		constexpr bool bIncludeUnloadedAssets = false;
		ISearchEngine& NodeSearchEngine = Frontend::ISearchEngine::Get();
		TArray<FMetaSoundClassInfo> AllClasses = NodeSearchEngine.FindAllClasses(Frontend::ISearchEngine::EResultVersion::All, bIncludeUnloadedAssets);

		FMetasoundFrontendRegistryContainer* NodeRegistry = FMetasoundFrontendRegistryContainer::Get();
		check(nullptr != NodeRegistry);

		IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get();
		for (const FMetaSoundClassInfo& NodeClass : AllClasses)
		{
			// Exclude template classes because they cannot be created directly from the node registry
			if (NodeClass.ClassType == EMetasoundFrontendClassType::Template)
			{
				continue;
			}
			else if (!bInIncludeDeprecatedNodes && EnumHasAnyFlags(NodeClass.AccessFlags, EMetasoundFrontendClassAccessFlags::Deprecated))
			{
				// Optionally skip deprecated classes. 
				continue;
			}

			// Exclude asset-defined node classes
			const FNodeRegistryKey NodeRegistryKey = NodeClass.ToRegistryKey();
			if (AssetManager && AssetManager->ContainsKey(NodeRegistryKey))
			{
				continue;
			}

			OutBeautifiedNames.Add(FString::Printf(TEXT("%s %s"), *NodeClass.ClassName.ToString(), *NodeClass.Version.ToString()));

			// Test commands are node registry keys
			OutTestCommands.Add(NodeRegistryKey.ToString());
		}
	}

	void CreateVariables(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData)
	{
		using namespace VertexDataPrivate;

		// There is currently no easy way to instantiate variables. The only way
		// they are instantiated is within the TVariableNode's IOperator. In order
		// to create Variable inputs to nodes, we 
		// - Create a TVariableNode
		// - Create a IOperator from the TVariableNode
		// - Access the variable from the outputs of the IOperator

		Frontend::IDataTypeRegistry& DataTypeRegistry = Frontend::IDataTypeRegistry::Get();

		for (const FInputBinding& Binding : OutVertexData)
		{
			const FInputDataVertex& InputVertex = Binding.GetVertex();

			if (const Frontend::IDataTypeRegistryEntry* VariableEntry = DataTypeRegistry.FindDataTypeRegistryEntry(InputVertex.DataTypeName))
			{
				if (VariableEntry->GetDataTypeInfo().bIsVariable)
				{
					// Find the data type name of the data type wrapped by the variable
					FName UnderlyingDataTypeName = *InputVertex.DataTypeName.ToString().Replace(TEXT(METASOUND_DATA_TYPE_NAME_VARIABLE_TYPE_SPECIFIER), TEXT(""));

					if (const Frontend::IDataTypeRegistryEntry* DataTypeEntry = DataTypeRegistry.FindDataTypeRegistryEntry(UnderlyingDataTypeName))
					{
						TSharedPtr<const FNodeClassMetadata> VariableNodeMetadata = DataTypeEntry->GetVariableClassMetadata();
						if (!VariableNodeMetadata)
						{
							continue;
						}

						// Create a variable node for the underlying data type
						TUniquePtr<INode> VariableNode = DataTypeEntry->CreateVariableNode(
							DataTypeRegistry.CreateDefaultLiteral(UnderlyingDataTypeName),
							FNodeData{"VariableNodeName", FGuid::NewGuid(), VariableNodeMetadata->DefaultInterface});

						if (VariableNode.IsValid())
						{
							// Create the variable operator for the underlying data type
							FInputVertexInterfaceData InputData(VariableNode->GetVertexInterface().GetInputInterface());
							FBuildOperatorParams BuildParams
							{
								*VariableNode,
								InOperatorSettings,
								InputData,
								FMetasoundEnvironment{}
							};
							FBuildResults OutResults;
							TUniquePtr<IOperator> VariableOperator = VariableNode->GetDefaultOperatorFactory()->CreateOperator(BuildParams, OutResults);

							if (VariableOperator.IsValid())
							{
								// Access the TVariable data referenced created within the operator.
								FOutputVertexInterfaceData OutOperatorData;
								VariableOperator->BindOutputs(OutOperatorData);
								if (const FAnyDataReference* VariableRef = OutOperatorData.FindDataReference(METASOUND_GET_PARAM_NAME(VariableNames::OutputVariable)))
								{
									OutVertexData.SetVertex(InputVertex.VertexName, *VariableRef);
								}
							}
						}
					}
				}
			}
		}
	}

	void CreateDefaultsAndVariables(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData)
	{
		Frontend::CreateDefaults(InOperatorSettings, OutVertexData);
		CreateVariables(InOperatorSettings, OutVertexData);
	}
}

#endif // WITH_EDITORONLY_DATA

#endif //WITH_DEV_AUTOMATION_TESTS

