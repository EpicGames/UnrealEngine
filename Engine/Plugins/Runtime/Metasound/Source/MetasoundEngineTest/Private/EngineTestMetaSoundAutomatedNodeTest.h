// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreFwd.h"
#include "MetasoundDataReference.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITORONLY_DATA

// Forward declare
namespace Audio
{
	class FMixerDevice;
}

namespace Metasound
{
	class FMetasoundEnvironment;
	class INode;
	class FOutputVertexInterfaceData;
	class FInputVertexInterfaceData;
}

namespace Metasound::EngineTest 
{
	// Return audio mixer device if one is available
	Audio::FMixerDevice* GetAudioMixerDevice();

	// Create an example environment that generally exists for a UMetaSoundSoruce
	FMetasoundEnvironment GetSourceEnvironmentForTest();

	// Return string for printing test errors
	FString GetPrettyName(const Frontend::FNodeRegistryKey& InRegistryKey);

	// Interface for data reference analyzers.
	struct IDataReferenceAnalyzer
	{
		virtual ~IDataReferenceAnalyzer() = default;
		virtual FAnyDataReference Copy(const FAnyDataReference& InDataRef) const = 0;
		virtual bool IsEqual(const FAnyDataReference& InLHS, const FAnyDataReference& InRHS) const = 0;
		virtual bool IsValid(const FAnyDataReference& InDataRef) const = 0;
		virtual FString ToString(const FAnyDataReference& InDataRef) const = 0;
	};

	// Convenience class for setting node input data reference values to default, min, max or random values. 
	struct FOutputVertexDataTestController
	{
		struct FAnalyzableOutput
		{
			FAnyDataReference CapturedValue;
			FAnyDataReference DataReference;
			FVertexName VertexName;
			TSharedPtr<const IDataReferenceAnalyzer> DataReferenceAnalyzer;

			void CaptureValue();

			bool IsDataReferenceValid() const;

			bool IsDataReferenceEqualToCapturedValue() const;

			FString DataReferenceToString() const;

			FString CapturedValueToString() const;
		};

		FOutputVertexDataTestController( const FOutputVertexInterface& InOutputInterface, const FOutputVertexInterfaceData& InOutputData);

		int32 GetNumAnalyzableOutputs() const;

		bool AreAllAnalyzableOutputsValid() const;

		void CaptureCurrentOutputValues();

		bool AreAllOutputValuesEqualToCapturedValues() const;

	private:

		TArray<FAnalyzableOutput> AnalyzableOutputs;
	};

	// Interface for mutating data references
	struct IDataReferenceMutator
	{
		virtual ~IDataReferenceMutator() = default;
		virtual void SetDefault(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const = 0;
		virtual void SetMax(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const = 0;
		virtual void SetMin(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const = 0; 
		virtual void SetRandom(const FOperatorSettings& InSettings, const FAnyDataReference& InDataRef) const = 0;
		virtual FAnyDataReference Copy(const FAnyDataReference& InDataRef) const = 0;
		virtual void SetValue(const FAnyDataReference& InSrcDataRef, const FAnyDataReference& InDstDataRef) const = 0;
		virtual FString ToString(const FAnyDataReference& InDataRef) const = 0;
	};

	// Captures current data references on interface
	using FInterfaceState = TSortedVertexNameMap<FAnyDataReference>;

	// Convenience class for setting node input data reference values to default, min, max or random values. 
	struct FInputVertexDataTestController
	{
		struct FMutableInput
		{
			FAnyDataReference DataReference;
			FVertexName VertexName;
			TSharedPtr<const IDataReferenceMutator> DataReferenceMutator;
		};

		FInputVertexDataTestController(const FOperatorSettings& InSettings, const FInputVertexInterface& InInputInterface, const FInputVertexInterfaceData& InInputData);

		int32 GetNumMutableInputs() const;
		FInterfaceState GetInterfaceState() const;

		void SetMutableInputsToState(const FInterfaceState& InState);
		void SetMutableInputsToMin();
		void SetMutableInputsToMax();
		void SetMutableInputsToDefault();
		void SetMutableInputsToRandom();
		TArray<FString> GetInputValueStrings() const;

	private:

		FOperatorSettings Settings;
		TArray<FMutableInput> MutableInputs;
	};


	// Create a node from a node registry key
	TUniquePtr<INode> CreateNodeFromRegistry(const Frontend::FNodeRegistryKey& InNodeRegistryKey);

	// Return an array of all currently registered MetaSound nodes. 
	void GetAllRegisteredNodes(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutNodeRegistryKeys, bool bInIncludeDeprecatedNodes);

	// Return an array of all currently registered native MetaSound nodes. 
	void GetAllRegisteredNativeNodes(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutNodeRegistryKeys, bool bInIncludeDeprecatedNodes);

	// Create any variables that exist on the vertex interface.
	void CreateVariables(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData);

	// Create default references and variables for the vertex interface data.
	void CreateDefaultsAndVariables(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData);

} // namespace Metasound::EngineTest

#endif // WITH_EDITORONLY_DATA 

#endif // WITH_DEV_AUTOMATION_TESTS

