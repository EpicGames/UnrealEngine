// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/AllOf.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EngineTestMetaSoundAutomatedNodeTest.h"
#include "Misc/AutomationTest.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundLog.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITORONLY_DATA

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMetasoundAutomatedNodeTest_Bind, "Audio.Metasound.AutomatedNodeTest.Bind", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FMetasoundAutomatedNodeTest_Bind::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	using namespace Metasound::EngineTest;
	constexpr bool bIncludeDeprecatedNodes = false;
	GetAllRegisteredNativeNodes(OutBeautifiedNames, OutTestCommands, bIncludeDeprecatedNodes);

	UE_LOGF(LogMetaSound, Verbose, "Found %d metasound nodes to test", OutTestCommands.Num());
}

bool FMetasoundAutomatedNodeTest_Bind::RunTest(const FString& InRegistryKeyString)
{
	using namespace Metasound;
	using namespace Metasound::EngineTest;
	using namespace Metasound::Frontend;

	static const FOperatorSettings OperatorSettings{48000  /* samplerate */, 100.f /* block rate */};
	static const FMetasoundEnvironment SourceEnvironment = GetSourceEnvironmentForTest();

	FNodeRegistryKey RegistryKey;
	if (!FNodeRegistryKey::Parse(InRegistryKeyString, RegistryKey))
	{
		AddError(FString::Printf(TEXT("Failed to parse registry key string %s"), *InRegistryKeyString));
		return false;
	}

	TUniquePtr<INode> Node = CreateNodeFromRegistry(RegistryKey);
	if (!Node.IsValid())
	{
		AddError(FString::Printf(TEXT("Failed to create node %s from registry"), *InRegistryKeyString));
		return false;
	}

	// Populate inputs to node. 
	const FInputVertexInterface InputInterface = Node->GetVertexInterface().GetInputInterface();
	FInputVertexInterfaceData BuildInputVertexData(InputInterface);
	CreateDefaultsAndVariables(OperatorSettings, BuildInputVertexData);

	TArray<FVertexDataState> BuildInputVertexDataState;
	GetVertexInterfaceDataState(BuildInputVertexData, BuildInputVertexDataState);

	// Create operator 
	FBuildOperatorParams BuildParams
	{
		*Node,
		OperatorSettings,
		BuildInputVertexData,
		SourceEnvironment
	};

	// Create the operator from the node factory
	FBuildResults BuildResults;
	TUniquePtr<IOperator> Operator = Node->GetDefaultOperatorFactory()->CreateOperator(BuildParams, BuildResults);
	if (!Operator.IsValid())
	{
		AddError(FString::Printf(TEXT("Failed to create operator from node %s - %s."), *InRegistryKeyString, *GetPrettyName(RegistryKey)));
		return false;
	}

	// Query the data on the operator
	FVertexInterfaceData AfterBuildVertexData;
	Operator->BindInputs(AfterBuildVertexData.GetInputs());
	Operator->BindOutputs(AfterBuildVertexData.GetOutputs());

	// Check that build and queried data is the same.
	for (const FInputDataVertex& InputVertex : InputInterface)
	{
		const bool bIsVertexBoundDuringBuild = BuildInputVertexData.IsVertexBound(InputVertex.VertexName);
		const bool bIsVertexBoundAfterBuild = AfterBuildVertexData.GetInputs().IsVertexBound(InputVertex.VertexName);
		
		if (bIsVertexBoundDuringBuild  && !bIsVertexBoundAfterBuild)
		{
			// This means that a data reference was passed into the CreateOperator(...)
			// function, but when IOperator::BindInputs(...) is called, the vertex 
			// is not bound with any data. If data is sent to the CreateOperator(...)
			// InputData, it should be reflected in the inputs of the IOperator through
			// the BindInputs(...) method.
			AddError(FString::Printf(TEXT("Input vertex (%s) of node (%s) does not bind supplied input data. Check that CreateOperator(...) and BindInputs(...) are correctly implemented for the node's operator."), *InputVertex.VertexName.ToString(), *GetPrettyName(RegistryKey)));
		}
		else if (bIsVertexBoundDuringBuild && bIsVertexBoundAfterBuild)
		{
			// Input value vertices should bind to equal values. In order to detect
			// correct functionality, we would need to check whether the values bound
			// to the vertices are equal. Unfortunately we don't support a method
			// for doing equality comparison based upon bound vertex data. 
			if (InputVertex.AccessType != EVertexAccessType::Value)
			{
				const FAnyDataReference* BuildRef = BuildInputVertexData.FindDataReference(InputVertex.VertexName);
				check(BuildRef);
				const FDataReferenceID BuildDataRefID = GetDataReferenceID(*BuildRef);

				const FAnyDataReference* AfterBuildRef = AfterBuildVertexData.GetInputs().FindDataReference(InputVertex.VertexName);
				check(AfterBuildRef);
				const FDataReferenceID AfterBuildDataRefID = GetDataReferenceID(*AfterBuildRef);

				if (BuildDataRefID != AfterBuildDataRefID)
				{
					// This means that a data reference was passed into the CreateOperator(...)
					// function, but when IOperator::BindInputs(...) is called, the vertex 
					// is bound to different data. If data is sent to the CreateOperator(...)
					// InputData, it should be reflected in the inputs of the IOperator through
					// the BindInputs(...) method and be unchanged in this scenario.
					AddError(FString::Printf(TEXT("Input vertex (%s) of node (%s) binds different data than the supplied at input data. Check that CreateOperator(...) and BindInputs(...) are correctly implemented for the node's operator."), *InputVertex.VertexName.ToString(), *GetPrettyName(RegistryKey)));
				}
			}
		}
	}

	// Cache current state of IOperator bindings
	TArray<FVertexDataState> AfterBuildInputVertexDataState;
	GetVertexInterfaceDataState(AfterBuildVertexData.GetInputs(), AfterBuildInputVertexDataState);

	// Create new input data to override existing bindings.
	FInputVertexInterfaceData UpdateToInputVertexData(Node->GetVertexInterface().GetInputInterface());
	Frontend::CreateDefaults(OperatorSettings, UpdateToInputVertexData); // Note: Do not rebind new TVariables. This is not allowed. 

	// Compare what is currently bound to what will be overridden
	TSortedVertexNameMap<FAnyDataReference> ExpectedUpdates;
	CompareVertexInterfaceDataToPriorState(UpdateToInputVertexData, AfterBuildInputVertexDataState, ExpectedUpdates);

	// Perform the rebind which will override the current inputs
	Operator->BindInputs(UpdateToInputVertexData);
	
	// Query the operator to find what the current inputs are.
	FVertexInterfaceData NewVertexData;
	Operator->BindInputs(NewVertexData.GetInputs());
	Operator->BindOutputs(NewVertexData.GetOutputs());

	// Compare the prior queried inputs to the new queried inputs to see what data
	// references were actually updated.
	TSortedVertexNameMap<FAnyDataReference> ActualUpdates;
	CompareVertexInterfaceDataToPriorState(NewVertexData.GetInputs(), AfterBuildInputVertexDataState, ActualUpdates);

	// Check that the expected updates were reflected in the IOperator::BindInputs call.
	for (const FInputDataVertex& InputVertex : InputInterface)
	{
		const bool bIsUpdateExpected = ExpectedUpdates.Contains(InputVertex.VertexName);
		const bool bIsUpdateActualized = ActualUpdates.Contains(InputVertex.VertexName);

		if (bIsUpdateExpected && !bIsUpdateActualized)
		{
			// This means that the IOperator was given new data references during
			// the call to IOperator::BindInputs(...), but there was no data bound
			// on subsequent calls to IOperator::BindInputs(...).  If new data is 
			// bound to the operator inputs, it should reflect that new data on subsequent
			// calls to IOperator::BindInputs(...)
			AddWarning(FString::Printf(TEXT("Input vertex (%s) of node (%s) does not reflect updated data reference when rebound. This node will not operator as expected in a dynamic graph. Check that CreateOperator(...) and BindInputs(...) are correctly implemented for the node's operator."), *InputVertex.VertexName.ToString(), *GetPrettyName(RegistryKey)));
		}
		else if (!bIsUpdateExpected && bIsUpdateActualized)
		{
			// This means that the IOperator was given new data references during
			// the call to IOperator::BindInputs(...), but there was no data bound
			// on subsequent calls to IOperator::BindInputs(...).  If new data is 
			// bound to the operator inputs, it should reflect that new data on subsequent
			// calls to IOperator::BindInputs(...)
			AddWarning(FString::Printf(TEXT("Input vertex (%s) of node (%s) reflects unintended updated of data reference when rebound. This node will not operator as expected in a dynamic graph. Check that CreateOperator(...) and BindInputs(...) are correctly implemented for the node's operator."), *InputVertex.VertexName.ToString(), *GetPrettyName(RegistryKey)));
		}
	}

	return true;
}

#endif // WITH_EDITORONLY_DATA

#endif //WITH_DEV_AUTOMATION_TESTS

