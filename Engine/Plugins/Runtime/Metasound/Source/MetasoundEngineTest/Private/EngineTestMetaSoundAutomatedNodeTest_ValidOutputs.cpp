// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/AllOf.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EngineTestMetaSoundAutomatedNodeTest.h"
#include "Misc/AutomationTest.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundLog.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITORONLY_DATA


IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMetasoundAutomatedNodeTest_ValidOutputs, "Audio.Metasound.AutomatedNodeTest.ValidOutputs", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FMetasoundAutomatedNodeTest_ValidOutputs::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	using namespace Metasound::EngineTest;
	constexpr bool bIncludeDeprecatedNodes = false;
	GetAllRegisteredNativeNodes(OutBeautifiedNames, OutTestCommands, bIncludeDeprecatedNodes);

	FAutomationTestFramework& TestFramework = FAutomationTestFramework::Get();
	for (const FString& ComplexTestName : OutBeautifiedNames)
	{
		bool Registered = TestFramework.RegisterComplexAutomationTestTags(this, ComplexTestName, TEXT("[MustPass]"));
		if (!Registered)
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to register automation test tags for test %ls", *ComplexTestName);
		}
	}

	UE_LOGF(LogMetaSound, Verbose, "Found %d metasound nodes to test", OutTestCommands.Num());
}

bool FMetasoundAutomatedNodeTest_ValidOutputs::RunTest(const FString& InRegistryKeyString)
{
	using namespace Metasound;
	using namespace Metasound::EngineTest;
	using namespace Metasound::Frontend;

	static const FOperatorSettings OperatorSettings{48000  /* samplerate */, 100.f /* block rate */};
	static const FMetasoundEnvironment SourceEnvironment = GetSourceEnvironmentForTest();

	FNodeRegistryKey RegistryKey;
	if (!FNodeRegistryKey::Parse(InRegistryKeyString, RegistryKey))
	{
		AddError(FString::Printf(TEXT("Failed to parse registry key string %s"), *InRegistryKeyString));
		return false;
	}

	TUniquePtr<INode> Node = CreateNodeFromRegistry(RegistryKey);
	if (!Node.IsValid())
	{
		AddError(FString::Printf(TEXT("Failed to create node %s from registry"), *InRegistryKeyString));
		return false;
	}

	// Populate inputs to node. 
	FInputVertexInterface InputInterface = Node->GetVertexInterface().GetInputInterface();
	FInputVertexInterfaceData InputVertexData(InputInterface);
	CreateDefaultsAndVariables(OperatorSettings, InputVertexData);

	FInputVertexDataTestController InputTester(OperatorSettings, InputInterface, InputVertexData);

	// Create operator 
	FBuildOperatorParams BuildParams
	{
		*Node,
		OperatorSettings,
		InputVertexData,
		SourceEnvironment
	};

	IOperator::FResetParams ResetParams
	{
		OperatorSettings,
		SourceEnvironment
	};

	// Convenience function for testing entire lifecycle of an individual operator 
	// with a variety of inputs.
	auto RunTestIteration = [&]()
	{
		// Create the operator from the node factory
		FBuildResults BuildResults;
		TUniquePtr<IOperator> Operator = Node->GetDefaultOperatorFactory()->CreateOperator(BuildParams, BuildResults);

		if (!Operator.IsValid())
		{
			AddError(FString::Printf(TEXT("Failed to create operator from node %s - %s."), *InRegistryKeyString, *GetPrettyName(RegistryKey)));
			return;
		}


		// Store a copy of the input values data values so the inputs
		// can be reset to this value later during the test.
		FInterfaceState InitialInputState = InputTester.GetInterfaceState();

		// Bind to inputs and output data of operator.
		FVertexInterface VertexInterface = Node->GetVertexInterface();
		FVertexInterfaceData VertexInterfaceData{VertexInterface};
		Operator->BindInputs(VertexInterfaceData.GetInputs());
		Operator->BindOutputs(VertexInterfaceData.GetOutputs());

		// Create output tester which will analyzer outputs of operator.
		FOutputVertexDataTestController OutputTester{VertexInterface.GetOutputInterface(), VertexInterfaceData.GetOutputs()};
		// Convenience method for printing errors if the OutputTester finds an error.
		auto CheckOutputValuesAreValid = [&]()
		{
			if (!OutputTester.AreAllAnalyzableOutputsValid())
			{
				AddError(FString::Printf(TEXT("Invalid output value encountered from node %s - %s."), *InRegistryKeyString, *GetPrettyName(RegistryKey)));
			}
		};
		// Initial values should all be valid.
		CheckOutputValuesAreValid();
		
		// Capture current state of outputs so they can be referenced at a later time.
		// The captured values are held within the OutputTester.
		OutputTester.CaptureCurrentOutputValues();

		IOperator::FExecuteFunction OpExecFunc = Operator->GetExecuteFunction();
		IOperator::FResetFunction OpResetFunc = Operator->GetResetFunction();
		// Test execute function with input variations
		if (OpExecFunc)
		{
			OpExecFunc(Operator.Get());
			CheckOutputValuesAreValid();

			if (InputTester.GetNumMutableInputs() > 0)
			{
				InputTester.SetMutableInputsToDefault();
				OpExecFunc(Operator.Get());
				CheckOutputValuesAreValid();

				InputTester.SetMutableInputsToMin();
				OpExecFunc(Operator.Get());
				CheckOutputValuesAreValid();

				InputTester.SetMutableInputsToMax();
				OpExecFunc(Operator.Get());
				CheckOutputValuesAreValid();

				InputTester.SetMutableInputsToRandom();
				OpExecFunc(Operator.Get());
				CheckOutputValuesAreValid();
			}
		}

		// Return inputs to initial state. 
		InputTester.SetMutableInputsToState(InitialInputState);

		if (OpResetFunc)
		{
			OpResetFunc(Operator.Get(), ResetParams);
			CheckOutputValuesAreValid();
		}
	};

	// Test entire operator lifecycle with different starting conditions if
	// any of the inputs are mutable
	InputTester.SetMutableInputsToDefault();
	RunTestIteration();

	if (InputTester.GetNumMutableInputs() > 0)
	{
		InputTester.SetMutableInputsToMin();
		RunTestIteration();
		InputTester.SetMutableInputsToMax();
		RunTestIteration();
		InputTester.SetMutableInputsToRandom();
		RunTestIteration();
	}

	return true;
}

#endif // WITH_EDITORONLY_DATA

#endif //WITH_DEV_AUTOMATION_TESTS

