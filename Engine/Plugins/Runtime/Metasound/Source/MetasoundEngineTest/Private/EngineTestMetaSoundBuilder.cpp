// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineTestMetaSoundBuilder.h"

#if WITH_DEV_AUTOMATION_TESTS

bool FMetaSoundSourceBuilderDisconnectInputLatentCommand::Update()
{
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	if (Builder)
	{
		Builder->DisconnectNodeInput(InputToDisconnect, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to disconnect MetaSound node's input"));
	}

	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool FMetaSoundSourceBuilderSetLiteralLatentCommand::Update()
{
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	if (Builder)
	{
		Builder->SetNodeInputDefault(NodeInput, NewValue, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to set MetaSound node's input default"));
	}

	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool FMetaSoundSourceBuilderRemoveNodeDefaultLiteralLatentCommand::Update()
{
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	if (Builder)
	{
		Builder->RemoveNodeInputDefault(NodeInput, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to remove MetaSound node's input default"));
	}

	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool FMetaSoundSourceBuilderCreateAndConnectTriGeneratorNodeLatentCommand::Update()
{
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	if (Builder)
	{
		// Tri Oscillator Node
		FMetaSoundNodeHandle TriNode = Builder->AddNodeByClassName({ "UE", "Triangle", "Audio" }, Result, 1);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && TriNode.IsSet(), TEXT("Failed to create node by class name 'UE:Triangle:Audio"));

		FMetaSoundBuilderNodeOutputHandle TriNodeAudioOutput = Builder->FindNodeOutputByName(TriNode, "Audio", Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to find Triangle Oscillator node output 'Audio'"));

		Builder->ConnectNodes(TriNodeAudioOutput, AudioOutNodeInput, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to connect 'Audio' Triangle Oscillator output to MetaSound graph's 'Mono Output'"));
	}

	return Result == EMetaSoundBuilderResult::Succeeded;
}

bool FBuilderRemoveFromRootLatentCommand::Update()
{
	if (Builder)
	{
		Builder->RemoveFromRoot();
		return true;
	}
	return false;
}

// Creates a collection of MetaSound patches from builders and exercises bindings all input and output interfaces by connecting and disconnecting using various builder API calls.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetaSoundBuilderInterfaceBindingConnectAndDisconnect, "Audio.Metasound.Builder.InterfaceBindingConnectAndDisconnect", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetaSoundBuilderInterfaceBindingConnectAndDisconnect, "Audio.Metasound.Builder.InterfaceBindingConnectAndDisconnect", "[MustPass]")
bool FAudioMetaSoundBuilderInterfaceBindingConnectAndDisconnect::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace Metasound::Engine;

	const TArray<FName> InputInterfaces =
	{
		InputFormatMonoInterface::GetVersion().Name,
		InputFormatStereoInterface::GetVersion().Name,
		InputFormatQuadInterface::GetVersion().Name,
		InputFormatFiveDotOneInterface::GetVersion().Name,
		InputFormatSevenDotOneInterface::GetVersion().Name
	};
	const TArray<FName> OutputInterfaces =
	{
		OutputFormatMonoInterface::GetVersion().Name,
		OutputFormatStereoInterface::GetVersion().Name,
		OutputFormatQuadInterface::GetVersion().Name,
		OutputFormatFiveDotOneInterface::GetVersion().Name,
		OutputFormatSevenDotOneInterface::GetVersion().Name
	};

	auto MakePatchName = [](FName InterfaceName) { return FString::Printf(TEXT("%s Patch"), *InterfaceName.ToString()); };
	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	for (const FName& InputInterface : InputInterfaces)
	{
		if (UMetaSoundPatch* InputBuilder = CreatePatchFromBuilder(*this, MakePatchName(InputInterface), { InputInterface }))
		{
			for (const FName& OutputInterface : OutputInterfaces)
			{
				if (UMetaSoundPatch* OutputBuilder = CreatePatchFromBuilder(*this, MakePatchName(OutputInterface), { OutputInterface }))
				{
					const FString BindBuilderName = FString::Printf(TEXT("%s to %s Binding"), *OutputInterface.ToString(), *InputInterface.ToString());
					UMetaSoundPatchBuilder& BindingBuilder = CreatePatchBuilderChecked(*this, *BindBuilderName, { InputInterface, OutputInterface });
					FMetaSoundNodeHandle InputInterfaceNode = BindingBuilder.AddNode(TScriptInterface<IMetaSoundDocumentInterface>(InputBuilder), Result);
					FMetaSoundNodeHandle OutputInterfaceNode = BindingBuilder.AddNode(TScriptInterface<IMetaSoundDocumentInterface>(OutputBuilder), Result);
					BindingBuilder.ConnectNodesByInterfaceBindings(OutputInterfaceNode, InputInterfaceNode, Result);
					AddErrorIfFalse(
						Result == EMetaSoundBuilderResult::Succeeded,
						FString::Printf(TEXT("Failed to connect nodes via binding with interface '%s' to '%s'"), *OutputInterface.ToString(), *InputInterface.ToString())
					);

					BindingBuilder.DisconnectNodesByInterfaceBindings(OutputInterfaceNode, InputInterfaceNode, Result);
					AddErrorIfFalse(
						Result == EMetaSoundBuilderResult::Succeeded,
						FString::Printf(TEXT("Failed to disconnect nodes via binding with interface '%s' to '%s'"), *OutputInterface.ToString(), *InputInterface.ToString())
					);

					BindingBuilder.ConnectNodeInputsToMatchingGraphInterfaceInputs(InputInterfaceNode, Result);
					AddErrorIfFalse(
						Result == EMetaSoundBuilderResult::Succeeded,
						FString::Printf(TEXT("Failed to connect input node with matching graph interface inputs: interface '%s'"), *InputInterface.ToString())
					);

					BindingBuilder.ConnectNodeOutputsToMatchingGraphInterfaceOutputs(OutputInterfaceNode, Result);
					AddErrorIfFalse(
						Result == EMetaSoundBuilderResult::Succeeded,
						FString::Printf(TEXT("Failed to connect output node with matching graph interface outputs: interface '%s'"), *OutputInterface.ToString())
					);
				}
			}
		}
	}

	return true;
}

// This test creates a MetaSound patch, then adds and connects sin oscillator, attempts to retrieve a default input set, then clears it, and finally retrieves the class default
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetaSoundNodeClassQueryFunctions, "Audio.Metasound.Builder.NodeClassQueryFunctions", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetaSoundNodeClassQueryFunctions, "Audio.Metasound.Builder.NodeClassQueryFunctions", "[MustPass]")
bool FAudioMetaSoundNodeClassQueryFunctions::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	constexpr float NodeDefaultFreq = 100.0f;

	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "DefaultLiteralAssignment_Test", { });

	// Sine Oscillator Node
	const FMetaSoundNodeHandle OscNode = Builder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNode.IsSet(), TEXT("Failed to create new MetaSound node by class name"));

	// Make connections:
	const FMetaSoundBuilderNodeInputHandle OscNodeFrequencyInput = Builder.FindNodeInputByName(OscNode, "Frequency", Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNodeFrequencyInput.IsSet(), TEXT("Failed to find Sine Oscillator node input 'Frequency'"));

	FMetasoundFrontendLiteral Literal;
	Literal.Set(100.0f);

	Builder.SetNodeInputDefault(OscNodeFrequencyInput, Literal, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to 'SetNodeInputDefault'"));

	Literal = Builder.GetNodeInputDefault(OscNodeFrequencyInput, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to 'GetNodeInputDefault'"));

	float Freq = 0.0f;
	bool bLiteralIsFloat = Literal.TryGet(Freq);
	AddErrorIfFalse(bLiteralIsFloat, TEXT("Failed to retrieve node literal as 'float'"));
	AddErrorIfFalse(FMath::IsNearlyEqual(NodeDefaultFreq, Freq), TEXT("'Freq' node default not set to provided default"));

	Builder.RemoveNodeInputDefault(OscNodeFrequencyInput, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to 'RemoveNodeInputDefault'"));

	Literal = Builder.GetNodeInputClassDefault(OscNodeFrequencyInput, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to 'GetNodeInputClassDefault' on c++ Sin Osc node"));

	bLiteralIsFloat = Literal.TryGet(Freq);
	AddErrorIfFalse(bLiteralIsFloat, TEXT("Failed to retrieve class input literal as 'float'"));
	AddErrorIfFalse(FMath::IsNearlyEqual(440.0f, Freq), TEXT("'Freq' node default not set to expected sin freq default of 440.0f Hz"));

	// Output Default Test
	{
		Literal.Set(123.0f);
		const FMetaSoundBuilderNodeInputHandle InputTest = Builder.AddGraphOutputNode("Output", GetMetasoundDataTypeName<float>(), Literal, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && InputTest.IsSet(), TEXT("Failed to create new MetaSound graph input"));

		bLiteralIsFloat = Literal.TryGet(Freq);
		AddErrorIfFalse(bLiteralIsFloat, TEXT("Failed to retrieve class input literal as 'float'"));
		AddErrorIfFalse(FMath::IsNearlyEqual(123.0f, Freq), TEXT("'Freq' node default not set to expected sin freq default of 440.0f Hz"));

		const bool bIsConstructorPin = Builder.GetNodeInputIsConstructorPin(InputTest);
		AddErrorIfFalse(!bIsConstructorPin, TEXT("Pin is not constructor pin but GetNodeInputIsConstructorPin is returning true"));
	}

	// Output Ctor tests
	{
		const FMetaSoundBuilderNodeInputHandle CtorTest = Builder.AddGraphOutputNode("CtorOutputTest", GetMetasoundDataTypeName<float>(), Literal, Result, true);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && CtorTest.IsSet(), TEXT("Failed to create new MetaSound graph input"));

		const bool bIsConstructorPin = Builder.GetNodeInputIsConstructorPin(CtorTest);
		AddErrorIfFalse(bIsConstructorPin, TEXT("Output handle is constructor pin but GetNodeInputIsConstructorPin is returning false"));
	}

	// Input Ctor tests
	{
		const FMetaSoundBuilderNodeOutputHandle CtorTest = Builder.AddGraphInputNode("CtorOutput", GetMetasoundDataTypeName<float>(), Literal, Result, true);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && CtorTest.IsSet(), TEXT("Failed to create new MetaSound graph input"));

		const bool bIsConstructorPin = Builder.GetNodeOutputIsConstructorPin(CtorTest);
		AddErrorIfFalse(bIsConstructorPin, TEXT("Input handle is constructor pin but GetNodeInputIsConstructorPin is returning false"));
	}

	return true;
}

// This test creates a MetaSound patch, adds a frequency input, connects a sin oscillator, and if the editor is loaded, and then attempts to inject template input nodes.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetaSoundInjectTemplateNodes, "Audio.Metasound.Builder.InjectInputTemplateNodes", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetaSoundInjectTemplateNodes::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	constexpr float NodeDefaultFreq = 100.0f;

	EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
	UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(*this, "TemplateInputInjection_Test", { });

	// Sine Oscillator Node
	const FMetaSoundNodeHandle OscNode = Builder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNode.IsSet(), TEXT("Failed to create new MetaSound node by class name"));

	auto MakeAndConnectSinInputToGraphInput = [&](FName InputName, FName TypeName, const FVector2D* Location, const FMetasoundFrontendLiteral& InLiteral, bool bIsCtorPin)
	{
		const FMetaSoundBuilderNodeOutputHandle InputNodeOutputHandle = Builder.AddGraphInputNode(InputName, TypeName, InLiteral, Result, bIsCtorPin);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && InputNodeOutputHandle.IsSet(), TEXT("Failed to create new MetaSound graph input"));

		const FMetaSoundBuilderNodeInputHandle OscNodeInput = Builder.FindNodeInputByName(OscNode, InputName, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNodeInput.IsSet(), TEXT("Failed to find Sine Oscillator node input"));

		Builder.ConnectNodes(InputNodeOutputHandle, OscNodeInput, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to connect input node to node input"));

#if WITH_EDITOR
		if (Location)
		{
			Builder.SetNodeLocation(FMetaSoundNodeHandle{ InputNodeOutputHandle.NodeID }, *Location, Result);
			AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to set input node location"));
		}
#endif // WITH_EDITOR
	};
	FMetasoundFrontendLiteral Literal;

	Literal.Set(true);
	MakeAndConnectSinInputToGraphInput("Enabled", GetMetasoundDataTypeName<bool>(), nullptr, Literal, true);

	Literal.Set(100.0f);
	FVector2D FreqLoc(100, 100);
	MakeAndConnectSinInputToGraphInput("Frequency", GetMetasoundDataTypeName<float>(), nullptr, Literal, false);

	Literal.Set(FMetasoundFrontendLiteral::FDefault { });
	FVector2D SyncLoc(300, 300);
	MakeAndConnectSinInputToGraphInput("Sync", GetMetasoundDataTypeName<FTrigger>(), nullptr, Literal, false);

#if WITH_EDITOR
	constexpr bool bForceNodeCreation = false;
	Builder.InjectInputTemplateNodes(bForceNodeCreation, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to inject input template node"));
#endif // WITH_EDITOR

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
