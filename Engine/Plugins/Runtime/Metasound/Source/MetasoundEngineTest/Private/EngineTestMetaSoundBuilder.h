// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/MetasoundInputFormatInterfaces.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Metasound.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDataReference.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundTrigger.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace EngineTestMetaSoundPatchBuilderPrivate
{
	inline UMetaSoundPatchBuilder& CreatePatchBuilderChecked(FAutomationTestBase& Test, FName BuilderName, const TArray<FName>& InterfaceNamesToAdd)
	{
		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
		UMetaSoundPatchBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreatePatchBuilder(BuilderName, Result);
		checkf(Builder, TEXT("Failed to create MetaSoundPatchBuilder '%s', required for all further testing."), *BuilderName.ToString());
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Builder created but CreatePatchBuilder did not result in 'Succeeded' state"));

		for (const FName& InterfaceName : InterfaceNamesToAdd)
		{
			Builder->AddInterface(InterfaceName, Result);
			Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, FString::Printf(TEXT("Failed to add initial interface '%s' to MetaSound patch"), *InterfaceName.ToString()));
		}

		return *Builder;
	}

	inline UMetaSoundPatch* CreatePatchFromBuilder(FAutomationTestBase& Test, const FString& PatchName, const TArray<FName>& InterfaceNamesToAdd)
	{
		UMetaSoundPatchBuilder& Builder = CreatePatchBuilderChecked(Test, FName(PatchName + TEXT(" Builder")), InterfaceNamesToAdd);

		UMetaSoundPatch* InputPatch = CastChecked<UMetaSoundPatch>(Builder.BuildNewMetaSound(FName(PatchName)).GetObject());
		Test.AddErrorIfFalse(InputPatch != nullptr, FString::Printf(TEXT("Failed to build MetaSound patch '%s'"), *PatchName));
		return InputPatch;
	}
} // namespace EngineTestMetaSoundPatchBuilderPrivate


DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FMetaSoundSourceBuilderDisconnectInputLatentCommand, FAutomationTestBase&, Test, UMetaSoundBuilderBase*, Builder, FMetaSoundBuilderNodeInputHandle, InputToDisconnect);

DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FMetaSoundSourceBuilderSetLiteralLatentCommand, FAutomationTestBase&, Test, UMetaSoundBuilderBase*, Builder, FMetaSoundBuilderNodeInputHandle, NodeInput, FMetasoundFrontendLiteral, NewValue);

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FMetaSoundSourceBuilderRemoveNodeDefaultLiteralLatentCommand, FAutomationTestBase&, Test, UMetaSoundBuilderBase*, Builder, FMetaSoundBuilderNodeInputHandle, NodeInput);

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FMetaSoundSourceBuilderCreateAndConnectTriGeneratorNodeLatentCommand, FAutomationTestBase&, Test, UMetaSoundSourceBuilder*, Builder, FMetaSoundBuilderNodeInputHandle, AudioOutNodeInput);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FBuilderRemoveFromRootLatentCommand, UMetaSoundBuilderBase*, Builder);

#endif // WITH_DEV_AUTOMATION_TESTS
