// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_DEV_AUTOMATION_TESTS
#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "EngineTestMetaSoundBuilder.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Math/UnrealMathUtility.h"
#include "MetasoundAssetSubsystem.h"
#include "MetasoundDataReference.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundLog.h"
#include "MetasoundSettings.h"
#include "MetasoundSource.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "Tests/AutomationCommon.h"
#include "Sound/SoundAttenuation.h"


namespace EngineTestMetaSoundSourcePrivate
{
	struct FInitTestBuilderSourceOutput
	{
		FMetaSoundBuilderNodeOutputHandle OnPlayOutput;
		FMetaSoundBuilderNodeInputHandle OnFinishedInput;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOutNodeInputs;
	};

	FString GetPluginContentDirectory()
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("Metasound"));
		if (ensure(Plugin.IsValid()))
		{
			return Plugin->GetContentDir();
		}
		return FString();
	}

	FString GetPathToTestFilesDir()
	{
		FString OutPath =  FPaths::Combine(GetPluginContentDirectory(), TEXT("Test"));

		OutPath = FPaths::ConvertRelativePathToFull(OutPath);
		FPaths::NormalizeDirectoryName(OutPath);
		
		return OutPath;
	}

	FString GetPathToGeneratedFilesDir()
	{
		FString OutPath = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("Metasounds"));

		OutPath = FPaths::ConvertRelativePathToFull(OutPath);
		FPaths::NormalizeDirectoryName(OutPath);
		
		return OutPath;
	}

	FString GetPathToGeneratedAssetsDir()
	{
		FString OutPath = TEXT("/Game/Metasound/Generated/");
		FPaths::NormalizeDirectoryName(OutPath);
		return OutPath;
	}

	Metasound::Frontend::FNodeHandle AddNode(Metasound::Frontend::IGraphController& InGraph, const Metasound::FNodeClassName& InClassName, int32 InMajorVersion)
	{
		Metasound::Frontend::FNodeHandle Node = Metasound::Frontend::INodeController::GetInvalidHandle();
		FMetasoundFrontendClass NodeClass;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ensure(Metasound::Frontend::ISearchEngine::Get().FindClassWithHighestMinorVersion(InClassName, InMajorVersion, NodeClass)))
		{
			Node = InGraph.AddNode(NodeClass.Metadata);
			check(Node->IsValid());
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Node;
	}

	UMetaSoundSourceBuilder& CreateSourceBuilder(
		FAutomationTestBase& Test,
		EMetaSoundOutputAudioFormat OutputFormat,
		bool bIsOneShot,
		EngineTestMetaSoundSourcePrivate::FInitTestBuilderSourceOutput& Output)
	{
		using namespace Audio;
		using namespace Metasound;
		using namespace Metasound::Engine;
		using namespace Metasound::Frontend;

		EMetaSoundBuilderResult Result;
		UMetaSoundSourceBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreateSourceBuilder(
			"Unit Test Graph Builder",
			Output.OnPlayOutput,
			Output.OnFinishedInput,
			Output.AudioOutNodeInputs,
			Result,
			OutputFormat,
			bIsOneShot);
		checkf(Builder, TEXT("Failed to create MetaSoundSourceBuilder"));
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Builder created but CreateSourceBuilder did not result in 'Succeeded' state"));

		return *Builder;
	}

	UMetaSoundSourceBuilder& CreateMonoSourceSinGenBuilder(
		FAutomationTestBase& Test,
		FMetaSoundBuilderNodeInputHandle* GenInputNodeFreq = nullptr,
		FMetaSoundBuilderNodeInputHandle* MonoOutNodeInput = nullptr,
		float InDefaultFreq = 100.f)
	{
		using namespace EngineTestMetaSoundPatchBuilderPrivate;
		using namespace EngineTestMetaSoundSourcePrivate;
		using namespace Metasound;
		using namespace Metasound::Engine;
		using namespace Metasound::Frontend;

		constexpr EMetaSoundOutputAudioFormat OutputFormat = EMetaSoundOutputAudioFormat::Mono;
		constexpr bool bIsOneShot = false;
		FInitTestBuilderSourceOutput Output;
		UMetaSoundSourceBuilder& Builder = CreateSourceBuilder(Test, OutputFormat, bIsOneShot, Output);

		EMetaSoundBuilderResult Result = EMetaSoundBuilderResult::Failed;
		if (MonoOutNodeInput)
		{
			*MonoOutNodeInput = { };
		}

		// Input on Play
		const FMetaSoundNodeHandle OnPlayOutputNode = Builder.FindGraphInputNode(SourceInterface::Inputs::OnPlay, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OnPlayOutputNode.IsSet(), TEXT("Failed to create MetaSound OnPlay input"));

		// Input Frequency
		FMetasoundFrontendLiteral DefaultFreq;
		DefaultFreq.Set(InDefaultFreq);
		const FMetaSoundBuilderNodeOutputHandle FrequencyNodeOutput = Builder.AddGraphInputNode("Frequency", GetMetasoundDataTypeName<float>(), DefaultFreq, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && FrequencyNodeOutput.IsSet(), TEXT("Failed to create new MetaSound graph input"));

		// Sine Oscillator Node
		const FMetaSoundNodeHandle OscNode = Builder.AddNodeByClassName({ "UE", "Sine", "Audio" }, Result, 1);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNode.IsSet(), TEXT("Failed to create new MetaSound node by class name"));

		// Make connections:
		const FMetaSoundBuilderNodeInputHandle OscNodeFrequencyInput = Builder.FindNodeInputByName(OscNode, "Frequency", Result);
		if (GenInputNodeFreq)
		{
			*GenInputNodeFreq = OscNodeFrequencyInput;
		}
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNodeFrequencyInput.IsSet(), TEXT("Failed to find Sine Oscillator node input 'Frequency'"));

		const FMetaSoundBuilderNodeOutputHandle OscNodeAudioOutput = Builder.FindNodeOutputByName(OscNode, "Audio", Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded && OscNodeAudioOutput.IsSet(), TEXT("Failed to find Sine Oscillator node output 'Audio'"));

		// Frequency input "Frequency" -> oscillator "Frequency"
		Builder.ConnectNodes(FrequencyNodeOutput, OscNodeFrequencyInput, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to connect 'Frequency' input to node input 'Frequency'"));

		// Oscillator to Output Node
		Test.AddErrorIfFalse(Output.AudioOutNodeInputs.Num() == 1, TEXT("Should only ever have one output node for mono"));
		if (MonoOutNodeInput)
		{
			*MonoOutNodeInput = Output.AudioOutNodeInputs.Last();
		}

		Builder.ConnectNodes(OscNodeAudioOutput, Output.AudioOutNodeInputs.Last(), Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to connect 'Audio' Sine Oscillator output to MetaSound graph's 'Mono Output'"));

		return Builder;
	}

	FMetasoundFrontendDocument CreateMonoSourceDocument()
	{
		using namespace Audio;
		using namespace Metasound;
		using namespace Metasound::Engine;
		using namespace Metasound::Frontend;

		// TODO: Move this to generate the basic document fully from builder. As of now, this test
		// uses a weird mix of builder API & controller API calls.

		FMetasoundFrontendDocument Document;
		Document.RootGraph.ID = FGuid::NewGuid();

		FMetaSoundFrontendDocumentBuilder::InitGraphClassMetadata(Document.RootGraph.Metadata);
		Document.RootGraph.InitDefaultGraphPage();

		FDocumentHandle DocumentHandle = IDocumentController::CreateDocumentHandle(Document);
		FGraphHandle RootGraph = DocumentHandle->GetRootGraph();
		check(RootGraph->IsValid());

		// Add default source & mono interface members (OnPlay, OnFinished & Mono Out)
		FModifyRootGraphInterfaces InterfaceTransform(
		{ },
		{
			SourceInterface::GetVersion(),
			SourceOneShotInterface::GetVersion(),
			OutputFormatMonoInterface::GetVersion()
		}); 
		InterfaceTransform.Transform(DocumentHandle);

		// Input on Play
		FNodeHandle OnPlayOutputNode = RootGraph->GetInputNodeWithName(SourceInterface::Inputs::OnPlay);
		check(OnPlayOutputNode->IsValid());

		// Input Frequency
		FMetasoundFrontendClassInput FrequencyInput;
		FrequencyInput.Name = "Frequency";
		FrequencyInput.TypeName = GetMetasoundDataTypeName<float>();
		FrequencyInput.VertexID = FGuid::NewGuid();
		FrequencyInput.InitDefault().Set(100.f);
		FNodeHandle FrequencyInputNode = RootGraph->AddInputVertex(FrequencyInput);
		check(FrequencyInputNode->IsValid());

		// Output On Finished
		FNodeHandle OnFinishedOutputNode = RootGraph->GetOutputNodeWithName(SourceOneShotInterface::Outputs::OnFinished);
		check(OnFinishedOutputNode->IsValid());

		// Output Audio
		FNodeHandle AudioOutputNode = RootGraph->GetOutputNodeWithName(OutputFormatMonoInterface::Outputs::MonoOut);
		check(AudioOutputNode->IsValid());

		// osc node
		FNodeHandle OscNode = AddNode(*RootGraph, { "UE", "Sine", "Audio" }, 1);

		// Make connections:

		// frequency input "Frequency" -> oscillator "Frequency"
		FOutputHandle OutputToConnect = FrequencyInputNode->GetOutputWithVertexName("Frequency");
		FInputHandle InputToConnect = OscNode->GetInputWithVertexName("Frequency");
		ensure(InputToConnect->Connect(*OutputToConnect));

		// oscillator to output
		OutputToConnect = OscNode->GetOutputWithVertexName("Audio");
		InputToConnect = AudioOutputNode->GetInputWithVertexName(OutputFormatMonoInterface::Outputs::MonoOut);
		ensure(InputToConnect->Connect(*OutputToConnect));

		return Document;
	}

	UAudioComponent* CreateTestComponent(FAutomationTestBase& Test, USoundBase* Sound = nullptr, bool bAddToRoot = true)
	{
		if (FAudioDevice* AudioDevice = GEngine->GetMainAudioDeviceRaw())
		{
			UAudioComponent* AudioComponent = NewObject<UAudioComponent>();
			Test.AddErrorIfFalse(AudioComponent != nullptr, "Failed to create test audio component");

			if (AudioComponent)
			{
				AudioComponent->bAutoActivate = false;
				AudioComponent->bIsUISound = true; // play while "paused"
				AudioComponent->AudioDeviceID = AudioDevice->DeviceID;

				AudioComponent->bAllowSpatialization = false;
				AudioComponent->SetVolumeMultiplier(1.0f);

				if (bAddToRoot)
				{
					AudioComponent->AddToRoot();
				}

				AudioComponent->SetSound(Sound);
			}

			return AudioComponent;
		}

		return nullptr;
	}
} // EngineTestMetaSoundSourcePrivate

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FAudioComponentPlayLatentCommand, UAudioComponent*, AudioComponent);

bool FAudioComponentPlayLatentCommand::Update()
{
	if (AudioComponent)
	{
		AudioComponent->Play();
		return true;
	}
	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FAudioComponentStopLatentCommand, UAudioComponent*, AudioComponent);

bool FAudioComponentStopLatentCommand::Update()
{
	if (AudioComponent)
	{
		AudioComponent->Stop();
		return true;
	}
	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FMetaSoundSourceLatentSetParamsCommand, UAudioComponent*, AudioComponent, TArray<FAudioParameter>, Params);

bool FMetaSoundSourceLatentSetParamsCommand::Update()
{
	if (AudioComponent)
	{
		AudioComponent->SetParameters(MoveTemp(Params));
		return true;
	}

	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FMetaSoundSourceBuilderAuditionLatentCommand, UMetaSoundSourceBuilder*, Builder, UAudioComponent*, AudioComponent, bool, bEnableLiveUpdates);

bool FMetaSoundSourceBuilderAuditionLatentCommand::Update()
{
	if (Builder && AudioComponent)
	{
		auto GetComponentSoundClassName = [this]() -> FName
		{
			if (const USoundBase* InitSound = AudioComponent->GetSound())
			{
				const TScriptInterface<const IMetaSoundDocumentInterface> InitMetaSound = InitSound;
				return InitMetaSound->GetConstDocument().RootGraph.Metadata.GetClassName().GetFullName();
			}

			return { };
		};

		// This is an inline test to ensure that the first time a builder is auditioned, its generating a new unique MetaSound Class Name.
		// Each subsequent call should maintain that name to avoid bloating the FName table/breaking references should this auditioned sound
		// be in anyway referenced.
		const FName InitClassName = GetComponentSoundClassName();
		Builder->Audition(nullptr, AudioComponent, { }, bEnableLiveUpdates);
		const FName BuiltClassName = GetComponentSoundClassName();

		const bool bInitNameGenerated = InitClassName.IsNone() && !BuiltClassName.IsNone();
		const bool bClassNameMaintained = InitClassName == BuiltClassName;
		if (bInitNameGenerated || bClassNameMaintained)
		{
			return true;
		}

		UE_LOGF(LogMetaSound, Error, "Latent test audition call resulted in generation of a new MetaSound class instead of re-using the existing class name");
	}
	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FAudioComponentRemoveFromRootLatentCommand, UAudioComponent*, AudioComponent);

bool FAudioComponentRemoveFromRootLatentCommand::Update()
{
	if (AudioComponent)
	{
		AudioComponent->RemoveFromRoot();
		return true;
	}
	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FMetaSoundSourceBuilderRemoveNodesLatentCommand, FAutomationTestBase&, Test, UMetaSoundSourceBuilder*, Builder, FMetaSoundNodeHandle, Node);

bool FMetaSoundSourceBuilderRemoveNodesLatentCommand::Update()
{
	if (Builder)
	{
		EMetaSoundBuilderResult Result;
		Builder->RemoveNode(Node, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Failed to remove node from MetaSound graph");
	}

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FMetaSoundSourceBuilderConnectNodesLatentCommand, FAutomationTestBase&, Test, UMetaSoundSourceBuilder*, Builder, FMetaSoundBuilderNodeOutputHandle, Output, FMetaSoundBuilderNodeInputHandle, Input);

bool FMetaSoundSourceBuilderConnectNodesLatentCommand::Update()
{
	if (Builder)
	{
		EMetaSoundBuilderResult Result;
		Builder->ConnectNodes(Output, Input, Result);
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to connect MetaSound nodes"));
	}

	return true;
}

// This test creates a MetaSound from the legacy controller document editing system and attempts to play it directly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceTest, "Audio.Metasound.PlayMetasoundSource", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetasoundSourceTest, "Audio.Metasound.PlayMetasoundSource", "[MustPass]")
bool FAudioMetasoundSourceTest::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundSourcePrivate;

	UMetaSoundSource* MetaSoundSource = NewObject<UMetaSoundSource>(GetTransientPackage(), FName(*LexToString(FGuid::NewGuid())));;
	if (ensure(nullptr != MetaSoundSource))
	{
		// TODO: Move to using builder to swap in doc
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MetaSoundSource->GetDocumentChecked() = CreateMonoSourceDocument();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (UAudioComponent* AudioComponent = CreateTestComponent(*this, MetaSoundSource))
		{
			ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentPlayLatentCommand(AudioComponent));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.f));
			ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));
			ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));
		}
	}

	return true;
 }

// This test creates a MetaSound source from a SourceBuilder, adds a simple sine tone generator with a connected graph input frequency, and attempts to audition it.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderTest, "Audio.Metasound.Builder.AuditionMetasoundSource", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetasoundSourceBuilderTest, "Audio.Metasound.Builder.AuditionMetasoundSource", "[MustPass]")
bool FAudioMetasoundSourceBuilderTest::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;

	FMetaSoundBuilderNodeInputHandle MonoOutNodeInput;
	UMetaSoundSourceBuilder& Builder = CreateMonoSourceSinGenBuilder(*this, nullptr, &MonoOutNodeInput);
	Builder.AddToRoot();

	if (UAudioComponent* AudioComponent = CreateTestComponent(*this))
	{
		constexpr bool bEnableLiveUpdate = false;
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.f));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));
		ADD_LATENT_AUTOMATION_COMMAND(FBuilderRemoveFromRootLatentCommand(&Builder));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));

		return true;
	}

	return false;
}

// This test creates a MetaSound source from a SourceBuilder, adds a simple sine tone generator with a connected graph input frequency, and attempts to change the frequency and audition it rapidly 100 times.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderTestSpamAudition, "Audio.Metasound.Builder.SpamAuditionMetasoundSource", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundSourceBuilderTestSpamAudition::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;

	FMetaSoundBuilderNodeInputHandle MonoOutNodeInput;
	FMetaSoundBuilderNodeInputHandle GenInputNodeFreq;
	UMetaSoundSourceBuilder& Builder = CreateMonoSourceSinGenBuilder(*this, &GenInputNodeFreq, &MonoOutNodeInput);
	Builder.AddToRoot();

	if (UAudioComponent* AudioComponent = CreateTestComponent(*this))
	{
		constexpr bool bEnableLiveUpdate = false;
		constexpr int32 NumTrials = 100;
		for (int32 Index = 0; Index < NumTrials; ++Index)
		{
			FMetasoundFrontendLiteral NewValue;
			NewValue.Set(FMath::RandRange(220.f, 2200.f));
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderSetLiteralLatentCommand(*this, &Builder, GenInputNodeFreq, NewValue));
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.05f));
		}

		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));
		ADD_LATENT_AUTOMATION_COMMAND(FBuilderRemoveFromRootLatentCommand(&Builder));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));

		return true;
	}

	return false;
}

// This test exercises auditioning multiple sources, both at the same time and attempting to audition live changes after stopping and restarting an audio component by:
// 1. Creates a MetaSound source from a SourceBuilder
// 2. Adds a simple sine tone generator with a connected graph input frequency
// 3. Auditions the builder via two AudioComponents
// 4. Sets the parameter input for frequency to a different value for the second component
// 5. Disconnects the parameter/graph input and sets the literal on the builder to a new frequency, which is observed on both components
// 6. Stops first the second component then the first.
// 7. Restarts audition of the original AudioComponent (which should continue after all the prior changes)
// 8. Swaps to use a tri tone generator
// 9. Stops original component, completing the test
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderLiveUpdateNode, "Audio.Metasound.Builder.LiveUpdateMultipleMetaSoundSources", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundSourceBuilderLiveUpdateNode::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundSourcePrivate;
	using namespace Metasound;
	using namespace Metasound::Engine;
	using namespace Metasound::Frontend;

	FMetaSoundBuilderNodeInputHandle GenInputNodeFreq;
	FMetaSoundBuilderNodeInputHandle MonoOutNodeInput;
	UMetaSoundSourceBuilder& Builder = CreateMonoSourceSinGenBuilder(*this, &GenInputNodeFreq, &MonoOutNodeInput, 440.f);
	Builder.AddToRoot();

	if (UAudioComponent* AudioComponent = CreateTestComponent(*this))
	{
		constexpr bool bEnableLiveUpdate = true;
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));

		// Send commands to more than one component. By setting param to 220.f on the second, listener can hear two
		// operators function on separate runtime graphs/generators
		if (UAudioComponent* AudioComponent2 = CreateTestComponent(*this))
		{
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent2, bEnableLiveUpdate));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));

			TArray<FAudioParameter> Params { FAudioParameter { "Frequency", 220.0f } };
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceLatentSetParamsCommand(AudioComponent2, Params));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.f));

			// Set a literal to ensure live updates still update as expected (nothing should happen until the next step when the edge
			// is disconnected, at which point we should hear 880Hz)
			FMetasoundFrontendLiteral NewFreq;
			NewFreq.Set(880.f);
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderSetLiteralLatentCommand(*this, &Builder, GenInputNodeFreq, NewFreq));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.5f));

			// Remove the edge to the graph input, at which point listener should hear the literal value of 880 via both test components.
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderDisconnectInputLatentCommand(*this, &Builder, GenInputNodeFreq));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.f));

			// Remove the second component, as the rest of the test is working with just the OG component.
			ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent2));
			ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent2));
		}
		else
		{
			return false;
		}
		
		// Stop and hear silence
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.f));

		// Restart audition to ensure it restarts as expected
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.f));

		// Disconnect graph audio output from existing sinosc output and connect to added triosc
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderCreateAndConnectTriGeneratorNodeLatentCommand(*this, &Builder, MonoOutNodeInput));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.f));

		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FBuilderRemoveFromRootLatentCommand(&Builder));

		return true;
	}

	return false;
}

// This test creates a MetaSound source from a SourceBuilder, adds a simple sine tone generator with a connected graph input frequency, attempts to audition it,
// disconnects frequency input, sets the sinosc frequency literal value to a new value, and finally removes the literal value default to have it return to the
// class default.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderLiveUpdateLiteral, "Audio.Metasound.Builder.LiveUpdateLiteralMetaSoundSource", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetasoundSourceBuilderLiveUpdateLiteral, "Audio.Metasound.Builder.LiveUpdateLiteralMetaSoundSource", "[MustPass]")
bool FAudioMetasoundSourceBuilderLiveUpdateLiteral::RunTest(const FString& Parameters)
{
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;

	FMetaSoundBuilderNodeInputHandle MonoOutNodeInput;
	FMetaSoundBuilderNodeInputHandle GenNodeFreqInput;
	UMetaSoundSourceBuilder& Builder = CreateMonoSourceSinGenBuilder(*this, &GenNodeFreqInput , &MonoOutNodeInput, 220.f);
	Builder.AddToRoot();

	if (UAudioComponent* AudioComponent = CreateTestComponent(*this))
	{
		constexpr bool bEnableLiveUpdate = true;
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.25f));

		// Disconnects freq input node output from sinosc freq input. Initially was set to 220Hz above, and node's default is 440Hz,
		// resulting in an octive pitch up.
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderDisconnectInputLatentCommand(*this, &Builder, GenNodeFreqInput));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.25f));

		// Sets literal value on the sinosc freq input to 880Hz, pitching an octive yet again from previous.
		FName DataTypeName;
		FMetasoundFrontendLiteral NewValue = UMetaSoundBuilderSubsystem::GetChecked().CreateFloatMetaSoundLiteral(880.f, DataTypeName);
		AddErrorIfFalse(DataTypeName == Metasound::GetMetasoundDataTypeName<float>(),
			"Setting MetaSound Float literal returns non-float DataTypeName.");
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderSetLiteralLatentCommand(*this, &Builder, GenNodeFreqInput, NewValue));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.25f));

		// Removes the literal value on the sinosc freq input set to 880Hz, reverting back to the class literal of 440Hz.
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderRemoveNodeDefaultLiteralLatentCommand(*this, &Builder, GenNodeFreqInput));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.25f));

		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.25f));

		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FBuilderRemoveFromRootLatentCommand(&Builder));

		return true;
	}

	return false;
}

// This test creates a MetaSound source from a SourceBuilder, then adds and finally removes an interface using the builder API, and verifies it as well as its members were added to the document.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderMutateInterface, "Audio.Metasound.Builder.MutateInterface", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundSourceBuilderMutateInterface::RunTest(const FString& Parameters)
{
	using namespace Audio;
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;
	using namespace Metasound::Frontend;

	constexpr EMetaSoundOutputAudioFormat OutputFormat = EMetaSoundOutputAudioFormat::Mono;
	constexpr bool bIsOneShot = false;
	FInitTestBuilderSourceOutput Output;
	UMetaSoundSourceBuilder& Builder = CreateSourceBuilder(*this, OutputFormat, bIsOneShot, Output);
	Builder.AddToRoot();

	EMetaSoundBuilderResult Result;

	// Test interface output mutation with oneshot interface
	Builder.AddInterface(SourceOneShotInterface::GetVersion().Name, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Returned failed state when adding 'OneShot' Interface to MetaSound using AddInterface Builder API call");

	Builder.FindGraphOutputNode(SourceOneShotInterface::Outputs::OnFinished, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Failed to add 'OnFinished' output to MetaSound using AddInterface Builder API call");

	bool bIsDeclared = Builder.InterfaceIsDeclared(SourceOneShotInterface::GetVersion().Name);
	AddErrorIfFalse(bIsDeclared, "'OneShot' Interface added but is not member of declaration list on MetaSound asset.");

	Builder.RemoveInterface(SourceOneShotInterface::GetVersion().Name, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Returned failed state when removing 'OneShot' Interface from MetaSound using RemoveInterface Builder API call");

	Builder.FindGraphOutputNode(SourceOneShotInterface::Outputs::OnFinished, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Failed, "Failed to remove 'OnFinished' output to MetaSound using RemoveInterface Builder API call");

	bIsDeclared = Builder.InterfaceIsDeclared(SourceOneShotInterface::GetVersion().Name);
	AddErrorIfFalse(!bIsDeclared, "'OneShot' Interface removed but remains member of declaration list on MetaSound asset.");


	// Test input mutation with attenuation interface
	Builder.AddInterface(AttenuationInterface::Name, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Returned failed state when adding 'Attenuation' Interface to MetaSound using AddInterface Builder API call");

	Builder.FindGraphInputNode(AttenuationInterface::Inputs::Distance, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Failed to add 'Distance' input to MetaSound using AddInterface Builder API call");

	bIsDeclared = Builder.InterfaceIsDeclared(AttenuationInterface::Name);
	AddErrorIfFalse(bIsDeclared, "'Attenuation' Interface added but is not member of declaration list on MetaSound asset.");

	Builder.RemoveInterface(AttenuationInterface::Name, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Returned failed state when removing 'Attenuation' Interface from MetaSound using RemoveInterface Builder API call");

	Builder.FindGraphInputNode(AttenuationInterface::Inputs::Distance, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Failed, "Failed to remove 'Distance' input to MetaSound using RemoveInterface Builder API call");

	bIsDeclared = Builder.InterfaceIsDeclared(AttenuationInterface::Name);
	AddErrorIfFalse(!bIsDeclared, "'Attenuation' Interface removed but remains member of declaration list on MetaSound asset.");

	return true;
}

// This test creates a MetaSound source from a SourceBuilder, then adds and removes various variable nodes to and from pages (if editor-only data is loaded and the project defines at least one page setting).
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderAddRemovePagedVariableNodes, "Audio.Metasound.Builder.AddRemovePagedVariableNodes", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundSourceBuilderAddRemovePagedVariableNodes::RunTest(const FString& Parameters)
{
	using namespace Audio;
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;
	using namespace Metasound;
	using namespace Metasound::Frontend;

	constexpr EMetaSoundOutputAudioFormat OutputFormat = EMetaSoundOutputAudioFormat::Mono;
	constexpr bool bIsOneShot = false;
	FInitTestBuilderSourceOutput Output;
	UMetaSoundSourceBuilder& Builder = CreateSourceBuilder(*this, OutputFormat, bIsOneShot, Output);
	Builder.AddToRoot();

	constexpr float SuperNotSpecialNumber = UE_EULERS_NUMBER * UE_GOLDEN_RATIO * UE_PI;
	FMetasoundFrontendLiteral DefaultValue;
	DefaultValue.Set(SuperNotSpecialNumber);
	EMetaSoundBuilderResult Result;

	static const FLazyName VariableName = "MyThing";

	FMetaSoundFrontendDocumentBuilder& DocBuilder = Builder.GetBuilder();

	Builder.AddGraphVariable(VariableName, GetMetasoundDataTypeName<float>(), DefaultValue, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to add graph variable"));

	{
		const FMetasoundFrontendVariable* Variable = DocBuilder.FindGraphVariable(VariableName);
		AddErrorIfFalse(Variable != nullptr, TEXT("Failed to retrieve frontend variable"));
	}

	const FMetasoundFrontendLiteral RetrievedDefault = Builder.GetGraphVariableDefault(VariableName, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to retrieve graph variable default"));

	AddErrorIfFalse(RetrievedDefault.IsEqual(DefaultValue), TEXT("Failed to retrieve graph variable default"));

	Builder.GetGraphVariableDefault(VariableName, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to retrieve graph variable default"));

	{
		const FMetaSoundNodeHandle SetNodeHandle = Builder.AddGraphVariableSetNode(VariableName, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to add graph variable setter node"));

		const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariableByNodeID(SetNodeHandle.NodeID);
		AddErrorIfFalse(FrontendVariable != nullptr, TEXT("Failed to find FrontendVariable by Setter NodeID"));
	}

	{
		const FMetaSoundNodeHandle GetNodeHandle = Builder.AddGraphVariableGetNode(VariableName, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to add graph variable getter node"));

		const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariableByNodeID(GetNodeHandle.NodeID);
		AddErrorIfFalse(FrontendVariable != nullptr, TEXT("Failed to find FrontendVariable by Getter NodeID"));
	}

	{
		const FMetaSoundNodeHandle GetNodeHandle = Builder.AddGraphVariableGetDelayedNode(VariableName, Result);
		AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to add graph variable delayed getter node"));

		const FMetasoundFrontendVariable* FrontendVariable = DocBuilder.FindGraphVariableByNodeID(GetNodeHandle.NodeID);
		AddErrorIfFalse(FrontendVariable != nullptr, TEXT("Failed to find FrontendVariable by Delayed Getter NodeID"));
	}

#if WITH_EDITORONLY_DATA
	bool bRanPageTests = false;
#endif // WITH_EDITORONLY_DATA

	if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
	{
		const TArray<FMetaSoundPageSettings>& PageSettings = Settings->GetProjectPageSettings();

#if WITH_EDITORONLY_DATA
		if (PageSettings.IsEmpty())
		{
			UE_LOGF(LogMetaSound, Display, "Skipping MetaSound Page Variable Tests: Project has no project-specific page settings implemented");
		}
		else
		{
			bRanPageTests = true;

			constexpr bool bDuplicateLastGraph = false;
			constexpr bool bSetAsBuildGraph = true;
			const FGuid PageID = PageSettings.Last().UniqueId;
			AddErrorIfFalse(PageID != Frontend::DefaultPageID, TEXT("Added PageID cannot be the default page ID"));
			DocBuilder.AddGraphPage(PageID, bDuplicateLastGraph, bSetAsBuildGraph);

			const FMetasoundFrontendVariable* PagedVariable = DocBuilder.AddGraphVariable(VariableName, GetMetasoundDataTypeName<int32>());
			int32 IntValue = (int32)(SuperNotSpecialNumber);
			AddErrorIfFalse(PagedVariable && PagedVariable->Literal.TryGet(IntValue) && IntValue == 0, TEXT("Failed to add graph variable or initialize paged variable value to 0"));

			for (int32 Index = 0; Index < 10; ++Index)
			{
				DocBuilder.AddGraphVariableDeferredAccessorNode(VariableName);
				DocBuilder.AddGraphVariableDeferredAccessorNode(VariableName, FGuid::NewGuid(), &Frontend::DefaultPageID);

				DocBuilder.AddGraphVariableAccessorNode(VariableName);
				DocBuilder.AddGraphVariableAccessorNode(VariableName, FGuid::NewGuid(), &Frontend::DefaultPageID);
			}

			AddErrorIfFalse(PagedVariable != nullptr, TEXT("Failed to add paged integer variable with same name as default float variable: Skipping additional page variable tests..."));

			if (PagedVariable)
			{
				const FMetasoundFrontendVariable* Variable = DocBuilder.FindGraphVariable(VariableName, &Frontend::DefaultPageID);
				AddErrorIfFalse(Variable != nullptr, TEXT("Failed to find graph variable getter node from default page (when build page ID is set to other ID"));

				if (Variable)
				{
					auto CheckNodeIDs = [this, &DocBuilder](const TArray<FGuid>& PagedNodeIDs, const TArray<FGuid>& DefaultNodeIDs, EMetasoundFrontendClassType ClassType)
					{
						TSet<FGuid> DefaultNodeIDSet;
						DefaultNodeIDSet.Append(DefaultNodeIDs);
						const TSet<FGuid> Intersection = DefaultNodeIDSet.Intersect(TSet<FGuid> (PagedNodeIDs));
						AddErrorIfFalse(Intersection.IsEmpty(), TEXT("Paged variable added variable node(s) to incorrect graph"));

						for (const FGuid& NodeID : PagedNodeIDs)
						{
							const FMetasoundFrontendClass* PagedVariableDependency = nullptr;
							if (const FMetasoundFrontendNode* PagedVariableNode = DocBuilder.FindNode(NodeID))
							{
								PagedVariableDependency = DocBuilder.FindDependency(PagedVariableNode->ClassID);
							}
						}

						for (const FGuid& NodeID : DefaultNodeIDs)
						{
							const FMetasoundFrontendClass* VariableDependency = nullptr;
							if (const FMetasoundFrontendNode* VariableNode = DocBuilder.FindNode(NodeID, &Frontend::DefaultPageID))
							{
								VariableDependency = DocBuilder.FindDependency(VariableNode->ClassID);
							}

							AddErrorIfFalse(VariableDependency && VariableDependency->Metadata.GetType() == ClassType, FString::Printf(
								TEXT("MetaSound Class missing/mismatch type (%s) for added variable node"),
								*FString(LexToString(ClassType))
							));
						}
					};

					CheckNodeIDs(PagedVariable->AccessorNodeIDs, Variable->AccessorNodeIDs, EMetasoundFrontendClassType::VariableAccessor);
					CheckNodeIDs(PagedVariable->DeferredAccessorNodeIDs, Variable->DeferredAccessorNodeIDs, EMetasoundFrontendClassType::VariableDeferredAccessor);
					CheckNodeIDs({ PagedVariable->VariableNodeID }, { Variable->VariableNodeID }, EMetasoundFrontendClassType::Variable);

					AddErrorIfFalse(!PagedVariable->MutatorNodeID.IsValid(), TEXT("Paged variable should have invalid mutator node ID as no mutator was added"));
					CheckNodeIDs({ /* PagedVariable->MutatorNodeID // Purposefully never added mutator as negation test */ }, { Variable->MutatorNodeID }, EMetasoundFrontendClassType::VariableMutator);
				}
			}
		}

#else // !WITH_EDITORONLY_DATA
		UE_LOGF(LogMetaSound, Display, "Skipping MetaSound Page Variable Tests: Test is running with editor data not loaded");
#endif // !WITH_EDITORONLY_DATA
	}

	Builder.RemoveGraphVariable(VariableName, Result);
	AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Failed to remove graph variable"));

#if WITH_EDITORONLY_DATA
	if (bRanPageTests)
	{
		DocBuilder.SetBuildPageID(Frontend::DefaultPageID);

		const FMetasoundFrontendVariable* PageVariable = DocBuilder.FindGraphVariable(VariableName);
		AddErrorIfFalse(PageVariable != nullptr, TEXT("Remove graph request should not have removed default variable as well (as build page ID was set)"));
	}
#endif // WITH_EDITORONLY_DATA

	Builder.RemoveFromRoot();
	return true;
}

// This test creates a MetaSound source from a SourceBuilder, then adds, removes and connects a variable and its respective nodes (get/get delayed/set) to multiple pages. Does not playback or live-audition.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundSourceBuilderAddRemoveNodes, "Audio.Metasound.Builder.AddRemoveNodes", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetasoundSourceBuilderAddRemoveNodes, "Audio.Metasound.Builder.AddRemoveNodes", "[MustPass]")
bool FAudioMetasoundSourceBuilderAddRemoveNodes::RunTest(const FString& Parameters)
{
	using namespace Audio;
	using namespace EngineTestMetaSoundPatchBuilderPrivate;
	using namespace EngineTestMetaSoundSourcePrivate;
	using namespace Metasound::Frontend;

	constexpr EMetaSoundOutputAudioFormat OutputFormat = EMetaSoundOutputAudioFormat::Mono;
	constexpr bool bIsOneShot = false;
	FInitTestBuilderSourceOutput Output;
	UMetaSoundSourceBuilder& Builder = CreateSourceBuilder(*this, OutputFormat, bIsOneShot, Output);
	Builder.AddToRoot();

	EMetaSoundBuilderResult Result;

	if (UAudioComponent* AudioComponent = CreateTestComponent(*this))
	{
		constexpr bool bEnableLiveUpdate = true;
		ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderAuditionLatentCommand(&Builder, AudioComponent, bEnableLiveUpdate));

		const FName OscType = "Sine";
		const TArray<float> ChromaticFreqs
		{
			293.66f,
			311.13f,
			329.63f,
			349.23f,
			369.99f,
			392.00f,
			415.30f,
			440.00f
		};

		TArray<FMetaSoundNodeHandle> GenNodes;
		for (int32 i = 0; i < 8; ++i)
		{
			FMetaSoundNodeHandle OscNode = Builder.AddNodeByClassName({ "UE", OscType, "Audio" }, Result, 1);
			AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, "Failed to add osc node to graph");

			if (Result == EMetaSoundBuilderResult::Succeeded)
			{
				GenNodes.Add(MoveTemp(OscNode));
			}
		}

		for (int32 i = 0; i < GenNodes.Num(); ++i)
		{
			FMetaSoundNodeHandle& OscNode = GenNodes[i];
			EMetaSoundBuilderResult ConnectResult;
			const FMetaSoundBuilderNodeOutputHandle OscNodeAudioOutput = Builder.FindNodeOutputByName(OscNode, "Audio", ConnectResult);
			AddErrorIfFalse(ConnectResult == EMetaSoundBuilderResult::Succeeded && OscNodeAudioOutput.IsSet(), TEXT("Failed to find oscillator node output 'Audio'"));

			FMetaSoundBuilderNodeInputHandle InputHandle = Builder.FindNodeInputByName(OscNode, "Frequency", ConnectResult);
			AddErrorIfFalse(ConnectResult == EMetaSoundBuilderResult::Succeeded && InputHandle.IsSet(), TEXT("Failed to find oscillator node input 'Frequency'"));

			FMetasoundFrontendLiteral Literal;
			Literal.Set(ChromaticFreqs[i]);
			Builder.SetNodeInputDefault(InputHandle, Literal, ConnectResult);
			AddErrorIfFalse(ConnectResult == EMetaSoundBuilderResult::Succeeded && OscNodeAudioOutput.IsSet(), TEXT("Failed to find oscillator node output 'Audio'"));

			Builder.FindGraphOutputNode(SourceOneShotInterface::Outputs::OnFinished, Result);
			AddErrorIfFalse(Result == EMetaSoundBuilderResult::Failed, "Failed to remove 'OnFinished' output to MetaSound using RemoveInterface Builder API call");

			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderConnectNodesLatentCommand(*this, &Builder, OscNodeAudioOutput, Output.AudioOutNodeInputs.Last()));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.125f));
			ADD_LATENT_AUTOMATION_COMMAND(FMetaSoundSourceBuilderRemoveNodesLatentCommand(*this, &Builder, GenNodes[i]));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.125f));
		}

		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.f));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentStopLatentCommand(AudioComponent));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.1f));
		ADD_LATENT_AUTOMATION_COMMAND(FBuilderRemoveFromRootLatentCommand(&Builder));
		ADD_LATENT_AUTOMATION_COMMAND(FAudioComponentRemoveFromRootLatentCommand(AudioComponent));
	}

	return true;
}

// This test attempts to load class info from asset tag data set on a MetaSound source and check its validity via the MetaSound Asset Subsystem.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetaSoundFindAssetClassInfo, "Audio.Metasound.Asset.FindAssetClassInfo", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAudioMetaSoundFindAssetClassInfo, "Audio.Metasound.Asset.FindAssetClassInfo", "[MustPass]")
bool FAudioMetaSoundFindAssetClassInfo::RunTest(const FString& Parameters)
{
#if WITH_EDITOR
	const FTopLevelAssetPath Path(FName("/Game/Assets/MetaSounds/AssetSubsystem/QueryClassInfoAutomatedTest_Preset"), FName("QueryClassInfoAutomatedTest_Preset"));

	FMetaSoundDocumentInfo DocInfo;
	FMetaSoundClassInterfaceInfo ClassInterfaceInfo;

	UMetaSoundAssetSubsystem* Subsystem = GEngine->GetEngineSubsystem<UMetaSoundAssetSubsystem>();
	AddErrorIfFalse(Subsystem != nullptr, TEXT("MetaSound Asset Subsystem could not be found. Skipping additional class query testing."));

	if (Subsystem)
	{
		const bool bAssetFound = Subsystem->FindAssetClassInfo(Path, DocInfo, ClassInterfaceInfo);

		// TODO: Don't want to require this asset to be included in plugin for deployment, so cannot fail test
		// if asset isn't found for now.
// 		AddErrorIfFalse(bAssetFound, FString::Format(
// 			TEXT("MetaSound asset class query failed to find or parse asset tag data for asset '{0}'."),
// 			{ Path.ToString() }));

		if (bAssetFound)
		{
			TScriptInterface<IMetaSoundDocumentInterface> DocInterface = FSoftObjectPath(Path).TryLoad();
			if (DocInterface.GetObject())
			{
				AddErrorIfFalse(DocInfo.bIsPreset, TEXT("MetaSound asset query tags did not return that asset is preset"));

				const FMetasoundFrontendDocument& Doc = DocInterface->GetConstDocument();
				const FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;

				const FText StringDisplayName = GraphClass.Metadata.GetDisplayName();
				AddErrorIfFalse(ClassInterfaceInfo.SearchInfo.ClassDisplayName.EqualTo(StringDisplayName), FString::Format(TEXT("MetaSound display name should have been '{0}'"), { StringDisplayName.ToString() }));

				const FText StringDesc = GraphClass.Metadata.GetDescription();
				AddErrorIfFalse(ClassInterfaceInfo.SearchInfo.ClassDescription.EqualTo(StringDesc), *FString::Format(TEXT("MetaSound description should have been '{0}'"), { StringDesc.ToString()}));

				AddErrorIfFalse(ClassInterfaceInfo.DefinedInterfaces.Num() == 0, TEXT("Cool! We support declared interfaces now! Update the test!"));

				AddErrorIfFalse(ClassInterfaceInfo.InheritedInterfaces.Num() == Doc.Interfaces.Num(), TEXT("MetaSound asset query tags did not return that asset implements number of interfaces as expected"));
				AddErrorIfFalse(ClassInterfaceInfo.Inputs.Num() == GraphClass.GetDefaultInterface().Inputs.Num(), TEXT("MetaSound asset query tags did not return that asset implements number of inputs expected"));
				AddErrorIfFalse(ClassInterfaceInfo.Outputs.Num() == GraphClass.GetDefaultInterface().Outputs.Num(), TEXT("MetaSound asset query tags did not return that asset implements number of outputs expected"));
			}
		}
	}
#endif // WITH_EDITOR

	return true;
}
#endif // WITH_DEV_AUTOMATION_TESTS
