// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundEnvironment.h"
#include "MetasoundFrontend.h"
#include "MetasoundSource.h"
#include "Kismet/GameplayStatics.h"
#include "Components/AudioComponent.h"
#include "AudioDevice.h"
#include "Misc/AutomationTest.h"

#include "Sound/SoundWave.h"

#include "MetasoundWave.h"
#include "MetasoundAudioBuffer.h"

#if WITH_DEV_AUTOMATION_TESTS

// Disabled for now until waveplayer implementation is extracted and place in a better location for testing
// (as nodes are moving to headerless implementation to better support hotfixing between engine versions.)
#if 0
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundWavePlayer_PlaySoundWave, "Audio.Metasound.WavePlayer.PlaySoundWave", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundWavePlayer_PlaySoundWave::RunTest(const FString& Parameters)
{	
	const TCHAR* TestAsset = TEXT("SoundWave'/Game/Tests/Audio/AttenuationFocusAzimuth/FocusAzimuthPinkNoise.FocusAzimuthPinkNoise'");
	FSoftObjectPath SoftPath(TestAsset);
	USoundWave* TestWave = Cast<USoundWave>(SoftPath.TryLoad());

#if WITH_EDITOR
	TestWave->CachePlatformData();
#endif //WITH_EDITOR

	AddErrorIfFalse(TestWave != nullptr, TEXT("Failed to load a SoundWave"));

	// Make a copy of the expected PCM.
	TArray<float> ExpectedFloat;
	if (const void* pExpected = TestWave->RawData.LockReadOnly() )
	{
		int32 NumSamples = TestWave->RawData.GetBulkDataSize()/sizeof(int16);
		ExpectedFloat.SetNum(NumSamples);
	
		const int16* Src16 = (const int16*) pExpected;
		for(int32 i = 0; i < NumSamples; ++i)
		{
			ExpectedFloat[i] = (float)Src16[i] / 32768.f;
		}		
		TestWave->RawData.Unlock();
	}
	
	using namespace Metasound;
	using namespace Audio;
	FWavePlayerNode Node(TEXT("WavePlayer"), FGuid::NewGuid());

	FOperatorFactorySharedRef Factory = Node.GetDefaultOperatorFactory();

	Metasound::FOperatorSettings OperatorSettings(48000.f, 1024);
	Metasound::FMetasoundEnvironment Environment;
	Metasound::FDataReferenceCollection DataReferenceCollection;

	Audio::FProxyDataInitParams Params { TEXT("WavePlayer") };
	TUniquePtr<IProxyData> WaveProxy = TestWave->CreateNewProxyData(Params);
	Metasound::FWaveAssetWriteRef Wave = Metasound::FWaveAssetWriteRef::CreateNew(WaveProxy);	
	DataReferenceCollection.AddDataReadReference(TEXT("Wave"), Wave);	

	TArray<TUniquePtr<Metasound::IOperatorBuildError>> BuildErrors;

	TUniquePtr<Metasound::IOperator> Operator = Factory->CreateOperator({Node, OperatorSettings, DataReferenceCollection, Environment}, BuildErrors);
	UTEST_TRUE("Made the operator", Operator.IsValid());
	UTEST_TRUE("We should have zero errors", BuildErrors.Num() == 0);
	   
	//THEN("It should produce some valid PCM output")
	{

		Metasound::FDataReferenceCollection OutputCollection = Operator->GetOutputs();
		Metasound::IOperator::FExecuteFunction Func = Operator->GetExecuteFunction();

		UTEST_TRUE("Has Audio Output", OutputCollection.ContainsDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft")));

		if (OutputCollection.ContainsDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft")))
		{
			Metasound::FAudioBufferReadRef Buffer = OutputCollection.GetDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft"));
			UTEST_NOT_NULL("Buffer is not null", Buffer->GetData());

			const float* Data = Buffer->GetData();
			float Phase = 0.0f;
			const float Frequency = 200.f;

			const int32 NumBuffers = (TestWave->RawData.GetBulkDataSize() / TestWave->NumChannels / sizeof(int16)) /
				OperatorSettings.GetNumFramesPerBlock();
			
			int32 ExpectedPos = 0;
			for (int32 j = 0; j < NumBuffers; j++)
			{
				Invoke(Func, Operator.Get());

				/*for (int32 i = 0; i < OperatorSettings.GetNumFramesPerBlock(); ++i)
				{
					UTEST_EQUAL_TOLERANCE("Compare signals are the same-ish-sorta",
						Data[i],
						ExpectedFloat[i],
						0.01f);
				}*/

				ExpectedPos += OperatorSettings.GetNumFramesPerBlock();
			}
		}
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundWavePlayer_DefaultMakesSilence, "Audio.Metasound.WavePlayer.DefaultMakesSilence", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundWavePlayer_DefaultMakesSilence::RunTest(const FString& Parameters)
{
	using namespace Metasound;
	using namespace Audio;
	FWavePlayerNode Node(TEXT("WavePlayer"), FGuid::NewGuid());
	
	FOperatorFactorySharedRef Factory = Node.GetDefaultOperatorFactory();

	Metasound::FOperatorSettings OperatorSettings(48000.f, 1024);
	Metasound::FMetasoundEnvironment Environment;
	Metasound::FDataReferenceCollection DataReferenceCollection;
	TArray<TUniquePtr<Metasound::IOperatorBuildError>> BuildErrors;

	TUniquePtr<Metasound::IOperator> Operator = Factory->CreateOperator({Node, OperatorSettings, DataReferenceCollection, Environment}, BuildErrors);
	UTEST_TRUE("Create operator",Operator.IsValid());
	
	UTEST_TRUE("Creating without any inputs will log some errors about bad inputs.",BuildErrors.Num() > 0);

	//THEN("It should correctly warn and produce silence")
	{
		Metasound::FDataReferenceCollection OutputCollection = Operator->GetOutputs();
		Metasound::IOperator::FExecuteFunction Func = Operator->GetExecuteFunction();
		
		UTEST_TRUE("Has Audio Output",OutputCollection.ContainsDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft")));

		if (OutputCollection.ContainsDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft")))
		{
			Metasound::FAudioBufferReadRef Buffer = OutputCollection.GetDataReadReference<Metasound::FAudioBuffer>(TEXT("AudioLeft"));
			UTEST_TRUE("Output Buffer is valid", nullptr != Buffer->GetData());
			UTEST_TRUE("Num of Frames per block match input", OperatorSettings.GetNumFramesPerBlock() == Buffer->Num());

			const int32 NumBuffers = 2;

			const float* Data = Buffer->GetData();
			for (int32 j = 0; j < NumBuffers; j++)
			{
				Invoke(Func, Operator.Get());

				for (int32 i = 0; i < OperatorSettings.GetNumFramesPerBlock(); i++)
				{
					// Should all be silent.
					UTEST_TRUE("", Data[i] == 0.0f);
				}
			}
		}
	}
	return true;
}

/*
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioMetasoundWavePlayer_EncodeDecodeOsscilator, "Audio.Metasound.WavePlayer.EncodeDecodeOsscilator", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioMetasoundWavePlayer_EncodeDecodeOsscilator::RunTest(const FString& Parameters)
{
	using namespace Metasound;
	using namespace Audio;
	FWavePlayerNode Node(TEXT("WavePlayer"));

	// Make a simple mono sequence in int16
	TArray<int16> Sequence;
	constexpr int32 SequenceLength = 48000;
	Sequence.SetNum(SequenceLength);

	// Oscillator.
	{
		const float DefaultFrequency = 200.f;
		float Phase = 0.f;
		float Frequency = DefaultFrequency;
		float Rate = 48000.f;

		for (int32 i = 0; i < SequenceLength; i++)
		{
			float OscFloat = FMath::Sin(Phase);
			Sequence[i] = static_cast<int16>(OscFloat * 32767.f);
			Phase += 2.0f * PI * Frequency / Rate;

			while (Phase > (2.0f * PI))
			{
				Phase -= (2.0f * PI);
			}
		}
	}

	// Find default codec / create encoder / encode signal.
	IEncoderInput::FFormat Format{ 1, 48000, Int16_Interleaved };
	ICodecRegistry::FCodecPtr Codec = ICodecRegistry::Get().FindDefaultCodec();
	TUniquePtr<IEncoderInput> InputObj = IEncoderInput::Create(Sequence, Format);
	UTEST_NOT_NULL("Default Codec", Codec);
	UTEST_TRUE("Can Encode",Codec->GetDetails().Features.HasFeature(FCodecFeatures::HasEncoder));
	UTEST_TRUE("Can Decode",Codec->GetDetails().Features.HasFeature(FCodecFeatures::HasDecoder));

	ICodec::FEncoderPtr Encoder = Codec->CreateEncoder(InputObj.Get());
	UTEST_TRUE("Encoder was created",Encoder.IsValid());

	TArray<uint8> CompressedBytes;
	if (Encoder.IsValid())
	{
		UTEST_TRUE("Encode passes",Encoder->Encode(InputObj.Get(), IEncoderOutput::Create(CompressedBytes).Get(), nullptr));
		UTEST_TRUE("Encode produced some data",CompressedBytes.Num() > 0);
	}

	

	//WHEN("Making a WavePlayer with valid inputs")
	{
		FOperatorFactorySharedRef Factory = Node.GetDefaultOperatorFactory();

		Metasound::FOperatorSettings OperatorSettings(48000.f, 1024);
		Metasound::FDataReferenceCollection DataReferenceCollection;

		Metasound::FWaveAssetWriteRef Wave = Metasound::FWaveAssetWriteRef::CreateNew(CompressedBytes);
		DataReferenceCollection.AddDataReadReference(TEXT("Wave"), Wave);

		TArray<TUniquePtr<Metasound::IOperatorBuildError>> BuildErrors;

		TUniquePtr<Metasound::IOperator> Operator = Factory->CreateOperator({Node, OperatorSettings, DataReferenceCollection}, BuildErrors);
		UTEST_TRUE("Made the operator",Operator.IsValid());
		UTEST_TRUE("We should have zero errors",BuildErrors.Num() == 0);

		//THEN("It should produce some valid PCM output")
		{
			Metasound::FDataReferenceCollection OutputCollection = Operator->GetOutputs();
			Metasound::IOperator::FExecuteFunction Func = Operator->GetExecuteFunction();
			Metasound::FAudioBufferReadRef Buffer = OutputCollection.GetDataReadReference<Metasound::FAudioBuffer>(TEXT("Audio"));
			UTEST_NOT_NULL("Buffer is not null",Buffer->GetData());

			const float* Data = Buffer->GetData();
			float Phase = 0.0f;
			const float Frequency = 200.f;

			const int32 NumBuffers = SequenceLength / OperatorSettings.GetNumFramesPerBlock();

			for (int32 j = 0; j < NumBuffers; j++)
			{
				Invoke(Func, Operator.Get());

				for (int32 i = 0; i < OperatorSettings.GetNumFramesPerBlock(); i++)
				{
					UTEST_EQUAL_TOLERANCE("Compare signals are the same-ish-sorta",Data[i], FMath::Sin(Phase),0.01f);
					//REQUIRE(Data[i] == Approx(FMath::Sin(Phase)).margin(0.01f));
					Phase += 2.0f * PI * Frequency / OperatorSettings.GetSampleRate();

					while (Phase > (2.0f * PI))
					{
						Phase -= (2.0f * PI);
					}
				}
			}
		}
	}
	
	// Got to the end.
	return true;
}
*/
#endif // 0

#endif //WITH_DEV_AUTOMATION_TESTS
