// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGenerator.h"
#include "MetasoundStandardNodesNames.h"
#include "NodeTestGraphBuilder.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Metasound::Test::Nodes::EnvelopeFollower
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetasoundEnvelopeFollowerNodeEnableTest,
		"Audio.Metasound.Nodes.EnvelopeFollower.Enable",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundEnvelopeFollowerNodeEnableTest::RunTest(const FString&)
	{
		// Make a graph with a sine oscillator going into an envelope follower
		TUniquePtr<FMetasoundGenerator> Generator;
		{
			FNodeTestGraphBuilder Builder;

			const Frontend::FNodeHandle Oscillator = Builder.AddNode({ StandardNodes::Namespace, TEXT("Sine"), StandardNodes::AudioVariant }, 1);
			const Frontend::FNodeHandle EnvelopeFollower = Builder.AddNode({StandardNodes::Namespace, TEXT("Envelope Follower"), TEXT("") }, 1);
			FNodeTestGraphBuilder::ConnectNodes(Oscillator, "Audio", EnvelopeFollower, "In");

			// Expose the enable pin on the envelope follower
			Builder.AddAndConnectDataReferenceInput(EnvelopeFollower, "Enable", GetMetasoundDataTypeName<bool>());

			// Expose the audio and float outputs of the envelope follower
			Builder.AddAndConnectDataReferenceOutput(EnvelopeFollower, "Envelope", GetMetasoundDataTypeName<float>());
			Builder.AddAndConnectDataReferenceOutput(EnvelopeFollower, "Audio Envelope", GetMetasoundDataTypeName<FAudioBuffer>());

			Generator = Builder.BuildGenerator();
		}
		UTEST_NOT_NULL("Created generator", Generator.Get());

		// Get the graph output refs
		TOptional<TDataReadReference<float>> FloatEnvelopeOutput = Generator->GetOutputReadReference<float>("Envelope");
		UTEST_TRUE("Got float envelope output", FloatEnvelopeOutput.IsSet());
		TOptional<TDataReadReference<FAudioBuffer>> AudioEnvelopeOutput = Generator->GetOutputReadReference<FAudioBuffer>("Audio Envelope");
		UTEST_TRUE("Got audio envelope output", AudioEnvelopeOutput.IsSet());
		
		// Enable, render a few blocks and expect some output
		const int32 BlockSize = Generator->OperatorSettings.GetNumFramesPerBlock();
		FAudioBuffer DummyBuffer{ BlockSize };

		Generator->SetInputValue("Enable", true);
		
		constexpr int32 NumBlocksToRender = 4;
		for (int32 i = 0; i < NumBlocksToRender; ++i)
		{
			Generator->OnGenerateAudio(DummyBuffer.GetData(), DummyBuffer.Num());
		}

		{
			UTEST_TRUE("Float envelope was non-zero", **FloatEnvelopeOutput > 0.0f);
			bool HadNonZeroEnvelope = false;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float Value = (*AudioEnvelopeOutput)->GetData()[i];
				if (Value > 0.0f)
				{
					HadNonZeroEnvelope = true;
					break;
				}
			}
			UTEST_TRUE("Audio envelope had non-zero output", HadNonZeroEnvelope);
		}

		// Disable, render a block and expect zero output
		Generator->SetInputValue("Enable", false);
		Generator->OnGenerateAudio(DummyBuffer.GetData(), DummyBuffer.Num());

		{
			UTEST_EQUAL("Float envelope was zero", **FloatEnvelopeOutput, 0.0f);
			bool HadNonZeroEnvelope = false;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float Value = (*AudioEnvelopeOutput)->GetData()[i];
				if (Value > 0.0f)
				{
					HadNonZeroEnvelope = true;
					break;
				}
			}
			UTEST_FALSE("Audio envelope had zero output", HadNonZeroEnvelope);
		}

		// Enable, render a few blocks and expect some output
		Generator->SetInputValue("Enable", true);
		for (int32 i = 0; i < NumBlocksToRender; ++i)
		{
			Generator->OnGenerateAudio(DummyBuffer.GetData(), DummyBuffer.Num());
		}

		{
			UTEST_TRUE("Float envelope was non-zero", **FloatEnvelopeOutput > 0.0f);
			bool HadNonZeroEnvelope = false;
			for (int32 i = 0; i < BlockSize; ++i)
			{
				const float Value = (*AudioEnvelopeOutput)->GetData()[i];
				if (Value > 0.0f)
				{
					HadNonZeroEnvelope = true;
					break;
				}
			}
			UTEST_TRUE("Audio envelope had non-zero output", HadNonZeroEnvelope);
		}
		
		return true;
	}

}

#endif
