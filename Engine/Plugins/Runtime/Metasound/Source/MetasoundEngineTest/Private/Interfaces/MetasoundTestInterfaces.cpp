// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundTestInterfaces.h"

#include "MetasoundFrontendDocument.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundTestInterfaces"

namespace Metasound::Test
{
#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Test.Update"
	namespace UpdateTestInterface_0_1
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 0, 1 } };
			return Version;
		}

		namespace Inputs
		{
			const FName InputTrigger = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("InputTrigger");
		}

		namespace Outputs
		{
			const FName OutputTrigger = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OutputTrigger");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(UpdateTestInterface_0_1::GetVersion().Name, UpdateTestInterface_0_1::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = true;
					constexpr bool bIsDefault = false;
					UClassOptions = TArray<FClassOptions>
					{
						{ InClassPath, bIsModifiable, bIsDefault }
					};

					Inputs =
					{
						{
							LOCTEXT("InputTrigger", "Input Trigger"),
							LOCTEXT("InputTrigger Description", "A trigger input used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<FTrigger>(),
							Inputs::InputTrigger
						}
					};

					Outputs =
					{
						{
							LOCTEXT("OutputTrigger", "Output Trigger"),
							LOCTEXT("OutputTrigger Description", "A trigger output used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<FTrigger>(),
							Outputs::OutputTrigger
						}
					};
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}
	}

	namespace UpdateTestInterface_0_2
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 0, 2 } };
			return Version;
		}

		namespace Inputs
		{
			const FName InputTrigger = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("InputTrigger");
			const FName InputFloat = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("InputFloat");
		}

		namespace Outputs
		{
			const FName OutputTrigger = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OutputTrigger");
			const FName OutputFloat = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OutputFloat");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(UpdateTestInterface_0_2::GetVersion().Name, UpdateTestInterface_0_2::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = true;
					constexpr bool bIsDefault = false;
					UClassOptions = TArray<FClassOptions>
					{
						{ InClassPath, bIsModifiable, bIsDefault }
					};

					Inputs =
					{
						{
							LOCTEXT("InputTrigger", "Input Trigger"),
							LOCTEXT("InputTrigger Description", "A trigger input used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<FTrigger>(),
							Inputs::InputTrigger
						},
						{
							LOCTEXT("InputFloat", "Input Float"),
							LOCTEXT("InputFloat Description", "A float input used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<float>(),
							Inputs::InputFloat
						}
					};

					Outputs =
					{
						{
							LOCTEXT("OutputTrigger", "Output Trigger"),
							LOCTEXT("OutputTrigger Description", "A trigger output used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<FTrigger>(),
							Outputs::OutputTrigger
						},
						{
							LOCTEXT("OutputFloat", "Output Float"),
							LOCTEXT("OutputFloat Description", "A float output used for testing Metasound Interfaces."),
							GetMetasoundDataTypeName<float>(),
							Outputs::OutputFloat
						}
					};
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}
	}
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE
}

#undef LOCTEXT_NAMESPACE