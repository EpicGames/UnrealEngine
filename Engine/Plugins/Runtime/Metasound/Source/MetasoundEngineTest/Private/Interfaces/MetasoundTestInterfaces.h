// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"
#include "IAudioParameterInterfaceRegistry.h"

struct FMetasoundFrontendVersion;

namespace Metasound::Test
{
#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Test.Update"
	namespace UpdateTestInterface_0_1
	{
		const FMetasoundFrontendVersion& GetVersion();

		namespace Inputs
		{
			const extern FName InputTrigger;
		}

		namespace Outputs
		{
			const extern FName OutputTrigger;
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
	}

	namespace UpdateTestInterface_0_2
	{
		const FMetasoundFrontendVersion& GetVersion();

		namespace Inputs
		{
			const extern FName InputTrigger;
			const extern FName InputFloat;
		}

		namespace Outputs
		{
			const extern FName OutputTrigger;
			const extern FName OutputFloat;
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
	}
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE
}