// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundGenerator.h"

#include "Interfaces/MetasoundOutputFormatInterfaces.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Metasound::Test::Generator::Dynamic
{
	TArray<FVertexName> GetAudioOutputVertexNames(const EMetaSoundOutputAudioFormat Format)
	{
		const Engine::FOutputAudioFormatInfo* FormatInfo = Engine::GetOutputAudioFormatInfo().Find(Format);
		return FormatInfo != nullptr ? FormatInfo->OutputVertexChannelOrder : TArray<FVertexName>{};
	}
	
	class FDynamicGeneratorBuilder
	{
	public:
		FDynamicGeneratorBuilder(FSampleRate SampleRate, int32 BlockSize)
			: OperatorSettings(SampleRate, static_cast<float>(SampleRate) / BlockSize)
			, Generator(OperatorSettings)
			, RenderBuffer(BlockSize)
		{
			check(OperatorSettings.GetNumFramesPerBlock() == BlockSize);

			// Add the minimum required interfaces so we don't get warnings when we run the test(s)
			// NB: if you start getting warnings, check FMetasoundDynamicGraphGenerator to see if the required
			// I/O has changed.
			// TODO: Add a future-proof way to do this
			AddInput<FTrigger>(Frontend::SourceInterface::Inputs::OnPlay, FGuid::NewGuid(), {});
			AddOutput<FAudioBuffer>(Engine::OutputFormatMonoInterface::Outputs::MonoOut, FGuid::NewGuid());

			// Make the generator
			FOperatorBuilderSettings BuilderSettings = FOperatorBuilderSettings::GetDefaultSettings();
			BuilderSettings.bEnableOperatorRebind = true;

			FMetasoundEnvironment Environment;
			Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, 123);
			
			FMetasoundDynamicGraphGeneratorInitParams InitParams
			{
				FGeneratorInitParams
				{
					.OperatorSettings = OperatorSettings,
					.BuilderSettings = MoveTemp(BuilderSettings),
					.Graph = MakeShared<FGraph>(Transactor.GetGraph()),
					.Environment = Environment,
					.AudioOutputNames = GetAudioOutputVertexNames(EMetaSoundOutputAudioFormat::Mono),
					.bBuildSynchronous = true,
					.AssetPath = FTopLevelAssetPath { FName(), FName("TestMetaSoundGenerator") }
				},
				Transactor.CreateTransformQueue(OperatorSettings, Environment, nullptr) // Create transaction queue
			};

			Generator.Init(MoveTemp(InitParams));
		}
		
		template<typename DataType>
		bool AddInput(const FVertexName& Name, const FGuid& NodeGuid, const FLiteral& DefaultLiteral)
		{
			TSharedPtr<const FNodeClassMetadata> InputClassMetadata = DataRegistry.GetInputClassMetadata(GetMetasoundDataTypeName<DataType>());
			if (!InputClassMetadata)
			{
				return false;
			}

			FVertexInterface InputNodeInterface = InputClassMetadata->DefaultInterface;
			// Vertex names must be set for input nodes
			InputNodeInterface.GetInputInterface().At(0).VertexName = Name;
			InputNodeInterface.GetOutputInterface().At(0).VertexName = Name;
			InputNodeInterface.GetInputInterface().At(0).SetDefaultLiteral(DefaultLiteral);

			TUniquePtr<INode> Node = DataRegistry.CreateInputNode(GetMetasoundDataTypeName<DataType>(), FNodeData{ Name, NodeGuid, InputNodeInterface });

			if (!Node.IsValid())
			{
				return false;
			}

			Transactor.AddNode(NodeGuid, MoveTemp(Node));

			const auto CreateDataReference = [](
				const FOperatorSettings& InSettings,
				const FName InDataType, 
				const FLiteral& InLiteral,
				const EDataReferenceAccessType InAccessType)
			{
				const Frontend::IDataTypeRegistry& DataRegistry2 = Frontend::IDataTypeRegistry::Get();
				return DataRegistry2.CreateDataReference(InDataType, InAccessType, InLiteral, InSettings);
			};
			
			Transactor.AddInputDataDestination(
				NodeGuid,
				Name,
				DefaultLiteral,
				CreateDataReference);

			return true;
		}

		void RemoveInput(const FVertexName& Name, const FGuid& NodeGuid)
		{
			Transactor.RemoveInputDataDestination(Name);
			Transactor.RemoveNode(NodeGuid);
		}

		template<typename DataType>
		bool AddOutput(const FVertexName& Name, const FGuid& NodeGuid)
		{
			TSharedPtr<const FNodeClassMetadata> OutputClassMetadata = DataRegistry.GetOutputClassMetadata(GetMetasoundDataTypeName<DataType>());
			if (!OutputClassMetadata)
			{
				return false;
			}
			FVertexInterface OutputNodeInterface = OutputClassMetadata->DefaultInterface;
			// Vertex names must be set for output nodes
			OutputNodeInterface.GetInputInterface().At(0).VertexName = Name;
			OutputNodeInterface.GetOutputInterface().At(0).VertexName = Name;

			TUniquePtr<INode> Node = DataRegistry.CreateOutputNode(GetMetasoundDataTypeName<DataType>(), FNodeData{ Name, NodeGuid, OutputNodeInterface });

			if (!Node.IsValid())
			{
				return false;
			}

			Transactor.AddNode(NodeGuid, MoveTemp(Node));
			Transactor.AddOutputDataSource(NodeGuid, Name);
			return true;
		}

		void RemoveOutput(const FVertexName& Name, const FGuid& NodeGuid)
		{
			Transactor.RemoveOutputDataSource(Name);
			Transactor.RemoveNode(NodeGuid);
		}

		void Execute()
		{
			Generator.OnGenerateAudio(RenderBuffer.GetData(), RenderBuffer.Num());
		}

		const FOperatorSettings OperatorSettings;
		FMetasoundDynamicGraphGenerator Generator;

	private:
		DynamicGraph::FDynamicOperatorTransactor Transactor;
		Frontend::IDataTypeRegistry& DataRegistry = Frontend::IDataTypeRegistry::Get();
		FAudioBuffer RenderBuffer;
	};
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetasoundGeneratorDynamicVertexInterfaceUpdatedTest,
		"Audio.Metasound.Generator.Dynamic.VertexInterfaceUpdated",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FMetasoundGeneratorDynamicVertexInterfaceUpdatedTest, "Audio.Metasound.Generator.Dynamic.VertexInterfaceUpdated", "[MustPass]")
	bool FMetasoundGeneratorDynamicVertexInterfaceUpdatedTest::RunTest(const FString& Parameters)
	{
		// Make a dynamic generator
		FDynamicGeneratorBuilder GeneratorBuilder{ 48000, 480 };

		// Register for vertex interface updates
		FVertexInterfaceData LatestInterfaceData;
		TArray<FVertexInterfaceChange> LatestInterfaceChanges;

		GeneratorBuilder.Generator.OnVertexInterfaceDataUpdated.AddLambda([&LatestInterfaceData](FVertexInterfaceData VertexInterfaceData)
		{
			LatestInterfaceData = MoveTemp(VertexInterfaceData);
		});

		GeneratorBuilder.Generator.OnVertexInterfaceDataUpdatedWithChanges.AddLambda([&LatestInterfaceChanges](const TArray<FVertexInterfaceChange>& VertexInterfaceChanges)
		{
			LatestInterfaceChanges = VertexInterfaceChanges;
		});

		// Add an input
		const FVertexName InputName = "SomeInput";
		const FGuid InputGuid = FGuid::NewGuid();
		{
			// Add the input
			constexpr float DefaultValue = 123.456f;
			UTEST_TRUE("Added input", GeneratorBuilder.AddInput<float>(InputName, InputGuid, DefaultValue));

			// Render to flush the transaction queue
			GeneratorBuilder.Execute();

			// Check that the input actually got added with the default
			const FAnyDataReference* InputRef = LatestInterfaceData.GetInputs().FindDataReference(InputName);
			UTEST_NOT_NULL("Vertex data contains input", InputRef);
			const float* Value = InputRef->GetValue<float>();
			UTEST_NOT_NULL("Value exists", Value);
			UTEST_EQUAL("Value is default", *Value, DefaultValue);

			// Check that the change was tracked
			const TArray<FVertexInterfaceChange> InputChanges = LatestInterfaceChanges.FilterByPredicate([InputName](const FVertexInterfaceChange& Other)
				{
					return Other.VertexName.IsEqual(InputName);
				});
			UTEST_EQUAL("There is only one expected change with our Input", InputChanges.Num(), 1);
			UTEST_EQUAL("Input addition is for the right Vertex", InputChanges[0].VertexName, InputName);
			UTEST_EQUAL("Input addition is for the right Vertex type", InputChanges[0].VertexType, EMetasoundFrontendClassType::Input);
			UTEST_EQUAL("Input addition is the Added type", InputChanges[0].ChangeType, Metasound::EVertexInterfaceChangeType::Added);
		}

		// Remove the input
		GeneratorBuilder.RemoveInput(InputName, InputGuid);
		{
			// Render to flush the transaction queue
			GeneratorBuilder.Execute();

			// Check that the input actually got removed
			const FAnyDataReference* InputRef = LatestInterfaceData.GetInputs().FindDataReference(InputName);
			UTEST_NULL("Vertex data does not contain input", InputRef);

			// Check that the change was tracked
			UTEST_EQUAL("Input removal is present in changes", LatestInterfaceChanges.Num(), 1);
			const FVertexInterfaceChange LastChange = LatestInterfaceChanges.Last();
			UTEST_EQUAL("Input removal is for the right Vertex", LastChange.VertexName, InputName);
			UTEST_EQUAL("Input removal is for the right Vertex type", LastChange.VertexType, EMetasoundFrontendClassType::Input);
			UTEST_EQUAL("Input removal is the Removed type", LastChange.ChangeType, Metasound::EVertexInterfaceChangeType::Removed);
		}

		// Add an output
		const FVertexName OutputName = "SomeOutput";
		const FGuid OutputGuid = FGuid::NewGuid();
		{
			UTEST_TRUE("Added output", GeneratorBuilder.AddOutput<int32>(OutputName, OutputGuid));

			// Render to flush the transaction queue
			GeneratorBuilder.Execute();
			
			// check that the output actually got added
			const FAnyDataReference* OutputRef = LatestInterfaceData.GetOutputs().FindDataReference(OutputName);
			UTEST_NOT_NULL("Vertex data contains output", OutputRef);

			// Check that the change was tracked
			UTEST_EQUAL("Output addition is present in changes", LatestInterfaceChanges.Num(), 1);
			const FVertexInterfaceChange LastChange = LatestInterfaceChanges.Last();
			UTEST_EQUAL("Output addition is for the right Vertex", LastChange.VertexName, OutputName);
			UTEST_EQUAL("Output addition is for the right Vertex type", LastChange.VertexType, EMetasoundFrontendClassType::Output);
			UTEST_EQUAL("Output addition is the Removed type", LastChange.ChangeType, Metasound::EVertexInterfaceChangeType::Added);
		}

		// Remove the output
		GeneratorBuilder.RemoveOutput(OutputName, OutputGuid);
		{
			// Render to flush the transaction queue
			GeneratorBuilder.Execute();

			// Check that the output actually got removed
			const FAnyDataReference* OutputRef = LatestInterfaceData.GetOutputs().FindDataReference(OutputName);
			UTEST_NULL("Vertex data does not contain output", OutputRef);

			// Check that the change was tracked
			UTEST_EQUAL("Output removal is present in changes", LatestInterfaceChanges.Num(), 1);
			const FVertexInterfaceChange LastChange = LatestInterfaceChanges.Last();
			UTEST_EQUAL("Output removal is for the right Vertex", LastChange.VertexName, OutputName);
			UTEST_EQUAL("Output removal is for the right Vertex type", LastChange.VertexType, EMetasoundFrontendClassType::Output);
			UTEST_EQUAL("Output removal is the Removed type", LastChange.ChangeType, Metasound::EVertexInterfaceChangeType::Removed);
		}
		
		return true;
	}
}

#endif
