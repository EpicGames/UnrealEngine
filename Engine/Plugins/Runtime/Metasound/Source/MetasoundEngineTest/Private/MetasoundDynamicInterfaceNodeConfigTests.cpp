// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "CQTest.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorBuilderSettings.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundEnvironment.h"
#include "MetasoundFrontendGraphBuilder.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundGenerator.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Interfaces/MetasoundOutputFormatInterfaces.h"

namespace Metasound::Test::DynamicInterfaceNodeConfigTests
{
	// Helper: get expanded vertex name for a sub-interface instance.
	// Instance 0 uses the base name, subsequent instances append _N.
	static FVertexName GetExpandedVertexName(const FVertexName& InBaseName, int32 InIndex)
	{
		if (InIndex == 0)
		{
			return InBaseName;
		}
		return FVertexName(*FString::Printf(TEXT("%s_%d"), *InBaseName.ToString(), InIndex));
	}
	// -----------------------------------------------------------------------
	// Test-only node with sub-interfaces.
	// Declares one fixed input and a "Channels" sub-interface with
	// Min=1, Max=8, Default=2 containing one input and one output per
	// instance.
	// -----------------------------------------------------------------------
	namespace SubInterfaceTestNode
	{
		static const FName ChannelsSubInterfaceName = "Channels";
		static const FVertexName GainInputName = "Gain";
		static const FVertexName AudioInputName = "Audio In";
		static const FVertexName AudioOutputName = "Audio Out";

		static FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				TInputDataVertex<float>(GainInputName, FDataVertexMetadata{ INVTEXT("Gain") }),
				FSubInterfaceDescription{ ChannelsSubInterfaceName, 1, 8, 2 },
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TInputDataVertex<float>(AudioInputName, FDataVertexMetadata{ INVTEXT("Audio input") })
				),
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TOutputDataVertex<float>(AudioOutputName, FDataVertexMetadata{ INVTEXT("Audio output") })
				)
			);
		}
	}

	// -----------------------------------------------------------------------
	// Test-only node with variant vertices.
	// Declares one input and one output with variant types float/int32.
	// -----------------------------------------------------------------------
	namespace VariantTestNode
	{
		static const FName ValueVariantName = "Value";
		static const FVertexName ValueInputName = "Value In";
		static const FVertexName ValueOutputName = "Value Out";

		static FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				TInputVariantVertex<float, int32>(ValueInputName, ValueVariantName, FDataVertexMetadata{ INVTEXT("Value input") }),
				TOutputVariantVertex<float, int32>(ValueOutputName, ValueVariantName, FDataVertexMetadata{ INVTEXT("Value output") })
			);
		}
	}

	// -----------------------------------------------------------------------
	// Test-only node with both sub-interfaces and variants.
	// -----------------------------------------------------------------------
	namespace CombinedTestNode
	{
		static const FName ChannelsSubInterfaceName = "Channels";
		static const FName TypeVariantName = "Type";
		static const FVertexName DataInputName = "Data In";
		static const FVertexName DataOutputName = "Data Out";

		static FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				FSubInterfaceDescription{ ChannelsSubInterfaceName, 1, 8, 2 },
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TInputVariantVertex<float, int32>(DataInputName, TypeVariantName, FDataVertexMetadata{ INVTEXT("Data input") })
				),
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TOutputVariantVertex<float, int32>(DataOutputName, TypeVariantName, FDataVertexMetadata{ INVTEXT("Data output") })
				)
			);
		}
	}

	// -----------------------------------------------------------------------
	// Registered test node for document serialize/deserialize round-trip.
	// Records sub-interface instance counts observed during CreateOperator
	// so tests can verify layouts survived the FMetasoundFrontendDocument
	// pipeline.
	// -----------------------------------------------------------------------
	namespace SerializeTestNode
	{
		static const FName ChannelsSubInterfaceName = "Channels";
		static const FVertexName GainInputName = "Gain";
		static const FVertexName AudioInputName = "Audio In";
		static const FVertexName AudioOutputName = "Audio Out";

		static FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				TInputDataVertex<float>(GainInputName, FDataVertexMetadata{ INVTEXT("Gain") }),
				FSubInterfaceDescription{ ChannelsSubInterfaceName, 1, 8, 2 },
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TInputDataVertex<FAudioBuffer>(AudioInputName, FDataVertexMetadata{ INVTEXT("Audio input") })
				),
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TOutputDataVertex<FAudioBuffer>(AudioOutputName, FDataVertexMetadata{ INVTEXT("Audio output") })
				)
			);
		}

		class FOperator : public FNoOpOperator
		{
		public:
			// Observable state: set by CreateOperator so tests can verify
			// what the operator saw after the document round-trip.
			static uint32 LastInputSubInterfaceInstanceCount;

			TOptional<TDataReadReference<float>> GainInput;
			TArray<TDataReadReference<FAudioBuffer>> AudioInputBuffers;
			TArray<TDataWriteReference<FAudioBuffer>> AudioOutputBuffers;

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto Create = []() -> FNodeClassMetadata
				{
					return FNodeClassMetadata
					{
						FNodeClassName{ "Test", "SubInterfaceSerialize", "" },
						1, // Major Version
						0, // Minor Version
						INVTEXT("SubInterface Serialize Test"),
						INVTEXT("Test node for sub-interface document round-trip"),
						{}, // Author
						{}, // Prompt if missing
						CreateTestClassInterface(),
						{}, // Category
						{}, // Keywords
						FNodeDisplayStyle()
					};
				};
				static const FNodeClassMetadata Info = Create();
				return Info;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				LastInputSubInterfaceInstanceCount = InParams.InputData.GetNumSubInterfaceInstances(ChannelsSubInterfaceName);

				TUniquePtr<FOperator> Op = MakeUnique<FOperator>();

				Op->GainInput = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(GainInputName, InParams.OperatorSettings);
				Op->AudioInputBuffers = InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<FAudioBuffer>(
					ChannelsSubInterfaceName, AudioInputName, InParams.OperatorSettings);

				// Create audio output buffers — output sub-interface count matches input
				// since both are on the same "Channels" sub-interface.
				const uint32 NumInstances = InParams.InputData.GetNumSubInterfaceInstances(ChannelsSubInterfaceName);
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					Op->AudioOutputBuffers.Emplace(TDataWriteReference<FAudioBuffer>::CreateNew(InParams.OperatorSettings));
				}

				return Op;
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				if (GainInput.IsSet())
				{
					InOutVertexData.BindReadVertex(GainInputName, GainInput.GetValue());
				}
				for (int32 i = 0; i < AudioInputBuffers.Num(); ++i)
				{
					InOutVertexData.BindReadVertex(GetExpandedVertexName(AudioInputName, i), AudioInputBuffers[i]);
				}
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				for (int32 i = 0; i < AudioOutputBuffers.Num(); ++i)
				{
					InOutVertexData.BindVertex(GetExpandedVertexName(AudioOutputName, i), AudioOutputBuffers[i]);
				}
			}
		};

		uint32 FOperator::LastInputSubInterfaceInstanceCount = 0;

		using FNode = TNodeFacade<FOperator>;
		METASOUND_REGISTER_NODE(FNode)
	}

	// -----------------------------------------------------------------------
	// Registered test node with FMetaSoundDynamicInterfaceNodeConfiguration.
	// Uses METASOUND_REGISTER_NODE_AND_CONFIGURATION to associate the
	// node with the production base configuration type.
	// FMetaSoundDynamicInterfaceNodeConfiguration::OverrideDefaultInterface looks up
	// the FClassInterface via INodeClassRegistry::GetChecked().FindClassInterface()
	// at configuration time.
	// -----------------------------------------------------------------------
	namespace ConfigTestNode
	{
		static const FName ChannelsSubInterfaceName = "Channels";
		static const FVertexName GainInputName = "Gain";
		static const FVertexName AudioInputName = "Audio In";
		static const FVertexName AudioOutputName = "Audio Out";

		FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				TInputDataVertex<float>(GainInputName, FDataVertexMetadata{ INVTEXT("Gain") }),
				FSubInterfaceDescription{ ChannelsSubInterfaceName, 1, 8, 2 },
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TInputDataVertex<float>(AudioInputName, FDataVertexMetadata{ INVTEXT("Audio input") })
				),
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TOutputDataVertex<float>(AudioOutputName, FDataVertexMetadata{ INVTEXT("Audio output") })
				)
			);
		}

		class FOperator : public FNoOpOperator
		{
		public:
			TOptional<TDataReadReference<float>> GainInput;
			TArray<TDataReadReference<float>> AudioInputs;
			TArray<TDataWriteReference<float>> AudioOutputs;

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto Create = []() -> FNodeClassMetadata
				{
					return FNodeClassMetadata
					{
						FNodeClassName{ "Test", "ConfigSubInterface", "" },
						1, // Major Version
						0, // Minor Version
						INVTEXT("Config SubInterface Test"),
						INVTEXT("Test node for sub-interface configuration round-trip"),
						{}, // Author
						{}, // Prompt if missing
						CreateTestClassInterface(),
						{}, // Category
						{}, // Keywords
						FNodeDisplayStyle()
					};
				};
				static const FNodeClassMetadata Info = Create();
				return Info;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				TUniquePtr<FOperator> Op = MakeUnique<FOperator>();

				Op->GainInput = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(GainInputName, InParams.OperatorSettings);
				Op->AudioInputs = InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<float>(
					ChannelsSubInterfaceName, AudioInputName, InParams.OperatorSettings);

				const uint32 NumInstances = InParams.InputData.GetNumSubInterfaceInstances(ChannelsSubInterfaceName);
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					Op->AudioOutputs.Emplace(TDataWriteReference<float>::CreateNew(0.0f));
				}

				return Op;
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				if (GainInput.IsSet())
				{
					InOutVertexData.BindReadVertex(GainInputName, GainInput.GetValue());
				}
				for (int32 i = 0; i < AudioInputs.Num(); ++i)
				{
					InOutVertexData.BindReadVertex(GetExpandedVertexName(AudioInputName, i), AudioInputs[i]);
				}
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				for (int32 i = 0; i < AudioOutputs.Num(); ++i)
				{
					InOutVertexData.BindVertex(GetExpandedVertexName(AudioOutputName, i), AudioOutputs[i]);
				}
			}
		};

		using FNode = TNodeFacade<FOperator>;
		METASOUND_REGISTER_NODE_AND_CONFIGURATION(FNode, FMetaSoundDynamicInterfaceNodeConfiguration)
	}
}

using namespace Metasound;
using namespace Metasound::Frontend;
using namespace Metasound::Test::DynamicInterfaceNodeConfigTests;

// =========================================================================
// FClassInterface Creation & SubInterface Layout Tests
// =========================================================================

TEST_CLASS(MetasoundClassInterfaceSubInterfaceTests, "Audio.Metasound.DynamicInterfaceNodeConfig.SubInterface")
{

	TEST_METHOD(ClassInterface_HasSubInterfaceDescriptions)
	{
		FClassInterface ClassInterface = SubInterfaceTestNode::CreateTestClassInterface();

		TConstArrayView<FSubInterfaceDescription> Descriptions = ClassInterface.GetSubInterfaceDescriptions();
		ASSERT_THAT(AreEqual(Descriptions.Num(), 1));
		ASSERT_THAT(AreEqual(Descriptions[0].SubInterfaceName, SubInterfaceTestNode::ChannelsSubInterfaceName));
		ASSERT_THAT(AreEqual(Descriptions[0].Min, 1u));
		ASSERT_THAT(AreEqual(Descriptions[0].Max, 8u));
		ASSERT_THAT(AreEqual(Descriptions[0].NumDefault, 2u));
	}

	// Group A: Consolidated vertex count test covering Min, Default, Custom, Max
	TEST_METHOD(CreateVertexInterface_VariousCounts_HasCorrectVertexCount)
	{
		FClassInterface ClassInterface = SubInterfaceTestNode::CreateTestClassInterface();

		// Test Min, Default, Custom, and Max counts
		for (const auto& [Num, ExpectedInputs, ExpectedOutputs] : TArray<TTuple<uint32, int32, int32>>{
			{1, 2, 1}, {2, 3, 2}, {4, 5, 4}, {8, 9, 8}})
		{
			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({ { SubInterfaceTestNode::ChannelsSubInterfaceName, Num } }, {});
			ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), ExpectedInputs));
			ASSERT_THAT(AreEqual(VertexInterface.GetOutputInterface().Num(), ExpectedOutputs));
		}
	}

	// Group B: Consolidated ForEachSubInterfaceInstance test checking both
	// input/output iteration count and per-instance vertex count
	TEST_METHOD(SubInterfaceLayout_ForEachInstance_IteratesCorrectCountWithOneVertexEach)
	{
		FClassInterface ClassInterface = SubInterfaceTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({ { SubInterfaceTestNode::ChannelsSubInterfaceName, 3 } }, {});

		int32 InputInstanceCount = 0;
		VertexInterface.GetInputInterface().ForEachSubInterfaceInstance(
			SubInterfaceTestNode::ChannelsSubInterfaceName,
			[this, &InputInstanceCount](TConstArrayView<FInputDataVertex> Vertices)
			{
				InputInstanceCount++;
				ASSERT_THAT(AreEqual(Vertices.Num(), 1));
			});
		ASSERT_THAT(AreEqual(InputInstanceCount, 3));

		int32 OutputInstanceCount = 0;
		VertexInterface.GetOutputInterface().ForEachSubInterfaceInstance(
			SubInterfaceTestNode::ChannelsSubInterfaceName,
			[this, &OutputInstanceCount](TConstArrayView<FOutputDataVertex> Vertices)
			{
				OutputInstanceCount++;
				ASSERT_THAT(AreEqual(Vertices.Num(), 1));
			});
		ASSERT_THAT(AreEqual(OutputInstanceCount, 3));
	}
};

// =========================================================================
// SubInterface Layout -> FVertexInterfaceData Round-Trip Tests
// =========================================================================

TEST_CLASS(MetasoundSubInterfaceVertexDataTests, "Audio.Metasound.DynamicInterfaceNodeConfig.SubInterfaceVertexData")
{

	// Group C+D+E: Consolidated test covering GetNumSubInterfaceInstances and
	// BindSubInterfaceVertices for both input and output across multiple counts
	TEST_METHOD(VertexInterfaceData_VariousCounts_GetNumAndBindSucceed)
	{
		FClassInterface ClassInterface = SubInterfaceTestNode::CreateTestClassInterface();

		for (uint32 NumInstances : { 1u, 2u, 4u, 8u })
		{
			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({ { SubInterfaceTestNode::ChannelsSubInterfaceName, NumInstances } }, {});

			// Input side: verify count and bind
			FInputVertexInterfaceData InputData(VertexInterface.GetInputInterface());
			ASSERT_THAT(AreEqual(InputData.GetNumSubInterfaceInstances(SubInterfaceTestNode::ChannelsSubInterfaceName), NumInstances));

			TArray<TDataWriteReference<float>> InputRefs;
			for (uint32 i = 0; i < NumInstances; ++i)
			{
				InputRefs.Emplace(TDataWriteReference<float>::CreateNew(0.0f));
			}

			InputData.BindSubInterfaceVertices(
				SubInterfaceTestNode::ChannelsSubInterfaceName,
				SubInterfaceTestNode::AudioInputName,
				InputRefs);
			ASSERT_THAT(IsTrue(InputData.IsVertexBound(SubInterfaceTestNode::AudioInputName)));

			// Output side: verify count and bind
			FOutputVertexInterfaceData OutputData(VertexInterface.GetOutputInterface());
			ASSERT_THAT(AreEqual(OutputData.GetNumSubInterfaceInstances(SubInterfaceTestNode::ChannelsSubInterfaceName), NumInstances));

			TArray<TDataWriteReference<float>> OutputRefs;
			for (uint32 i = 0; i < NumInstances; ++i)
			{
				OutputRefs.Emplace(TDataWriteReference<float>::CreateNew(0.0f));
			}

			OutputData.BindSubInterfaceVertices(
				SubInterfaceTestNode::ChannelsSubInterfaceName,
				SubInterfaceTestNode::AudioOutputName,
				OutputRefs);
			ASSERT_THAT(IsTrue(OutputData.IsVertexBound(SubInterfaceTestNode::AudioOutputName)));
		}
	}
};

// =========================================================================
// Variant Configuration Tests
// =========================================================================

TEST_CLASS(MetasoundClassInterfaceVariantTests, "Audio.Metasound.DynamicInterfaceNodeConfig.Variant")
{

	TEST_METHOD(ClassInterface_HasVariantDescriptions)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		TConstArrayView<FVariantDescription> Descriptions = ClassInterface.GetVariantDescriptions();
		ASSERT_THAT(AreEqual(Descriptions.Num(), 1));
		ASSERT_THAT(AreEqual(Descriptions[0].VariantName, VariantTestNode::ValueVariantName));
		ASSERT_THAT(AreEqual(Descriptions[0].DataTypes.Num(), 2));
	}

	TEST_METHOD(CreateVertexInterface_FloatVariant_HasFloatVertices)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({}, { { VariantTestNode::ValueVariantName, GetMetasoundDataTypeName<float>() } });

		ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 1));
		ASSERT_THAT(AreEqual(VertexInterface.GetOutputInterface().Num(), 1));

		// Verify the input vertex has float type
		const FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find(VariantTestNode::ValueInputName);
		ASSERT_THAT(IsNotNull(InputVertex));
		ASSERT_THAT(AreEqual(InputVertex->DataTypeName, GetMetasoundDataTypeName<float>()));

		// Verify the output vertex has float type
		const FOutputDataVertex* OutputVertex = VertexInterface.GetOutputInterface().Find(VariantTestNode::ValueOutputName);
		ASSERT_THAT(IsNotNull(OutputVertex));
		ASSERT_THAT(AreEqual(OutputVertex->DataTypeName, GetMetasoundDataTypeName<float>()));
	}

	TEST_METHOD(CreateVertexInterface_Int32Variant_HasInt32Vertices)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({}, { { VariantTestNode::ValueVariantName, GetMetasoundDataTypeName<int32>() } });

		const FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find(VariantTestNode::ValueInputName);
		ASSERT_THAT(IsNotNull(InputVertex));
		ASSERT_THAT(AreEqual(InputVertex->DataTypeName, GetMetasoundDataTypeName<int32>()));

		const FOutputDataVertex* OutputVertex = VertexInterface.GetOutputInterface().Find(VariantTestNode::ValueOutputName);
		ASSERT_THAT(IsNotNull(OutputVertex));
		ASSERT_THAT(AreEqual(OutputVertex->DataTypeName, GetMetasoundDataTypeName<int32>()));
	}

	TEST_METHOD(CreateVertexInterface_DefaultVariant_HasUnresolvedType)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		// Empty variant config — variant vertex type is unresolved (None)
		// until a variant selection is explicitly provided.
		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface();

		const FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find(VariantTestNode::ValueInputName);
		ASSERT_THAT(IsNotNull(InputVertex));
		ASSERT_THAT(AreEqual(InputVertex->DataTypeName, FName(NAME_None)));
	}

	TEST_METHOD(FindVariantDescriptionForInput_FindsCorrectVariant)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		const FVariantDescription* Description = ClassInterface.FindVariantDescriptionForInput(VariantTestNode::ValueInputName);
		ASSERT_THAT(IsNotNull(Description));
		ASSERT_THAT(AreEqual(Description->VariantName, VariantTestNode::ValueVariantName));
	}

	TEST_METHOD(FindVariantDescriptionForOutput_FindsCorrectVariant)
	{
		FClassInterface ClassInterface = VariantTestNode::CreateTestClassInterface();

		const FVariantDescription* Description = ClassInterface.FindVariantDescriptionForOutput(VariantTestNode::ValueOutputName);
		ASSERT_THAT(IsNotNull(Description));
		ASSERT_THAT(AreEqual(Description->VariantName, VariantTestNode::ValueVariantName));
	}
};

// =========================================================================
// Combined SubInterface + Variant Tests
// =========================================================================

TEST_CLASS(MetasoundClassInterfaceCombinedTests, "Audio.Metasound.DynamicInterfaceNodeConfig.Combined")
{

	TEST_METHOD(CreateVertexInterface_SubInterfaceAndVariant_HasCorrectCounts)
	{
		FClassInterface ClassInterface = CombinedTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
			{ { CombinedTestNode::ChannelsSubInterfaceName, 3 } },
			{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<float>() } });

		// 3 sub-interface instances, each with 1 input and 1 output
		ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 3));
		ASSERT_THAT(AreEqual(VertexInterface.GetOutputInterface().Num(), 3));
	}

	TEST_METHOD(CreateVertexInterface_SubInterfaceAndVariant_VerticesHaveCorrectType)
	{
		FClassInterface ClassInterface = CombinedTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
			{ { CombinedTestNode::ChannelsSubInterfaceName, 2 } },
			{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<int32>() } });

		// All input vertices should be int32 (from variant config)
		for (const FInputDataVertex& Vertex : VertexInterface.GetInputInterface())
		{
			ASSERT_THAT(AreEqual(Vertex.DataTypeName, GetMetasoundDataTypeName<int32>()));
		}

		// All output vertices should be int32
		for (const FOutputDataVertex& Vertex : VertexInterface.GetOutputInterface())
		{
			ASSERT_THAT(AreEqual(Vertex.DataTypeName, GetMetasoundDataTypeName<int32>()));
		}
	}

	TEST_METHOD(SubInterfaceLayout_WithVariant_ForEachInstanceWorks)
	{
		FClassInterface ClassInterface = CombinedTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
			{ { CombinedTestNode::ChannelsSubInterfaceName, 3 } },
			{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<float>() } });

		int32 InputInstanceCount = 0;
		VertexInterface.GetInputInterface().ForEachSubInterfaceInstance(
			CombinedTestNode::ChannelsSubInterfaceName,
			[&InputInstanceCount](TConstArrayView<FInputDataVertex> Vertices)
			{
				InputInstanceCount++;
			});

		ASSERT_THAT(AreEqual(InputInstanceCount, 3));

		int32 OutputInstanceCount = 0;
		VertexInterface.GetOutputInterface().ForEachSubInterfaceInstance(
			CombinedTestNode::ChannelsSubInterfaceName,
			[&OutputInstanceCount](TConstArrayView<FOutputDataVertex> Vertices)
			{
				OutputInstanceCount++;
			});

		ASSERT_THAT(AreEqual(OutputInstanceCount, 3));
	}

	TEST_METHOD(VertexInterfaceData_SubInterfaceAndVariant_BindSucceeds)
	{
		FClassInterface ClassInterface = CombinedTestNode::CreateTestClassInterface();

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
			{ { CombinedTestNode::ChannelsSubInterfaceName, 2 } },
			{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<float>() } });
		FInputVertexInterfaceData InputData(VertexInterface.GetInputInterface());

		ASSERT_THAT(AreEqual(InputData.GetNumSubInterfaceInstances(CombinedTestNode::ChannelsSubInterfaceName), 2u));

		TArray<TDataWriteReference<float>> FloatRefs;
		for (uint32 i = 0; i < 2; ++i)
		{
			FloatRefs.Emplace(TDataWriteReference<float>::CreateNew(0.0f));
		}

		InputData.BindSubInterfaceVertices(
			CombinedTestNode::ChannelsSubInterfaceName,
			CombinedTestNode::DataInputName,
			FloatRefs);

		ASSERT_THAT(IsTrue(InputData.IsVertexBound(CombinedTestNode::DataInputName)));
	}

	TEST_METHOD(IndependentConfigChanges_SubInterfaceCountAndVariantType)
	{
		FClassInterface ClassInterface = CombinedTestNode::CreateTestClassInterface();

		// Config A: 2 instances, float type
		{
			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
				{ { CombinedTestNode::ChannelsSubInterfaceName, 2 } },
				{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<float>() } });
			ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 2));
			for (const FInputDataVertex& Vertex : VertexInterface.GetInputInterface())
			{
				ASSERT_THAT(AreEqual(Vertex.DataTypeName, GetMetasoundDataTypeName<float>()));
			}
		}

		// Config B: 4 instances, int32 type -- changed independently
		{
			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
				{ { CombinedTestNode::ChannelsSubInterfaceName, 4 } },
				{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<int32>() } });
			ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 4));
			for (const FInputDataVertex& Vertex : VertexInterface.GetInputInterface())
			{
				ASSERT_THAT(AreEqual(Vertex.DataTypeName, GetMetasoundDataTypeName<int32>()));
			}
		}

		// Config C: 2 instances, int32 type -- only count changed from B
		{
			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(
				{ { CombinedTestNode::ChannelsSubInterfaceName, 2 } },
				{ { CombinedTestNode::TypeVariantName, GetMetasoundDataTypeName<int32>() } });
			ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 2));
			for (const FInputDataVertex& Vertex : VertexInterface.GetInputInterface())
			{
				ASSERT_THAT(AreEqual(Vertex.DataTypeName, GetMetasoundDataTypeName<int32>()));
			}
		}
	}
};

// =========================================================================
// FClassInterface <-> FVertexInterface Conversion Tests
//
// These tests document the REQUIRED behavior for the FClassInterface type
// changes in the design spec. They will fail until FClassInterface is made
// constructible from FVertexInterface and convertible back.
// =========================================================================

TEST_CLASS(MetasoundClassInterfaceConversionTests, "Audio.Metasound.DynamicInterfaceNodeConfig.Conversion")
{

	TEST_METHOD(ConstructFromVertexInterface_NoSubInterfacesOrVariants)
	{
		FVertexInterface VertexInterface(
			FInputVertexInterface(TInputDataVertex<float>("In", FDataVertexMetadata{})),
			FOutputVertexInterface(TOutputDataVertex<float>("Out", FDataVertexMetadata{}))
		);

		FClassInterface ClassInterface(VertexInterface);

		ASSERT_THAT(AreEqual(ClassInterface.GetSubInterfaceDescriptions().Num(), 0));
		ASSERT_THAT(AreEqual(ClassInterface.GetVariantDescriptions().Num(), 0));

		// Round-trip: should produce equivalent vertex interface
		FVertexInterface RoundTripped = ClassInterface.CreateVertexInterface();
		ASSERT_THAT(IsTrue(RoundTripped == VertexInterface));
	}

	TEST_METHOD(ClassInterface_IsCopyable)
	{
		FClassInterface Original = Test::DynamicInterfaceNodeConfigTests::SubInterfaceTestNode::CreateTestClassInterface();
		FClassInterface Copy(Original);

		ASSERT_THAT(AreEqual(Copy.GetSubInterfaceDescriptions().Num(), 1));

		FVertexInterface OriginalVI = Original.CreateVertexInterface(
			{ { FName("Channels"), 2u } }, {});
		FVertexInterface CopyVI = Copy.CreateVertexInterface(
			{ { FName("Channels"), 2u } }, {});

		ASSERT_THAT(IsTrue(OriginalVI == CopyVI));
	}

	TEST_METHOD(ExplicitConversion_ProducesDefaultInterface)
	{
		FClassInterface ClassInterface = Test::DynamicInterfaceNodeConfigTests::SubInterfaceTestNode::CreateTestClassInterface();

		// Explicit conversion should produce the default-configured interface
		FVertexInterface Implicit = FVertexInterface(ClassInterface);
		FVertexInterface Explicit = ClassInterface.CreateVertexInterface();

		ASSERT_THAT(IsTrue(Implicit == Explicit));
	}

	TEST_METHOD(ClassInterface_WithoutSubInterfaces_ProducesCorrectDefault)
	{
		// A FClassInterface with no sub-interfaces or variants should
		// produce a FVertexInterface identical to what you'd get from
		// a plain FVertexInterface construction.
		FClassInterface ClassInterface = CreateClassInterface(
			TInputDataVertex<float>("In", FDataVertexMetadata{}),
			TOutputDataVertex<float>("Out", FDataVertexMetadata{})
		);

		ASSERT_THAT(AreEqual(ClassInterface.GetSubInterfaceDescriptions().Num(), 0));
		ASSERT_THAT(AreEqual(ClassInterface.GetVariantDescriptions().Num(), 0));

		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface();
		ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 1));
		ASSERT_THAT(AreEqual(VertexInterface.GetOutputInterface().Num(), 1));

		const FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find("In");
		ASSERT_THAT(IsNotNull(InputVertex));
		ASSERT_THAT(AreEqual(InputVertex->DataTypeName, GetMetasoundDataTypeName<float>()));
	}
};

// =========================================================================
// Document Serialize/Deserialize Round-Trip Tests
//
// These tests verify that sub-interface layouts survive the full pipeline:
// Registered Node -> FMetasoundFrontendDocument -> FGraphBuilder::CreateGraph
// -> operator creation. Uses UMetaSoundSourceBuilder to build the document
// (not the deprecated FrontendController API). The registered test node
// records what it observes in CreateOperator so we can verify state after
// the round-trip.
// =========================================================================

TEST_CLASS(MetasoundSubInterfaceSerializeTests, "Audio.Metasound.DynamicInterfaceNodeConfig.Serialize")
{

	static const FMetasoundFrontendClassName SerializeTestNodeClassName;

	// Helper: create a source builder, add the sub-interface test node,
	// and return the builder. The source builder provides a properly
	// initialized FMetasoundFrontendDocument with OnPlay/AudioOut wired.
	UMetaSoundSourceBuilder& CreateTestSourceBuilder(
		FMetaSoundBuilderNodeOutputHandle& OutOnPlay,
		FMetaSoundBuilderNodeInputHandle& OutOnFinished,
		TArray<FMetaSoundBuilderNodeInputHandle>& OutAudioOuts)
	{
		EMetaSoundBuilderResult Result;
		UMetaSoundSourceBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreateSourceBuilder(
			"SubInterfaceSerializeTest",
			OutOnPlay,
			OutOnFinished,
			OutAudioOuts,
			Result,
			EMetaSoundOutputAudioFormat::Mono,
			true);
		// ensureAlwaysMsgf instead of check() — logs error + callstack without
		// aborting the test runner. If Builder is null, the null dereference will
		// be caught by the test runner's SEH handler and the test fails gracefully.
		ensureAlwaysMsgf(Builder != nullptr, TEXT("CreateSourceBuilder returned null"));
		ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("CreateSourceBuilder failed"));
		return *Builder;
	}

	// Helper: build an FFrontendGraph from the builder's document and
	// create a generator from it.
	TUniquePtr<FMetasoundGenerator> BuildGeneratorFromBuilder(UMetaSoundSourceBuilder& Builder)
	{
		const FMetasoundFrontendDocument& Document = Builder.GetConstBuilder().GetConstDocumentChecked();
		const FTopLevelAssetPath AssetPath(GetTransientPackage()->GetFName(), FName("SerializeTest"));

		TUniquePtr<FFrontendGraph> Graph = FGraphBuilder::CreateGraph(Document, AssetPath);
		if (!Graph)
		{
			return nullptr;
		}

		FOperatorBuilderSettings BuilderSettings;
		BuilderSettings.bFailOnAnyError = true;
		BuilderSettings.bPopulateInternalDataReferences = true;
		BuilderSettings.bValidateEdgeDataTypesMatch = true;
		BuilderSettings.bValidateNoCyclesInGraph = true;
		BuilderSettings.bValidateNoDuplicateInputs = true;
		BuilderSettings.bValidateOperatorOutputsAreBound = true;
		BuilderSettings.bValidateVerticesExist = true;

		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 SamplesPerBlock = 256;
		const FOperatorSettings OperatorSettings{ SampleRate, static_cast<float>(SampleRate) / SamplesPerBlock };

		FMetasoundEnvironment Environment;
		Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, 123);

		FGeneratorInitParams GeneratorInitParams
		{
			.OperatorSettings = OperatorSettings,
			.BuilderSettings = MoveTemp(BuilderSettings),
			.Graph = MakeShared<const FFrontendGraph, ESPMode::ThreadSafe>(*Graph),
			.Environment = Environment,
			.AudioOutputNames = { Metasound::Engine::OutputFormatMonoInterface::Outputs::MonoOut },
			.bBuildSynchronous = true,
			.AssetPath = FTopLevelAssetPath(FName(), "TestMetasound")
		};

		return MakeUnique<FMetasoundConstGraphGenerator>(MoveTemp(GeneratorInitParams));
	}

	TEST_METHOD(RegisteredNode_BuildsThroughDocument)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		TUniquePtr<FMetasoundGenerator> Generator = BuildGeneratorFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Generator.Get()));
	}

	TEST_METHOD(DocumentRoundTrip_SubInterfaceInstanceCount_IsPreserved)
	{
		// Reset the observable state
		SerializeTestNode::FOperator::LastInputSubInterfaceInstanceCount = 0;

		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Connect the test node's first audio output to the graph's audio output
		// so the node is not pruned during graph optimization.
		FMetaSoundBuilderNodeOutputHandle AudioOutHandle = Builder.FindNodeOutputByName(NodeHandle, SerializeTestNode::AudioOutputName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));
		ASSERT_THAT(IsTrue(AudioOuts.Num() > 0));
		Builder.ConnectNodes(AudioOutHandle, AudioOuts[0], Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		TUniquePtr<FMetasoundGenerator> Generator = BuildGeneratorFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Generator.Get()));

		// The registered node has 2 default sub-interface instances.
		// After going through the document pipeline, the operator should
		// still observe 2 sub-interface instances on its input data.
		ASSERT_THAT(AreEqual(SerializeTestNode::FOperator::LastInputSubInterfaceInstanceCount, 2u));
	}

};

const FMetasoundFrontendClassName MetasoundSubInterfaceSerializeTests::SerializeTestNodeClassName = { "Test", "SubInterfaceSerialize", "" };

// =========================================================================
// FMetaSoundDynamicInterfaceNodeConfiguration Round-Trip Tests
//
// These tests verify that when a node with FMetaSoundDynamicInterfaceNodeConfiguration
// is added to a document and configured with a non-default sub-interface
// count, the sub-interface layout data survives the full pipeline:
// Document -> FGraphBuilder::CreateGraph -> FNode -> FVertexInterface.
// =========================================================================

TEST_CLASS(MetasoundDynamicInterfaceNodeConfigRoundTripTests, "Audio.Metasound.DynamicInterfaceNodeConfig.ConfigRoundTrip")
{

	static const FMetasoundFrontendClassName ConfigTestNodeClassName;

	UMetaSoundSourceBuilder& CreateTestSourceBuilder(
		FMetaSoundBuilderNodeOutputHandle& OutOnPlay,
		FMetaSoundBuilderNodeInputHandle& OutOnFinished,
		TArray<FMetaSoundBuilderNodeInputHandle>& OutAudioOuts)
	{
		EMetaSoundBuilderResult Result;
		UMetaSoundSourceBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreateSourceBuilder(
			"ConfigRoundTripTest",
			OutOnPlay,
			OutOnFinished,
			OutAudioOuts,
			Result,
			EMetaSoundOutputAudioFormat::Mono,
			true);
		// ensureAlwaysMsgf instead of check() — logs error + callstack without
		// aborting the test runner. If Builder is null, the null dereference will
		// be caught by the test runner's SEH handler and the test fails gracefully.
		ensureAlwaysMsgf(Builder != nullptr, TEXT("CreateSourceBuilder returned null"));
		ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("CreateSourceBuilder failed"));
		return *Builder;
	}

	TEST_METHOD(AddNodeWithConfig_DefaultCount_DocumentHasCorrectVertexCount)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(ConfigTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Default config has NumChannels=2, so the node should have:
		// 1 fixed input (Gain) + 2 sub-interface inputs (Audio In) = 3 inputs
		// 2 sub-interface outputs (Audio Out) = 2 outputs
		const FMetasoundFrontendNode* Node = Builder.GetConstBuilder().FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(Node));
		ASSERT_THAT(AreEqual(Node->Interface.Inputs.Num(), 3));
		ASSERT_THAT(AreEqual(Node->Interface.Outputs.Num(), 2));
	}

	TEST_METHOD(SetNodeConfig_NonDefaultCount_DocumentHasCorrectVertexCount)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(ConfigTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Change configuration to 4 channels (non-default)
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
		NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>().SubInterfaceCounts.Add(ConfigTestNode::ChannelsSubInterfaceName, 4);

		bool bConfigSet = Builder.GetBuilder().SetNodeConfiguration(NodeHandle.NodeID, MoveTemp(NewConfig));
		ASSERT_THAT(IsTrue(bConfigSet));

		// After config change: 1 fixed + 4 sub-interface inputs = 5 inputs, 4 outputs
		const FMetasoundFrontendNode* Node = Builder.GetConstBuilder().FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(Node));
		ASSERT_THAT(AreEqual(Node->Interface.Inputs.Num(), 5));
		ASSERT_THAT(AreEqual(Node->Interface.Outputs.Num(), 4));
	}

	TEST_METHOD(GraphRoundTrip_NonDefaultSubInterfaceCount_VertexInterfaceHasCorrectInputCount)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(ConfigTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Set non-default: 4 channels
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
		NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>().SubInterfaceCounts.Add(ConfigTestNode::ChannelsSubInterfaceName, 4);
		Builder.GetBuilder().SetNodeConfiguration(NodeHandle.NodeID, MoveTemp(NewConfig));

		// Build FGraph from document
		const FMetasoundFrontendDocument& Document = Builder.GetConstBuilder().GetConstDocumentChecked();
		const FTopLevelAssetPath AssetPath(GetTransientPackage()->GetFName(), FName("ConfigRoundTripTest"));
		TUniquePtr<FFrontendGraph> Graph = FGraphBuilder::CreateGraph(Document, AssetPath);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		// Find the config test node in the graph
		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		const FVertexInterface& VertexInterface = GraphNode->GetVertexInterface();

		// Verify vertex counts: 1 fixed + 4 sub-interface inputs, 4 outputs
		ASSERT_THAT(AreEqual(VertexInterface.GetInputInterface().Num(), 5));
		ASSERT_THAT(AreEqual(VertexInterface.GetOutputInterface().Num(), 4));
	}

	// Group F: Consolidated input + output sub-interface layout preservation test
	TEST_METHOD(GraphRoundTrip_NonDefaultSubInterfaceCount_SubInterfaceLayoutIsPreserved)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(ConfigTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Set non-default: 4 channels
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
		NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>().SubInterfaceCounts.Add(ConfigTestNode::ChannelsSubInterfaceName, 4);
		Builder.GetBuilder().SetNodeConfiguration(NodeHandle.NodeID, MoveTemp(NewConfig));

		// Build FGraph from document
		const FMetasoundFrontendDocument& Document = Builder.GetConstBuilder().GetConstDocumentChecked();
		const FTopLevelAssetPath AssetPath(GetTransientPackage()->GetFName(), FName("ConfigRoundTripTest"));
		TUniquePtr<FFrontendGraph> Graph = FGraphBuilder::CreateGraph(Document, AssetPath);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		const FVertexInterface& VertexInterface = GraphNode->GetVertexInterface();

		// Verify input sub-interface layout: 4 instances
		int32 InputInstanceCount = 0;
		VertexInterface.GetInputInterface().ForEachSubInterfaceInstance(
			ConfigTestNode::ChannelsSubInterfaceName,
			[&InputInstanceCount](TConstArrayView<FInputDataVertex> Vertices)
			{
				InputInstanceCount++;
			});
		ASSERT_THAT(AreEqual(InputInstanceCount, 4));

		// Verify output sub-interface layout: 4 instances
		int32 OutputInstanceCount = 0;
		VertexInterface.GetOutputInterface().ForEachSubInterfaceInstance(
			ConfigTestNode::ChannelsSubInterfaceName,
			[&OutputInstanceCount](TConstArrayView<FOutputDataVertex> Vertices)
			{
				OutputInstanceCount++;
			});
		ASSERT_THAT(AreEqual(OutputInstanceCount, 4));
	}

	TEST_METHOD(GraphRoundTrip_NonDefaultSubInterfaceCount_VertexInterfaceDataHasCorrectInstanceCount)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(ConfigTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Set non-default: 4 channels
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
		NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>().SubInterfaceCounts.Add(ConfigTestNode::ChannelsSubInterfaceName, 4);
		Builder.GetBuilder().SetNodeConfiguration(NodeHandle.NodeID, MoveTemp(NewConfig));

		const FMetasoundFrontendDocument& Document = Builder.GetConstBuilder().GetConstDocumentChecked();
		const FTopLevelAssetPath AssetPath(GetTransientPackage()->GetFName(), FName("ConfigRoundTripTest"));
		TUniquePtr<FFrontendGraph> Graph = FGraphBuilder::CreateGraph(Document, AssetPath);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		const FVertexInterface& VertexInterface = GraphNode->GetVertexInterface();

		// Construct FInputVertexInterfaceData from the graph node's vertex interface.
		// This is the same path the operator builder uses when creating operators.
		// GetNumSubInterfaceInstances should return 4 if the layout survived.
		FInputVertexInterfaceData InputData(VertexInterface.GetInputInterface());
		uint32 NumInstances = InputData.GetNumSubInterfaceInstances(ConfigTestNode::ChannelsSubInterfaceName);
		ASSERT_THAT(AreEqual(NumInstances, 4u));
	}
};

const FMetasoundFrontendClassName MetasoundDynamicInterfaceNodeConfigRoundTripTests::ConfigTestNodeClassName = { "Test", "ConfigSubInterface", "" };

// =========================================================================
// FGraphBuilder FClassInterface Path Parity Tests (REQ-GB-1 through REQ-GB-5)
//
// These tests verify that the FClassInterface code path in
// FGraphBuilder::CreateVertexInterface maintains parity with the old
// flat-vertex-list code path. Each test maps to a requirement in
// MetaSoundDynamicInterfaceNodeConfigDesign.md.
// =========================================================================

TEST_CLASS(MetasoundGraphBuilderParityTests, "Audio.Metasound.DynamicInterfaceNodeConfig.GraphBuilderParity")
{

	static const FMetasoundFrontendClassName SerializeTestNodeClassName;

	UMetaSoundSourceBuilder& CreateTestSourceBuilder(
		FMetaSoundBuilderNodeOutputHandle& OutOnPlay,
		FMetaSoundBuilderNodeInputHandle& OutOnFinished,
		TArray<FMetaSoundBuilderNodeInputHandle>& OutAudioOuts)
	{
		EMetaSoundBuilderResult Result;
		UMetaSoundSourceBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreateSourceBuilder(
			"GraphBuilderParityTest",
			OutOnPlay,
			OutOnFinished,
			OutAudioOuts,
			Result,
			EMetaSoundOutputAudioFormat::Mono,
			true);
		// ensureAlwaysMsgf instead of check() — logs error + callstack without
		// aborting the test runner. If Builder is null, the null dereference will
		// be caught by the test runner's SEH handler and the test fails gracefully.
		ensureAlwaysMsgf(Builder != nullptr, TEXT("CreateSourceBuilder returned null"));
		ensureAlwaysMsgf(Result == EMetaSoundBuilderResult::Succeeded, TEXT("CreateSourceBuilder failed"));
		return *Builder;
	}

	// Helper: build an FFrontendGraph from the builder's document.
	TUniquePtr<FFrontendGraph> BuildGraphFromBuilder(UMetaSoundSourceBuilder& Builder)
	{
		const FMetasoundFrontendDocument& Document = Builder.GetConstBuilder().GetConstDocumentChecked();
		const FTopLevelAssetPath AssetPath(GetTransientPackage()->GetFName(), FName("GraphBuilderParityTest"));
		return FGraphBuilder::CreateGraph(Document, AssetPath);
	}

	// -----------------------------------------------------------------------
	// REQ-GB-1: Vertex AccessType Must Be Respected
	//
	// The FClassInterface path should produce vertices whose AccessType
	// matches the frontend document's per-vertex AccessType. Since
	// sub-interface prototype vertices use EVertexAccessType::Reference
	// (default), the built graph vertices should also be Reference.
	// -----------------------------------------------------------------------

	// Group G: Consolidated AccessType test covering all input and output vertices
	TEST_METHOD(REQGB1_AllVertices_HaveCorrectAccessType)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		TUniquePtr<FFrontendGraph> Graph = BuildGraphFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		// All input vertices (fixed + sub-interface expanded) should have
		// Reference access type — matching the prototype and the frontend document.
		for (const FInputDataVertex& Vertex : GraphNode->GetVertexInterface().GetInputInterface())
		{
			ASSERT_THAT(IsTrue(Vertex.AccessType == EVertexAccessType::Reference));
		}

		// All output vertices should also have Reference access type.
		for (const FOutputDataVertex& Vertex : GraphNode->GetVertexInterface().GetOutputInterface())
		{
			ASSERT_THAT(IsTrue(Vertex.AccessType == EVertexAccessType::Reference));
		}
	}

	// -----------------------------------------------------------------------
	// REQ-GB-2: Literal Defaults Must Apply to Expanded Vertex Names
	//
	// The fixed "Gain" input has a literal default set through the document.
	// After the FClassInterface path, the built vertex should retain its
	// literal default. This tests the literal overlay in CreateVertexInterface.
	// -----------------------------------------------------------------------

	TEST_METHOD(REQGB2_FixedInputVertex_LiteralDefaultIsApplied)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Set a literal value on the Gain input through the builder
		Builder.SetNodeInputDefault(NodeHandle, SerializeTestNode::GainInputName, 0.75f, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		TUniquePtr<FFrontendGraph> Graph = BuildGraphFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		// The Gain vertex should have a literal default after the round-trip
		const FInputDataVertex* GainVertex = GraphNode->GetVertexInterface().GetInputInterface().Find(SerializeTestNode::GainInputName);
		ASSERT_THAT(IsNotNull(GainVertex));

		// Verify the literal exists and has the correct value
		const FLiteral& Literal = GainVertex->GetDefaultLiteral();
		const float* FloatValue = Literal.Value.TryGet<float>();
		ASSERT_THAT(IsNotNull(FloatValue));
		ASSERT_THAT(IsNear(*FloatValue, 0.75f, KINDA_SMALL_NUMBER));
	}

	// -----------------------------------------------------------------------
	// REQ-GB-3: Output Vertex Metadata Consistency
	//
	// The old flat-vertex path strips FDataVertexMetadata on outputs
	// (constructs with FDataVertexMetadata{}). The FClassInterface path
	// preserves metadata from the prototype. This test documents the
	// current behavior: metadata IS preserved on the FClassInterface path.
	// -----------------------------------------------------------------------

	TEST_METHOD(REQGB3_ClassInterfacePath_OutputMetadataIsPreserved)
	{
		// For a plain registered node (no sub-interfaces), the old path
		// strips metadata. We verify this with the direct C++ flat path:
		// construct FOutputDataVertex with metadata, then verify the old
		// path would strip it.
		//
		// Since we can't easily invoke the old path for an arbitrary node,
		// this test documents the expectation: metadata should be empty
		// on nodes built via the flat path.
		//
		// For now, verify the FClassInterface path does preserve metadata:
		FClassInterface ClassInterface = SerializeTestNode::CreateTestClassInterface();
		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface({ { SerializeTestNode::ChannelsSubInterfaceName, 2 } }, {});

		// FClassInterface::CreateVertexInterface preserves prototype metadata.
		// The prototype was created with FDataVertexMetadata{INVTEXT("Audio output")}.
		// Verify that the metadata is non-empty on at least one output vertex.
		bool bFoundNonEmptyMetadata = false;
		for (const FOutputDataVertex& Vertex : VertexInterface.GetOutputInterface())
		{
#if WITH_EDITORONLY_DATA
			if (!Vertex.Metadata.Description.IsEmpty())
			{
				bFoundNonEmptyMetadata = true;
				break;
			}
#endif
		}

#if WITH_EDITORONLY_DATA
		// Document current behavior: FClassInterface path preserves metadata.
		// REQ-GB-3 asks us to decide whether to strip or preserve.
		ASSERT_THAT(IsTrue(bFoundNonEmptyMetadata));
#endif
	}

	TEST_METHOD(REQGB3_GraphBuilder_SubInterfaceNode_OutputMetadataFromPrototype)
	{
		// When a sub-interface node is built through FGraphBuilder, the
		// FClassInterface path produces output vertices. Verify that the
		// built graph node's output vertices have the metadata from the
		// FClassInterface prototype (not stripped empty).
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		TUniquePtr<FFrontendGraph> Graph = BuildGraphFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

#if WITH_EDITORONLY_DATA
		// The old path would produce vertices with empty FDataVertexMetadata.
		// The new FClassInterface path preserves metadata from the prototype.
		// This test documents the inconsistency identified in REQ-GB-3.
		bool bFoundNonEmptyMetadata = false;
		for (const FOutputDataVertex& Vertex : GraphNode->GetVertexInterface().GetOutputInterface())
		{
			if (!Vertex.Metadata.Description.IsEmpty())
			{
				bFoundNonEmptyMetadata = true;
				break;
			}
		}
		ASSERT_THAT(IsTrue(bFoundNonEmptyMetadata));
#endif
	}

	// -----------------------------------------------------------------------
	// REQ-GB-4: Environment Variables Must Come From Frontend Document
	//
	// The old path calls CreateEnvironmentVertexInterface(InNode.Interface)
	// to build environment vertices from the frontend document. The
	// FClassInterface path currently uses the environment from
	// FClassInterface::CreateVertexInterface(), which comes from the
	// registered prototype.
	//
	// This test verifies that the graph node has the correct environment
	// variables — they should match the frontend document, not just the
	// prototype.
	// -----------------------------------------------------------------------

	// Group H: Consolidated environment test — names matching implies count matching
	TEST_METHOD(REQGB4_SubInterfaceNode_EnvironmentMatchesFrontendDocument)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Get environment vertex names from the frontend document
		const FMetasoundFrontendNode* FrontendNode = Builder.GetConstBuilder().FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(FrontendNode));

		TArray<FName> FrontendEnvNames;
		for (const FMetasoundFrontendVertex& EnvVertex : FrontendNode->Interface.Environment)
		{
			FrontendEnvNames.Add(EnvVertex.Name);
		}

		// Build graph
		TUniquePtr<FFrontendGraph> Graph = BuildGraphFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));

		const FEnvironmentVertexInterface& EnvInterface = GraphNode->GetVertexInterface().GetEnvironmentInterface();

		// The graph node's environment variable count should match
		// the frontend document's environment variable count.
		ASSERT_THAT(AreEqual(EnvInterface.Num(), FrontendEnvNames.Num()));

		// Every environment variable in the frontend document should
		// exist in the built graph node's environment interface.
		for (const FName& EnvName : FrontendEnvNames)
		{
			const FEnvironmentVertex* Vertex = EnvInterface.Find(EnvName);
			ASSERT_THAT(IsNotNull(Vertex));
		}
	}

	// -----------------------------------------------------------------------
	// REQ-GB-5: Frontend Document Consistency Validation
	//
	// The old path asserts that InClassInterface.Inputs.Num() ==
	// InNode.Interface.Inputs.Num(). The FClassInterface path bypasses
	// this check. Verify that a well-formed document still builds
	// successfully (positive validation).
	// -----------------------------------------------------------------------

	TEST_METHOD(REQGB5_WellFormedDocument_BuildsSuccessfully)
	{
		FMetaSoundBuilderNodeOutputHandle OnPlay;
		FMetaSoundBuilderNodeInputHandle OnFinished;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOuts;
		UMetaSoundSourceBuilder& Builder = CreateTestSourceBuilder(OnPlay, OnFinished, AudioOuts);

		EMetaSoundBuilderResult Result;
		FMetaSoundNodeHandle NodeHandle = Builder.AddNodeByClassName(SerializeTestNodeClassName, Result);
		ASSERT_THAT(IsTrue(Result == EMetaSoundBuilderResult::Succeeded));

		// Verify the document's node has consistent vertex counts with the class
		const FMetasoundFrontendNode* FrontendNode = Builder.GetConstBuilder().FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(FrontendNode));

		// The frontend node should have the expected vertex counts
		// (default config: 1 Gain + 2 Audio In = 3 inputs, 2 Audio Out = 2 outputs)
		ASSERT_THAT(AreEqual(FrontendNode->Interface.Inputs.Num(), 3));
		ASSERT_THAT(AreEqual(FrontendNode->Interface.Outputs.Num(), 2));

		// Build should succeed without errors
		TUniquePtr<FFrontendGraph> Graph = BuildGraphFromBuilder(Builder);
		ASSERT_THAT(IsNotNull(Graph.Get()));

		// The built node should exist and have the correct vertex counts
		const INode* GraphNode = Graph->FindNode(NodeHandle.NodeID);
		ASSERT_THAT(IsNotNull(GraphNode));
		ASSERT_THAT(AreEqual(GraphNode->GetVertexInterface().GetInputInterface().Num(), 3));
		ASSERT_THAT(AreEqual(GraphNode->GetVertexInterface().GetOutputInterface().Num(), 2));
	}
};

const FMetasoundFrontendClassName MetasoundGraphBuilderParityTests::SerializeTestNodeClassName = { "Test", "SubInterfaceSerialize", "" };

#endif // WITH_DEV_AUTOMATION_TESTS
