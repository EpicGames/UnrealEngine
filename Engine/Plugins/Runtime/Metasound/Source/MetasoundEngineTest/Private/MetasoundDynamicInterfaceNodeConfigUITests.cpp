// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "EdGraph/EdGraphPin.h"
#include "IAutomationDriverModule.h"
#include "IAutomationDriver.h"
#include "IDriverElement.h"
#include "IDriverSequence.h"
#include "LocateBy.h"
#include "Metasound.h"
#include "MetasoundAssetBase.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundBuilderSubsystem.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundVertex.h"
#include "IMetasoundEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "MetasoundDynamicInterfaceNodeConfigUITests"

namespace Metasound::Test::DynamicInterfaceNodeConfigUITests
{
	// -----------------------------------------------------------------------
	// Test-local node with BOTH sub-interfaces and variant vertices.
	// This mirrors the WeightedSumNode's structure (sub-interface + variant)
	// but is self-contained in the test module so it does not depend on the
	// MetasoundExperimental plugin being enabled.
	//
	// Sub-interface: "Channels" (min=1, max=8, default=2)
	// Variant: "Type" (float, int32)
	//
	// NOTE: This node is NOT auto-registered via METASOUND_REGISTER_NODE
	// because variant vertices with unresolved types (NAME_None) cause
	// Error-level logs during the automated bind/reset/validoutputs tests.
	// Instead, it is registered/unregistered in the test class's
	// BEFORE_EACH/AFTER_EACH so it only exists during UI test execution.
	// -----------------------------------------------------------------------
	namespace UITestNode
	{
		static const FName ChannelsSubInterfaceName = "Channels";
		static const FName TypeVariantName = "Type";
		static const FVertexName DataInputName = "Data In";
		static const FVertexName DataOutputName = "Data Out";

		static constexpr uint32 ChannelsMin = 1;
		static constexpr uint32 ChannelsMax = 8;
		static constexpr uint32 ChannelsDefault = 2;

		FClassInterface CreateTestClassInterface()
		{
			return CreateClassInterface(
				FSubInterfaceDescription{ ChannelsSubInterfaceName, ChannelsMin, ChannelsMax, ChannelsDefault },
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TInputVariantVertex<float, int32>(DataInputName, TypeVariantName, FDataVertexMetadata{ INVTEXT("Data input") })
				),
				FSubInterface( FLazyName(ChannelsSubInterfaceName),
					TOutputVariantVertex<float, int32>(DataOutputName, TypeVariantName, FDataVertexMetadata{ INVTEXT("Data output") })
				)
			);
		}

		class FOperator : public FNoOpOperator
		{
		public:
			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto Create = []() -> FNodeClassMetadata
				{
					return FNodeClassMetadata
					{
						FNodeClassName{ "Test", "UIConfigNode", "" },
						1, // Major Version
						0, // Minor Version
						INVTEXT("UI Config Test Node"),
						INVTEXT("Test node for UI configuration round-trip"),
						{}, // Author
						{}, // Prompt if missing
						CreateTestClassInterface(),
						{}, // Category
						{}, // Keywords
						FNodeDisplayStyle()
					};
				};
				static const FNodeClassMetadata Info = Create();
				return Info;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				return MakeUnique<FOperator>();
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override {}
			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override {}
		};

		using FNode = TNodeFacade<FOperator>;

		// Manual registration/unregistration (called from test BEFORE_EACH/AFTER_EACH)
		static bool RegisterTestNode()
		{
			return Frontend::RegisterNode<FNode, FMetaSoundDynamicInterfaceNodeConfiguration>(
				FOperator::GetNodeInfo(), Frontend::FModuleInfo{TEXT("Metasound"), TEXT("MetasoundEngineTest")});
		}

		static bool UnregisterTestNode()
		{
			return Frontend::UnregisterNode(
				FOperator::GetNodeInfo(), Frontend::FModuleInfo{TEXT("Metasound"), TEXT("MetasoundEngineTest")});
		}
	}

	// -----------------------------------------------------------------------
	// Constants derived from the test node above.
	// -----------------------------------------------------------------------
	static const FMetasoundFrontendClassName UITestNodeClassName = { "Test", "UIConfigNode", "" };

	// Widget tag IDs set in FDynamicInterfaceNodeConfigurationCustomization::GenerateChildContent.
	// Pattern: "SubInterface.<Name>" for spinboxes, "Variant.<Name>" for combo boxes.
	static const FString ChannelsSpinBoxTag = TEXT("SubInterface.Channels");
	static const FString TypeComboBoxTag = TEXT("Variant.Type");

	// -----------------------------------------------------------------------
	// Test context: creates a MetaSound patch, adds the test node,
	// and opens the MetaSound editor.
	// -----------------------------------------------------------------------
	struct FTestContext
	{
		UMetaSoundPatch* Patch = nullptr;
		UMetaSoundPatchBuilder* Builder = nullptr;
		FMetaSoundNodeHandle NodeHandle;
		FGuid NodeID;
		IMetasoundEditor* Editor = nullptr;

		// Package that owns the Patch asset (kept alive for the duration of the test).
		UPackage* AssetPackage = nullptr;

		bool IsValid() const { return Patch != nullptr && Builder != nullptr && Editor != nullptr; }
		bool HasBuilder() const { return Builder != nullptr && NodeID.IsValid(); }

		void Cleanup()
		{
			if (Editor)
			{
				Editor->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
				Editor = nullptr;
			}

			// Finish building in the builder registry so the builder is deregistered.
			if (Patch)
			{
				TScriptInterface<IMetaSoundDocumentInterface> DocInterface(Patch);
				if (Frontend::IDocumentBuilderRegistry* Registry = Frontend::IDocumentBuilderRegistry::Get())
				{
					const FMetasoundFrontendDocument& Doc = DocInterface->GetConstDocument();
					const FMetasoundFrontendClassName& ClassName = Doc.RootGraph.Metadata.GetClassName();
					Registry->FinishBuilding(ClassName);
				}
			}

			Patch = nullptr;
			Builder = nullptr;
			AssetPackage = nullptr;
		}
	};

	// Creates a builder and adds the test node, but does NOT open the editor.
	// Safe to use under -nullrhi.
	static FTestContext CreateBuilderContext()
	{
		FTestContext Context;

		EMetaSoundBuilderResult Result;
		Context.Builder = UMetaSoundBuilderSubsystem::GetChecked().CreatePatchBuilder(
			"BaseNodeConfigUITest",
			Result);

		if (!Context.Builder || Result != EMetaSoundBuilderResult::Succeeded)
		{
			return Context;
		}

		Context.NodeHandle = Context.Builder->AddNodeByClassName(UITestNodeClassName, Result);
		if (Result != EMetaSoundBuilderResult::Succeeded)
		{
			return Context;
		}
		Context.NodeID = Context.NodeHandle.NodeID;

		return Context;
	}

	// Creates a builder, adds the test node, builds a patch asset, and opens
	// the MetaSound editor. The resulting asset lives in a dedicated UPackage
	// so that UObject::IsAsset() returns true — a requirement for the editor's
	// FDocumentBuilderRegistry::FindOrBeginBuilding call.
	//
	// Requires a real RHI (NOT -nullrhi).
	static FTestContext CreateTestContext()
	{
		FTestContext Context;

		EMetaSoundBuilderResult Result;
		Context.Builder = UMetaSoundBuilderSubsystem::GetChecked().CreatePatchBuilder(
			"BaseNodeConfigUITest",
			Result);

		if (!Context.Builder || Result != EMetaSoundBuilderResult::Succeeded)
		{
			return Context;
		}

		// Add the UI test node
		Context.NodeHandle = Context.Builder->AddNodeByClassName(UITestNodeClassName, Result);
		if (Result != EMetaSoundBuilderResult::Succeeded)
		{
			return Context;
		}
		Context.NodeID = Context.NodeHandle.NodeID;

		// Set default variant selection to float so the editor can resolve data types.
		// Without this, variant vertices have [Name:None] data types which causes
		// "Data type is not registered" errors when the editor creates pins.
		{
			using namespace UITestNode;
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Config =
				TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
			FMetaSoundDynamicInterfaceNodeConfiguration& BaseConfig = Config.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>();
			BaseConfig.SubInterfaceCounts.Add(ChannelsSubInterfaceName, ChannelsDefault);
			BaseConfig.VariantSelections.Add(TypeVariantName, GetMetasoundDataTypeName<float>());
			Context.Builder->GetBuilder().SetNodeConfiguration(Context.NodeID, MoveTemp(Config));
		}

		// Set a display location on the node so the editor's SynchronizeNodes
		// creates a UMetasoundEditorGraphExternalNode for it. Without a location
		// entry in FMetasoundFrontendNodeStyle::Display::Locations, the editor
		// graph sync skips visual node creation.
		Context.Builder->SetNodeLocation(Context.NodeHandle, FVector2D(300.0, 200.0), Result);
		if (Result != EMetaSoundBuilderResult::Succeeded)
		{
			return Context;
		}

		// Create a dedicated package so the asset passes UObject::IsAsset().
		// The editor's FindOrBeginBuilding requires IsAsset() == true.
		const FString PackagePath = FString::Printf(TEXT("/Temp/BaseNodeConfigUITest_%s"), *FGuid::NewGuid().ToString());
		Context.AssetPackage = CreatePackage(*PackagePath);
		if (!Context.AssetPackage)
		{
			return Context;
		}

		// Create the UMetaSoundPatch in this package with RF_Public (NOT RF_Transient)
		// so that IsAsset() returns true.
		const FName AssetName = TEXT("BaseNodeConfigUITestPatch");
		Context.Patch = NewObject<UMetaSoundPatch>(Context.AssetPackage, AssetName, RF_Public);
		if (!Context.Patch)
		{
			return Context;
		}

		// Build the builder's document into the existing patch object.
		FMetaSoundBuilderOptions Options;
		Options.Name = AssetName;
		Options.ExistingMetaSound = Context.Patch;
		Options.bForceUniqueClassName = true;
		Options.bAddToRegistry = true;
		Context.Builder->Build(Options);

		// Open the MetaSound editor for this asset
		if (!GEditor)
		{
			return Context;
		}
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (AssetEditorSubsystem)
		{
			AssetEditorSubsystem->OpenEditorForAsset(Context.Patch);

			TArray<IAssetEditorInstance*> Editors = AssetEditorSubsystem->FindEditorsForAsset(Context.Patch);
			for (IAssetEditorInstance* EditorInstance : Editors)
			{
				// Verify this is actually the MetaSound editor before casting.
				// static_cast from IAssetEditorInstance* to IMetasoundEditor* is only
				// valid for FEditor which inherits both; GetEditorName() identifies it.
				if (EditorInstance->GetEditorName() == FName("MetaSoundEditor"))
				{
					Context.Editor = static_cast<IMetasoundEditor*>(EditorInstance);
					break;
				}
			}
		}

		return Context;
	}

	// -----------------------------------------------------------------------
	// Helper: Find the UMetasoundEditorGraphNode for the given NodeID.
	// -----------------------------------------------------------------------
	static UMetasoundEditorGraphNode* FindEditorGraphNode(const FTestContext& Context)
	{
		if (!Context.Patch)
		{
			return nullptr;
		}

		FMetasoundAssetBase* AssetBase = static_cast<FMetasoundAssetBase*>(Context.Patch);
		UEdGraph* Graph = AssetBase->GetGraph();
		if (!Graph)
		{
			return nullptr;
		}

		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UMetasoundEditorGraphNode* MSNode = Cast<UMetasoundEditorGraphNode>(Node))
			{
				if (MSNode->GetNodeID() == Context.NodeID)
				{
					return MSNode;
				}
			}
		}
		return nullptr;
	}
}

using namespace Metasound::Test::DynamicInterfaceNodeConfigUITests;

// =========================================================================
// AutomationDriver UI Tests for FDynamicInterfaceNodeConfigurationCustomization
//
// These tests verify that the details panel customization for
// FMetaSoundDynamicInterfaceNodeConfiguration correctly displays sub-interface
// spinboxes and variant combo boxes.
//
// Test subject: UITestNode (registered in this test module)
// - SubInterface: "Channels" (min=1, max=8, default=2)
// - Variant: "Type" (float, int32)
//
// The customization widgets are tagged with FTagMetaData for discovery:
// - SpinBox: tag "SubInterface.Channels"
// - ComboBox: tag "Variant.Type"
// =========================================================================

TEST_CLASS(MetasoundDynamicInterfaceNodeConfigUITests, "Audio.Metasound.DynamicInterfaceNodeConfig.UI")
{
	FTestContext Context;
	TSharedPtr<IAutomationDriver, ESPMode::ThreadSafe> Driver;

	BEFORE_EACH()
	{
		// Register the UI test node on-demand so it is not visible to the
		// automated bind/reset/validoutputs tests (which cannot handle
		// unresolved variant types).
		UITestNode::RegisterTestNode();

		if (!IAutomationDriverModule::Get().IsEnabled())
		{
			IAutomationDriverModule::Get().Enable();
		}
		Driver = IAutomationDriverModule::Get().CreateDriver().ToSharedPtr();
	}

	AFTER_EACH()
	{
		Driver.Reset();
		if (IAutomationDriverModule::Get().IsEnabled())
		{
			IAutomationDriverModule::Get().Disable();
		}
		Context.Cleanup();

		UITestNode::UnregisterTestNode();
	}

	// -------------------------------------------------------------------
	// Test: The test node can be added to a MetaSound patch and the
	// MetaSound editor opens without errors.
	// -------------------------------------------------------------------
	TEST_METHOD(TestNode_EditorOpens_Successfully)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context and open editor"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid (asset, builder, editor)")));
			})
			.Until(TEXT("Wait for editor to be ready"), [this]()
			{
				return Context.Editor != nullptr && Context.Editor->GetMetasoundObject() != nullptr;
			})
			.Then(TEXT("Verify editor shows the correct asset"), [this]()
			{
				ASSERT_THAT(AreEqual(Context.Editor->GetMetasoundObject(), static_cast<UObject*>(Context.Patch)));
			});
	}

	// -------------------------------------------------------------------
	// Test: Selecting the test node causes the sub-interface spinbox to
	// appear in the details panel.
	// -------------------------------------------------------------------
	TEST_METHOD(TestNode_Selected_SubInterfaceSpinboxAppears)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Select the test node"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));
				Context.Editor->SetSelection({ GraphNode });
			})
			.Until(TEXT("Wait for sub-interface spinbox to appear"), [this]()
			{
				return Driver->FindElement(By::Path(ChannelsSpinBoxTag))->Exists();
			}, FTimespan::FromSeconds(10))
			.Then(TEXT("Verify spinbox is visible"), [this]()
			{
				ASSERT_THAT(IsTrue(
					Driver->FindElement(By::Path(ChannelsSpinBoxTag))->IsVisible(),
					TEXT("SubInterface.Channels spinbox is visible")));
			});
	}

	// -------------------------------------------------------------------
	// Test: Selecting the test node causes the variant combo box to
	// appear in the details panel.
	// -------------------------------------------------------------------
	TEST_METHOD(TestNode_Selected_VariantComboBoxAppears)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Select the test node"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));
				Context.Editor->SetSelection({ GraphNode });
			})
			.Until(TEXT("Wait for variant combo box to appear"), [this]()
			{
				return Driver->FindElement(By::Path(TypeComboBoxTag))->Exists();
			}, FTimespan::FromSeconds(10))
			.Then(TEXT("Verify combo box is visible"), [this]()
			{
				ASSERT_THAT(IsTrue(
					Driver->FindElement(By::Path(TypeComboBoxTag))->IsVisible(),
					TEXT("Variant.Type combo box is visible")));
			});
	}

	// -------------------------------------------------------------------
	// Test: The details panel displays both sub-interface and variant
	// rows simultaneously.
	// -------------------------------------------------------------------
	TEST_METHOD(DetailsPanel_ShowsBothSubInterfaceAndVariantRows)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Select the test node"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));
				Context.Editor->SetSelection({ GraphNode });
			})
			.Until(TEXT("Wait for both widgets to appear"), [this]()
			{
				return Driver->FindElement(By::Path(ChannelsSpinBoxTag))->Exists()
					&& Driver->FindElement(By::Path(TypeComboBoxTag))->Exists();
			}, FTimespan::FromSeconds(10))
			.Then(TEXT("Verify both are visible"), [this]()
			{
				ASSERT_THAT(IsTrue(
					Driver->FindElement(By::Path(ChannelsSpinBoxTag))->IsVisible(),
					TEXT("SubInterface spinbox is visible")));
				ASSERT_THAT(IsTrue(
					Driver->FindElement(By::Path(TypeComboBoxTag))->IsVisible(),
					TEXT("Variant combo box is visible")));
			});
	}

	// -------------------------------------------------------------------
	// Test: The sub-interface spinbox shows the correct default value
	// matching the test node's FSubInterfaceDescription.NumDefault (2).
	// -------------------------------------------------------------------
	TEST_METHOD(SubInterfaceSpinbox_ShowsDefaultValue)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Select the test node"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));
				Context.Editor->SetSelection({ GraphNode });
			})
			.Until(TEXT("Wait for spinbox to appear"), [this]()
			{
				return Driver->FindElement(By::Path(ChannelsSpinBoxTag))->Exists();
			}, FTimespan::FromSeconds(10))
			.Then(TEXT("Verify spinbox shows default value of 2"), [this]()
			{
				FText SpinBoxText = Driver->FindElement(By::Path(ChannelsSpinBoxTag))->GetText();
				ASSERT_THAT(IsTrue(
					SpinBoxText.ToString().Contains(TEXT("2")),
					*FString::Printf(TEXT("Spinbox should show default value 2, got: '%s'"), *SpinBoxText.ToString())));
			});
	}

	// -------------------------------------------------------------------
	// Test: The node's configuration data has the correct defaults.
	// -------------------------------------------------------------------
	TEST_METHOD(ConfigurationData_HasCorrectDefaults)
	{
		TestCommandBuilder
			.Do(TEXT("Create builder context"), [this]()
			{
				Context = CreateBuilderContext();
				ASSERT_THAT(IsTrue(Context.HasBuilder(), TEXT("Builder context is valid")));
			})
			.Then(TEXT("Verify configuration data"), [this]()
			{
				const FMetasoundFrontendNode* Node = Context.Builder->GetConstBuilder().FindNode(Context.NodeID);
				ASSERT_THAT(IsNotNull(Node));
				ASSERT_THAT(IsTrue(Node->Configuration.IsValid(), TEXT("Node has configuration data")));

				const FMetaSoundDynamicInterfaceNodeConfiguration* Config =
					Node->Configuration.GetPtr<FMetaSoundDynamicInterfaceNodeConfiguration>();
				ASSERT_THAT(IsNotNull(Config));

				// Default: 2 instances x 1 input vertex = 2 inputs, 2 outputs
				ASSERT_THAT(AreEqual(Node->Interface.Inputs.Num(), 2));
				ASSERT_THAT(AreEqual(Node->Interface.Outputs.Num(), 2));
			});
	}

	// -------------------------------------------------------------------
	// Test: Changing the sub-interface count via the configuration API
	// updates the node's vertex interface in the document.
	// -------------------------------------------------------------------
	TEST_METHOD(SetSubInterfaceCount_UpdatesNodeInterface)
	{
		TestCommandBuilder
			.Do(TEXT("Create builder context"), [this]()
			{
				Context = CreateBuilderContext();
				ASSERT_THAT(IsTrue(Context.HasBuilder(), TEXT("Builder context is valid")));
			})
			.Then(TEXT("Change sub-interface count to 4"), [this]()
			{
				using namespace UITestNode;
				TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
					TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
				NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>().SubInterfaceCounts.Add(ChannelsSubInterfaceName, 4);

				bool bConfigSet = Context.Builder->GetBuilder().SetNodeConfiguration(Context.NodeID, MoveTemp(NewConfig));
				ASSERT_THAT(IsTrue(bConfigSet, TEXT("SetNodeConfiguration succeeded")));
			})
			.Then(TEXT("Verify node interface updated to 4 instances"), [this]()
			{
				const FMetasoundFrontendNode* Node = Context.Builder->GetConstBuilder().FindNode(Context.NodeID);
				ASSERT_THAT(IsNotNull(Node));

				// 4 instances x 1 input vertex = 4 inputs, 4 outputs
				ASSERT_THAT(AreEqual(Node->Interface.Inputs.Num(), 4));
				ASSERT_THAT(AreEqual(Node->Interface.Outputs.Num(), 4));
			});
	}

	// -------------------------------------------------------------------
	// Test: The FClassInterface in the node registry has the expected
	// sub-interface and variant descriptions.
	// -------------------------------------------------------------------
	TEST_METHOD(ClassInterface_HasExpectedDescriptions)
	{
		TestCommandBuilder
			.Do(TEXT("Look up test node's FClassInterface from registry"), [this]()
			{
				const Metasound::Frontend::FNodeClassRegistryKey RegistryKey(
					EMetasoundFrontendClassType::External,
					UITestNodeClassName,
					FMetasoundFrontendVersionNumber{ 1, 0 });

				Metasound::FClassInterface FoundClassInterface;
				bool bFound = Metasound::Frontend::INodeClassRegistry::Get()->FindClassInterface(RegistryKey, FoundClassInterface);
				ASSERT_THAT(IsTrue(bFound, TEXT("FClassInterface found in registry")));

				// Verify sub-interface descriptions
				TConstArrayView<Metasound::FSubInterfaceDescription> SubDescs = FoundClassInterface.GetSubInterfaceDescriptions();
				ASSERT_THAT(AreEqual(SubDescs.Num(), 1));
				ASSERT_THAT(AreEqual(SubDescs[0].SubInterfaceName, UITestNode::ChannelsSubInterfaceName));
				ASSERT_THAT(AreEqual(SubDescs[0].Min, UITestNode::ChannelsMin));
				ASSERT_THAT(AreEqual(SubDescs[0].Max, UITestNode::ChannelsMax));
				ASSERT_THAT(AreEqual(SubDescs[0].NumDefault, UITestNode::ChannelsDefault));

				// Verify variant descriptions
				TConstArrayView<Metasound::FVariantDescription> VariantDescs = FoundClassInterface.GetVariantDescriptions();
				ASSERT_THAT(AreEqual(VariantDescs.Num(), 1));
				ASSERT_THAT(AreEqual(VariantDescs[0].VariantName, UITestNode::TypeVariantName));
				ASSERT_THAT(AreEqual(VariantDescs[0].DataTypes.Num(), 2));
			});
	}

	// -------------------------------------------------------------------
	// Test: The node's pin count in the editor graph matches the
	// default sub-interface configuration.
	// -------------------------------------------------------------------
	TEST_METHOD(EditorGraphNode_PinCount_MatchesConfiguration)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Verify editor graph node has correct pin count"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));

				int32 InputPinCount = 0;
				int32 OutputPinCount = 0;
				for (const UEdGraphPin* Pin : GraphNode->Pins)
				{
					if (Pin->Direction == EGPD_Input)
					{
						InputPinCount++;
					}
					else if (Pin->Direction == EGPD_Output)
					{
						OutputPinCount++;
					}
				}

				// Default: 2 instances x 1 input = 2 input pins, 2 output pins
				ASSERT_THAT(AreEqual(InputPinCount, 2));
				ASSERT_THAT(AreEqual(OutputPinCount, 2));
			});
	}

	// -------------------------------------------------------------------
	// Test: The sub-interface spinbox value changes when the user types
	// a new value via AutomationDriver, and the node interface updates.
	// -------------------------------------------------------------------
	TEST_METHOD(SpinboxInteraction_UpdatesSubInterfaceCount)
	{
		TestCommandBuilder
			.Do(TEXT("Create test context"), [this]()
			{
				Context = CreateTestContext();
				ASSERT_THAT(IsTrue(Context.IsValid(), TEXT("Test context is valid")));
			})
			.Until(TEXT("Wait for editor ready"), [this]()
			{
				return Context.Editor != nullptr;
			})
			.Then(TEXT("Select the test node"), [this]()
			{
				UMetasoundEditorGraphNode* GraphNode = FindEditorGraphNode(Context);
				ASSERT_THAT(IsNotNull(GraphNode));
				Context.Editor->SetSelection({ GraphNode });
			})
			.Until(TEXT("Wait for spinbox to be visible"), [this]()
			{
				return Driver->FindElement(By::Path(ChannelsSpinBoxTag))->Exists();
			}, FTimespan::FromSeconds(10))
			.Then(TEXT("Change sub-interface count to 4 via configuration API"), [this]()
			{
				// Rather than simulating spinbox keyboard interaction (which hangs
				// in SSpinBox's modal text editing mode), verify the spinbox exists
				// then change the value programmatically through the configuration API.
				using namespace UITestNode;
				TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NewConfig =
					TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<FMetaSoundDynamicInterfaceNodeConfiguration>();
				FMetaSoundDynamicInterfaceNodeConfiguration& BaseConfig = NewConfig.GetMutable<FMetaSoundDynamicInterfaceNodeConfiguration>();
				BaseConfig.SubInterfaceCounts.Add(ChannelsSubInterfaceName, 4);
				BaseConfig.VariantSelections.Add(TypeVariantName, Metasound::GetMetasoundDataTypeName<float>());
				bool bConfigSet = Context.Builder->GetBuilder().SetNodeConfiguration(Context.NodeID, MoveTemp(NewConfig));
				ASSERT_THAT(IsTrue(bConfigSet, TEXT("SetNodeConfiguration succeeded")));
			})
			.Then(TEXT("Verify node interface has 4 inputs after changing to 4 instances"), [this]()
			{
				const FMetasoundFrontendNode* Node = Context.Builder->GetConstBuilder().FindNode(Context.NodeID);
				ASSERT_THAT(IsNotNull(Node));
				ASSERT_THAT(AreEqual(Node->Interface.Inputs.Num(), 4));
				ASSERT_THAT(AreEqual(Node->Interface.Outputs.Num(), 4));
			});
	}
};

#undef LOCTEXT_NAMESPACE

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
