// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundTestInterfaces.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundSource.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST

namespace Metasound
{
	class METASOUNDENGINETEST_API FMetasoundEngineTestModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			FModuleManager::Get().LoadModuleChecked("MetasoundFrontend");
			FModuleManager::Get().LoadModuleChecked("MetasoundStandardNodes");
			FModuleManager::Get().LoadModuleChecked("MetasoundEngine");

			METASOUND_REGISTER_ITEMS_IN_MODULE

			using namespace Metasound::Test;

			Audio::IAudioParameterInterfaceRegistry& Registry = Audio::IAudioParameterInterfaceRegistry::Get();
			Registry.RegisterInterface(UpdateTestInterface_0_1::CreateInterface(*UMetaSoundSource::StaticClass()));
			Registry.RegisterInterface(UpdateTestInterface_0_2::CreateInterface(*UMetaSoundSource::StaticClass()));
		}

		virtual void ShutdownModule() override
		{
			METASOUND_UNREGISTER_ITEMS_IN_MODULE
		}
	};
}

IMPLEMENT_MODULE(Metasound::FMetasoundEngineTestModule, MetasoundEngineTest);




