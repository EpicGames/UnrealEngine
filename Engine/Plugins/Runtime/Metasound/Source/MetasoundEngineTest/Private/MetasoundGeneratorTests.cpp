// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "Analysis/MetasoundFrontendAnalyzerView.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerToTime.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Metasound::Test::Generator
{
	using GraphBuilder = FNodeTestGraphBuilder;
	using namespace Frontend;
	
	template<typename DataType>
	TUniquePtr<FMetasoundGenerator> BuildPassthroughGraph(
		FAutomationTestBase& Test,
		const FName& InputName,
		const FName& OutputName,
		const FSampleRate SampleRate,
		const int32 NumSamplesPerBlock,
		FGuid* OutputGuid = nullptr)
	{
		GraphBuilder Builder;
		const FNodeHandle InputNode = Builder.AddInput(InputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const FNodeHandle OutputNode = Builder.AddOutput(OutputName, Metasound::GetMetasoundDataTypeName<DataType>());
		const FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(InputName);
		const FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(OutputName);

		if (!Test.TestTrue("Connected input to output", InputToConnect->Connect(*OutputToConnect)))
		{
			return nullptr;
		}

		if (nullptr != OutputGuid)
		{
			*OutputGuid = OutputNode->GetID();
		}

		// have to add an audio output for the generator to render
		Builder.AddOutput("Audio", Metasound::GetMetasoundDataTypeName<FAudioBuffer>());
		
		return Builder.BuildGenerator(SampleRate, NumSamplesPerBlock);
	}
	
	template<typename DataType>
	bool RunSimpleOutputTest(FAutomationTestBase& Test, DataType ExpectedValue)
	{
		const FName InputName = "MyInput";
		const FName OutputName = "MyOutput";
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;

		// Build a passthrough graph
		const TUniquePtr<FMetasoundGenerator> Generator =
			BuildPassthroughGraph<DataType>(Test, InputName, OutputName, SampleRate, NumSamplesPerBlock);
		if (!Test.TestNotNull("Generator built", Generator.Get()))
		{
			return false;
		}
		
		// Set the input value
		const TOptional<TDataWriteReference<DataType>> InputWriteRef = Generator->GetInputWriteReference<DataType>(InputName);
		if (!Test.TestTrue("Got the write ref", InputWriteRef.IsSet()))
		{
			return false;
		}
		*InputWriteRef.GetValue() = ExpectedValue;

		// Render a block
		{
			TArray<float> Buffer;
			Buffer.Reserve(NumSamplesPerBlock);
			if (!Test.TestEqual(
				"Generated the right number of samples.",
				Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
				NumSamplesPerBlock))
			{
				return false;
			}
		}

		// Check the output
		const TOptional<TDataReadReference<DataType>> OutputReadRef = Generator->GetOutputReadReference<DataType>(OutputName);
		if (!Test.TestTrue("Got the read ref", OutputReadRef.IsSet()))
		{
			return false;
		}
		if (!Test.TestEqual("Input was passed through", *OutputReadRef.GetValue(), ExpectedValue))
		{
			return false;
		}

		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOutputSimpleFloatTest,
	"Audio.Metasound.Generator.Outputs.Simple.Float",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOutputSimpleFloatTest::RunTest(const FString&)
	{
		return RunSimpleOutputTest<float>(*this, 123.456f);
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOutputSimpleIntTest,
	"Audio.Metasound.Generator.Outputs.Simple.Int",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOutputSimpleIntTest::RunTest(const FString&)
	{
		return RunSimpleOutputTest<int32>(*this, 123456);
	}
		
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOutputSimpleBoolTest,
	"Audio.Metasound.Generator.Outputs.Simple.Bool",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOutputSimpleBoolTest::RunTest(const FString&)
	{
		return RunSimpleOutputTest<bool>(*this, true);
	}

	template<typename DataType>
	bool RunSetInputTest(FAutomationTestBase& Test, DataType ExpectedValue)
	{
		const FName InputName = "MyInput";
		const FName OutputName = "MyOutput";
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;

		// Build a passthrough graph
		const TUniquePtr<FMetasoundGenerator> Generator =
			BuildPassthroughGraph<DataType>(Test, InputName, OutputName, SampleRate, NumSamplesPerBlock);
		if (!Test.TestNotNull("Generator built", Generator.Get()))
		{
			return false;
		}
		
		// Set the input value
		Generator->SetInputValue<DataType>(InputName, ExpectedValue);

		// Render a block
		{
			TArray<float> Buffer;
			Buffer.Reserve(NumSamplesPerBlock);
			if (!Test.TestEqual(
				"Generated the right number of samples.",
				Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
				NumSamplesPerBlock))
			{
				return false;
			}
		}

		// Check the output
		const TOptional<TDataReadReference<DataType>> OutputReadRef = Generator->GetOutputReadReference<DataType>(OutputName);
		if (!Test.TestTrue("Got the read ref", OutputReadRef.IsSet()))
		{
			return false;
		}
		if (!Test.TestEqual("Input was passed through", *OutputReadRef.GetValue(), ExpectedValue))
		{
			return false;
		}

		return true;
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorSetInputFloatTest,
	"Audio.Metasound.Generator.SetInput.Float",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorSetInputFloatTest::RunTest(const FString&)
	{
		return RunSetInputTest<float>(*this, 123.456f);
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorSetInputIntTest,
	"Audio.Metasound.Generator.SetInput.Int",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorSetInputIntTest::RunTest(const FString&)
	{
		return RunSetInputTest<int32>(*this, 123456);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorSetInputBoolTest,
	"Audio.Metasound.Generator.SetInput.Bool",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorSetInputBoolTest::RunTest(const FString&)
	{
		return RunSetInputTest<bool>(*this, true);
	}

	template<typename DataType>
	bool RunApplyToInputValueTest(FAutomationTestBase& Test, TFunctionRef<void(DataType&)> InFunc, DataType ExpectedValue)
	{
		const FName InputName = "MyInput";
		const FName OutputName = "MyOutput";
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;

		// Build a passthrough graph
		const TUniquePtr<FMetasoundGenerator> Generator =
			BuildPassthroughGraph<DataType>(Test, InputName, OutputName, SampleRate, NumSamplesPerBlock);
		if (!Test.TestNotNull("Generator built", Generator.Get()))
		{
			return false;
		}
		
		// Set the input value
		Generator->ApplyToInputValue(InputName, InFunc);

		// Render a block
		{
			TArray<float> Buffer;
			Buffer.Reserve(NumSamplesPerBlock);
			if (!Test.TestEqual(
				"Generated the right number of samples.",
				Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
				NumSamplesPerBlock))
			{
				return false;
			}
		}

		// Check the output
		const TOptional<TDataReadReference<DataType>> OutputReadRef = Generator->GetOutputReadReference<DataType>(OutputName);
		if (!Test.TestTrue("Got the read ref", OutputReadRef.IsSet()))
		{
			return false;
		}
		if (!Test.TestEqual("Input was passed through", *OutputReadRef.GetValue(), ExpectedValue))
		{
			return false;
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorApplyToInputFloatTest,
	"Audio.Metasound.Generator.ApplyToInput.Float",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorApplyToInputFloatTest::RunTest(const FString&)
	{
		using DataType = float;
		constexpr DataType ExpectedValue = 123.456f;
		const TUniqueFunction<void(DataType&)> Fn = [ExpectedValue](DataType& Value)
		{
			Value = ExpectedValue;
		};
		return RunApplyToInputValueTest<DataType>(*this, Fn, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorApplyToInputIntTest,
	"Audio.Metasound.Generator.ApplyToInput.Int",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorApplyToInputIntTest::RunTest(const FString&)
	{
		using DataType = int32;
		constexpr DataType ExpectedValue = 123456;
		const TUniqueFunction<void(DataType&)> Fn = [ExpectedValue](DataType& Value)
		{
			Value = ExpectedValue;
		};
		return RunApplyToInputValueTest<DataType>(*this, Fn, ExpectedValue);
	}
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorApplyToInputBoolTest,
	"Audio.Metasound.Generator.ApplyToInput.Bool",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorApplyToInputBoolTest::RunTest(const FString&)
	{
		using DataType = bool;
		constexpr DataType ExpectedValue = true;
		const TUniqueFunction<void(DataType&)> Fn = [ExpectedValue](DataType& Value)
		{
			Value = ExpectedValue;
		};
		return RunApplyToInputValueTest<DataType>(*this, Fn, ExpectedValue);
	}

	template<typename DataType>
	bool RunOutputAnalyzerForwardValueTest(FAutomationTestBase& Test, DataType ExpectedValue, FName AnalyzerName)
	{
		const FName InputName = "MyInput";
		const FName OutputName = "MyOutput";
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;

		// Build a passthrough graph
		FGuid OutputNodeId;
		const TUniquePtr<FMetasoundGenerator> Generator =
			BuildPassthroughGraph<DataType>(Test, InputName, OutputName, SampleRate, NumSamplesPerBlock, &OutputNodeId);
		if (!Test.TestNotNull("Generator built", Generator.Get()))
		{
			return false;
		}

		// Add an analyzer to the output
		FAnalyzerAddress AnalyzerAddress;
		AnalyzerAddress.DataType = GetMetasoundDataTypeName<DataType>();
		AnalyzerAddress.InstanceID = 1234;
		AnalyzerAddress.OutputName = OutputName;
		AnalyzerAddress.AnalyzerName = AnalyzerName;
		AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
		AnalyzerAddress.NodeID = OutputNodeId;
		Generator->AddOutputVertexAnalyzer(AnalyzerAddress);


		// Add an analyzer view to watch the output
		FMetasoundAnalyzerView AnalyzerView{ MoveTemp(AnalyzerAddress) };
		AnalyzerView.BindToAllOutputs(Generator->OperatorSettings);
		
		// Set the input value
		Generator->SetInputValue(InputName, ExpectedValue);

		// Render a block
		{
			TArray<float> Buffer;
			Buffer.Reserve(NumSamplesPerBlock);
			if (!Test.TestEqual(
				"Generated the right number of samples.",
				Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
				NumSamplesPerBlock))
			{
				return false;
			}
		}

		// Check the output
		DataType Value;

		if (!Test.TestTrue("Got output data.", AnalyzerView.TryGetOutputData("Value", Value)))
		{
			return false;
		}

		return Test.TestEqual("Input was passed through", Value, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOututAnalyzerForwardValueFloatTest,
	"Audio.Metasound.Generator.OutputAnalyzer.ForwardValue.Float",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOututAnalyzerForwardValueFloatTest::RunTest(const FString&)
	{
		using DataType = float;
		constexpr DataType ExpectedValue = 678.345f;
		return RunOutputAnalyzerForwardValueTest<DataType>(*this, ExpectedValue, "UE.Forward.Float");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOututAnalyzerForwardValueIntTest,
	"Audio.Metasound.Generator.OutputAnalyzer.ForwardValue.Int",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOututAnalyzerForwardValueIntTest::RunTest(const FString&)
	{
		using DataType = int32;
		constexpr DataType ExpectedValue = 678345;
		return RunOutputAnalyzerForwardValueTest<DataType>(*this, ExpectedValue, "UE.Forward.Int32");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOututAnalyzerForwardValueBoolTest,
	"Audio.Metasound.Generator.OutputAnalyzer.ForwardValue.Bool",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOututAnalyzerForwardValueBoolTest::RunTest(const FString&)
	{
		using DataType = bool;
		constexpr DataType ExpectedValue = true;
		return RunOutputAnalyzerForwardValueTest<DataType>(*this, ExpectedValue, "UE.Forward.Bool");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOututAnalyzerForwardValueStringTest,
	"Audio.Metasound.Generator.OutputAnalyzer.ForwardValue.String",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOututAnalyzerForwardValueStringTest::RunTest(const FString&)
	{
		using FDataType = FString;
		const FDataType ExpectedValue { "unexpected value" };
		return RunOutputAnalyzerForwardValueTest<FDataType>(*this, ExpectedValue, "UE.Forward.String");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundGeneratorOutputAnalyzerTriggerToTimeTest,
	"Audio.Metasound.Generator.OutputAnalyzer.TriggerToTime",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundGeneratorOutputAnalyzerTriggerToTimeTest::RunTest(const FString&)
	{
		const FName InputName = "MyInput";
		const FName OutputName = "MyOutput";
		constexpr FSampleRate SampleRate = 48000;
		constexpr int32 NumSamplesPerBlock = 128;

		// Build a passthrough graph
		FGuid OutputNodeId;
		const TUniquePtr<FMetasoundGenerator> Generator =
			BuildPassthroughGraph<FTrigger>(*this, InputName, OutputName, SampleRate, NumSamplesPerBlock, &OutputNodeId);
		UTEST_NOT_NULL("Generator build", Generator.Get());

		// Add an analyzer to the output
		FAnalyzerAddress AnalyzerAddress;
		AnalyzerAddress.DataType = GetMetasoundDataTypeName<FTrigger>();
		AnalyzerAddress.InstanceID = 1234;
		AnalyzerAddress.OutputName = OutputName;
		AnalyzerAddress.AnalyzerName = FVertexAnalyzerTriggerToTime::GetAnalyzerName();
		AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
		AnalyzerAddress.AnalyzerMemberName = FVertexAnalyzerTriggerToTime::FOutputs::GetValue().Name;
		AnalyzerAddress.NodeID = OutputNodeId;
		Generator->AddOutputVertexAnalyzer(AnalyzerAddress);

		// make the interval such that the triggers don't line up exactly with the beginning of the block after the first block
		constexpr int32 TriggerIntervalSamples = NumSamplesPerBlock / 3 + NumSamplesPerBlock / 2; 
		constexpr double TriggerIntervalSeconds = static_cast<double>(TriggerIntervalSamples) / SampleRate;
		double NextTriggerReceivedSeconds = 0;
		bool CallbackSuccess = true;
		int32 NumTriggersReceived = 0;
		
		// Subscribe for updates
		Generator->OnOutputChanged.AddLambda(
			[this, &NextTriggerReceivedSeconds, &AnalyzerAddress, &CallbackSuccess, &NumTriggersReceived, TriggerIntervalSeconds]
			(const FName AnalyzerName, const FName OutputName, const FName AnalyzerOutputName, TSharedPtr<IOutputStorage> OutputData)
			{
				CallbackSuccess = TestEqual("Data types match", OutputData->GetDataTypeName(), GetMetasoundDataTypeName<FTime>())
				&& TestEqual("Analyzer names match", AnalyzerName, AnalyzerAddress.AnalyzerName)
				&& TestEqual("Output names match", OutputName, AnalyzerAddress.OutputName)
				&& TestEqual("Analyzer output names match", AnalyzerOutputName, AnalyzerAddress.AnalyzerMemberName);

				if (!CallbackSuccess)
				{
					return;
				}

				const FTime Time = static_cast<TOutputStorage<FTime>*>(OutputData.Get())->Get();
				CallbackSuccess = TestEqual("Time is as expected", Time.GetSeconds(), NextTriggerReceivedSeconds);
				NextTriggerReceivedSeconds += TriggerIntervalSeconds;
				++NumTriggersReceived;
			});
		
		// Get the input
		const TOptional<FTriggerWriteRef> TriggerIn = Generator->GetInputWriteReference<FTrigger>(InputName);
		UTEST_TRUE("Trigger input is valid", TriggerIn.IsSet());

		int32 NextTriggerBlockOffset = 0;
		constexpr int32 NumBlocksToTest = 20;

		for (int32 i = 0; i < NumBlocksToTest; ++i)
		{
			int32 NumTriggersThisBlock = 0;
			NumTriggersReceived = 0;
			
			// Set the triggers for this block
			{
				while (NextTriggerBlockOffset < NumSamplesPerBlock)
				{
					(*TriggerIn)->TriggerFrame(NextTriggerBlockOffset);
					NextTriggerBlockOffset += TriggerIntervalSamples;
					++NumTriggersThisBlock;
				}
				NextTriggerBlockOffset -= NumSamplesPerBlock;
			}
		
			// Render a block
			{
				TArray<float> Buffer;
				Buffer.Reserve(NumSamplesPerBlock);
				UTEST_EQUAL("Generated the right number of samples.",
					Generator->OnGenerateAudio(Buffer.GetData(), NumSamplesPerBlock),
					NumSamplesPerBlock);
			}

			UTEST_TRUE(FString::Printf(TEXT("Callback success on iteration %i"), i), CallbackSuccess);
			UTEST_EQUAL(FString::Printf(TEXT("Correct number of triggers on iteration %i"), i), NumTriggersReceived, NumTriggersThisBlock);
		}

		return true;
	}
}

#endif