// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundTestInterfaces.h"
#include "MetasoundBuilderSubsystem.h"
#include "../../MetasoundFrontend/Public/MetasoundFrontendDocumentVersioning.h"
#include "MetasoundSource.h"
#include "MetasoundStandardNodesNames.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Metasound::Test::InterfaceUpdate
{
	struct FInitTestBuilderSourceOutput
	{
		FMetaSoundBuilderNodeOutputHandle OnPlayOutput;
		FMetaSoundBuilderNodeInputHandle OnFinishedInput;
		TArray<FMetaSoundBuilderNodeInputHandle> AudioOutNodeInputs;
	};

	UMetaSoundSourceBuilder& CreateSourceBuilder(
		FAutomationTestBase& Test,
		EMetaSoundOutputAudioFormat OutputFormat,
		bool bIsOneShot,
		FInitTestBuilderSourceOutput& Output)
	{
		using namespace Audio;
		using namespace Metasound;
		using namespace Metasound::Engine;
		using namespace Metasound::Frontend;

		EMetaSoundBuilderResult Result;
		UMetaSoundSourceBuilder* Builder = UMetaSoundBuilderSubsystem::GetChecked().CreateSourceBuilder(
			"Unit Test Graph Builder",
			Output.OnPlayOutput,
			Output.OnFinishedInput,
			Output.AudioOutNodeInputs,
			Result,
			OutputFormat,
			bIsOneShot);
		checkf(Builder, TEXT("Failed to create MetaSoundSourceBuilder"));
		Test.AddErrorIfFalse(Result == EMetaSoundBuilderResult::Succeeded, TEXT("Builder created but CreateSourceBuilder did not result in 'Succeeded' state"));

		return *Builder;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetasoundInterfaceMinorUpdateTest,
		"Audio.Metasound.Interface.MinorVersionUpdate",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(
		FMetasoundInterfaceMinorUpdateTest,
		"Audio.Metasound.Interface.MinorVersionUpdate",
		"[MustPass]")
	bool FMetasoundInterfaceMinorUpdateTest::RunTest(const FString&)
	{
		using namespace Audio;
		using namespace Metasound::Frontend;

		FInitTestBuilderSourceOutput BuilderSourceOutput;
		UMetaSoundSourceBuilder& MetasoundSourceBuilder = CreateSourceBuilder(*this, EMetaSoundOutputAudioFormat::Mono, true, BuilderSourceOutput);

		FMetaSoundFrontendDocumentBuilder& MetasoundFrontendBuilder = MetasoundSourceBuilder.GetBuilder();

		TArray<FMetasoundFrontendInterface> TestInterfaceVersions;

		IAudioParameterInterfaceRegistry::Get().IterateInterfaces([&TestInterfaceVersions](FParameterInterfacePtr InterfacePtr)
			{
				if (InterfacePtr->GetName() == UpdateTestInterface_0_1::GetVersion().Name)
				{
					TestInterfaceVersions.Add(InterfacePtr);
				}
			});

		TestEqual(TEXT("There should be two versions of the test interface"), TestInterfaceVersions.Num(), 2);

		const FMetasoundFrontendInterface TestInterfaceV0_1 = TestInterfaceVersions[0];
		const FMetasoundFrontendInterface TestInterfaceV0_2 = TestInterfaceVersions[1];

		TestEqual("Test Interface V0.1 version numbers should be correct", TestInterfaceV0_1.Metadata.Version.Number.Major, 0);
		TestEqual("Test Interface V0.1 version numbers should be correct", TestInterfaceV0_1.Metadata.Version.Number.Minor, 1);
		TestEqual("Test Interface V0.2 version numbers should be correct", TestInterfaceV0_2.Metadata.Version.Number.Major, 0);
		TestEqual("Test Interface V0.2 version numbers should be correct", TestInterfaceV0_2.Metadata.Version.Number.Minor, 2);

		{
			FModifyInterfaceOptions ModifyInterfaceOptions({ }, { TestInterfaceV0_1 });

			MetasoundFrontendBuilder.ModifyInterfaces(MoveTemp(ModifyInterfaceOptions));
		}

		const bool bHasInterfaceV0_1 = MetasoundFrontendBuilder.IsInterfaceDeclared(TestInterfaceV0_1.Metadata.Version);

		TestTrue("Test Interface V0.1 should be declared at this point", bHasInterfaceV0_1);
		
		EMetaSoundBuilderResult BuilderResult{ EMetaSoundBuilderResult::Failed };

		// This block creates a chain of Trigger Input -> Trigger Toggle -> SuperOscillator, Trigger Toggle -> Trigger Output
		if (FMetaSoundNodeHandle SuperOscillatorNodeHandle = MetasoundSourceBuilder.AddNodeByClassName(FMetasoundFrontendClassName(StandardNodes::Namespace, FName("SuperOscillatorMono"), StandardNodes::AudioVariant), BuilderResult);
			SuperOscillatorNodeHandle.IsSet() && BuilderResult == EMetaSoundBuilderResult::Succeeded)
		{
			FMetaSoundNodeHandle TriggerToggleNodeHandle = MetasoundSourceBuilder.AddNodeByClassName(FMetasoundFrontendClassName(StandardNodes::Namespace, FName("Trigger Toggle"), ""), BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to add trigger toggle node");

			FMetaSoundBuilderNodeInputHandle TriggerToggleOnInput = MetasoundSourceBuilder.FindNodeInputByName(TriggerToggleNodeHandle, FName("On"), BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find \"On\" input pin of trigger toggle node");

			FMetaSoundNodeHandle InterfaceInputNodeHandle = MetasoundSourceBuilder.FindGraphInputNode(UpdateTestInterface_0_1::Inputs::InputTrigger, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find interface input node");
			 
			FMetaSoundBuilderNodeOutputHandle InterfaceInputNodeOutputHandle = MetasoundSourceBuilder.FindNodeOutputByName(InterfaceInputNodeHandle, UpdateTestInterface_0_1::Inputs::InputTrigger, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find interface input node output pin handle");

			MetasoundSourceBuilder.ConnectNodes(InterfaceInputNodeOutputHandle, TriggerToggleOnInput, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to connect interface trigger input pin to Trigger Toggle Node's \"On\" pin.");

			FMetaSoundBuilderNodeInputHandle SuperOscEnabledInput = MetasoundSourceBuilder.FindNodeInputByName(SuperOscillatorNodeHandle, FName("Enabled"), BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find \"Enabled\" input pin of SuperOscillator node");

			FMetaSoundBuilderNodeOutputHandle TriggerToggleValueOutput = MetasoundSourceBuilder.FindNodeOutputByName(TriggerToggleNodeHandle, FName("Value"), BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find \"Value\" output pin of trigger toggle node");

			MetasoundSourceBuilder.ConnectNodes(TriggerToggleValueOutput, SuperOscEnabledInput, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to connect Trigger Toggle Node's \"Value\" pin to SuperOscillator node's \"Enabled\" pin.");

			FMetaSoundNodeHandle InterfaceOutputNodeHandle = MetasoundSourceBuilder.FindGraphOutputNode(UpdateTestInterface_0_1::Outputs::OutputTrigger, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find interface output node");

			FMetaSoundBuilderNodeInputHandle InterfaceOutputNodeInputHandle = MetasoundSourceBuilder.FindNodeInputByName(InterfaceOutputNodeHandle, UpdateTestInterface_0_1::Outputs::OutputTrigger, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find interface input node output pin handle");

			FMetaSoundBuilderNodeOutputHandle TriggerToggleOutOutput = MetasoundSourceBuilder.FindNodeOutputByName(TriggerToggleNodeHandle, FName("Out"), BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to find \"Out\" output pin of trigger toggle node");

			MetasoundSourceBuilder.ConnectNodes(TriggerToggleOutOutput, InterfaceOutputNodeInputHandle, BuilderResult);
			AddErrorIfFalse(BuilderResult == EMetaSoundBuilderResult::Succeeded, "Failed to connect Trigger Toggle Node's \"Out\" pin to graph output pin.");
		}
		else
		{
			AddError("Failed to add superoscillator node to test graph");
			return false;
		}

		{
			const FMetasoundFrontendClassInput* InputTriggerNode = MetasoundFrontendBuilder.FindGraphInput(UpdateTestInterface_0_1::Inputs::InputTrigger);
			if (TestNotNull("Graph has the trigger input defined by interface V0.1", InputTriggerNode))
			{
				TestEqual("Input Trigger Node should have FTrigger as type", InputTriggerNode->TypeName, GetMetasoundDataTypeName<FTrigger>());
			}

			const FMetasoundFrontendClassOutput* OutputTriggerNode = MetasoundFrontendBuilder.FindGraphOutput(UpdateTestInterface_0_1::Outputs::OutputTrigger);
			if (TestNotNull("Graph has the trigger output defined by interface V0.1", OutputTriggerNode))
			{
				TestEqual("Output Trigger Node should have FTrigger as type", OutputTriggerNode->TypeName, GetMetasoundDataTypeName<FTrigger>());
			}
		}

		TScriptInterface<IMetaSoundDocumentInterface> MetasoundSourceInterface = MetasoundSourceBuilder.BuildNewMetaSound(FName("TestMetaSoundSource"));
		if (!TestNotNull("Built MetasoundDocumentInterface is null", MetasoundSourceInterface.GetObject()))
		{
			return false;
		}

		TObjectPtr<UMetaSoundSource> MetasoundSource = Cast<UMetaSoundSource>(MetasoundSourceInterface.GetObject());
		if (!TestNotNull("Cast to Metasound Source failed", MetasoundSource.Get()))
		{
			return false;
		}

		// This will internally throw an error if there are problems when versioning the asset
		FVersioningManager::Get().VersionInterfaces(MetasoundFrontendBuilder);

		return true;
	}
}

#endif
