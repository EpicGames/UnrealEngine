// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOutput.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTime.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Metasound::Test::GeneratorOutput
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMetasoundOutputIsTypeTest,
		"Audio.Metasound.Output.IsType",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(
		FMetasoundOutputIsTypeTest,
		"Audio.Metasound.Output.IsType",
		"[MustPass]")
	bool FMetasoundOutputIsTypeTest::RunTest(const FString&)
	{
		FMetaSoundOutput Output;
		
		Output.Init(float{ 0.0f });
		UTEST_TRUE("float is float", Output.IsType<float>());
		UTEST_FALSE("float is not int32", Output.IsType<int32>());
		UTEST_FALSE("float is not bool", Output.IsType<bool>());
		UTEST_FALSE("float is not FString", Output.IsType<FString>());
		UTEST_FALSE("float is not FTime", Output.IsType<FTime>());

		Output.Init(int32{ 0 });
		UTEST_FALSE("int32 is not float", Output.IsType<float>());
		UTEST_TRUE("int32 is int32", Output.IsType<int32>());
		UTEST_FALSE("int32 is not bool", Output.IsType<bool>());
		UTEST_FALSE("int32 is not FString", Output.IsType<FString>());
		UTEST_FALSE("int32 is not FTime", Output.IsType<FTime>());
		
		Output.Init(bool{ false });
		UTEST_FALSE("bool is not float", Output.IsType<float>());
		UTEST_FALSE("bool is not int32", Output.IsType<int32>());
		UTEST_TRUE("bool is bool", Output.IsType<bool>());
		UTEST_FALSE("bool is not FString", Output.IsType<FString>());
		UTEST_FALSE("bool is not FTime", Output.IsType<FTime>());
		
		Output.Init(FString{ "hi!" });
		UTEST_FALSE("FString is not float", Output.IsType<float>());
		UTEST_FALSE("FString is not int32", Output.IsType<int32>());
		UTEST_FALSE("FString is not bool", Output.IsType<bool>());
		UTEST_TRUE("FString is FString", Output.IsType<FString>());
		UTEST_FALSE("FString is not FTime", Output.IsType<FTime>());

		constexpr float TimeSeconds = 123.456f;
		Output.Init(FTime::FromSeconds(TimeSeconds));
		UTEST_FALSE("FTime is not float", Output.IsType<float>());
		UTEST_FALSE("FTime is not int32", Output.IsType<int32>());
		UTEST_FALSE("FTime is not bool", Output.IsType<bool>());
		UTEST_FALSE("FTime is not FString", Output.IsType<FString>());
		UTEST_TRUE("FTime is FTime", Output.IsType<FTime>());
		
		return true;
	}

	template<typename DataType>
	bool RunOutputGetSetTest(FAutomationTestBase& Test, const DataType& InitialValue, const DataType& ExpectedValue)
	{
		FMetaSoundOutput Output;
		Output.Init<DataType>(InitialValue);
		DataType Value;
		
		if (!Test.TestTrue("Got value", Output.Get<DataType>(Value)))
		{
			return false;
		}
		if (!Test.TestEqual("Value equals initial", Value, InitialValue))
		{
			return false;
		}
		if (!Test.TestTrue("Set value", Output.Set<DataType>(ExpectedValue)))
		{
			return false;
		}
		if (!Test.TestTrue("Got value", Output.Get<DataType>(Value)))
		{
			return false;
		}
		return Test.TestEqual("Value equals expected", Value, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundOutputGetSetFloatTest,
	"Audio.Metasound.Output.GetSet.Float",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundOutputGetSetFloatTest::RunTest(const FString&)
	{
		using DataType = float;
		constexpr DataType InitialValue = 321.654f;
		constexpr DataType ExpectedValue = 123.456f;
		return RunOutputGetSetTest<DataType>(*this, InitialValue, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundOutputGetSetInt32Test,
	"Audio.Metasound.Output.GetSet.Int32",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundOutputGetSetInt32Test::RunTest(const FString&)
	{
		using DataType = int32;
		constexpr DataType InitialValue = 321654;
		constexpr DataType ExpectedValue = 123456;
		return RunOutputGetSetTest<DataType>(*this, InitialValue, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundOutputGetSetBoolTest,
	"Audio.Metasound.Output.GetSet.Bool",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundOutputGetSetBoolTest::RunTest(const FString&)
	{
		using DataType = bool;
		constexpr DataType InitialValue = false;
		constexpr DataType ExpectedValue = true;
		return RunOutputGetSetTest<DataType>(*this, InitialValue, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundOutputGetSetStringTest,
	"Audio.Metasound.Output.GetSet.String",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundOutputGetSetStringTest::RunTest(const FString&)
	{
		using FDataType = FString;
		const FDataType InitialValue{ "hello" };
		const FDataType ExpectedValue{ "goodbye" };
		return RunOutputGetSetTest<FDataType>(*this, InitialValue, ExpectedValue);
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMetasoundOutputGetSetFTimeTest,
	"Audio.Metasound.Output.GetSet.FTime",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMetasoundOutputGetSetFTimeTest::RunTest(const FString&)
	{
		using FDataType = FTime;
		const FDataType InitialValue{ FTime::FromSeconds(123.456f) };
		const FDataType ExpectedValue{ FTime::FromSeconds(654.321f) };
		return RunOutputGetSetTest<FDataType>(*this, InitialValue, ExpectedValue);
	}
}

#endif
