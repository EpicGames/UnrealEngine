// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTestGraphBuilder.h"
#include "Containers/Set.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundOperatorBuilder.h"

namespace Metasound::Test
{
	using namespace Frontend;

	FNodeTestGraphBuilder::FNodeTestGraphBuilder()
	{
		Document.RootGraph.ID = FGuid::NewGuid();
		Document.RootGraph.Metadata.SetClassName({ "Metasound", "TestNodes", *LexToString(FGuid::NewGuid()) });
		Document.RootGraph.Metadata.SetType(EMetasoundFrontendClassType::Graph);
		Document.RootGraph.InitDefaultGraphPage();

		DocumentHandle = IDocumentController::CreateDocumentHandle(Document);
		RootGraph = DocumentHandle->GetRootGraph();
		check(RootGraph->IsValid());
	}

	FNodeHandle FNodeTestGraphBuilder::AddNode(const FNodeClassName& ClassName, int32 MajorVersion) const
	{
		check(RootGraph->IsValid());

		FNodeHandle Node = INodeController::GetInvalidHandle();
		FMetasoundFrontendClass NodeClass;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ISearchEngine::Get().FindClassWithHighestMinorVersion(ClassName, MajorVersion, NodeClass))
		{
			Node = RootGraph->AddNode(NodeClass.Metadata);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return Node;
	}


	FNodeHandle FNodeTestGraphBuilder::GetInput(const FName& InName) const
	{
		return RootGraph->GetInputNodeWithName(InName);
	}

	FNodeHandle FNodeTestGraphBuilder::GetOutput(const FName& InName) const
	{
		return RootGraph->GetOutputNodeWithName(InName);
	}

	FNodeHandle FNodeTestGraphBuilder::AddInput(
		const FName& InputName,
		const FName& TypeName,
		EMetasoundFrontendVertexAccessType AccessType) const
	{
		check(RootGraph->IsValid());

		FMetasoundFrontendClassInput Input;
		Input.Name = InputName;
		Input.TypeName = TypeName;
		Input.AccessType = AccessType;
		Input.VertexID = FGuid::NewGuid();
		Input.InitDefault();

		return RootGraph->AddInputVertex(Input);
	}

	FNodeHandle FNodeTestGraphBuilder::AddOutput(const FName& OutputName, const FName& TypeName)
	{
		check(RootGraph->IsValid());

		FMetasoundFrontendClassOutput Output;
		Output.Name = OutputName;
		Output.TypeName = TypeName;
		Output.VertexID = FGuid::NewGuid();
		FNodeHandle NodeHandle = RootGraph->AddOutputVertex(Output);

		static const FName AudioBufferTypeName = GetMetasoundDataTypeName<FAudioBuffer>();
		if (TypeName == AudioBufferTypeName)
		{
			AudioOutputNames.Add(OutputName);
		}

		return NodeHandle;
	}

	TUniquePtr<FMetasoundGenerator> FNodeTestGraphBuilder::BuildGenerator(FSampleRate SampleRate, int32 SamplesPerBlock) const
	{
		// Because a generator is tied to the concept of a source, go ahead and add the "OnPlay" input.
		// Otherwise we get warnings when we run our tests.
		AddInput(SourceInterface::Inputs::OnPlay, Metasound::GetMetasoundDataTypeName<FTrigger>());
		
		UPackage* TransientPackage = GetTransientPackage();
		check(TransientPackage);
		const FTopLevelAssetPath UnknownAsset(TransientPackage->GetFName(), FName("UnknownAsset"));
		if (const TUniquePtr<FFrontendGraph> Graph = Frontend::FGraphBuilder::CreateGraph(Document, UnknownAsset))
		{
			FOperatorBuilderSettings BuilderSettings;
			BuilderSettings.bFailOnAnyError = true;
			BuilderSettings.bPopulateInternalDataReferences = true;
			BuilderSettings.bValidateEdgeDataTypesMatch = true;
			BuilderSettings.bValidateNoCyclesInGraph = true;
			BuilderSettings.bValidateNoDuplicateInputs = true;
			BuilderSettings.bValidateOperatorOutputsAreBound = true;
			BuilderSettings.bValidateVerticesExist = true;

			const FOperatorSettings OperatorSettings{SampleRate, static_cast<float>(SampleRate) / SamplesPerBlock};

			FMetasoundEnvironment Environment;
			Environment.SetValue<uint64>(CoreInterface::Environment::InstanceID, 123);
			
			FGeneratorInitParams GeneratorInitParams
			{
				.OperatorSettings = OperatorSettings,
				.BuilderSettings = MoveTemp(BuilderSettings),
				.Graph = MakeShared<const FFrontendGraph, ESPMode::ThreadSafe>(*Graph),
				.Environment = Environment,
				.AudioOutputNames = AudioOutputNames,
				.bBuildSynchronous = true, // Don't need to wait for latent tasks
				.AssetPath = FTopLevelAssetPath(FName(), "TestMetasound")
			};
			return MakeUnique<FMetasoundConstGraphGenerator>(MoveTemp(GeneratorInitParams));
		}

		return nullptr;
	}

	TUniquePtr<FMetasoundGenerator> FNodeTestGraphBuilder::MakeSingleNodeGraph(
		const FNodeClassName& ClassName,
		const int32 MajorVersion,
		const FSampleRate SampleRate,
		const int32 SamplesPerBlock)
	{
		FNodeTestGraphBuilder Builder;
		FNodeHandle NodeHandle = Builder.AddNode(ClassName, MajorVersion);

		if (!NodeHandle->IsValid())
		{
			return nullptr;
		}

		// add the inputs and connect them
		for (FInputHandle Input : NodeHandle->GetInputs())
		{
			FNodeHandle InputNode = Builder.AddInput(Input->GetName(), Input->GetDataType(), Input->GetVertexAccessType());

			if (!InputNode->IsValid())
			{
				return nullptr;
			}

			FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(Input->GetName());
			FInputHandle InputToConnect = NodeHandle->GetInputWithVertexName(Input->GetName());

			// set the input to the default, if there is one
			if (const FMetasoundFrontendLiteral* Default = InputToConnect->GetClassDefaultLiteral(); nullptr != Default)
			{
				const FGuid InputId = Builder.RootGraph->GetVertexIDForInputVertex(Input->GetName());
				Builder.RootGraph->SetDefaultInput(InputId, *Default, {});
			}

			if (!InputToConnect->Connect(*OutputToConnect))
			{
				return nullptr;
			}
		}

		// add the outputs and connect them
		for (FOutputHandle Output : NodeHandle->GetOutputs())
		{
			FNodeHandle OutputNode = Builder.AddOutput(Output->GetName(), Output->GetDataType());

			if (!OutputNode->IsValid())
			{
				return nullptr;
			}

			FOutputHandle OutputToConnect = NodeHandle->GetOutputWithVertexName(Output->GetName());
			FInputHandle InputToConnect = OutputNode->GetInputWithVertexName(Output->GetName());

			if (!InputToConnect->Connect(*OutputToConnect))
			{
				return nullptr;
			}
		}

		// have to make an audio output for the generator to do anything
		if (Builder.AudioOutputNames.IsEmpty())
		{
			Builder.AddOutput("AudioOut", GetMetasoundDataTypeName<FAudioBuffer>());
		}

		// build the graph
		return Builder.BuildGenerator(SampleRate, SamplesPerBlock);
	}

	bool FNodeTestGraphBuilder::ConnectNodes(const Frontend::FNodeHandle& LeftNode, const FName& OutputName, const Frontend::FNodeHandle& RightNode, const FName& InputName)
	{
		if (!LeftNode->IsValid() || !RightNode->IsValid())
		{
			return false;
		}

		FOutputHandle OutputToConnect = LeftNode->GetOutputWithVertexName(OutputName);
		FInputHandle InputToConnect = RightNode->GetInputWithVertexName(InputName);

		if (!OutputToConnect->IsValid() || !InputToConnect->IsValid())
		{
			return false;
		}
		return InputToConnect->Connect(*OutputToConnect);
	}

	bool FNodeTestGraphBuilder::ConnectNodes(const Frontend::FNodeHandle& LeftNode, const Frontend::FNodeHandle& RightNode, const FName& InputOutputName)
	{
		return ConnectNodes(LeftNode, InputOutputName, RightNode, InputOutputName);
	}

	bool FNodeTestGraphBuilder::AddAndConnectDataReferenceInputs(const Frontend::FNodeHandle& NodeHandle)
	{
		// add the inputs and connect them
		for (FInputHandle Input : NodeHandle->GetInputs())
		{
			if (Input->GetVertexAccessType() != EMetasoundFrontendVertexAccessType::Reference)
			{
				continue;
			}

			FNodeHandle InputNode = AddInput(Input->GetName(), Input->GetDataType(), Input->GetVertexAccessType());

			if (!InputNode->IsValid())
			{
				return false;
			}

			FOutputHandle OutputToConnect = InputNode->GetOutputWithVertexName(Input->GetName());
			FInputHandle InputToConnect = NodeHandle->GetInputWithVertexName(Input->GetName());

			if (!InputToConnect->Connect(*OutputToConnect))
			{
				return false;
			}
		}

		return true;
	}

	FNodeHandle FNodeTestGraphBuilder::AddAndConnectDataReferenceInput(const FNodeHandle& NodeToConnect, const FName& InputName, const FName& TypeName, const FName& NodeName) const
	{
		FName NameToUse = NodeName.IsNone() ? InputName : NodeName;
		FNodeHandle InputNode = AddInput(NameToUse, TypeName);
		ConnectNodes(InputNode, NameToUse, NodeToConnect, InputName);
		return InputNode;
	}

	FNodeHandle FNodeTestGraphBuilder::AddAndConnectDataReferenceOutput(const FNodeHandle& NodeToConnect, const FName& OutputName, const FName& TypeName, const FName& NodeName)
	{
		FName NameToUse = NodeName.IsNone() ? OutputName : NodeName;
		FNodeHandle OutputNode = AddOutput(NameToUse, TypeName);
		ConnectNodes(NodeToConnect, OutputName, OutputNode, NameToUse);
		return OutputNode;
	}

}
