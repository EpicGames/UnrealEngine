// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundGenerator.h"
#include "MetasoundNodeInterface.h"

#define UE_API METASOUNDENGINETEST_API

namespace Metasound::Test
{
	/** Helper to make testing nodes simpler. */
	class FNodeTestGraphBuilder
	{
	public:
		UE_API FNodeTestGraphBuilder();

		/** 
		* Add a node to the graph 
		*
		* @param ClassName		Class name of the node to add
		* @param MajorVersion	Major version of the node to add
		* @return				NodeHandle of the added node
		*/
		UE_API Frontend::FNodeHandle AddNode(const FNodeClassName& ClassName, int32 MajorVersion) const;

		/**
		* Add an input node from the graph by name
		*
		* @param Name			The name of the input node to get
		* @return				NodeHandle, which can be invalid if an input with the given name doesn't exist in the graph
		*/
		UE_API Frontend::FNodeHandle GetInput(const FName& Name) const;

		/**
		* Add an output node from the graph by name
		*
		* @param Name			The name of the output node to get
		* @return				NodeHandle, which can be invalid if an input with the given name doesn't exist in the graph
		*/
		UE_API Frontend::FNodeHandle GetOutput(const FName& Name) const;

		/**
		* Add an input node from the graph by name, type, and access type
		*
		* @param InputName		The name of the input node to add
		* @param TypeName		The type of the node to add (use: GetMetasoundDataTypeName<DataType>());
		* @param AccessType		Whether the node should be a Value, Reference, or Unset type
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		UE_API Frontend::FNodeHandle AddInput(
			const FName& InputName,
			const FName& TypeName,
			EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Reference) const;

		/**
		* Add a constructor input node from the graph by name and type
		*
		* @param InputName		The name of the input node to add
		* @param Value			The value for the node to hold. The type of the value will determine the type for the node.
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		template<typename DataType>
		Frontend::FNodeHandle AddConstructorInput(const FName& InputName, const DataType& Value) const
		{
			check(RootGraph->IsValid());

			FMetasoundFrontendClassInput Input;
			Input.Name = InputName;
			Input.TypeName = GetMetasoundDataTypeName<DataType>();
			Input.VertexID = FGuid::NewGuid();
			Input.AccessType = EMetasoundFrontendVertexAccessType::Value;
			Input.InitDefault().Set(Value);
			return RootGraph->AddInputVertex(Input);
		}

		/**
		* Add a data reference output node from the graph by name and type
		*
		* @param InputName		The name of the output node to add
		* @param TypeName		The type for the node to reference
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		UE_API Frontend::FNodeHandle AddOutput(const FName& OutputName, const FName& TypeName);

		/** 
		* Adds a constructor input node to the graph and connects it as an input to the NodeToConnect
		* 
		* @param NodeToConnect	The node to connect the newly added constructor input node to
		* @param InputName		The name of the input pin to connect the node to, and the name to use for the new node.
		* @param Value			The value for the node to hold. The type of the value will determine the type for the node.
		* @param NodeName		Optional name to use for the created constructor input node
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		template <typename DataType>
		Frontend::FNodeHandle AddAndConnectConstructorInput(const Frontend::FNodeHandle& NodeToConnect, const FName& InputName, const DataType& Value, const FName& NodeName = NAME_None) const
		{
			FName NameToUse = NodeName.IsNone() ? InputName : NodeName;
			Frontend::FNodeHandle InputNode = AddConstructorInput<DataType>(NameToUse, Value);
			ConnectNodes(InputNode, NameToUse, NodeToConnect, InputName);
			return InputNode;
		}

		/**
		 * Adds a constructor input node to the graph and connects it as an input to the NodeToConnect.
		 * This version handles input data types that don't have direct support in FLiteral.
		 *
		 * @tparam InputDataType The data type of the input vertex
		 * @tparam LiteralDataType The data type of the literal used to set the input vertex
		 * @param NodeToConnect The node to connect the newly added constructor input node to
		 * @param InputName The name of the input pin to connect the node to, and the name to use for the new node.
		 * @param Value The value for the node to hold. The type of the value will determine the type for the node.
		 * @param NodeName Optional name to use for the created constructor input node
		 * @return NodeHandle of the added node, can be invalid if failed to add
		 */
		template<typename InputDataType, typename LiteralDataType>
		Frontend::FNodeHandle AddAndConnectConstructorInput(
			const Frontend::FNodeHandle& NodeToConnect,
			const FName& InputName,
			const LiteralDataType& Value,
			const FName& NodeName = NAME_None) const
		{
			check(RootGraph->IsValid());

			const FName NameToUse = NodeName.IsNone() ? InputName : NodeName;
			
			FMetasoundFrontendClassInput Input;
			Input.Name = NameToUse;
			Input.TypeName = GetMetasoundDataTypeName<InputDataType>();
			Input.VertexID = FGuid::NewGuid();
			Input.AccessType = EMetasoundFrontendVertexAccessType::Value;
			Input.InitDefault().Set(Value);
			
			Frontend::FNodeHandle InputNode = RootGraph->AddInputVertex(Input);

			ConnectNodes(InputNode, NameToUse, NodeToConnect, InputName);
			
			return InputNode;
		}

		/** 
		* Adds data reference input nodes for each input on the NodeHandle and connects them to the inputs of the NodeHandle
		*
		* @param NodeHandle		The node to connect the input reference nodes to
		* @return				whether all the nodes were successfully created and connected
		*/
		UE_API bool AddAndConnectDataReferenceInputs(const Frontend::FNodeHandle& NodeHandle);

		/** 
		* Adds a data reference input node with InputName and connects it to the node's input with the same name
		*
		* @param NodeToConnect	The node to connect the newly added constructor input node to
		* @param InputName		The name of the input pin to connect the node to, and the name to use for the new node.
		* @param TypeName		The type name of the data reference
		* @param NodeName		Optional name to use for the created data reference input node
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		UE_API Frontend::FNodeHandle AddAndConnectDataReferenceInput(const Frontend::FNodeHandle& NodeToConnect, const FName& InputName, const FName& TypeName, const FName& NodeName = NAME_None) const;

		/** Adds a data reference output node with OutputName and connects it to the node's output with the same name
		* 	
		* @param NodeToConnect	The node to connect the newly added constructor input node to
		* @param InputName		The name of the output pin to connect the node to, and the name to use for the new node.
		* @param TypeName		The type name of the data reference
		* @param NodeName		Optoinal name to use for the created data reference output node
		* @return				NodeHandle of the added node, can be invalid if failed to add
		*/
		UE_API Frontend::FNodeHandle AddAndConnectDataReferenceOutput(const Frontend::FNodeHandle& NodeToConnect, const FName& OutputName, const FName& TypeName, const FName& NodeName = NAME_None);

		/** 
		* Call last to create a generator from the graph constructed using the other method. 
		* 
		* @param SampleRate			The Sample Rate the generator should render audio at
		* @param SamplesPerBlock	The number of samples to render per block
		* @return					A unique pointer to the Metasound Generator created
		*/
		UE_API TUniquePtr<FMetasoundGenerator> BuildGenerator(FSampleRate SampleRate = 48000, int32 SamplesPerBlock = 256) const;

		/** 
		* Helper that will add a single node, wire up the node's inputs and outputs, and hand back the graph's operator 
		*
		* @param ClassName			The class name of the node to add
		* @param MajorVersion		The version of the node to use
		* @param SampleRate			The sample rate the generator should render audio at
		* @param SamplesPerBlock	The number of samples to render per block
		* @return					A unique pointer to the Metasound Generator created
		*/
		static UE_API TUniquePtr<FMetasoundGenerator> MakeSingleNodeGraph(
			const FNodeClassName& ClassName,
			int32 MajorVersion,
			FSampleRate SampleRate = 48000,
			int32 SamplesPerBlock = 256);
		
		/** 
		* Connects an Input (Left) Node's output with name OutputName to another (Right) node's input with name InputName 
		*
		* @param LeftNode		The "left hand" side node to connect from
		* @param OutputName		The name of the output pin to connect from the left node
		* @param RightNode		The "right hand" side node to connect to
		* @param InputName		The name of the input pin to connect to the right node
		* @bool					true if able to connect the nodes (ie. all nodes were valid, all input/outputs were valid)
		*/
		static UE_API bool ConnectNodes(const Frontend::FNodeHandle& LeftNode, const FName& OutputName, const Frontend::FNodeHandle& RightNode, const FName& InputName);

		/**
		* Connects an Input (Left) Node's output with name OutputName to another (Right) node's input with name InputName
		*
		* @param LeftNode			The "left hand" side node to connect from
		* @param RightNode			The "right hand" side node to connect to
		* @param InputOutputName	The name of the input pin to connect to the right node
		* @bool						true if able to connect the nodes (ie. all nodes were valid, all input/outputs were valid)
		*/
		static UE_API bool ConnectNodes(const Frontend::FNodeHandle& LeftNode, const Frontend::FNodeHandle& RightNode, const FName& InputOutputName);

	private:
		FMetasoundFrontendDocument Document;
		Frontend::FDocumentHandle DocumentHandle = Frontend::IDocumentController::GetInvalidHandle();
		Frontend::FGraphHandle RootGraph = Frontend::IGraphController::GetInvalidHandle();
		TArray<FVertexName> AudioOutputNames;
	};
}

#undef UE_API
