// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundFrontend : ModuleRules
	{
		public MetasoundFrontend(ReadOnlyTargetRules Target) : base(Target)
		{
			NumIncludedBytesPerUnityCPPOverride = 294912; // best unity size found from using UBT ProfileUnitySizes mode

			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AudioExtensions",
					"Core",
					"CoreUObject",
					"Json",
					"JsonUtilities",
					"Serialization",
					"SignalProcessing",
					"MetasoundGraphCore",
					"AudioChannelAgnosticCore",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Projects",	// For IPluginManager
				});

			if (Target.bBuildWithEditorOnlyData)
			{
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"Slate",
						"SlateCore"
					}
				);
			}

			// In UE 5.6, registered node are expected to support the constructor 
			// signature Constructor(FNodeData, TSharedRef<const FNodeClassMetadata>)
			// Because there are many existing nodes, it may take time to update 
			// them. For convenience, the deprecations related to this change are
			// configurable via a preprocessor macro so that the deprecation warnings 
			// do not drown out other compiler errors and warnings.
			PublicDefinitions.Add("UE_METASOUND_DISABLE_5_6_NODE_REGISTRATION_DEPRECATION_WARNINGS=0");

			PrivateDefinitions.Add("METASOUND_PLUGIN=Metasound");
			PrivateDefinitions.Add("METASOUND_MODULE=MetasoundFrontend");

			// Disable experimental warnings
			bValidateExperimentalApi = false;
		}
	}
}
