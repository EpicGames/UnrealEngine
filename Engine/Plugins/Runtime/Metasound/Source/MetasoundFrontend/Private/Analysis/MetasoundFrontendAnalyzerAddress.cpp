// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "MetasoundArrayNodesRegistration.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundFrontendDataTypeTraits.h"
#include "Misc/AssertionMacros.h"


namespace Metasound
{
	namespace Frontend
	{
		FAnalyzerAddress::FAnalyzerAddress(const FString& InAddressString)
		{
			TArray<FString> Tokens;
			if (ensureAlwaysMsgf(InAddressString.ParseIntoArray(Tokens, METASOUND_ANALYZER_PATH_SEPARATOR) == 7, TEXT("Invalid Analyzer Address String Format")))
			{
				InstanceID = static_cast<uint64>(FCString::Atoi64(*Tokens[0]));
				NodeID = FGuid(Tokens[1]);
				OutputName = *Tokens[2];
				DataType = *Tokens[3];
				AnalyzerName = *Tokens[4];
				AnalyzerInstanceID = FGuid(Tokens[5]);
				AnalyzerMemberName = *Tokens[6];
			}
		}

		FName FAnalyzerAddress::GetAddressType() const
		{
			return "Analyzer";
		}

		FName FAnalyzerAddress::GetDataType() const
		{
			return DataType;
		}

		TUniquePtr<FTransmissionAddress> FAnalyzerAddress::Clone() const
		{
			return TUniquePtr<FTransmissionAddress>(new FAnalyzerAddress(*this));
		}

		uint32 FAnalyzerAddress::GetHash() const
		{
			uint32 AddressHash = HashCombineFast(AnalyzerInstanceID.A, GetTypeHashHelper(AnalyzerMemberName));
			AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(AnalyzerName));
			AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(DataType));
			AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(InstanceID));
			AddressHash = HashCombineFast(AddressHash, NodeID.A);
			AddressHash = HashCombineFast(AddressHash, GetTypeHashHelper(OutputName));

			return AddressHash;
		}

		bool FAnalyzerAddress::IsEqual(const FTransmissionAddress& InOther) const
		{
			if (InOther.GetAddressType() != GetAddressType())
			{
				return false;
			}

			const FAnalyzerAddress& OtherAddr = static_cast<const FAnalyzerAddress&>(InOther);
			return OtherAddr.AnalyzerInstanceID == AnalyzerInstanceID
				&& OtherAddr.AnalyzerMemberName == AnalyzerMemberName
				&& OtherAddr.AnalyzerName == AnalyzerName
				&& OtherAddr.DataType == DataType
				&& OtherAddr.InstanceID == InstanceID
				&& OtherAddr.NodeID == NodeID
				&& OtherAddr.OutputName == OutputName;
		}

		FString FAnalyzerAddress::ToString() const
		{
			return FString::Join(TArray<FString>
			{
				*FString::Printf(TEXT("%lld"), InstanceID),
				*NodeID.ToString(),
				*OutputName.ToString(),
				*DataType.ToString(),
				*AnalyzerName.ToString(),
				*AnalyzerInstanceID.ToString(),
				*AnalyzerMemberName.ToString()
			}, METASOUND_ANALYZER_PATH_SEPARATOR);
		}
	} // namespace Frontend

	template<>
	struct TEnableArrayNodes<Frontend::FAnalyzerAddress>
	{
		static constexpr bool Value = false;
	};

	template<>
	struct TEnableTransmissionNodeRegistration<Frontend::FAnalyzerAddress>
	{
		static constexpr bool Value = false;
	};

	template<typename FromDataType>
	struct TEnableAutoConverterNodeRegistration<FromDataType, Frontend::FAnalyzerAddress>
	{
		static constexpr bool Value = false;
	};

	template<>
	struct TEnableConstructorVertex<Frontend::FAnalyzerAddress>
	{
		static constexpr bool Value = false;
	};

	REGISTER_METASOUND_DATATYPE(Frontend::FAnalyzerAddress, "AnalyzerAddress");
} // namespace Metasound
