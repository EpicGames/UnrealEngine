// Copyright Epic Games, Inc. All Rights Reserved.
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerForwardValue.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerDensity.h"
#include "MetasoundPrimitives.h"


namespace Metasound
{
	namespace Frontend
	{
		class FVertexAnalyzerRegistry : public IVertexAnalyzerRegistry
		{
			TMap<FName, TUniquePtr<IVertexAnalyzerFactory>> AnalyzerFactoryRegistry;

		public:
			template<typename TAnalyzerFactoryClass>
			void RegisterAnalyzerFactory()
			{
				TUniquePtr<IVertexAnalyzerFactory> Factory(new TAnalyzerFactoryClass());
				AnalyzerFactoryRegistry.Emplace(TAnalyzerFactoryClass::GetAnalyzerName(), MoveTemp(Factory));
			}
			
			FVertexAnalyzerRegistry() = default;
			virtual ~FVertexAnalyzerRegistry() override = default;

			virtual const IVertexAnalyzerFactory* FindAnalyzerFactory(FName InAnalyzerName) const override
			{
				const TUniquePtr<IVertexAnalyzerFactory>* Factory = AnalyzerFactoryRegistry.Find(InAnalyzerName);
				if (nullptr == Factory)
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to find registered MetaSound Analyzer Factory with name '%ls'", *InAnalyzerName.ToString());
					return nullptr;
				}

				check(Factory->IsValid());
				return Factory->Get();
			}

			virtual void RegisterAnalyzerFactory(FName AnalyzerName, TUniquePtr<IVertexAnalyzerFactory>&& Factory) override
			{
				check(!AnalyzerFactoryRegistry.Contains(AnalyzerName));
				AnalyzerFactoryRegistry.Emplace(AnalyzerName, MoveTemp(Factory));
			}
		};

		IVertexAnalyzerRegistry& IVertexAnalyzerRegistry::Get()
		{
			static FVertexAnalyzerRegistry Registry;
			return Registry;
		}
	} // namespace Frontend
} // namespace Metasound
