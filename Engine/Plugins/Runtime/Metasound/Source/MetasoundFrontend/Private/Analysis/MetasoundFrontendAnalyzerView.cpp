// Copyright Epic Games, Inc. All Rights Reserved.
#include "Analysis/MetasoundFrontendAnalyzerView.h"

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"
#include "Analysis/MetasoundFrontendGraphAnalyzer.h"
#include "Math/UnrealMathUtility.h"
#include "MetasoundAssetBase.h"
#include "MetasoundFrontendController.h"
#include "MetasoundOperatorSettings.h"
#include "Templates/Function.h"


namespace Metasound
{
	namespace Frontend
	{
		FMetasoundAnalyzerView::FMetasoundAnalyzerView(FAnalyzerAddress&& InAnalyzerAddress)
			: AnalyzerAddress(MoveTemp(InAnalyzerAddress))
		{
		}

		void FMetasoundAnalyzerView::BindToAllOutputs(const FOperatorSettings& InOperatorSettings)
		{
			const IVertexAnalyzerFactory* Factory = IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(AnalyzerAddress.AnalyzerName);
			if (ensureMsgf(Factory, TEXT("Failed to bind AnalyzerView to all Analyzer outputs: Missing factory definition for analyzer with name '%s'"), *AnalyzerAddress.AnalyzerName.ToString()))
			{
				for (const FAnalyzerOutput& Output : Factory->GetAnalyzerOutputs())
				{
					FAnalyzerAddress OutputAddress = AnalyzerAddress;
					OutputAddress.AnalyzerMemberName = Output.Name;
					OutputAddress.DataType = Output.DataType;
					const FReceiverInitParams ReceiverParams { InOperatorSettings };
					IReceiver* Receiver = FDataTransmissionCenter::Get().RegisterNewReceiver(OutputAddress, ReceiverParams).Release();
					OutputReceivers.Add({ Output.Name, TSharedPtr<IReceiver>(Receiver) });
				}
			}
		}

		bool FMetasoundAnalyzerView::UnbindOutput(FName InOutputName)
		{
			return OutputReceivers.Remove(InOutputName) > 0;
		}

		TArray<FMetasoundAnalyzerView::FBoundOutputDescription> FMetasoundAnalyzerView::GetBoundOutputDescriptions() const
		{
			TArray<FBoundOutputDescription> OutputDescriptions;
			
			for (auto& Receiver : OutputReceivers)
			{
				if (!ensure(Receiver.Value.IsValid()))
				{
					continue;
				}

				OutputDescriptions.Add({ Receiver.Key, Receiver.Value->GetDataType() });
			}

			return OutputDescriptions;
		}
	} // namespace Frontend
} // namespace Metasound
