// Copyright Epic Games, Inc. All Rights Reserved.
#include "Analysis/MetasoundFrontendGraphAnalyzer.h"

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "MetasoundDataReference.h"
#include "MetasoundPrimitives.h"


namespace Metasound
{
	namespace Frontend
	{
		FGraphAnalyzer::FGraphAnalyzer(const FOperatorSettings& InSettings, uint64 InInstanceID, FNodeVertexDataReferenceMap&& InReferences)
			: OperatorSettings(InSettings)
			, InstanceID(InInstanceID)
			, InternalDataReferences(MoveTemp(InReferences))
		{
			using namespace Frontend;
			FGraphAnalyzerAddress GraphAddress(InstanceID);
			ActiveAnalyzerReceiver = FDataTransmissionCenter::Get().RegisterNewReceiver(GraphAddress, FReceiverInitParams { InSettings });
		}

		void FGraphAnalyzer::Execute()
		{
			// 1. Check if any message has been received that determines what analyzers are active.
			TSet<FAnalyzerAddress> ReceiverAddresses;
			bool bUpdateReceivers = false;
			if (ActiveAnalyzerReceiver.IsValid())
			{
				TReceiver<TArray<FAnalyzerAddress>>& AnalyzerReceiver = ActiveAnalyzerReceiver->GetAs<TReceiver<TArray<FAnalyzerAddress>>>();
				if (AnalyzerReceiver.CanPop())
				{
					bUpdateReceivers = true;
					TArray<FAnalyzerAddress> AnalyzerAddresses;
					AnalyzerReceiver.Pop(AnalyzerAddresses);
					ReceiverAddresses.Append(AnalyzerAddresses);
				}
			}

			// 2. Check if any message has been received that determines what analyzers are active and remove stale analyzers.
			if (bUpdateReceivers)
			{
				for (int32 i = Analyzers.Num() - 1; i >= 0; --i)
				{
					TUniquePtr<IVertexAnalyzer>& Analyzer = Analyzers[i];
					check(Analyzer.IsValid());
					const FAnalyzerAddress& AnalyzerAddress = Analyzer->GetAnalyzerAddress();
					if (ReceiverAddresses.Contains(AnalyzerAddress))
					{
						// Found the matching active analyzer list entry for this vertex analyzer.
						ReceiverAddresses.Remove(AnalyzerAddress);

						// Keep this vertex analyzer alive:
						continue;
					}

					Analyzers.RemoveAtSwap(i, EAllowShrinking::No);
				}
			}

			// 3. If message received to update analyzers, create missing analyzers
			if (bUpdateReceivers)
			{
				for (const FAnalyzerAddress& AnalyzerAddress : ReceiverAddresses)
				{
					const FDataReferenceCollection* Collection = InternalDataReferences.Find(AnalyzerAddress.NodeID);
					// TODO: This currently fails for composed graphs.  Figure out how to differentiate addresses for composed graphs or refactor to only support parent graph analysis.
// 						if (ensureMsgf(Collection != nullptr, TEXT("Failed to create MetaSoundAnalyzer: DataReferenceCollection for node analyzer at address '%s' not found."), *AnalyzerKey))
					if (Collection)
					{
						if (const IVertexAnalyzerFactory* Factory = IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(AnalyzerAddress.AnalyzerName))
						{
							if (const FAnyDataReference* DataRef = Collection->FindDataReference(AnalyzerAddress.OutputName))
							{
								FCreateAnalyzerParams Params{ AnalyzerAddress, OperatorSettings, *DataRef };
								TUniquePtr<IVertexAnalyzer> NewAnalyzer = Factory->CreateAnalyzer(Params);
								Analyzers.Add(MoveTemp(NewAnalyzer));
							}
						}
					}
				}
			}

			// 4. Execute active analyzers post update.
			for (TUniquePtr<Frontend::IVertexAnalyzer>& Analyzer : Analyzers)
			{
				Analyzer->Execute();
			}
		}
	} // namespace Frontend
} // namespace Metasound
