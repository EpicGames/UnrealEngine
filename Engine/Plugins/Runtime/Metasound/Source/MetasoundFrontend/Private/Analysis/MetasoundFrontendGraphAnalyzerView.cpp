// Copyright Epic Games, Inc. All Rights Reserved.
#include "Analysis/MetasoundFrontendGraphAnalyzerView.h"

#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendGraphAnalyzer.h"
#include "Math/UnrealMathUtility.h"
#include "MetasoundAssetBase.h"
#include "MetasoundFrontendController.h"
#include "MetasoundOperatorSettings.h"
#include "Templates/Function.h"


namespace Metasound
{
	namespace Frontend
	{
		namespace GraphAnalyzerViewPrivate
		{
			using FIterateOutputFunctionRef = TFunctionRef<void(const FConstOutputHandle&, const FAnalyzerAddress& AnalyzerAddress)>;
			void IterateOutputsSupportingAnalyzer(const FConstGraphHandle& GraphHandle, uint64 InInstanceID, FName InAnalyzerName, bool bInRequiresConnection, FIterateOutputFunctionRef InFunc)
			{
				const IVertexAnalyzerFactory* Factory = IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(InAnalyzerName);
				if (!Factory)
				{
					return;
				}

				const FName AnalyzerDataType = Factory->GetDataType();
				GraphHandle->IterateConstNodes([&](FConstNodeHandle NodeHandle)
				{
					if (NodeHandle->GetClassMetadata().GetType() != EMetasoundFrontendClassType::Output)
					{
						NodeHandle->IterateConstOutputs([&](FConstOutputHandle OutputHandle)
						{
							const bool bInclude = !bInRequiresConnection || (OutputHandle->IsConnected() && bInRequiresConnection);
							if (bInclude && OutputHandle->GetDataType() == AnalyzerDataType)
							{
								FAnalyzerAddress AnalyzerAddress;
								AnalyzerAddress.DataType = AnalyzerDataType;
								AnalyzerAddress.InstanceID = InInstanceID;
								AnalyzerAddress.NodeID = NodeHandle->GetID();
								AnalyzerAddress.OutputName = OutputHandle->GetName();
								AnalyzerAddress.AnalyzerName = InAnalyzerName;
								AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();

								InFunc(OutputHandle, AnalyzerAddress);
							}
						});
					}
				});
			}
		} // namespace GraphAnalyzerViewPrivate

		FMetasoundGraphAnalyzerView::FMetasoundGraphAnalyzerView(const FMetasoundAssetBase& InAssetBase, uint64 InInstanceID, const FOperatorSettings& InOperatorSettings)
			: InstanceID(InInstanceID)
			, MetaSoundAsset(&InAssetBase)
			, OperatorSettings(InOperatorSettings)
		{
			// TODO: Mirrored from FMetaSoundParameterTransmitter. Fix here when this is refactored.
			const float DelayTimeInSeconds = 0.1f;
			FSenderInitParams SenderParams { OperatorSettings, DelayTimeInSeconds };

			const FGraphAnalyzerAddress AnalyzerAddress(InInstanceID);
			ActiveAnalyzerSender = FDataTransmissionCenter::Get().RegisterNewSender(AnalyzerAddress, SenderParams);
		}

		FMetasoundGraphAnalyzerView::~FMetasoundGraphAnalyzerView()
		{
			// Only unregister the data channel if we had a sender using that 
			// data channel. This protects against removing the data channel 
			// multiple times. Multiple removals of data channels has caused
			// race conditions between newly created transmitters and transmitters
			// being cleaned up.
			if (ActiveAnalyzerSender.IsValid())
			{
				const FGraphAnalyzerAddress AnalyzerAddress(InstanceID);
				ActiveAnalyzerSender.Reset();
				FDataTransmissionCenter::Get().UnregisterDataChannel(AnalyzerAddress);
			}
		}

		void FMetasoundGraphAnalyzerView::AddAnalyzerForAllSupportedOutputs(FName InAnalyzerName, bool bInRequiresConnection)
		{
			using namespace GraphAnalyzerViewPrivate;

			FConstGraphHandle GraphHandle = GetMetaSoundAssetChecked().GetRootGraphHandle();
			const IVertexAnalyzerFactory* Factory = IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(InAnalyzerName);
			if (!Factory)
			{
				return;
			}

			IterateOutputsSupportingAnalyzer(GraphHandle, InstanceID, InAnalyzerName, bInRequiresConnection, [this, &Factory, &InAnalyzerName](FConstOutputHandle OutputHandle, const FAnalyzerAddress& AnalyzerAddress)
			{
				ActiveAnalyzers.Add(AnalyzerAddress);

				const TArray<FAnalyzerOutput>& AnalyzerOutputs = Factory->GetAnalyzerOutputs();
				for (const FAnalyzerOutput& AnalyzerOutput : AnalyzerOutputs)
				{
					FAnalyzerAddress OutputReceiverAddress = AnalyzerAddress;
					OutputReceiverAddress.AnalyzerMemberName = AnalyzerOutput.Name;
					OutputReceiverAddress.DataType = AnalyzerOutput.DataType;

					FMetasoundGraphAnalyzerOutputKey OutputKey { AnalyzerAddress.NodeID, AnalyzerAddress.OutputName };
					FMetasoundAnalyzerView NewView(MoveTemp(OutputReceiverAddress));
					NewView.BindToAllOutputs(OperatorSettings);
					AnalyzerViews.FindOrAdd(OutputKey).Add(MoveTemp(NewView));
				}
			});

			SendActiveAnalyzers();
		}

		FGuid FMetasoundGraphAnalyzerView::AddAnalyzerForSpecifiedOutput(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, FName InAnalyzerMemberName)
		{
			const IVertexAnalyzerFactory* Factory = IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(InAnalyzerName);
			if (!Factory)
			{
				return {};
			}

			FAnalyzerAddress AnalyzerAddress;
			AnalyzerAddress.DataType = Factory->GetDataType();
			AnalyzerAddress.InstanceID = InstanceID;
			AnalyzerAddress.NodeID = InNodeID;
			AnalyzerAddress.OutputName = InOutputName;
			AnalyzerAddress.AnalyzerName = InAnalyzerName;
			AnalyzerAddress.AnalyzerInstanceID = FGuid::NewGuid();
			AnalyzerAddress.AnalyzerMemberName = InAnalyzerMemberName;
			ActiveAnalyzers.Add(AnalyzerAddress);

			const TArray<FAnalyzerOutput>& AnalyzerOutputs = Factory->GetAnalyzerOutputs();
			for (const FAnalyzerOutput& AnalyzerOutput : AnalyzerOutputs)
			{
				FAnalyzerAddress OutputReceiverAddress = AnalyzerAddress;
				OutputReceiverAddress.AnalyzerMemberName = AnalyzerOutput.Name;
				OutputReceiverAddress.DataType = AnalyzerOutput.DataType;

				FMetasoundGraphAnalyzerOutputKey OutputKey{ AnalyzerAddress.NodeID, AnalyzerAddress.OutputName };
				FMetasoundAnalyzerView NewView(MoveTemp(OutputReceiverAddress));
				NewView.BindToAllOutputs(OperatorSettings);
				AnalyzerViews.FindOrAdd(OutputKey).Add(MoveTemp(NewView));
			}

			SendActiveAnalyzers();

			return AnalyzerAddress.AnalyzerInstanceID;
		}

		TArray<FMetasoundAnalyzerView*> FMetasoundGraphAnalyzerView::GetAnalyzerViews(FName InAnalyzerName)
		{
			TArray<FMetasoundAnalyzerView*> ViewsToReturn;

			for (TPair<FMetasoundGraphAnalyzerOutputKey, TArray<FMetasoundAnalyzerView>>& Pair : AnalyzerViews)
			{
				for (FMetasoundAnalyzerView& View : Pair.Value)
				{
					if (View.AnalyzerAddress.AnalyzerName == InAnalyzerName)
					{
						ViewsToReturn.Add(&View);
					}
				}
			}

			return ViewsToReturn;
		}

		TArray<const FMetasoundAnalyzerView*> FMetasoundGraphAnalyzerView::GetAnalyzerViews(FName InAnalyzerName) const
		{
			TArray<const FMetasoundAnalyzerView*> ViewsToReturn;

			for (const TPair<FMetasoundGraphAnalyzerOutputKey, TArray<FMetasoundAnalyzerView>>& Pair : AnalyzerViews)
			{
				for (const FMetasoundAnalyzerView& View : Pair.Value)
				{
					if (View.AnalyzerAddress.AnalyzerName == InAnalyzerName)
					{
						ViewsToReturn.Add(&View);
					}
				}
			}

			return ViewsToReturn;
		}

		TArray<FMetasoundAnalyzerView*> FMetasoundGraphAnalyzerView::GetAnalyzerViewsForOutput(const FGuid& InNodeID, FName InOutputName, FName InAnalyzerName)
		{
			TArray<FMetasoundAnalyzerView*> ViewsToReturn;

			const FMetasoundGraphAnalyzerOutputKey OutputKey { InNodeID, InOutputName };
			if (TArray<FMetasoundAnalyzerView>* OutputAnalyzerViews = AnalyzerViews.Find(OutputKey))
			{
				for (FMetasoundAnalyzerView& View : *OutputAnalyzerViews)
				{
					if (View.AnalyzerAddress.AnalyzerName == InAnalyzerName)
					{
						ViewsToReturn.Add(&View);
					}
				}
			}

			return ViewsToReturn;
		}

		TArray<const FMetasoundAnalyzerView*> FMetasoundGraphAnalyzerView::GetAnalyzerViewsForOutput(const FGuid& InNodeID, FName InOutputName, FName InAnalyzerName) const
		{
			TArray<const FMetasoundAnalyzerView*> ViewsToReturn;

			const FMetasoundGraphAnalyzerOutputKey OutputKey { InNodeID, InOutputName };
			if (const TArray<FMetasoundAnalyzerView>* OutputAnalyzerViews = AnalyzerViews.Find(OutputKey))
			{
				for (const FMetasoundAnalyzerView& View : *OutputAnalyzerViews)
				{
					if (View.AnalyzerAddress.AnalyzerName == InAnalyzerName)
					{
						ViewsToReturn.Add(&View);
					}
				}
			}

			return ViewsToReturn;
		}

		void FMetasoundGraphAnalyzerView::RemoveAnalyzerForAllSupportedOutputs(FName InAnalyzerName)
		{
			TArray<FMetasoundGraphAnalyzerOutputKey> OutputsToRemove;
			for (TPair<FMetasoundGraphAnalyzerOutputKey, TArray<FMetasoundAnalyzerView>>& AnalyzerPair : AnalyzerViews)
			{
				TArray<FMetasoundAnalyzerView>& Views = AnalyzerPair.Value;
				Views.RemoveAll([InAnalyzerName](const FMetasoundAnalyzerView& View)
				{
					return View.AnalyzerAddress.AnalyzerName == InAnalyzerName;
				});

				if (Views.IsEmpty())
				{
					OutputsToRemove.Add(AnalyzerPair.Key);
				}
			}

			for (const FMetasoundGraphAnalyzerOutputKey& Key : OutputsToRemove)
			{
				AnalyzerViews.Remove(Key);
			}

			TArray<FAnalyzerAddress> AnalyzerAddresses = ActiveAnalyzers.Array();
			for (const FAnalyzerAddress& AnalyzerAddress : AnalyzerAddresses)
			{
				if (AnalyzerAddress.AnalyzerName == InAnalyzerName)
				{
					ActiveAnalyzers.Remove(AnalyzerAddress);
					break;
				}
			}

			SendActiveAnalyzers();
		}

		void FMetasoundGraphAnalyzerView::RemoveAnalyzerInstance(FName InAnalyzerName, const FGuid& InAnalyzerInstanceID)
		{
			TArray<FMetasoundGraphAnalyzerOutputKey> OutputsToRemove;
			for (TPair<FMetasoundGraphAnalyzerOutputKey, TArray<FMetasoundAnalyzerView>>& AnalyzerPair : AnalyzerViews)
			{
				TArray<FMetasoundAnalyzerView>& Views = AnalyzerPair.Value;
				Views.RemoveAll([InAnalyzerName, InAnalyzerInstanceID](const FMetasoundAnalyzerView& View)
					{
						return View.AnalyzerAddress.AnalyzerName == InAnalyzerName && View.AnalyzerAddress.AnalyzerInstanceID == InAnalyzerInstanceID;
					});

				if (Views.IsEmpty())
				{
					OutputsToRemove.Add(AnalyzerPair.Key);
				}
			}

			for (const FMetasoundGraphAnalyzerOutputKey& Key : OutputsToRemove)
			{
				AnalyzerViews.Remove(Key);
			}

			TArray<FAnalyzerAddress> AnalyzerAddresses = ActiveAnalyzers.Array();
			for (const FAnalyzerAddress& AnalyzerAddress : AnalyzerAddresses)
			{
				if (AnalyzerAddress.AnalyzerName == InAnalyzerName && AnalyzerAddress.AnalyzerInstanceID == InAnalyzerInstanceID)
				{
					ActiveAnalyzers.Remove(AnalyzerAddress);
					break;
				}
			}

			SendActiveAnalyzers();
		}

		bool FMetasoundGraphAnalyzerView::HasAnalyzerInstance(FName InAnalyzerName, const FGuid& InAnalyzerInstanceID) const
		{
			for (const FAnalyzerAddress& AnalyzerAddress : ActiveAnalyzers)
			{
				if (AnalyzerAddress.AnalyzerName == InAnalyzerName && AnalyzerAddress.AnalyzerInstanceID == InAnalyzerInstanceID)
				{
					return true;
				}
			}

			return false;
		}

		const FMetasoundAssetBase& FMetasoundGraphAnalyzerView::GetMetaSoundAssetChecked() const
		{
			check(MetaSoundAsset);
			return *MetaSoundAsset;
		}

		void FMetasoundGraphAnalyzerView::SendActiveAnalyzers()
		{
			TArray<FString> ActiveAnalyzerStrings;
			Algo::Transform(ActiveAnalyzers, ActiveAnalyzerStrings, [](const FAnalyzerAddress& Address) { return Address.ToString(); });
			ActiveAnalyzerSender->PushLiteral(MoveTemp(ActiveAnalyzerStrings));
		}
	} // namespace Frontend
} // namespace Metasound
