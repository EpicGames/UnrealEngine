// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundFrontendVertexAnalyzerAudioBuffer.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"


namespace Metasound
{
	namespace Frontend
	{
		const FAnalyzerOutput& FVertexAnalyzerAudioBuffer::FOutputs::GetValue()
		{
			static FAnalyzerOutput Value = { "AudioBuffer", GetMetasoundDataTypeName<FAudioBuffer>() };
			return Value;
		}

		const FName& FVertexAnalyzerAudioBuffer::GetAnalyzerName()
		{
			static const FName AnalyzerName = "UE.Audio.AudioBuffer"; return AnalyzerName;
		}

		const FName& FVertexAnalyzerAudioBuffer::GetDataType()
		{
			return GetMetasoundDataTypeName<FAudioBuffer>();
		}

		FVertexAnalyzerAudioBuffer::FVertexAnalyzerAudioBuffer(const FCreateAnalyzerParams& InParams)
			: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
			, AudioBuffer(TDataWriteReference<FAudioBuffer>::CreateNew())
		{
			FVertexAnalyzerBase::BindOutputData<FAudioBuffer>(FOutputs::GetValue().Name, InParams.OperatorSettings, FAudioBufferReadRef(AudioBuffer));
		}

		void FVertexAnalyzerAudioBuffer::Execute()
		{
			*AudioBuffer = GetVertexData<FAudioBuffer>();
			MarkOutputDirty();
		}
	} // namespace Frontend
} // namespace Metasound
