// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundFrontendVertexAnalyzerEnvelopeFollower.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"


namespace Metasound
{
	namespace Frontend
	{
		const FAnalyzerOutput& FVertexAnalyzerEnvelopeFollower::FOutputs::GetValue()
		{
			static FAnalyzerOutput Value = { "EnvelopeValue", GetMetasoundDataTypeName<float>() };
			return Value;
		}

		const FName& FVertexAnalyzerEnvelopeFollower::GetAnalyzerName()
		{
			static const FName AnalyzerName = "UE.Audio.EnvelopeFollower"; return AnalyzerName;
		}

		const FName& FVertexAnalyzerEnvelopeFollower::GetDataType()
		{
			return GetMetasoundDataTypeName<FAudioBuffer>();
		}

		FVertexAnalyzerEnvelopeFollower::FVertexAnalyzerEnvelopeFollower(const FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, EnvelopeValue(TDataWriteReference<float>::CreateNew())
		{
			Audio::FEnvelopeFollowerInitParams Params;
			Params.Mode = Audio::EPeakMode::RootMeanSquared;
			Params.SampleRate = InParams.OperatorSettings.GetSampleRate();
			Params.NumChannels = 1;
			Params.AttackTimeMsec = 10;
			Params.ReleaseTimeMsec = 10;
			EnvelopeFollower.Init(Params);

			FVertexAnalyzerBase::BindOutputData<float>(FOutputs::GetValue().Name, InParams.OperatorSettings, TDataReadReference<float>(EnvelopeValue));
		}

		void FVertexAnalyzerEnvelopeFollower::Execute()
		{
			const FAudioBuffer& AudioBuffer = GetVertexData<FAudioBuffer>();
			EnvelopeFollower.ProcessAudio(AudioBuffer.GetData(), AudioBuffer.Num());

			check(EnvelopeFollower.GetEnvelopeValues().Num() == 1);
			*EnvelopeValue = EnvelopeFollower.GetEnvelopeValues().Last();

			MarkOutputDirty();
		}
	} // namespace Frontend
} // namespace Metasound
