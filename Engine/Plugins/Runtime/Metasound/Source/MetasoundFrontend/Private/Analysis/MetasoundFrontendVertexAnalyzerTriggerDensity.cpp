// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerDensity.h"

#include "MetasoundDataReference.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"
#include "MetasoundTrigger.h"


namespace Metasound
{
	namespace Frontend
	{
		const FAnalyzerOutput& FVertexAnalyzerTriggerDensity::FOutputs::GetValue()
		{
			static const FAnalyzerOutput Value = { "TriggerDensity", GetMetasoundDataTypeName<float>() };
			return Value;
		}

		const FName& FVertexAnalyzerTriggerDensity::GetAnalyzerName()
		{
			static const FName AnalyzerName = "UE.Trigger.Density";
			return AnalyzerName;
		}

		const FName& FVertexAnalyzerTriggerDensity::GetDataType()
		{
			return GetMetasoundDataTypeName<FTrigger>();
		}

		FVertexAnalyzerTriggerDensity::FVertexAnalyzerTriggerDensity(const FCreateAnalyzerParams& InParams)
			: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
			, EnvelopeValue(TDataWriteReference<float>::CreateNew())
			, NumFramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
		{
			Audio::FEnvelopeFollowerInitParams Params;
			Params.Mode = Audio::EPeakMode::Peak;
			Params.SampleRate = InParams.OperatorSettings.GetSampleRate();
			Params.NumChannels = 1;
			Params.AttackTimeMsec = 0;
			Params.ReleaseTimeMsec = 120;
			EnvelopeFollower.Init(Params);

			FVertexAnalyzerBase::BindOutputData<float>(FOutputs::GetValue().Name, InParams.OperatorSettings, TDataReadReference<float>(EnvelopeValue));
		}

		FVertexAnalyzerTriggerDensity::~FVertexAnalyzerTriggerDensity() = default;

		void FVertexAnalyzerTriggerDensity::Execute()
		{
			const FTrigger& Trigger = GetVertexData<FTrigger>();

			ScratchBuffer.Reset();
			ScratchBuffer.AddZeroed(NumFramesPerBlock);
			for (int32 i = 0; i < Trigger.Num(); ++i)
			{
				const int32 TriggerIndex = Trigger[i];
				// Can trigger in the future so ignore those beyond the current block
				if (TriggerIndex < ScratchBuffer.Num())
				{
					ScratchBuffer[TriggerIndex] = 1.0f;
				}
			}

			EnvelopeFollower.ProcessAudio(ScratchBuffer.GetData(), ScratchBuffer.Num());

			check(EnvelopeFollower.GetEnvelopeValues().Num() == 1);
			*EnvelopeValue = EnvelopeFollower.GetEnvelopeValues().Last();

			MarkOutputDirty();
		}
	} // namespace Frontend
} // namespace Metasound
