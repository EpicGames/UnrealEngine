// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/MetasoundFrontendVertexAnalyzerTriggerToTime.h"

namespace Metasound
{
	namespace Frontend
	{
		const FName& FVertexAnalyzerTriggerToTime::GetAnalyzerName()
		{
			static const FName AnalyzerName = "UE.Trigger.ToTime";
			return AnalyzerName;
		}

		const FName& FVertexAnalyzerTriggerToTime::GetDataType()
		{
			return GetMetasoundDataTypeName<FTrigger>();
		}

		const FAnalyzerOutput& FVertexAnalyzerTriggerToTime::FOutputs::GetValue()
		{
			static FAnalyzerOutput Value = { "LastTriggerTime", GetMetasoundDataTypeName<FTime>() };
			return Value;
		}

		const TArray<FAnalyzerOutput>& FVertexAnalyzerTriggerToTime::FFactory::GetAnalyzerOutputs() const
		{
			static const TArray<FAnalyzerOutput> Outputs { FOutputs::GetValue() };
			return Outputs;
		}

		FVertexAnalyzerTriggerToTime::FVertexAnalyzerTriggerToTime(const FCreateAnalyzerParams& InParams)
		: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
		, SampleRate(InParams.OperatorSettings.GetSampleRate())
		, FramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
		, SecondsPerBlock(static_cast<double>(FramesPerBlock) / SampleRate)
		, LastTriggerTime(TDataWriteReference<FTime>::CreateNew())
		{
			BindOutputData(FOutputs::GetValue().Name, InParams.OperatorSettings, TDataReadReference{ LastTriggerTime });
		}

		void FVertexAnalyzerTriggerToTime::Execute()
		{
			const FTrigger& Triggers = GetVertexData<FTrigger>();

			Triggers.ExecuteBlock(
				[](int32, int32) {},
				[this](const int32 TriggerFrame, int32)
				{
					const double OffsetSeconds = static_cast<double>(TriggerFrame) / SampleRate;
					*LastTriggerTime = FTime::FromSeconds(OffsetSeconds + BlockStartTime);
					MarkOutputDirty();
				});

			BlockStartTime += SecondsPerBlock;
		}
	}
}
