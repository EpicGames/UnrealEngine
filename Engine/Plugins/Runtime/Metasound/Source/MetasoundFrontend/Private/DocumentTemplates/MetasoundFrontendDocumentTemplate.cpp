// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendSearchEngine.h"


#define LOCTEXT_NAMESPACE "MetaSound"

bool FMetaSoundFrontendDocumentTemplate::ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound::Frontend;

	TSharedRef<FDocumentModifyDelegates> ModifyDelegates = MakeShared<FDocumentModifyDelegates>(OutBuilder.GetDocumentDelegates());
	OutBuilder.ClearDocument(ModifyDelegates);

#if WITH_EDITORONLY_DATA
	OutBuilder.SetIsGraphEditable(false);
#endif // WITH_EDITORONLY_DATA

	// Add required interfaces, but disable editing to ensure document maintains valid state
	// (ex. MetaSound sources need  the source interface to properly initialize a null-rendering
	// MetaSound generator.)
	TArray<FMetasoundFrontendVersion> InitVersions = ISearchEngine::Get().FindUClassDefaultInterfaceVersions(OutBuilder.GetBuilderClassPath());
	FModifyInterfaceOptions Options({ }, MoveTemp(InitVersions));
	const bool bInterfacesModified = OutBuilder.ModifyInterfaces(MoveTemp(Options));
	ensureMsgf(bInterfacesModified, TEXT("Failed to apply default interfaces when configuring default MetaSound template"));

	return true;
}

#if WITH_EDITOR
Metasound::Frontend::FOnConfigPropertyChangedDelegate& FMetaSoundFrontendDocumentTemplate::GetPropertyChangedDelegate() const
{
	return PropertyChangedDelegate;
}

const FMetaSoundFrontendDocumentTemplate::FEditorOptions& FMetaSoundFrontendDocumentTemplate::FEditorOptions::GetDefaultOptions()
{
	static const FEditorOptions Default;
	return Default;
}

const FMetaSoundFrontendDocumentTemplate::FEditorOptions& FMetaSoundFrontendDocumentTemplate::GetEditorOptions() const
{
	return FEditorOptions::GetDefaultOptions();
}

EDataValidationResult FMetaSoundFrontendDocumentTemplate::IsDataValid(const FMetaSoundFrontendDocumentBuilder& Builder, FDataValidationContext& InOutContext) const
{
	return EDataValidationResult::Valid;
}

void FMetaSoundFrontendDocumentTemplate::OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
}

void FMetaSoundFrontendDocumentTemplate::OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	PropertyChangedDelegate.Broadcast(InEvent);
}
#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE
