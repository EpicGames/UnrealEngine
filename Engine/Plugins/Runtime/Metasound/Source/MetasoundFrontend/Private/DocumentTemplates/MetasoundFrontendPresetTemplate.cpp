// Copyright Epic Games, Inc. All Rights Reserved.
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"

#include "Algo/AnyOf.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "MetasoundAssetManager.h"
#endif // WITH_EDITOR

#include "MetasoundDataTypeGetTraits.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundGlobals.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetaSound"

namespace Metasound::Frontend
{
	bool bEnforcePresetParentReferenceable = false;
	FAutoConsoleVariableRef CVarMetaSoundEnforcePresetParentReferenceable(
		TEXT("au.MetaSound.EnforcePresetParentReferenceable"),
		bEnforcePresetParentReferenceable,
		TEXT("If 'true', FMetaSoundFrontendPresetTemplate ignorse reference when configuring document "
		"(i.e. leaving graph implementation empty) and fails to pass data validation when the parent "
		"class lacks the Referenceable access flag. Defaults to 'false' to preserve cook validity of "
		"pre-existing assets."),
		ECVF_Default);

	namespace PresetDocTemplatePrivate
	{
		// This likely should be deprecated in favor of using template data directly
		void BroadcastInputInheritanceUpdated(const FMetaSoundFrontendDocumentBuilder& Builder, const FGuid& InNodeID)
		{
			auto IsNode = [&InNodeID](const FMetasoundFrontendClassInput& Input) { return Input.NodeID == InNodeID; };
			const int32 Index = Builder.GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs.IndexOfByPredicate(IsNode);
			const_cast<FMetaSoundFrontendDocumentBuilder&>(Builder).GetDocumentDelegates()
				.InterfaceDelegates.OnInputInheritsDefaultChanged.Broadcast(Index);
		}

		bool IsInheritableInputNode(const FMetaSoundFrontendDocumentBuilder& Builder, const FGuid& NodeID)
		{
			const FMetasoundFrontendNode* Node = Builder.FindNode(NodeID);
			if (!Node)
			{
				return false;
			}

			const FMetasoundFrontendClass* Class = Builder.FindDependency(Node->ClassID);
			if (!Class)
			{
				return false;
			}

			if (Class->Metadata.GetType() != EMetasoundFrontendClassType::Input)
			{
				return false;
			}

			const FMetasoundFrontendClassInput* Input = Builder.FindGraphInput(Node->Name);
			if (!Input)
			{
				return false;
			}

			auto IsDefaultConstructed = [](const FMetasoundFrontendClassInputDefault& Default) { return Default.Literal.GetType() == EMetasoundFrontendLiteralType::None; };
			if (Algo::AnyOf(Input->GetDefaults(), IsDefaultConstructed))
			{
				return false;
			}

			if (Input->TypeName == GetMetasoundDataTypeName<FTrigger>())
			{
				return false;
			}

			return true;
		}
	} // namespace PresetDocTemplatePrivate
} // namespace Metasound::Frontend

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendPresetTemplate::ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound::Frontend;

	if (UObject* RefObject = Parent.GetObject())
	{
		if (RefObject->IsA(&OutBuilder.GetConstDocumentInterfaceChecked().GetBaseMetaSoundUClass()))
		{
			const FMetaSoundFrontendDocumentBuilder& ParentBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Parent);

			// Gated by CVar (off by default) to avoid invalidating existing project content that already references parents with the flag cleared.
			if (bEnforcePresetParentReferenceable)
			{
				const EMetasoundFrontendClassAccessFlags ParentAccessFlags = ParentBuilder.GetConstDocumentChecked().RootGraph.Metadata.GetAccessFlags();
				if (!EnumHasAnyFlags(ParentAccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
				{
					UE_LOGF(LogMetaSound, Warning,
						"Preset '%ls' configuration failed: parent class '%ls' lacks the '%ls' access flag.",
						*OutBuilder.GetDebugName(),
						*RefObject->GetName(),
						*LexToString(EMetasoundFrontendClassAccessFlags::Referenceable));
					return Super::ConfigureDocument(OutBuilder);
				}
			}

			OutBuilder.SetIsGraphEditable(false);

			// Apply root graph transform
			FRebuildPresetRootGraph RebuildPresetRootGraph(ParentBuilder);
			return RebuildPresetRootGraph.Transform(OutBuilder);
		}
	}

	// Need to configure using parent template call when reference is invalid as by default, no referenced parent is set and
	// document must be valid when default constructed for testing purposes.  Data Validation provides place to surface error
	// (i.e. when saving or reserializing).
	return Super::ConfigureDocument(OutBuilder);
}

EDataValidationResult FMetaSoundFrontendPresetTemplate::IsDataValid(const FMetaSoundFrontendDocumentBuilder& Builder, FDataValidationContext& InOutContext) const
{
	using namespace Metasound::Frontend;

	EDataValidationResult Result = EDataValidationResult::Valid;
	if (UObject* RefObject = Parent.GetObject())
	{
		if (!RefObject->IsA(&Builder.GetConstDocumentInterfaceChecked().GetBaseMetaSoundUClass()))
		{
			Result = EDataValidationResult::Invalid;
			InOutContext.AddError(FText::Format(LOCTEXT("MetaSoundPreset_InvalidReferenceTypeFormat", "The referenced node type must match the base MetaSound class when converting "
				"builder '{0}' to a preset (ex. source preset must reference another source)"),
				FText::FromString(Builder.GetDebugName())
			));
		}
		// Gated by CVar (off by default) to avoid failing cook on pre-existing assets whose parent
		// classes do not have the flag set.
		else if (bEnforcePresetParentReferenceable)
		{
			if (const IMetaSoundDocumentInterface* ParentDocInterface = Cast<IMetaSoundDocumentInterface>(RefObject))
			{
				const EMetasoundFrontendClassAccessFlags ParentAccessFlags = ParentDocInterface->GetConstDocument().RootGraph.Metadata.GetAccessFlags();
				if (!EnumHasAnyFlags(ParentAccessFlags, EMetasoundFrontendClassAccessFlags::Referenceable))
				{
					Result = EDataValidationResult::Invalid;
					InOutContext.AddError(FText::Format(
						LOCTEXT("MetaSoundPreset_ParentNotReferenceableFormat",
							"Preset '{0}' references parent '{1}' whose class lacks the Referenceable access flag."),
						FText::FromString(Builder.GetDebugName()),
						FText::FromString(RefObject->GetName())));
				}
			}
		}
	}
	else
	{
		Result = EDataValidationResult::Invalid;
		InOutContext.AddError(FText::Format(LOCTEXT("MetaSoundPreset_ReferenceUnsetFormat", "Preset '%s' generation failed: Referenced class not set."), FText::FromString(Builder.GetDebugName())));
	}

	return Result;
}
#endif // WITH_EDITORONLY_DATA

TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> FMetaSoundFrontendPresetTemplate::ConstructVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) const
{
	using namespace Metasound::Frontend;

	// Presets only need ConfigVertexMetadata for inputs, so don't construct otherwise
	if (PresetDocTemplatePrivate::IsInheritableInputNode(Builder, InVertex.NodeID))
	{
		return TInstancedStruct<FPresetVertexMetadata>::Make();
	}

	return { };
}

#if WITH_EDITOR
const FMetaSoundFrontendDocumentTemplate::FEditorOptions& FMetaSoundFrontendPresetTemplate::GetEditorOptions() const
{
	static const FEditorOptions PresetOptions
	{
		.bMemberEditingEnabled = true,
		.bGraphEditorVisible = true
	};
	return PresetOptions;
}

void FMetaSoundFrontendPresetTemplate::OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound::Frontend;

	for (UObject* SelectedObject : SelectedObjects)
	{
		if (SelectedObject)
		{
			if (&OutBuilder.GetConstDocumentInterfaceChecked().GetBaseMetaSoundUClass() == SelectedObject->GetClass())
			{
				Parent = SelectedObject;

				if (FMetasoundAssetBase* ParentAsset = IMetaSoundAssetManager::GetChecked().GetAsAsset(*SelectedObject))
				{
					const bool bCanExecute = Metasound::CanEverExecuteGraph();
					if (bCanExecute)
					{
						ParentAsset->UpdateAndRegisterForExecution(IMetaSoundAssetManager::GetChecked().GetRegistrationOptions());
					}
				}
				return;
			}
		}
	}
}

void FMetaSoundFrontendPresetTemplate::OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder)
{
	using namespace Metasound::Frontend;

	if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundFrontendPresetTemplate, Parent))
	{
		if (UObject* ParentObject = Parent.GetObject())
		{
			if (FMetasoundAssetBase* ParentAsset = IMetaSoundAssetManager::GetChecked().GetAsAsset(*ParentObject))
			{
				const bool bCreatesLoop = OutBuilder.GetMetasoundAsset().AddingReferenceCausesLoop(*ParentAsset);
				if (bCreatesLoop)
				{
					UE_LOGF(LogMetaSound, Error, "Parent could not be set to '%ls': Causes reference loop.", *ParentObject->GetFullName());
					Parent = nullptr;
				}
				else
				{
					const bool bCanExecute = Metasound::CanEverExecuteGraph();
					if (bCanExecute)
					{
						ParentAsset->UpdateAndRegisterForExecution(IMetaSoundAssetManager::GetChecked().GetRegistrationOptions());
					}
				}
			}
		}
	}
}

bool FMetaSoundFrontendPresetVertexMetadata::IsDefaultEditable() const
{
	return bOverrideInheritedDefault;
}
#endif // WITH_EDITOR

void FMetaSoundFrontendPresetVertexMetadata::OnDefaultReset(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex)
{
	using namespace Metasound::Frontend;

	bOverrideInheritedDefault = false;
	PresetDocTemplatePrivate::BroadcastInputInheritanceUpdated(const_cast<FMetaSoundFrontendDocumentBuilder&>(Builder), InVertex.NodeID);
}

void FMetaSoundFrontendPresetVertexMetadata::OnDefaultUpdated(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex)
{
	using namespace Metasound::Frontend;

	// When actively configuring the document, ignore calls to defaults being
	// updated as this means the request isn't being applied by a user/client code.
	if (!Builder.IsConfiguringDocument())
	{
		bOverrideInheritedDefault = true;
		PresetDocTemplatePrivate::BroadcastInputInheritanceUpdated(const_cast<FMetaSoundFrontendDocumentBuilder&>(Builder), InVertex.NodeID);
	}
}
#undef LOCTEXT_NAMESPACE
