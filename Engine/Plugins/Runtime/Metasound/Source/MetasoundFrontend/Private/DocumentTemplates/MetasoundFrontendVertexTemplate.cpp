// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentTemplates/MetasoundFrontendDocumentVertexTemplate.h"

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"


#define LOCTEXT_NAMESPACE "MetaSound"

FMetaSoundFrontendDocumentTemplateVertexMetadata* FMetaSoundFrontendDocumentVertexTemplate::AddVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex)
{
	if (TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>* Entry = VertexMetadata.Find(InVertex.NodeID))
	{
		return Entry->GetMutablePtr();
	}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> NewMetadata = ConstructVertexMetadata(Builder, InVertex);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	if (NewMetadata.IsValid())
	{
		return VertexMetadata.Add(InVertex.NodeID, MoveTemp(NewMetadata)).GetMutablePtr();
	}

	return nullptr;
}

TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> FMetaSoundFrontendDocumentVertexTemplate::ConstructVertexMetadata(
	const FMetaSoundFrontendDocumentBuilder& Builder,
	const FMetasoundFrontendClassVertex& InVertex) const
{
	return { };
}

bool FMetaSoundFrontendDocumentVertexTemplate::ContainsVertexMetadata(const FGuid& NodeID) const
{
	return VertexMetadata.Contains(NodeID);
}

bool FMetaSoundFrontendDocumentVertexTemplate::RemoveVertexMetadata(const FGuid& NodeID)
{
	return VertexMetadata.Remove(NodeID) > 0;
}
#undef LOCTEXT_NAMESPACE
