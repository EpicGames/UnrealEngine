// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundFrontendInterface.h"

#include "Algo/ForEach.h"
#include "MetasoundFrontendDocument.h"


namespace Metasound
{
	namespace Frontend
	{
		namespace InterfacePrivate
		{
			bool IsLessThanVertex(const FMetasoundFrontendVertex& VertexA, const FMetasoundFrontendVertex& VertexB)
			{
				// For interfaces we can ignore metadata when comparing
				if (VertexA.Name == VertexB.Name)
				{
					return VertexA.TypeName.FastLess(VertexB.TypeName);
				}
				return VertexA.Name.FastLess(VertexB.Name);
			}

			bool IsLessThanEnvironmentVariable(const FMetasoundFrontendClassEnvironmentVariable& EnvironmentA, const FMetasoundFrontendClassEnvironmentVariable& EnvironmentB)
			{
				// For interfaces we can ignore metadata when comparing
				if (EnvironmentA.Name == EnvironmentB.Name)
				{
					return EnvironmentA.TypeName.FastLess(EnvironmentB.TypeName);
				}
				return EnvironmentA.Name.FastLess(EnvironmentB.Name);
			}

			bool IsEquivalentEnvironmentVariable(const FMetasoundFrontendClassEnvironmentVariable& EnvironmentA, const FMetasoundFrontendClassEnvironmentVariable& EnvironmentB)
			{
				// For interfaces we can ignore display name and tooltip
				return (EnvironmentA.Name == EnvironmentB.Name) && (EnvironmentA.TypeName == EnvironmentB.TypeName);
			}

			template<typename Type>
			TArray<const Type*> MakePointerToArrayElements(const TArray<Type>& InArray)
			{
				TArray<const Type*> PointerArray;
				for (const Type& Element : InArray)
				{
					PointerArray.Add(&Element);
				}
				return PointerArray;
			}

			template<typename FrontendTypeA, typename FrontendTypeB, typename ComparisonType>
			void SetDifference(const TArray<FrontendTypeA>& InSetA, const TArray<FrontendTypeB>& InSetB, TArray<const FrontendTypeA*>& OutUniqueA, TArray<const FrontendTypeB*>& OutUniqueB, ComparisonType Compare)
			{
				TArray<const FrontendTypeA*> SortedA = MakePointerToArrayElements(InSetA);
				SortedA.Sort(Compare);

				TArray<const FrontendTypeB*> SortedB = MakePointerToArrayElements(InSetB);
				SortedB.Sort(Compare);

				int32 IndexA = 0;
				int32 IndexB = 0;

				while ((IndexA < SortedA.Num()) && (IndexB < SortedB.Num()))
				{
					if (Compare(*SortedA[IndexA], *SortedB[IndexB]))
					{
						OutUniqueA.Add(SortedA[IndexA]);
						IndexA++;
					}
					else if (Compare(*SortedB[IndexB], *SortedA[IndexA]))
					{
						OutUniqueB.Add(SortedB[IndexB]);
						IndexB++;
					}
					else
					{
						// Equal. Increment both indices.
						IndexA++;
						IndexB++;
					}
				}

				if (IndexA < SortedA.Num())
				{
					OutUniqueA.Append(&SortedA[IndexA], SortedA.Num() - IndexA);
				}

				if (IndexB < SortedB.Num())
				{
					OutUniqueB.Append(&SortedB[IndexB], SortedB.Num() - IndexB);
				}
			}

			template<typename FrontendTypeA, typename FrontendTypeB, typename ComparisonType>
			bool IsSetEquivalent(const TArray<FrontendTypeA>& InSetA, const TArray<FrontendTypeB>& InSetB, ComparisonType Compare)
			{
				TArray<const FrontendTypeA*> UniqueA;
				TArray<const FrontendTypeB*> UniqueB;

				SetDifference(InSetA, InSetB, UniqueA, UniqueB, Compare);

				return (UniqueA.Num() == 0) && (UniqueB.Num() == 0);
			}
			
			template<typename FrontendTypeA, typename FrontendTypeB, typename ComparisonType>
			bool IsSetIncluded(const TArray<FrontendTypeA>& InSubset, const TArray<FrontendTypeB>& InSuperset, ComparisonType Compare)
			{
				TArray<const FrontendTypeA*> UniqueSubset;
				TArray<const FrontendTypeB*> UniqueSuperset;

				SetDifference(InSubset, InSuperset, UniqueSubset, UniqueSuperset, Compare);

				return UniqueSubset.Num() == 0;
			}

			template<typename FrontendTypeA, typename FrontendTypeB, typename ComparisonType>
			int32 SetDifferenceCount(const TArray<FrontendTypeA>& InSetA, const TArray<FrontendTypeB>& InSetB, ComparisonType Compare)
			{
				TArray<const FrontendTypeA*> UniqueA;
				TArray<const FrontendTypeB*> UniqueB;

				SetDifference(InSetA, InSetB, UniqueA, UniqueB, Compare);

				return UniqueA.Num() + UniqueB.Num();
			}
		} // namespace InterfacePrivate

		bool IsSubsetOfInterface(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendInterface& InSupersetInterface)
		{
			using namespace InterfacePrivate;

			const bool bIsInputSetIncluded = IsSetIncluded(InSubsetInterface.Inputs, InSupersetInterface.Inputs, &IsLessThanVertex);

			const bool bIsOutputSetIncluded = IsSetIncluded(InSubsetInterface.Outputs, InSupersetInterface.Outputs, &IsLessThanVertex);

			const bool bIsEnvironmentVariableSetIncluded = IsSetIncluded(InSubsetInterface.Environment, InSupersetInterface.Environment, &IsLessThanEnvironmentVariable);

			return bIsInputSetIncluded && bIsOutputSetIncluded && bIsEnvironmentVariableSetIncluded;
		}

		bool IsSubsetOfClass(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendClass& InSupersetClass)
		{
			return IsSubsetOfClassInterface(InSubsetInterface, InSupersetClass.GetDefaultInterface());
		}

		bool IsSubsetOfClassInterface(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendClassInterface& InSupersetClassInterface)
		{
			// TODO: Environment variables are ignored as they are poorly supported and classes describe which environment variables are required,
			// not which are supported 
			using namespace InterfacePrivate;

			const bool bIsInputSetIncluded = IsSetIncluded(InSubsetInterface.Inputs, InSupersetClassInterface.Inputs, &IsLessThanVertex);
			const bool bIsOutputSetIncluded = IsSetIncluded(InSubsetInterface.Outputs, InSupersetClassInterface.Outputs, &IsLessThanVertex);

			return bIsInputSetIncluded && bIsOutputSetIncluded;
		}

		bool IsEquivalentInterface(const FMetasoundFrontendInterface& InInputInterface, const FMetasoundFrontendInterface& InTargetInterface)
		{
			using namespace InterfacePrivate;

			const bool bIsInputSetEquivalent = IsSetEquivalent(InTargetInterface.Inputs, InInputInterface.Inputs, &IsLessThanVertex);
			const bool bIsOutputSetEquivalent = IsSetEquivalent(InTargetInterface.Outputs, InInputInterface.Outputs, &IsLessThanVertex);
			const bool bIsEnvironmentVariableSetEquivalent = IsSetEquivalent(InTargetInterface.Environment, InInputInterface.Environment, &IsLessThanEnvironmentVariable);

			return bIsInputSetEquivalent && bIsOutputSetEquivalent && bIsEnvironmentVariableSetEquivalent;
		}

		bool IsEqualInterface(const FMetasoundFrontendInterface& InInputInterface, const FMetasoundFrontendInterface& InTargetInterface)
		{
			const bool bIsMetadataEqual = InInputInterface.Metadata.Version == InTargetInterface.Metadata.Version;
			if (bIsMetadataEqual)
			{
				// If metadata is equal, then inputs/outputs/environment variables should also be equal. 
				// Consider updating interface version info if this ensure is hit.
				return ensure(IsEquivalentInterface(InInputInterface, InTargetInterface));
			}

			return false;
		}

		bool DeclaredInterfaceVersionsMatch(const FMetasoundFrontendDocument& InDocumentA, const FMetasoundFrontendDocument& InDocumentB)
		{
			if (InDocumentA.Interfaces.Num() != InDocumentB.Interfaces.Num())
			{
				return false;
			}

			return InDocumentA.Interfaces.Includes(InDocumentB.Interfaces);
		}

		int32 InputOutputDifferenceCount(const FMetasoundFrontendClass& InClass, const FMetasoundFrontendInterface& InInterface)
		{
			return InputOutputDifferenceCount(InClass.GetDefaultInterface(), InInterface);
		}

		int32 InputOutputDifferenceCount(const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendInterface& InInterface)
		{
			using namespace InterfacePrivate;

			int32 DiffCount = SetDifferenceCount(InClassInterface.Inputs, InInterface.Inputs, &IsLessThanVertex);
			DiffCount += SetDifferenceCount(InClassInterface.Outputs, InInterface.Outputs, &IsLessThanVertex);

			return DiffCount;
		}

		int32 InputOutputDifferenceCount(const FMetasoundFrontendInterface& InInterfaceA, const FMetasoundFrontendInterface& InInterfaceB)
		{
			using namespace InterfacePrivate;

			int32 DiffCount = SetDifferenceCount(InInterfaceA.Inputs, InInterfaceB.Inputs, &IsLessThanVertex);
			DiffCount += SetDifferenceCount(InInterfaceA.Outputs, InInterfaceB.Outputs, &IsLessThanVertex);

			return DiffCount;
		}

		void GatherRequiredEnvironmentVariables(const FMetasoundFrontendGraphClass& InRootGraph, const TArray<FMetasoundFrontendClass>& InDependencies, const TArray<FMetasoundFrontendGraphClass>& InSubgraphs, TArray<FMetasoundFrontendClassEnvironmentVariable>& OutEnvironmentVariables)
		{
			auto GatherRequiredEnvironmentVariablesFromClass = [&](const FMetasoundFrontendClass& InClass)
			{
				using namespace InterfacePrivate;

				// Environment variables are maintained on the default interface. Changing the environment variables in a
				// class interface override is not supported. 
				for (const FMetasoundFrontendClassEnvironmentVariable& EnvVar : InClass.GetDefaultInterface().Environment)
				{
					if (EnvVar.bIsRequired)
					{
						auto IsEquivalentEnvVar = [&](const FMetasoundFrontendClassEnvironmentVariable& OtherEnvVar)
						{
							return IsEquivalentEnvironmentVariable(EnvVar, OtherEnvVar);
						};

						// Basically same as TArray::AddUnique except uses the `IsEquivalentEnvironmentVariable` instead of `operator==` to test for uniqueness.
						if (nullptr == OutEnvironmentVariables.FindByPredicate(IsEquivalentEnvVar))
						{
							OutEnvironmentVariables.Add(EnvVar);
						}
					}
				}
			};

			GatherRequiredEnvironmentVariablesFromClass(InRootGraph);
			Algo::ForEach(InDependencies, GatherRequiredEnvironmentVariablesFromClass);
			Algo::ForEach(InSubgraphs, GatherRequiredEnvironmentVariablesFromClass);
		}

		const FMetasoundFrontendInterface* FindMostSimilarInterfaceSupportingEnvironment(const FMetasoundFrontendGraphClass& InRootGraph, const TArray<FMetasoundFrontendClass>& InDependencies, const TArray<FMetasoundFrontendGraphClass>& InSubgraphs, const TArray<FMetasoundFrontendInterface>& InCandidateInterfaces)
		{
			using namespace InterfacePrivate;

			const FMetasoundFrontendInterface* Result = nullptr;

			TArray<const FMetasoundFrontendInterface*> CandidateInterfaces = MakePointerToArrayElements(InCandidateInterfaces);

			TArray<FMetasoundFrontendClassEnvironmentVariable> AllEnvironmentVariables;
			GatherRequiredEnvironmentVariables(InRootGraph, InDependencies, InSubgraphs, AllEnvironmentVariables);

			// Remove all interfaces which do not provide all environment variables.
			auto DoesNotProvideAllEnvironmentVariables = [&](const FMetasoundFrontendInterface* CandidateInterface)
			{
				return !IsSetIncluded(AllEnvironmentVariables, CandidateInterface->Environment, &IsLessThanEnvironmentVariable);
			};

			CandidateInterfaces.RemoveAll(DoesNotProvideAllEnvironmentVariables);

			// Return the interface with the least amount of differences.
			auto DifferencesFromRootGraph = [&](const FMetasoundFrontendInterface* CandidateInterface) -> int32
			{ 
				return InputOutputDifferenceCount(InRootGraph.GetDefaultInterface(), *CandidateInterface);
			};

			Algo::SortBy(CandidateInterfaces, DifferencesFromRootGraph);

			if (0 == CandidateInterfaces.Num())
			{
				return nullptr;
			}

			return CandidateInterfaces[0];
		}
		
		const FMetasoundFrontendInterface* FindMostSimilarInterfaceSupportingEnvironment(const FMetasoundFrontendDocument& InDocument, const TArray<FMetasoundFrontendInterface>& InCandidateInterfaces)
		{
			return FindMostSimilarInterfaceSupportingEnvironment(InDocument.RootGraph, InDocument.Dependencies, InDocument.Subgraphs, InCandidateInterfaces);
		}
	} // namespace Frontend
} // namespace Metasound
