// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundFrontendInterfaceBindingRegistry.h"

#include "HAL/PlatformTime.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceBindingRegistryPrivate.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundTrace.h"


namespace Metasound::Frontend
{
	FInterfaceBindingRegistryEntry::FInterfaceBindingRegistryEntry(const FMetasoundFrontendInterfaceBinding& InBinding)
		: OutputInterfaceVersion(InBinding.OutputInterfaceVersion)
		, VertexBindings(InBinding.VertexBindings)
		, BindingPriority(InBinding.BindingPriority)
	{
	}

	const FMetasoundFrontendVersion& FInterfaceBindingRegistryEntry::GetOutputInterfaceVersion() const
	{
		return OutputInterfaceVersion;
	}

	const TArray<FMetasoundFrontendInterfaceVertexBinding>& FInterfaceBindingRegistryEntry::GetVertexBindings() const
	{
		return VertexBindings;
	}

	int32 FInterfaceBindingRegistryEntry::GetBindingPriority() const
	{
		return BindingPriority;
	}

	bool FInterfaceBindingRegistry::FindInterfaceBindingEntries(const FMetasoundFrontendVersion& InInputInterfaceVersion, TArray<const FInterfaceBindingRegistryEntry*>& OutEntries) const
	{
		TArray<const TUniquePtr<FInterfaceBindingRegistryEntry>*> FoundEntries;
		Entries.MultiFindPointer(InInputInterfaceVersion, FoundEntries);

		OutEntries.Reset();
		Algo::Transform(FoundEntries, OutEntries, [](const TUniquePtr<FInterfaceBindingRegistryEntry>* FoundEntry)
		{
			const FInterfaceBindingRegistryEntry* EntryPtr = FoundEntry->Get();
			return EntryPtr;
		});

		return !OutEntries.IsEmpty();
	}

	void FInterfaceBindingRegistry::RegisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, TUniquePtr<FInterfaceBindingRegistryEntry>&& InEntry)
	{
		Entries.Add(InInputInterfaceVersion, MoveTemp(InEntry));
	}

	bool FInterfaceBindingRegistry::UnregisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, const FMetasoundFrontendVersion& InOutputInterfaceVersion)
	{
		TArray<const TUniquePtr<FInterfaceBindingRegistryEntry>*> FoundEntries;
		Entries.MultiFindPointer(InInputInterfaceVersion, FoundEntries);

		for (const TUniquePtr<FInterfaceBindingRegistryEntry>* EntryPtr : FoundEntries)
		{
			const FInterfaceBindingRegistryEntry* Entry = (*EntryPtr).Get();
			check(Entry);
			if (Entry->GetOutputInterfaceVersion() == InOutputInterfaceVersion)
			{
				if (Entries.RemoveSingle(InInputInterfaceVersion, *EntryPtr) > 0)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool FInterfaceBindingRegistry::UnregisterAllInterfaceBindings(const FMetasoundFrontendVersion& InInputInterfaceVersion)
	{
		return Entries.Remove(InInputInterfaceVersion) > 0;
	}

	IInterfaceBindingRegistry& IInterfaceBindingRegistry::Get()
	{
		static FInterfaceBindingRegistry Registry;
		return Registry;
	}
} // namespace Metasound::Frontend
