// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Interfaces/MetasoundFrontendInterfaceBindingRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "UObject/NoExportTypes.h"


namespace Metasound::Frontend
{
	class FInterfaceBindingRegistry : public IInterfaceBindingRegistry
	{
	public:
		FInterfaceBindingRegistry() = default;

		virtual ~FInterfaceBindingRegistry() = default;

		virtual bool FindInterfaceBindingEntries(const FMetasoundFrontendVersion& InInputInterfaceVersion, TArray<const FInterfaceBindingRegistryEntry*>& OutEntries) const override;

		virtual void RegisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, TUniquePtr<FInterfaceBindingRegistryEntry>&& InEntry) override;

		virtual bool UnregisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, const FMetasoundFrontendVersion& InOutputInterfaceVersion) override;

		virtual bool UnregisterAllInterfaceBindings(const FMetasoundFrontendVersion& InInputInterfaceVersion) override;

	private:
		// MultiMap of interface bindings keyed by the shared input interface version for an array of connection-priority sorted bindings
		TMultiMap<FMetasoundFrontendVersion, TUniquePtr<FInterfaceBindingRegistryEntry>> Entries;
	};
} // namespace Metasound::Frontend
