// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"

#include "AudioParameter.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/MetasoundFrontendInterfaceBindingRegistry.h"
#include "MetasoundFrontendInterfaceRegistryPrivate.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundTrace.h"
#include "Misc/ScopeLock.h"


#ifndef UE_METASOUND_ENABLE_INTERFACE_VALIDATION
	#define UE_METASOUND_ENABLE_INTERFACE_VALIDATION 0
#endif // !define UE_METASOUND_ENABLE_INTERFACE_VALIDATION

namespace Metasound::Frontend
{
	FInterfaceRegistry::FInterfaceRegistry()
	: TransactionBuffer(MakeShared<TTransactionBuffer<FInterfaceRegistryTransaction>>())
	{
	}

	bool FInterfaceRegistry::RegisterInterface(TUniquePtr<IInterfaceRegistryEntry>&& InEntry)
	{
		METASOUND_LLM_SCOPE;

		FInterfaceRegistryTransaction::FTimeType TransactionTime = FPlatformTime::Cycles64();
		if (InEntry.IsValid())
		{
			const FMetasoundFrontendVersion& Version = InEntry->GetInterface().Metadata.Version;
			if (ContainsInterface(Version))
			{
				UE_LOGF(LogMetaSound, Warning, "Registration of interface overwriting previously registered interface [%ls]", *Version.ToString());
						
				FInterfaceRegistryTransaction Transaction { FInterfaceRegistryTransaction::ETransactionType::InterfaceUnregistration, Version, TransactionTime };
				TransactionBuffer->AddTransaction(MoveTemp(Transaction));
			}

#if UE_METASOUND_ENABLE_INTERFACE_VALIDATION
			// Don't run vertex name validation warning for deprecated registry entries. Some of these may be
			// versioning schema that have a subsequent version or versions fixing the very problem this log is reporting.
			if (!InEntry->IsDeprecated())
			{
				const FName InterfaceNamespace = InEntry->GetInterface().Version.Name;
				auto LogIfMismatch = [this, &InterfaceNamespace](FName VertexName)
				{
					FName VertexNamespace;
					if (!IsInterfaceVertexNameValid(InterfaceNamespace, VertexName, &VertexNamespace))
					{
						UE_LOGF(LogMetaSound, Warning, "Interface '%ls' contains vertex '%ls' with mismatched namespace '%ls': "
							"All interface-defined vertices' must start with matching interface namespace (See AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE/AUDIO_PARAMETER_INTERFACE_NAMESPACE macro to ensure convention is followed). "
							"Failing to fix relationship via interface versioning will fail validation/cook in future builds.", *InterfaceNamespace.ToString(), *VertexName.ToString(), *VertexNamespace.ToString());
					}
				};

				for (const FMetasoundFrontendClassVertex& Vertex : InEntry->GetInterface().Inputs)
				{
					LogIfMismatch(Vertex.Name);
				}

				for (const FMetasoundFrontendClassVertex& Vertex : InEntry->GetInterface().Outputs)
				{
					LogIfMismatch(Vertex.Name);
				}
			}
#endif // UE_METASOUND_ENABLE_INTERFACE_VALIDATION

			FInterfaceRegistryTransaction Transaction{FInterfaceRegistryTransaction::ETransactionType::InterfaceRegistration, Version, TransactionTime};
			TransactionBuffer->AddTransaction(MoveTemp(Transaction));

			{
				UE::TScopeLock Lock(EntriesCriticalSection);
				Entries.Add(Version, MoveTemp(InEntry));
			}
			return true;
		}

		return false;
	}

	bool FInterfaceRegistry::ContainsInput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InInput) const
	{
		auto IsVertex = [&InInput](const FMetasoundFrontendVertex& InterfaceInput)
		{
			return FMetasoundFrontendVertex::IsFunctionalEquivalent(InInput, InterfaceInput);
		};

		{
			UE::TScopeLock Lock(EntriesCriticalSection);
			if (const TUniquePtr<IInterfaceRegistryEntry>* Entry = Entries.Find(InVersion))
			{
				check(Entry);
				return (*Entry)->GetInterface().Inputs.ContainsByPredicate(IsVertex);
			}
		}

		return false;
	}

	bool FInterfaceRegistry::ContainsInterface(const FMetasoundFrontendVersion& InVersion) const
	{
		UE::TScopeLock Lock(EntriesCriticalSection);
		return Entries.Contains(InVersion);
	}

	bool FInterfaceRegistry::ContainsOutput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InOutput) const
	{
		auto IsVertex = [&InOutput](const FMetasoundFrontendVertex& InterfaceInput)
		{
			return FMetasoundFrontendVertex::IsFunctionalEquivalent(InOutput, InterfaceInput);
		};

		{
			UE::TScopeLock Lock(EntriesCriticalSection);
			if (const TUniquePtr<IInterfaceRegistryEntry>* Entry = Entries.Find(InVersion))
			{
				check(Entry);
				return (*Entry)->GetInterface().Outputs.ContainsByPredicate(IsVertex);
			}
		}

		return false;
	}

	bool FInterfaceRegistry::FindInterface(const FMetasoundFrontendVersion& InVersion, FMetasoundFrontendInterface& OutInterface) const
	{
		UE::TScopeLock Lock(EntriesCriticalSection);
		if (const TUniquePtr<IInterfaceRegistryEntry>* Entry = Entries.Find(InVersion))
		{
			OutInterface = Entry->Get()->GetInterface();
			return true;
		}

		return false;
	}

	bool FInterfaceRegistry::FindInterfaceRouter(const FMetasoundFrontendVersion& InVersion, FName& OutRouter) const
	{
		UE::TScopeLock Lock(EntriesCriticalSection);
		if (const TUniquePtr<IInterfaceRegistryEntry>* Entry = Entries.Find(InVersion); Entry && Entry->IsValid())
		{
			OutRouter = (*Entry)->GetRouterName();
			return true;
		}

		return false;
	}

	TSharedPtr<IBuilderVersionTransform> FInterfaceRegistry::FindInterfaceUpdateTransform(const FMetasoundFrontendVersion& InVersion) const
	{
		UE::TScopeLock Lock(EntriesCriticalSection);
		if (const TUniquePtr<IInterfaceRegistryEntry>* Entry = Entries.Find(InVersion); Entry && Entry->IsValid())
		{
			return (*Entry)->GetUpdateTransform();
		}

		return { };
	}

	TUniquePtr<FInterfaceTransactionStream> FInterfaceRegistry::CreateTransactionStream()
	{
		return MakeUnique<FInterfaceTransactionStream>(TransactionBuffer);
	}

	bool IsValidInterfaceRegistryKey(const FInterfaceRegistryKey& InKey)
	{
		return !InKey.IsEmpty();
	}

	FInterfaceRegistryKey GetInterfaceRegistryKey(const FMetasoundFrontendVersion& InInterfaceVersion)
	{
		return FString::Format(TEXT("{0}_{1}.{2}"), { InInterfaceVersion.Name.ToString(), InInterfaceVersion.Number.Major, InInterfaceVersion.Number.Minor });

	}

	FInterfaceRegistryKey GetInterfaceRegistryKey(const FMetasoundFrontendInterface& InInterface)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetInterfaceRegistryKey(InInterface.Metadata.Version);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FInterfaceRegistryTransaction::FInterfaceRegistryTransaction(ETransactionType InType, const FInterfaceRegistryKey&, const FMetasoundFrontendVersion& InInterfaceVersion, FInterfaceRegistryTransaction::FTimeType InTimestamp)
	: Type(InType)
	, InterfaceVersion(InInterfaceVersion)
	, Timestamp(InTimestamp)
	{
	}

	FInterfaceRegistryTransaction::FInterfaceRegistryTransaction(ETransactionType InType, const FMetasoundFrontendVersion& InInterfaceVersion, FInterfaceRegistryTransaction::FTimeType InTimestamp)
		: Type(InType)
		, InterfaceVersion(InInterfaceVersion)
		, Timestamp(InTimestamp)
	{
	}

	FInterfaceRegistryTransaction::ETransactionType FInterfaceRegistryTransaction::GetTransactionType() const
	{
		return Type;
	}

	const FMetasoundFrontendVersion& FInterfaceRegistryTransaction::GetInterfaceVersion() const
	{
		return InterfaceVersion;
	}

	const FInterfaceRegistryKey& FInterfaceRegistryTransaction::GetInterfaceRegistryKey() const
	{
		static const FInterfaceRegistryKey InvalidKey;
		return InvalidKey;
	}

	FInterfaceRegistryTransaction::FTimeType FInterfaceRegistryTransaction::GetTimestamp() const
	{
		return Timestamp;
	}

	FInterfaceRegistry& FInterfaceRegistry::Get()
	{
		static FInterfaceRegistry Registry;
		return Registry;
	}

	bool FInterfaceRegistry::IsInterfaceVertexNameValid(FName InterfaceNamespace, FName FullVertexName, FName* VertexNamespace) const
	{
		FName Namespace;
		FName Name;
		Audio::FParameterPath::SplitName(FullVertexName, Namespace, Name);
		if (VertexNamespace)
		{
			*VertexNamespace = Namespace;
		}
		return InterfaceNamespace == Namespace;
	}

	IInterfaceRegistry& IInterfaceRegistry::Get()
	{
		return FInterfaceRegistry::Get();
	}
} // namespace Metasound::Frontend
