// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/NoExportTypes.h"
#include "UObject/TopLevelAssetPath.h"


namespace Metasound::Frontend
{
	using FInterfaceTransactionStream = TTransactionStream<FInterfaceRegistryTransaction>;

	class FInterfaceRegistry : public IInterfaceRegistry
	{
	public:
		static FInterfaceRegistry& Get();

		FInterfaceRegistry();

		virtual ~FInterfaceRegistry() = default;

		virtual bool ContainsInput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InInput) const override;
		virtual bool ContainsInterface(const FMetasoundFrontendVersion& InVersion) const override;
		virtual bool ContainsOutput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InOutput) const override;

		virtual bool FindInterface(const FMetasoundFrontendVersion& InVersion, FMetasoundFrontendInterface& OutInterface) const override;
		virtual bool FindInterfaceRouter(const FMetasoundFrontendVersion& InVersion, FName& OutRouter) const override;
		virtual TSharedPtr<IBuilderVersionTransform> FindInterfaceUpdateTransform(const FMetasoundFrontendVersion& InVersion) const override;
		virtual bool RegisterInterface(TUniquePtr<IInterfaceRegistryEntry>&& InEntry) override;

		TUniquePtr<FInterfaceTransactionStream> CreateTransactionStream();

	private:
		bool IsInterfaceVertexNameValid(FName InterfaceNamespace, FName FullVertexName, FName* VertexNamespace) const;

		using FInterfaceTransactionBuffer = TTransactionBuffer<FInterfaceRegistryTransaction>;

		mutable FTransactionallySafeCriticalSection EntriesCriticalSection;
		TMap<FMetasoundFrontendVersion, TUniquePtr<IInterfaceRegistryEntry>> Entries;

		TSharedRef<FInterfaceTransactionBuffer> TransactionBuffer;
	};
} // namespace Metasound::Frontend
