// Copyright Epic Games, Inc. All Rights Reserved.
#include "Interfaces/MetasoundFrontendSourceInterface.h"

#include "AudioParameter.h"
#include "MetasoundDataReference.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "Metasound"

namespace Metasound::Frontend
{
	namespace SourceInterfacePrivate
	{
		TArray<Audio::FParameterInterface::FClassOptions> GetDefaultSourceClassOptions(const FTopLevelAssetPath& InClassPath, bool bIsModifiable)
		{
			constexpr bool bIsDefault = true;
			return TArray<Audio::FParameterInterface::FClassOptions>
			{
				{ InClassPath, bIsModifiable, bIsDefault }
			};
		}
	}

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source.OneShot"
	namespace SourceOneShotInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Outputs
		{
			const FName OnFinished = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OnFinished");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(SourceOneShotInterface::GetVersion().Name, SourceOneShotInterface::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = true;
					UClassOptions = SourceInterfacePrivate::GetDefaultSourceClassOptions(InClassPath, bIsModifiable);

					Outputs =
					{
						{
							LOCTEXT("OnFinished", "On Finished"),
							LOCTEXT("OnFinishedDescription", "Trigger executed to initiate stopping the source."),
							GetMetasoundDataTypeName<FTrigger>(),
							Outputs::OnFinished,
							LOCTEXT("OnFinishedWarning", "\"On Finished\" should be connected for OneShot MetaSound sources. For sources with undefined duration (e.g. looping), remove the OneShot interface and use an audio component to avoid leaking the source."),
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Source.OneShot.Description",
						"For MetaSounds that are not looping nor infinite. Provides an output trigger to tell when the MetaSound is finished.");
#endif
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}
	} // namespace SourceOneShotInterface

#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source.StartTime"
	namespace SourceStartTimeInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName StartTime = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("StartTime");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(SourceStartTimeInterface::GetVersion().Name, SourceStartTimeInterface::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = true;
					constexpr bool bIsDefault = false;
					UClassOptions = TArray<FClassOptions>
					{
						{ InClassPath, bIsModifiable, bIsDefault }
					};

					Inputs =
					{
						{
							.DisplayName = LOCTEXT("StartTime", "Start Time"),
							.Description = LOCTEXT("StartTime Description", "[0, Duration] The StartTime of the source passed into the MetaSound from the Gameplay API."),
							.DataType = GetMetasoundDataTypeName<FTime>(),
							.InitValue = FAudioParameter(Inputs::StartTime, 0.f)
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Source.StartTime.Description",
						"Provides the start time (from 0) to seek into the sound as its played. Works from Gameplay API and Sequencer.\n"
						"The MetaSound itself will need to determine what to do with this input.\n"
						"In the simplest case, a MetaSound with a single Wave Player can route StartTime into the Wave Player's StartTime.");
#endif
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}
	} // namespace SourceStartTimeInterface

#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE


#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source"
	namespace SourceInterfaceV1_0
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 0 } };
			return Version;
		}

		namespace Inputs
		{
			const FName OnPlay = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OnPlay");
		}

		namespace Outputs
		{
			const FName OnFinished = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OnFinished");
		}

		namespace Environment
		{
			const FName DeviceID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("AudioDeviceID");
			const FName GraphName = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("GraphName");
			const FName IsPreview = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("IsPreviewSound");
			const FName SoundUniqueID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("SoundUniqueID");
			const FName TransmitterID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("TransmitterID");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(SourceInterfaceV1_0::GetVersion().Name, SourceInterfaceV1_0::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = false;
					constexpr bool bIsDefault = false;
					UClassOptions = TArray<FClassOptions>
					{
						{ InClassPath, bIsModifiable, bIsDefault }
					};

					Inputs =
					{
						{
							LOCTEXT("OnPlay", "On Play"),
							LOCTEXT("OnPlayDescription", "Trigger executed when the source is played."),
							GetMetasoundDataTypeName<FTrigger>(),
							{ Inputs::OnPlay, false }
						}
					};

					Outputs =
					{
						{
							LOCTEXT("OnFinished", "On Finished"),
							LOCTEXT("OnFinishedDescription", "Trigger executed to initiate stopping the source."),
							GetMetasoundDataTypeName<FTrigger>(),
							Outputs::OnFinished
						}
					};

					Environment =
					{
						{
							LOCTEXT("AudioDeviceIDDisplayName", "Audio Device ID"),
							LOCTEXT("AudioDeviceIDDescription", "ID of AudioDevice source is played from."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint32)
							Environment::DeviceID
						},
						{
							LOCTEXT("GraphNameDisplayName", "Graph Name"),
							LOCTEXT("GraphNameDescription", "Name of source graph (for debugging/logging)."),
							GetMetasoundDataTypeName<FString>(),
							Environment::GraphName
						},
						{
							LOCTEXT("IsPreviewSoundDisplayName", "Is Preview Sound"),
							LOCTEXT("IsPreviewSoundDescription", "Whether source is being played as a previewed sound."),
							GetMetasoundDataTypeName<bool>(),
							Environment::IsPreview
						},
						{
							LOCTEXT("TransmitterIDDisplayName", "Transmitter ID"),
							LOCTEXT("TransmitterIDDescription", "ID used by Transmission System to generate a unique send address for each source instance."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint64)
							Environment::TransmitterID
						},
						{
							LOCTEXT("SoundUniqueIdDisplayName", "Sound Unique ID"),
							LOCTEXT("SoundUniqueIdDescription", "ID of unique source instance."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint32)
							Environment::SoundUniqueID
						}
					};
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}
	} // namespace SourceInterfaceV1_0

	namespace SourceInterface
	{
		const FMetasoundFrontendVersion& GetVersion()
		{
			static const FMetasoundFrontendVersion Version = { AUDIO_PARAMETER_INTERFACE_NAMESPACE, { 1, 1 } };
			return Version;
		}

		namespace Inputs
		{
			const FName OnPlay = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("OnPlay");
		}

		namespace Environment
		{
			const FName DeviceID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("AudioDeviceID");
			const FName GraphName = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("GraphName");
			const FName IsPreview = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("IsPreviewSound");
			const FName SoundUniqueID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("SoundUniqueID");
			const FName TransmitterID = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("TransmitterID");
			const FName AudioMixerNumOutputFrames = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("AudioMixerNumOutputFrames");
			const FName AudioMixerSampleRate = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("AudioMixerSampleRate");
			const FName SourceFormatName = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("SourceFormatName");
		}

		Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass)
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface(const FTopLevelAssetPath& InClassPath)
					: FParameterInterface(SourceInterface::GetVersion().Name, SourceInterface::GetVersion().Number.ToInterfaceVersion())
				{
					constexpr bool bIsModifiable = false;
					UClassOptions = SourceInterfacePrivate::GetDefaultSourceClassOptions(InClassPath, bIsModifiable);

					Inputs =
					{
						{
							LOCTEXT("OnPlay", "On Play"),
							LOCTEXT("OnPlayDescription", "Trigger executed when the source is played."),
							GetMetasoundDataTypeName<FTrigger>(),
							{ Inputs::OnPlay, false }
						}
					};

					Environment =
					{
						{
							LOCTEXT("AudioDeviceIDDisplayName1", "Audio Device ID"),
							LOCTEXT("AudioDeviceIDDescription2", "ID of AudioDevice source is played from."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint32)
							Environment::DeviceID
						},
						{
							LOCTEXT("GraphNameDisplayName", "Graph Name"),
							LOCTEXT("AudioDeviceIDDescription3", "Name of source graph (for debugging/logging)."),
							GetMetasoundDataTypeName<FString>(),
							Environment::GraphName
						},
						{
							LOCTEXT("IsPreviewSoundDisplayName", "Is Preview Sound"),
							LOCTEXT("IsPreviewSoundDescription4", "Whether source is being played as a previewed sound."),
							GetMetasoundDataTypeName<bool>(),
							Environment::IsPreview
						},
						{
							LOCTEXT("TransmitterIDDisplayName", "Transmitter ID"),
							LOCTEXT("TransmitterIDDescription", "ID used by Transmission System to generate a unique send address for each source instance."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint64)
							Environment::TransmitterID
						},
						{
							LOCTEXT("SoundUniqueDisplayName", "Sound Unique ID"),
							LOCTEXT("SoundUniqueDescription", "ID of unique source instance."),
							FName(), // TODO: Align environment data types with environment (ex. this is actually set/get as a uint32)
							Environment::SoundUniqueID
						},
						{
							LOCTEXT("AudioMixerOutputFramesDisplayName", "Audio Mixer Output Frames"),
							LOCTEXT("AudioMixerOutputFramesDescription", "The number of output frames rendered per buffer in the audio mixer."),
							FName(), 
							Environment::AudioMixerNumOutputFrames
						},
						{
							LOCTEXT("AudioMixerSampleRateDisplayName", "Audio Mixer Sample Rate"),
							LOCTEXT("AudioMixerSampleRateDescription", "The sample rate of the audio mixer."),
							FName(), 
							Environment::AudioMixerSampleRate
						},
						{
							LOCTEXT("SourceFormatNameDisplayName", "Source Format"),
							LOCTEXT("SourceFormatNameDescription", "The \'Source\' channel agnostic format of the graph"),
							GetMetasoundDataTypeName<FString>(),	// <-- TODO: Metasound can't support FName parameters, so we must pass as a string.
							Environment::SourceFormatName
						}
					};
				}
			};

			return MakeShared<FInterface>(InClass.GetClassPathName());
		}

		bool FUpdateInterface::Transform(Frontend::FDocumentHandle InDocument) const
		{
			using namespace Frontend;

			// When upgrading, we only want to add the one-shot interface if the MetaSound actually has the OnFinished trigger connected.
			bool bIsOnFinishedConnected = false;
			InDocument->GetRootGraph()->IterateConstNodes([&](Frontend::FConstNodeHandle NodeHandle)
			{
				NodeHandle->IterateConstInputs([&](Frontend::FConstInputHandle InputHandle)
				{
					if (InputHandle->GetName() == SourceInterfaceV1_0::Outputs::OnFinished)
					{
						bIsOnFinishedConnected = InputHandle->IsConnected();
					}
				});
			}, EMetasoundFrontendClassType::Output);

			const TArray<FMetasoundFrontendVersion> InterfacesToRemove
			{
				SourceInterfaceV1_0::GetVersion()
			};

			TArray<FMetasoundFrontendVersion> InterfacesToAdd
			{
				SourceInterface::GetVersion()
			};

			if (bIsOnFinishedConnected)
			{
				InterfacesToAdd.Add(SourceOneShotInterface::GetVersion());
			}

			Frontend::FModifyRootGraphInterfaces InterfaceTransform(InterfacesToRemove, InterfacesToAdd);
			return InterfaceTransform.Transform(InDocument);
		}

	} // namespace SourceInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE
} // namespace Metasound::Frontend
#undef LOCTEXT_NAMESPACE
