// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundArrayNodes.h"
#include "Misc/EnumClassFlags.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace ArrayNodeVertexNames
	{
		const FLazyName InputInitialArrayName{"Array"};
#if WITH_EDITOR
		const FText InputInitialArrayTooltip = LOCTEXT("InitialArrayTooltip", "Initial Array");
		const FText InputInitialArrayDisplayName = LOCTEXT("InitialArrayDisplayName", "Init Array"); 
#else
		const FText InputInitialArrayTooltip = FText::GetEmpty();
		const FText InputInitialArrayDisplayName = FText::GetEmpty();
#endif

		DEFINE_METASOUND_PARAM(InputArray, "Array", "Input Array.")
		DEFINE_METASOUND_PARAM(InputLeftArray, "Left Array", "Input Left Array.")
		DEFINE_METASOUND_PARAM(InputRightArray, "Right Array", "Input Right Array.")
		DEFINE_METASOUND_PARAM(InputTriggerGet, "Trigger", "Trigger to get value.")
		DEFINE_METASOUND_PARAM(InputTriggerSet, "Trigger", "Trigger to set value.")
		DEFINE_METASOUND_PARAM(InputIndex, "Index", "Index in Array.")
		DEFINE_METASOUND_PARAM(InputStartIndex, "Start Index", "First index to include.")
		DEFINE_METASOUND_PARAM(InputEndIndex, "End Index", "Last index to include.")
		DEFINE_METASOUND_PARAM(InputValue, "Value", "Value to set.")

		DEFINE_METASOUND_PARAM(OutputNum, "Num", "Number of elements in the array.")
		DEFINE_METASOUND_PARAM(OutputValue, "Element", "Value of element at array index.")
		DEFINE_METASOUND_PARAM(OutputArrayConcat, "Array", "Array after concatenation.")
		DEFINE_METASOUND_PARAM(OutputArraySet, "Array", "Array after setting.")
		DEFINE_METASOUND_PARAM(OutputArraySubset, "Array", "Subset of input array.")
		DEFINE_METASOUND_PARAM(OutputLastIndex, "Last Index", "Last index of the array.")
	}

	namespace MetasoundArrayNodesPrivate
	{
		const FLazyName ArrayNamespace{"Array"};

		const FLazyName ArrayNumOperatorName{"Num"};
		constexpr int32 ArrayNumNodeMajorVersion = 1;
		constexpr int32 ArrayNumNodeMinorVersion = 0;

		const FLazyName ArrayGetOperatorName{"Get"};
		constexpr int32 ArrayGetNodeMajorVersion = 1;
		constexpr int32 ArrayGetNodeMinorVersion = 0;

		const FLazyName ArraySetOperatorName{"Set"};
		constexpr int32 ArraySetNodeMajorVersion = 1;
		constexpr int32 ArraySetNodeMinorVersion = 0;

		const FLazyName ArraySubsetOperatorName{"Subset"};
		constexpr int32 ArraySubsetNodeMajorVersion = 1;
		constexpr int32 ArraySubsetNodeMinorVersion = 0;

		const FLazyName ArrayConcatOperatorName{"Concat"}; 
		constexpr int32 ArrayConcatNodeMajorVersion = 1;
		constexpr int32 ArrayConcatNodeMinorVersion = 0;

		const FLazyName ArrayLastIndexOperatorName{"GetLastIndex"};
		constexpr int32 ArrayLastIndexNodeMajorVersion = 1;
		constexpr int32 ArrayLastIndexNodeMinorVersion = 0;

		FNodeClassName CreateArrayNodeClassName(const FName& InOperatorName, const FName& InDataTypeName)
		{
			return FNodeClassName{ ArrayNamespace, InOperatorName, InDataTypeName };
		}

		FNodeClassMetadata CreateArrayNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface, int32 MajorVersion, int32 MinorVersion, bool bIsDeprecated)
		{
			FNodeClassMetadata Metadata
			{
				CreateArrayNodeClassName(InOperatorName, InDataTypeName),
				MajorVersion, 
				MinorVersion,
				InDisplayName, 
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ METASOUND_LOCTEXT("ArrayCategory", "Array") },
				{ METASOUND_LOCTEXT("MetasoundArrayKeyword", "Array") },
			};

			if (bIsDeprecated)
			{
				EnumAddFlags(Metadata.AccessFlags, ENodeClassAccessFlags::Deprecated);
			}

			return Metadata;
		}

		FVertexInterface CreateArrayNumInterface(const FName& InArrayDataTypeName)
		{
			using namespace ArrayNodeVertexNames;

			return FVertexInterface {
				FInputVertexInterface(
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputArray))
				),
				FOutputVertexInterface(
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputNum))
				)
			};
		}

		FVertexInterface CreateArrayGetInterface(const FName& InArrayDataTypeName, const FName& InDataTypeName)
		{
			using namespace ArrayNodeVertexNames;
			return FVertexInterface {
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerGet)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputArray)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIndex))
				),
				FOutputVertexInterface(
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputValue), InDataTypeName, METASOUND_GET_PARAM_METADATA(OutputValue), EVertexAccessType::Reference)
				)
			};
		}

		FVertexInterface CreateArraySetInterface(const FName& InArrayDataTypeName, const FName& InDataTypeName)
		{
			using namespace ArrayNodeVertexNames;
			return FVertexInterface{ 
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerSet)),
					FInputDataVertex(InputInitialArrayName, InArrayDataTypeName, FDataVertexMetadata { InputInitialArrayTooltip, InputInitialArrayDisplayName }),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIndex)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputValue), InDataTypeName, METASOUND_GET_PARAM_METADATA(InputValue))
				),
				FOutputVertexInterface(
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputArraySet), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(OutputArraySet), EVertexAccessType::Reference)

				)
			};
		}

		FVertexInterface CreateArrayConcatInterface(const FName& InArrayDataTypeName)
		{
			using namespace ArrayNodeVertexNames;

			return FVertexInterface{
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerGet)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputLeftArray), InArrayDataTypeName,METASOUND_GET_PARAM_METADATA(InputLeftArray)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputRightArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputRightArray))
				),
				FOutputVertexInterface(
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputArrayConcat), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(OutputArrayConcat), EVertexAccessType::Reference)
				)
			};
		}

		FVertexInterface CreateArraySubsetInterface(const FName& InArrayDataTypeName)
		{
			using namespace ArrayNodeVertexNames;

			return FVertexInterface {
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerGet)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputArray)),

					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartIndex)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEndIndex))

				),
				FOutputVertexInterface(
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputArraySubset), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(OutputArraySubset), EVertexAccessType::Reference)

				)
			};
		}

		FVertexInterface CreateArrayLastIndexInterface(const FName& InArrayDataTypeName)
		{
			using namespace ArrayNodeVertexNames;

			return FVertexInterface {
				FInputVertexInterface(
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputArray))
				),
				FOutputVertexInterface(
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputLastIndex))
				)
			};
		}

		Frontend::FNodeClassRegistryKey CreateArrayNumNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArrayNumOperatorName, InArrayDataTypeName), ArrayNumNodeMajorVersion, ArrayNumNodeMinorVersion);
		}

		Frontend::FNodeClassRegistryKey CreateArrayGetNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArrayGetOperatorName, InArrayDataTypeName), ArrayGetNodeMajorVersion, ArrayGetNodeMinorVersion);
		}

		Frontend::FNodeClassRegistryKey CreateArraySetNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArraySetOperatorName, InArrayDataTypeName), ArraySetNodeMajorVersion, ArraySetNodeMinorVersion);
		}

		Frontend::FNodeClassRegistryKey CreateArraySubsetNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArraySubsetOperatorName, InArrayDataTypeName), ArraySubsetNodeMajorVersion, ArraySubsetNodeMinorVersion);
		}

		Frontend::FNodeClassRegistryKey CreateArrayConcatNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArrayConcatOperatorName, InArrayDataTypeName), ArrayConcatNodeMajorVersion, ArrayConcatNodeMinorVersion);
		}

		Frontend::FNodeClassRegistryKey CreateArrayLastIndexNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, CreateArrayNodeClassName(ArrayLastIndexOperatorName, InArrayDataTypeName), ArrayLastIndexNodeMajorVersion, ArrayLastIndexNodeMinorVersion);
		}


		FNodeClassMetadata CreateArrayNumNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArrayNumDisplayNamePattern", "Num ({0})", InArrayTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArrayNumDescription", "Number of elements in the array");

			FNodeClassMetadata NodeClassMetadata = MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayNumOperatorName, NodeDisplayName, NodeDescription, CreateArrayNumInterface(InArrayDataTypeName), ArrayNumNodeMajorVersion, ArrayNumNodeMinorVersion);

			NodeClassMetadata.Keywords.Append({
				METASOUND_LOCTEXT("ArrayNumKeyword_Length", "length"),
				METASOUND_LOCTEXT("ArrayNumKeyword_Size", "size"),
			});
			return NodeClassMetadata;
		}


		FNodeClassMetadata CreateArrayGetNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArrayGetDisplayNamePattern", "Get ({0})", InArrayTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArrayGetDescription", "Get element at index in array.");
			const FVertexInterface NodeInterface = MetasoundArrayNodesPrivate::CreateArrayGetInterface(InArrayDataTypeName, InElementDataTypeName);

			return MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayGetOperatorName, NodeDisplayName, NodeDescription, NodeInterface, ArrayGetNodeMajorVersion, ArrayGetNodeMinorVersion);
		}


		FNodeClassMetadata CreateArraySetNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArraySetDisplayNamePattern", "Set ({0})", InArrayTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArraySetDescription", "Set element at index in array.");
			const FVertexInterface NodeInterface = MetasoundArrayNodesPrivate::CreateArraySetInterface(InArrayDataTypeName, InElementDataTypeName);
		
			return MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArraySetOperatorName, NodeDisplayName, NodeDescription, NodeInterface, ArraySetNodeMajorVersion, ArraySetNodeMinorVersion);
		}
		

		FNodeClassMetadata CreateArraySubsetNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArraySubsetDisplayNamePattern", "Subset ({0})", InArrayDataTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArraySubsetDescription", "Subset array on trigger.");
			const FVertexInterface NodeInterface = MetasoundArrayNodesPrivate::CreateArraySubsetInterface(InArrayDataTypeName);
		
			FNodeClassMetadata NodeClassMetadata = MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArraySubsetOperatorName, NodeDisplayName, NodeDescription, NodeInterface, ArraySubsetNodeMajorVersion, ArraySubsetNodeMinorVersion);

			NodeClassMetadata.Keywords.Append({
				METASOUND_LOCTEXT("ArraySubsetKeyword_Slice", "slice"),
			});

			return NodeClassMetadata;
		}

		FNodeClassMetadata CreateArrayConcatNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArrayConcatDisplayNamePattern", "Concatenate ({0})", InArrayDataTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArrayConcatDescription", "Concatenates two arrays on trigger.");
			const FVertexInterface NodeInterface = MetasoundArrayNodesPrivate::CreateArrayConcatInterface(InArrayDataTypeName);
		
			FNodeClassMetadata NodeClassMetadata = MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayConcatOperatorName, NodeDisplayName, NodeDescription, NodeInterface, ArrayConcatNodeMajorVersion, ArrayConcatNodeMinorVersion);

			NodeClassMetadata.Keywords.Append({
				METASOUND_LOCTEXT("ArrayConcatKeyword_Append", "append"),
			});

			return NodeClassMetadata;
		}

		FNodeClassMetadata CreateArrayLastIndexNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArrayLastIndexDisplayNamePattern", "Get Last Index ({0})", InArrayDataTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArrayLastIndexDescription", "Last index of the array");
			const FVertexInterface NodeInterface = MetasoundArrayNodesPrivate::CreateArrayLastIndexInterface(InArrayDataTypeName);
			return MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayLastIndexOperatorName, NodeDisplayName, NodeDescription, NodeInterface, ArrayLastIndexNodeMajorVersion, ArrayLastIndexNodeMinorVersion);
		}
	}
}

#undef LOCTEXT_NAMESPACE
