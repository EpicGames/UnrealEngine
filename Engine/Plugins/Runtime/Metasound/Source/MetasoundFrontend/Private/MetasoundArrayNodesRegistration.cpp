// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundArrayNodesRegistration.h"

#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendRegistryKey.h"

namespace Metasound::Frontend
{
	bool UnregisterNodeIfRegistered(INodeClassRegistry& InRegistry, const FNodeClassRegistryKey& InKey)
	{
		if (InRegistry.IsNodeRegistered(InKey))
		{
			return InRegistry.UnregisterNode(InKey);
		}
		return true;
	}

	bool UnregisterArrayNodes(const FName& InArrayDataTypeName, const FModuleInfo& InModuleInfo)
	{
		using namespace MetasoundArrayNodesPrivate;

		bool bSuccess = true;

		INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();

		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArrayNumNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArrayGetNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArraySetNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArraySubsetNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArrayConcatNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, ArrayShuffleNodePrivate::CreateArrayShuffleNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, ArrayRandomNodePrivate::CreateArrayRandomNodeClassRegistryKey(InArrayDataTypeName));
		bSuccess = bSuccess && UnregisterNodeIfRegistered(NodeRegistry, CreateArrayLastIndexNodeClassRegistryKey(InArrayDataTypeName));

		return bSuccess;
	}
}
