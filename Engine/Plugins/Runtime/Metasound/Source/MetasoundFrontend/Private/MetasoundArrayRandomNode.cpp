// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundArrayRandomNode.h"
#include "Containers/CircularQueue.h"
#include "Internationalization/Text.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"
#include "MetasoundArrayNodes.h"
#include "MetasoundEnumRegistrationMacro.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(ESharedStateBehaviorType, FEnumSharedStateBehaviorType, "SharedStateBehaviorType")
		DEFINE_METASOUND_ENUM_ENTRY(ESharedStateBehaviorType::SameNode, \
			"SameNodeDescription", "Same Node", "SameNodeTT", \
			"State is shared with other instances of this individual node regardless of the MetaSound it is in."),
		DEFINE_METASOUND_ENUM_ENTRY(ESharedStateBehaviorType::SameNodeInComposition, \
			"SameNodeInCompositionDescription", "Same Node in Composition", "SSameNodeInCompositionTT", \
			"State is shared with other instances of this node with the same parent MetaSound graph(s). Useful for differentiating shared state between nodes used in different presets or multiple composed graphs."),
		DEFINE_METASOUND_ENUM_ENTRY(ESharedStateBehaviorType::SameData, \
			"SameDataDescription", "Same Data", "SameDataTT", \
			"State is shared with other nodes with the same input array data (by value) regardless of where the node is located. Useful for sharing state regardless of graph composition or between multiple nodes within a single MetaSound using the same input data. Input array type must implement a hash function.")
	DEFINE_METASOUND_ENUM_END()

	namespace ArrayRandomNodePrivate
	{
		const FLazyName ArrayRandomOperatorName{"Random Get"};
		constexpr int32 ArrayRandomNodeMajorVersion = 1;
		constexpr int32 ArrayRandomNodeMinorVersion = 1;

		Frontend::FNodeClassRegistryKey CreateArrayRandomNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, MetasoundArrayNodesPrivate::CreateArrayNodeClassName(ArrayRandomOperatorName, InArrayDataTypeName), ArrayRandomNodeMajorVersion, ArrayRandomNodeMinorVersion);
		}

		FVertexInterface CreateArrayRandomNodeInterface(const FName& InArrayDataTypeName, const FName& InElementDataTypeName)
		{
			using namespace ArrayNodeRandomGetVertexNames;

			return FVertexInterface {
				FInputVertexInterface{
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerNextValue)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerResetSeed)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputRandomArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputRandomArray)),
					TInputDataVertex<TArray<float>>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputWeights)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSeed), -1),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputNoRepeatOrder), 1),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEnableSharedState), false),
					TInputConstructorVertex<FEnumSharedStateBehaviorType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSharedStateBehavior), (int32)ESharedStateBehaviorType::SameNodeInComposition)
				},
				FOutputVertexInterface{
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnNext)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnReset)),
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(ShuffleOutputValue), InElementDataTypeName, METASOUND_GET_PARAM_METADATA(ShuffleOutputValue), EVertexAccessType::Reference),
					TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputIndex))
				}
			};
		}

		FNodeClassMetadata CreateArrayRandomNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("RandomArrayGetNode_OpDisplayNamePattern", "Random Get ({0})", InArrayDataTypeDisplayText);

			const FText NodeDescription = METASOUND_LOCTEXT("RandomArrayGetNode_Description", "Randomly retrieve data from input array using the supplied weights.");

			return MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayRandomOperatorName, NodeDisplayName, NodeDescription, CreateArrayRandomNodeInterface(InArrayDataTypeName, InElementDataTypeName), ArrayRandomNodeMajorVersion, ArrayRandomNodeMinorVersion);
		}
	}

	FArrayRandomGet::FArrayRandomGet(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder)
	{
		UpdateState(InSeed, InMaxIndex, InWeights, InNoRepeatOrder);
	}

	void FArrayRandomGet::Init(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder)
	{
		UpdateState(InSeed, InMaxIndex, InWeights, InNoRepeatOrder);
	}

	void FArrayRandomGet::UpdateState(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder)
	{
		SetSeed(InSeed);
		MaxIndex = InMaxIndex;
		SetNoRepeatOrder(InNoRepeatOrder);
		check(!InNoRepeatOrder || !PreviousIndicesQueue->IsFull());
		SetRandomWeights(InWeights);
	}

	void FArrayRandomGet::SetSeed(int32 InSeed)
	{
		if (!bRandomStreamInitialized || InSeed != Seed)
		{
			Seed = InSeed;
			if (InSeed == INDEX_NONE)
			{
				RandomStream.Initialize(FPlatformTime::Cycles());
			}
			else
			{
				RandomStream.Initialize(InSeed);
			}

			ResetSeed();
			bRandomStreamInitialized = true;
		}
	}

	void FArrayRandomGet::SetNoRepeatOrder(int32 InNoRepeatOrder)
	{
		// Only set the no repeats if the array length is larger than 2, otherwise, ignore no-repeats as it won't work anyway (it will end up repeating a clear pattern)
		// Make sure the no repeat order is between 0 and half of the max index
		// This is to prevent clear patterns in the output that causes confusion by user. By clamping to half the max index we achieve maximum no-repeats behavior, which is the intent of a user trying to maximize no repeats.
		if (MaxIndex > 2)
		{
			// Check for -1 on no-repeat order. This means to utilize the maximum non-repetition, matched to the array size. This prevents user from having to adjust their no-repeats to match the array size themselves.
			if (InNoRepeatOrder == -1)
			{
				InNoRepeatOrder = MaxIndex / 2;
			}
			else
			{
				InNoRepeatOrder = FMath::Clamp(InNoRepeatOrder, 0, MaxIndex / 2);
			}
		}
		else
		{
			// If we don't have enough indices, we fallback to a no-repeat order of 0
			InNoRepeatOrder = 0;
		}

		if (InNoRepeatOrder != NoRepeatOrder)
		{
			PreviousIndicesQueue = MakeUnique<TCircularQueue<int32>>(InNoRepeatOrder + 2);
			PreviousIndices.Reset();
			NoRepeatOrder = InNoRepeatOrder;
		}
	}

	void FArrayRandomGet::SetRandomWeights(const TArray<float>& InRandomWeights)
	{
		if (InRandomWeights != RandomWeights)
		{
			RandomWeights = InRandomWeights;
		}
	}

	void FArrayRandomGet::ResetSeed()
	{
		RandomStream.Reset();
	}

	float FArrayRandomGet::ComputeTotalWeight()
	{
		float TotalWeight = 0.0f;
		if (RandomWeights.Num() > 0)
		{
			for (int32 i = 0; i < MaxIndex; ++i)
			{
				// If the index exists in previous indices, continue
				if (PreviousIndices.Contains(i))
				{
					continue;
				}

				// We modulus on the weight array to determine weights for the input array
				// I.e. if weights is 2 elements, the weights will alternate in application to the input array
				TotalWeight += RandomWeights[i % RandomWeights.Num()];
			}
		}
		return TotalWeight;
	}

	// Returns the next random weighted value in the array indices. 
	int32 FArrayRandomGet::NextValue()
	{
		// First compute the total size of the weights
		bool bHasWeights = RandomWeights.Num() > 0;
		float TotalWeight = 0.0f;
		if (bHasWeights)
		{
			TotalWeight = ComputeTotalWeight();
			if (TotalWeight == 0.0f && PreviousIndices.Num() > 0)
			{
				PreviousIndices.Reset();
				PreviousIndicesQueue->Empty();
				TotalWeight = ComputeTotalWeight();
			}

			// Weights might have been set with all 0.0s. If that's the case, we treat as if there were no weights set.
			bHasWeights = (TotalWeight > 0.0f);
		}

		if (!bHasWeights)
		{
			TotalWeight = (float)(FMath::Max(MaxIndex - PreviousIndices.Num(), 1));
		}
		check(TotalWeight > 0.0f);
	

		// Make a random choice based on the total weight
		float Choice = RandomStream.FRandRange(0.0f, TotalWeight);
		// Now find the index this choice matches up to
		TotalWeight = 0.0f;
		int32 ChosenIndex = INDEX_NONE;
		for (int32 i = 0; i < MaxIndex; ++i)
		{
			if (PreviousIndices.Contains(i))
			{
				continue;
			}

			float NextTotalWeight = TotalWeight;
			if (bHasWeights)
			{
				check(RandomWeights.Num() > 0);
				NextTotalWeight += RandomWeights[i % RandomWeights.Num()];
			}
			else
			{
				NextTotalWeight += 1.0f;
			}

			if (Choice >= TotalWeight && Choice < NextTotalWeight)
			{
				ChosenIndex = i;
				break;
			}
			TotalWeight = NextTotalWeight;
		}
		check(ChosenIndex != INDEX_NONE);

		// Dequeue and remove the oldest previous index
		if (NoRepeatOrder > 0)
		{
			if (PreviousIndices.Num() == NoRepeatOrder)
			{
				check(PreviousIndicesQueue->Count() == PreviousIndices.Num());
				int32 OldPrevIndex;
				PreviousIndicesQueue->Dequeue(OldPrevIndex);
				PreviousIndices.Remove(OldPrevIndex);
				check(PreviousIndices.Num() == NoRepeatOrder - 1);
			}

			check(PreviousIndices.Num() < NoRepeatOrder);
			check(!PreviousIndicesQueue->IsFull());

			bool bSuccess = PreviousIndicesQueue->Enqueue(ChosenIndex);
			check(bSuccess);
			check(!PreviousIndices.Contains(ChosenIndex));
			PreviousIndices.Add(ChosenIndex);
			check(PreviousIndicesQueue->Count() == PreviousIndices.Num());
		}

		return ChosenIndex;
	}

	FSharedStateRandomGetManager& FSharedStateRandomGetManager::Get()
	{
		static FSharedStateRandomGetManager RGM;
		return RGM;
	}

	inline bool ShouldStompSharedState(const FArrayRandomGet& InRandomGet, const InitSharedStateArgs InArgs)
	{
		return
			InArgs.bIsPreviewSound										// If it's a preview sound
			|| InRandomGet.GetMaxIndex() != InArgs.NumElements			// Size of Array has changed
			|| InRandomGet.GetNoRepeatOrder() != InArgs.NoRepeatOrder;	// Repeat order has changed
	}

	void FSharedStateRandomGetManager::InitSharedState(InitSharedStateArgs& InArgs)
	{
		FScopeLock Lock(&CritSect);

		// Look if there's already shared state. In certain cases we should stomp it by re-adding...
		if (const TUniquePtr<FArrayRandomGet>* FoundExisting = RandomGets.Find(InArgs.SharedStateId))
		{
			if (FoundExisting && FoundExisting->IsValid())
			{
				const FArrayRandomGet& Existing = *FoundExisting->Get();
				if (!ShouldStompSharedState(Existing, InArgs))
				{
					return;
				}
			}
		}

		// Add state/Stomp existing.
		RandomGets.Add(InArgs.SharedStateId, MakeUnique<FArrayRandomGet>(InArgs.Seed, InArgs.NumElements, InArgs.Weights, InArgs.NoRepeatOrder));
	}

	int32 FSharedStateRandomGetManager::NextValue(const FGuid& InSharedStateId)
	{
		FScopeLock Lock(&CritSect);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		return (*RG)->NextValue();
	}

	int32 FSharedStateRandomGetManager::NextValue(const FGuid& InSharedStateId, InitSharedStateArgs& InStateArgs)
	{
		FScopeLock Lock(&CritSect);
		InitOrUpdate(InStateArgs);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		return (*RG)->NextValue();
	}

	void FSharedStateRandomGetManager::SetSeed(const FGuid& InSharedStateId, int32 InSeed)
	{
		FScopeLock Lock(&CritSect);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		(*RG)->SetSeed(InSeed);
	}

	void FSharedStateRandomGetManager::SetNoRepeatOrder(const FGuid& InSharedStateId, int32 InNoRepeatOrder)
	{
		FScopeLock Lock(&CritSect);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		(*RG)->SetNoRepeatOrder(InNoRepeatOrder);
	}

	void FSharedStateRandomGetManager::SetRandomWeights(const FGuid& InSharedStateId, const TArray<float>& InRandomWeights)
	{
		FScopeLock Lock(&CritSect);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		(*RG)->SetRandomWeights(InRandomWeights);
	}

	void FSharedStateRandomGetManager::ResetSeed(const FGuid& InSharedStateId)
	{
		FScopeLock Lock(&CritSect);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		(*RG)->ResetSeed();
	}
	
	void FSharedStateRandomGetManager::ResetSeed(const FGuid& InSharedStateId, InitSharedStateArgs& InStateArgs)
	{
		FScopeLock Lock(&CritSect);
		InitOrUpdate(InStateArgs);
		TUniquePtr<FArrayRandomGet>* RG = RandomGets.Find(InSharedStateId);
		(*RG)->ResetSeed();
	}

	void FSharedStateRandomGetManager::InitOrUpdate(InitSharedStateArgs& InStateArgs)
	{
		if (TUniquePtr<FArrayRandomGet>* FoundExisting = RandomGets.Find(InStateArgs.SharedStateId))
		{
			(*FoundExisting)->UpdateState(InStateArgs.Seed, InStateArgs.NumElements, InStateArgs.Weights, InStateArgs.NoRepeatOrder);
		}
		else
		{
			RandomGets.Add(InStateArgs.SharedStateId, MakeUnique<FArrayRandomGet>(InStateArgs.Seed, InStateArgs.NumElements, InStateArgs.Weights, InStateArgs.NoRepeatOrder));
		}
	}
}

#undef LOCTEXT_NAMESPACE // MetasoundFrontend
