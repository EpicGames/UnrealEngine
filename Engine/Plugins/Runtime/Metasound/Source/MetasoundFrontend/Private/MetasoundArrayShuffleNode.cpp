// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundArrayShuffleNode.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace ArrayShuffleNodePrivate
	{
		const FLazyName ArrayShuffleOperatorName = "Shuffle";
		constexpr int32 ArrayShuffleNodeMajorVersion = 1;
		constexpr int32 ArrayShuffleNodeMinorVersion = 0;

		Frontend::FNodeClassRegistryKey CreateArrayShuffleNodeClassRegistryKey(const FName& InArrayDataTypeName)
		{
			return Frontend::FNodeClassRegistryKey(EMetasoundFrontendClassType::External, MetasoundArrayNodesPrivate::CreateArrayNodeClassName(ArrayShuffleOperatorName, InArrayDataTypeName), ArrayShuffleNodeMajorVersion, ArrayShuffleNodeMinorVersion);
		}

		FVertexInterface CreateArrayShuffleNodeInterface(const FName& InArrayDataTypeName, const FName& InElementDataTypeName)
		{
			using namespace ArrayNodeShuffleVertexNames;

			return FVertexInterface {
				FInputVertexInterface{
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerNext)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerShuffle)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerReset)),
					FInputDataVertex(METASOUND_GET_PARAM_NAME(InputShuffleArray), InArrayDataTypeName, METASOUND_GET_PARAM_METADATA(InputShuffleArray)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputShuffleSeed), -1),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAutoShuffle), true),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputShuffleEnableSharedState), false)
					},
				FOutputVertexInterface{
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnNext)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnShuffle)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnResetSeed)),
					FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputShuffledValue), InElementDataTypeName, METASOUND_GET_PARAM_METADATA(OutputShuffledValue), EVertexAccessType::Reference)
				}
			};
		}

		FNodeClassMetadata CreateArrayShuffleNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText)
		{
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ArrayOpArrayShuffleDisplayNamePattern", "Shuffle ({0})", InArrayDataTypeDisplayText);
			const FText NodeDescription = METASOUND_LOCTEXT("ArrayOpArrayShuffleDescription", "Output next element of a shuffled array on trigger.");

			return MetasoundArrayNodesPrivate::CreateArrayNodeClassMetadata(InArrayDataTypeName, ArrayShuffleOperatorName, NodeDisplayName, NodeDescription, CreateArrayShuffleNodeInterface(InArrayDataTypeName, InElementDataTypeName), ArrayShuffleNodeMajorVersion, ArrayShuffleNodeMinorVersion);
		}
	}

	/**
	* FArrayIndexShuffler
	*/

	FArrayIndexShuffler::FArrayIndexShuffler(int32 InSeed, int32 InMaxIndices)
	{
		Init(InSeed, InMaxIndices);
	}

	void FArrayIndexShuffler::Init(int32 InSeed, int32 InMaxIndices)
	{
		SetSeed(InSeed);
		if (InMaxIndices > 0)
		{
			ShuffleIndices.AddUninitialized(InMaxIndices);
			for (int32 i = 0; i < ShuffleIndices.Num(); ++i)
			{
				ShuffleIndices[i] = i;
			}
			ShuffleArray();
		}
	}

	void FArrayIndexShuffler::SetSeed(int32 InSeed)
	{
		if (InSeed == INDEX_NONE)
		{
			RandomStream.Initialize(FPlatformTime::Cycles());
		}
		else
		{
			RandomStream.Initialize(InSeed);
		}

		ResetSeed();
	}

	void FArrayIndexShuffler::ResetSeed()
	{
		RandomStream.Reset();
	}

	bool FArrayIndexShuffler::NextValue(bool bAutoShuffle, int32& OutIndex)
	{
		bool bShuffled = false;
		if (CurrentIndex == ShuffleIndices.Num())
		{
			if (bAutoShuffle)
			{
				ShuffleArray();
				bShuffled = true;
			}
			else
			{
				CurrentIndex = 0;
			}
		}

		check(CurrentIndex < ShuffleIndices.Num());
		PrevValue = ShuffleIndices[CurrentIndex];
		OutIndex = PrevValue;
		++CurrentIndex;

		return bShuffled;
	}

	// Shuffle the array with the given max indices
	void FArrayIndexShuffler::ShuffleArray()
	{
		// Randomize the shuffled array by randomly swapping indicies
		for (int32 i = 0; i < ShuffleIndices.Num(); ++i)
		{
			RandomSwap(i, 0, ShuffleIndices.Num() - 1);
		}

		// Reset the current index back to 0
		CurrentIndex = 0;

		// Fix up the new current index if the value is our previous value and we have an array larger than 1
		if (ShuffleIndices.Num() > 1 && ShuffleIndices[CurrentIndex] == PrevValue)
		{
			RandomSwap(0, 1, ShuffleIndices.Num() - 1);
		}
	}

	void FArrayIndexShuffler::RandomSwap(int32 InCurrentIndex, int32 InStartIndex, int32 InEndIndex)
	{
		int32 ShuffleIndex = RandomStream.RandRange(InStartIndex, InEndIndex);
		int32 Temp = ShuffleIndices[ShuffleIndex];
		ShuffleIndices[ShuffleIndex] = ShuffleIndices[InCurrentIndex];
		ShuffleIndices[InCurrentIndex] = Temp;
	}

	namespace ArrayNodeGetGlobalArrayKeyVertexNames
	{
		const FVertexName& GetInputNamespaceName()
		{
			static const FVertexName Name = TEXT("Namespace");
			return Name;
		}

		const FVertexName& GetInputArraySizeName()
		{
			static const FVertexName Name = TEXT("Array Size");
			return Name;
		}

		const FVertexName& GetInputSeedName()
		{
			static const FVertexName Name = TEXT("Seed");
			return Name;
		}

		const FVertexName& GetOutputArrayKeyName()
		{
			static const FVertexName Name = TEXT("Global Array Key");
			return Name;
		}
	}
}

#undef LOCTEXT_NAMESPACE
