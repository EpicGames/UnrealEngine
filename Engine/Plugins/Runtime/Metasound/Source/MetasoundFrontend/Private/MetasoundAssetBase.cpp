// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAssetBase.h"

#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Containers/Set.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "IAudioParameterTransmitter.h"
#include "Interfaces/MetasoundFrontendInterface.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Internationalization/Text.h"
#include "IStructSerializerBackend.h"
#include "Logging/LogMacros.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeClassRegistryPrivate.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundGlobals.h"
#include "MetasoundGraph.h"
#include "MetasoundJsonBackend.h"
#include "MetasoundLog.h"
#include "MetasoundParameterPack.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundTrace.h"
#include "MetasoundVertex.h"
#include "StructSerializer.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Templates/SharedPointer.h"
#include "UObject/MetaData.h"

#define LOCTEXT_NAMESPACE "MetaSound"

namespace Metasound
{
	namespace Frontend
	{
		namespace AssetBasePrivate
		{
			// Zero values means, that these don't do anything.
			static float BlockRateOverride = 0;
			static int32 SampleRateOverride = 0;

			void DepthFirstTraversal(const FMetasoundAssetBase& InInitAsset, TFunctionRef<TSet<const FMetasoundAssetBase*>(const FMetasoundAssetBase&)> InVisitFunction)
			{
				// Non recursive depth first traversal.
				TArray<const FMetasoundAssetBase*> Stack({ &InInitAsset });
				TSet<const FMetasoundAssetBase*> Visited;

				while (!Stack.IsEmpty())
				{
					const FMetasoundAssetBase* CurrentNode = Stack.Pop();
					if (!Visited.Contains(CurrentNode))
					{
						TArray<const FMetasoundAssetBase*> Children = InVisitFunction(*CurrentNode).Array();
						Stack.Append(Children);

						Visited.Add(CurrentNode);
					}
				}
			}

			// Registers node by copying document. Updates to document require re-registration.
			// This registry entry does not support node creation as it is only intended to be
			// used when serializing MetaSounds in contexts not requiring any runtime model to
			// be generated (ex. cooking commandlets that don't play or are validating MetaSounds, etc.).
			class FDocumentNodeRegistryEntryForSerialization : public INodeClassRegistryEntry
			{
			public:
				FDocumentNodeRegistryEntryForSerialization(const FMetasoundFrontendDocument& InDocument, const FTopLevelAssetPath& InAssetPath)
					: Interfaces(InDocument.Interfaces)
					, FrontendClass(InDocument.RootGraph)
					, AssetPath(InAssetPath)
				{
					// Copy FrontendClass to preserve original document.
					FrontendClass.Metadata.SetType(EMetasoundFrontendClassType::External);
				}

				FDocumentNodeRegistryEntryForSerialization(const FDocumentNodeRegistryEntryForSerialization& InOther) = default;

				virtual ~FDocumentNodeRegistryEntryForSerialization() = default;

				virtual TUniquePtr<INode> CreateNode(FNodeData) const override { return nullptr; }

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return FrontendClass;
				}

				virtual const TSet<FMetasoundFrontendVersion>* GetImplementedInterfaces() const override
				{
					return &Interfaces;
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					UE_LOGF(LogMetaSound, Warning, "Proxy data is not available for non runtime node %ls only used for serialization, so interface will not include object literals. Please ensure calling this function is intended.", *FrontendClass.Metadata.GetClassName().ToString())
					return CreateDefaultVertexInterfaceFromClassNoProxy(FrontendClass);
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					UE_LOGF(LogMetaSound, Error, "ClassInterfaces for assets are not available during cook. Cannot create interface for %ls", *FrontendClass.Metadata.GetClassName().ToString())
					return InvalidClassInterface;
				}

				virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const override
				{
					// Document nodes do not currently support node configuration.
					return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
				}

				virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override
				{
					// No node configuration supported for this node type, so only compatible if setting to invalid (null) configuration
					return !InNodeConfiguration.IsValid();
				}

			private:
				TSet<FMetasoundFrontendVersion> Interfaces;
				FMetasoundFrontendClass FrontendClass;
				FTopLevelAssetPath AssetPath;
				static const FClassInterface InvalidClassInterface;
			};

			const FClassInterface FDocumentNodeRegistryEntryForSerialization::InvalidClassInterface;

		} // namespace AssetBasePrivate

		FConsoleVariableMulticastDelegate CVarMetaSoundBlockRateChanged;

		FAutoConsoleVariableRef CVarMetaSoundBlockRate(
			TEXT("au.MetaSound.BlockRate"),
			AssetBasePrivate::BlockRateOverride,
			TEXT("Sets block rate (blocks per second) of MetaSounds.\n")
			TEXT("Default: 100.0f, Min: 1.0f, Max: 1000.0f"),
			FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) { CVarMetaSoundBlockRateChanged.Broadcast(Var); }),
			ECVF_Default);

		FConsoleVariableMulticastDelegate CVarMetaSoundSampleRateChanged;
		FAutoConsoleVariableRef CVarMetaSoundSampleRate(
			TEXT("au.MetaSound.SampleRate"),
			AssetBasePrivate::SampleRateOverride,
			TEXT("Overrides the sample rate of metasounds. Negative values default to audio mixer sample rate.\n")
			TEXT("Default: 0, Min: 8000, Max: 96000"),
			FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) { CVarMetaSoundSampleRateChanged.Broadcast(Var); }),
			ECVF_Default);

		float GetBlockRateOverride()
		{
			if(AssetBasePrivate::BlockRateOverride > 0)
			{
				return FMath::Clamp(AssetBasePrivate::BlockRateOverride, 
					GetBlockRateClampRange().GetLowerBoundValue(), 
					GetBlockRateClampRange().GetUpperBoundValue()
				);
			}
			return AssetBasePrivate::BlockRateOverride;
		}

		FConsoleVariableMulticastDelegate& GetBlockRateOverrideChangedDelegate()
		{
			return CVarMetaSoundBlockRateChanged;
		}

		int32 GetSampleRateOverride()
		{
			if (AssetBasePrivate::SampleRateOverride > 0)
			{
				return FMath::Clamp(AssetBasePrivate::SampleRateOverride, 
					GetSampleRateClampRange().GetLowerBoundValue(),
					GetSampleRateClampRange().GetUpperBoundValue()
				);
			}
			return AssetBasePrivate::SampleRateOverride;
		}

		FConsoleVariableMulticastDelegate& GetSampleRateOverrideChangedDelegate()
		{
			return CVarMetaSoundSampleRateChanged;
		}
		
		TRange<float> GetBlockRateClampRange()
		{
			return TRange<float>(1.f,1000.f);
		}

		TRange<int32> GetSampleRateClampRange()
		{
			return TRange<int32>(8000, 96000);
		}
	} // namespace Frontend
} // namespace Metasound

const FString FMetasoundAssetBase::FileExtension(TEXT(".metasound"));

TSharedPtr<Audio::IProxyData> FMetasoundAssetBase::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	using namespace Metasound::Frontend;
	
	if (!IsRegistered())
	{
		UE_LOGF(LogMetaSound, Error, "Cannot make proxy for MetaSound %ls. Derived classes should ensure that they are registered before calling into FMetasoundAssetBase::CreateProxyData", *GetOwningAssetName());
		return nullptr;
	}

	const FGraphRegistryKey& Key = GetGraphRegistryKey();
	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = GetOwningAsset();

	// check if graph key has changed
	if (Proxy)
	{
		if (Proxy->GetGraphRegistryKey() != Key)
		{
			Proxy = nullptr;
		}
	}

	// check interfaces have changed
	if (Proxy)
	{
		int32 Num = Proxy->GetInterfaces().Num();
		if (Num != Proxy->GetInterfaces().Union(DocInterface->GetConstDocument().Interfaces).Num())
		{
			Proxy = nullptr;
		}
	}

	// recreate the proxy if necessary
	if (!Proxy)
	{
		if (ensure(Key.IsValid()))
		{
			FMetasoundAssetProxy::FParameters Args;
			Args.Interfaces = DocInterface->GetConstDocument().Interfaces;
			Args.GraphRegistryKey = Key;
			Proxy = MakeShared<FMetasoundAssetProxy>(Args);
		}
		else
		{
			UObject* Owner = GetOwningAsset();
			check(Owner);
			const UClass* Class = Owner->GetClass();
			check(Class);
			const FString ClassName = Class->GetName();
			UE_LOGF(LogMetaSound, Error, "Unable to create proxy data. Failed for MetaSound node class '%ls' of UObject class '%ls'", *GetOwningAssetName(), *ClassName);
		}
	}

	return Proxy;
}

void FMetasoundAssetBase::UpdateAndRegisterForExecution()
{
	// This is handling the case of a call where no registration options have been
	// explicitly set. As a fallback, use the default page ID. 
	Metasound::Frontend::FMetaSoundAssetRegistrationOptions RegOptions;
	TArray<FGuid> PageOrder({Metasound::Frontend::DefaultPageID});
	RegOptions.PageOrder = PageOrder;
	
	UpdateAndRegisterForExecution(RegOptions);
}

void FMetasoundAssetBase::UpdateAndRegisterForExecution(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions& InRegistrationOptions)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	// Graph registration must only happen on one thread to avoid race conditions on graph registration.
	checkf(IsInGameThread(), TEXT("MetaSound %s graph can only be registered on the GameThread"), *GetOwningAssetName());
	checkf(Metasound::CanEverExecuteGraph(), TEXT("Cannot generate proxies/runtime graph when graph execution is not enabled."));

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSoundAssetBase::UpdateAndRegisterForExecution);
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MetaSoundAssetBase::UpdateAndRegisterForExecution  asset %s"), *this->GetOwningAssetName()));
	if (!InRegistrationOptions.bForceReregister)
	{
		if (IsRegistered())
		{
			return;
		}
	}

#if WITH_EDITOR
	FMetaSoundFrontendDocumentBuilder* DocBuilder = nullptr;
	if (InRegistrationOptions.bRebuildReferencedAssetClasses)
	{
		RebuildReferencedAssetClasses();
	}
#endif // WITH_EDITOR

	if (InRegistrationOptions.bRegisterDependencies)
	{
		RegisterAssetDependencies(InRegistrationOptions);
	}

	UObject* Owner = GetOwningAsset();
	check(Owner);

	// This should not be necessary as it should be added on asset load,
	// but currently registration is required to be called prior to adding
	// an object-defined graph class to the registry so it was placed here.
	IMetaSoundAssetManager::GetChecked().AddOrUpdateFromObject(*Owner);

	// Auto update must be done after all referenced asset classes are registered
	if (InRegistrationOptions.bAutoUpdate)
	{
#if WITH_EDITORONLY_DATA
		bool bDidUpdate = false;
		
		// Only attempt asset versioning if owner is asset (dependency versioning on runtime MetaSound instances isn't supported nor necessary).
		if (Owner->IsAsset())
		{
			DocBuilder = &IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Owner);
			FAutoUpdateRootGraph AutoUpdateTransform(GetOwningAssetName(), InRegistrationOptions.bAutoUpdateLogWarningOnDroppedConnection);
			bDidUpdate = AutoUpdateTransform.Transform(*DocBuilder);
		}
#else // !WITH_EDITORONLY_DATA
		constexpr bool bDidUpdate = false;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
		if (bDidUpdate || InRegistrationOptions.bForceViewSynchronization)
		{
			GetModifyContext().SetForceRefreshViews();
		}
#endif // WITH_EDITOR
	}
	else
	{
#if WITH_EDITOR
		if (InRegistrationOptions.bForceViewSynchronization)
		{
			GetModifyContext().SetForceRefreshViews();
		}
#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	// Must be completed after auto-update to ensure all non-transient referenced dependency data is up-to-date (ex.
	// class version), which is required for most accurately caching current registry metadata.
	if (DocBuilder)
	{
		DocBuilder->CacheRegistryMetadata();
	}
#endif // WITH_EDITOR

	GraphRegistryKey = FNodeClassRegistry::Get().RegisterGraph(Owner, InRegistrationOptions.PageOrder);
	if (!GraphRegistryKey.IsValid())
	{
		UClass* Class = Owner->GetClass();
		check(Class);
		const FString ClassName = Class->GetName();
		const FString AssetName = Owner->GetName();
		UE_LOGF(LogMetaSound, Error, "Registration failed for MetaSound node class '%ls' of UObject class '%ls'", *AssetName, *ClassName);
	}
}


#if WITH_EDITORONLY_DATA
void FMetasoundAssetBase::UpdateAndRegisterForSerialization(FName InCookPlatformName)
{
	using namespace Metasound::Frontend;
	FMetaSoundAssetCookOptions CookOptions{ .bStripUnusedPages = !InCookPlatformName.IsNone() };

	if (CookOptions.bStripUnusedPages)
	{
		const IDocumentBuilderRegistry& DocBuilderRegistry = IDocumentBuilderRegistry::GetChecked();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CookOptions.PagesToTarget = DocBuilderRegistry.GetCookedTargetPages(InCookPlatformName);
		CookOptions.PageOrder = DocBuilderRegistry.GetCookedPageOrder(InCookPlatformName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UpdateAndRegisterForSerialization(CookOptions);
			
}
void FMetasoundAssetBase::UpdateAndRegisterForSerialization(const Metasound::Frontend::FMetaSoundAssetCookOptions& InCookOptions)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSoundAssetBase::UpdateAndRegisterForSerialization);

	// If already registered, nothing to condition for presaving
	if (IsRegistered())
	{
		return;
	}

	FDocumentIDGenerator::FScopeDeterminism DeterminismScope(true);
	UpdateAndRegisterReferencesForSerialization(InCookOptions);

	UObject* Owner = GetOwningAsset();
	check(Owner);

	IMetaSoundAssetManager::GetChecked().AddOrUpdateFromObject(*Owner);

	bool bDidUpdate = false;

	FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Owner);

	if (InCookOptions.bStripUnusedPages)
	{
		bDidUpdate |= StripUnusedPages(DocBuilder, InCookOptions.PagesToTarget, InCookOptions.PageOrder);
	}

	// Auto update must be done after all referenced asset classes are registered
#if WITH_EDITORONLY_DATA
	constexpr bool bAutoUpdateLogWarningOnDroppedConnection = true;
	FAutoUpdateRootGraph AutoUpdateTransform(GetOwningAssetName(), bAutoUpdateLogWarningOnDroppedConnection);
	bDidUpdate |= AutoUpdateTransform.Transform(DocBuilder);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	if (bDidUpdate)
	{
		GetModifyContext().SetForceRefreshViews();
	}
#endif // WITH_EDITOR

#if WITH_EDITOR
	// Must be completed after auto-update to ensure all non-transient referenced dependency data is up-to-date (ex.
	// class version), which is required for most accurately caching current registry metadata.
	DocBuilder.CacheRegistryMetadata();
#endif // WITH_EDITOR

	{
		// Performs document transforms on local copy, which reduces document footprint & renders transforming unnecessary at runtime
		const bool bContainsTemplateDependency = DocBuilder.ContainsDependencyOfType(EMetasoundFrontendClassType::Template);
		if (bContainsTemplateDependency)
		{
			DocBuilder.TransformTemplateNodes();
		}

		if (GraphRegistryKey.IsValid())
		{
			FNodeClassRegistry::Get().UnregisterNode(GraphRegistryKey.NodeKey);
			GraphRegistryKey = { };
		}

		// Need to register the node so that it is available for other graphs, but avoids creating proxies.
		// This is accomplished by using a special node registration object which reflects the necessary
		// information for the node registry, but does not create the runtime graph model (i.e. INodes).
		TScriptInterface<IMetaSoundDocumentInterface> DocInterface(Owner);
		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		const FTopLevelAssetPath AssetPath = DocInterface->GetAssetPathChecked();
		TUniquePtr<INodeClassRegistryEntry> RegistryEntry = MakeUnique<AssetBasePrivate::FDocumentNodeRegistryEntryForSerialization>(Document, AssetPath);

		const FNodeClassRegistryKey NodeKey = FNodeClassRegistry::Get().RegisterNode(MoveTemp(RegistryEntry));
		GraphRegistryKey = FGraphRegistryKey { NodeKey, AssetPath, Owner->GetUniqueID()};
	}

	if (!GraphRegistryKey.IsValid())
	{
		const UClass* Class = Owner->GetClass();
		check(Class);
		const FString ClassName = Class->GetName();
		UE_LOGF(LogMetaSound, Error, "Presave failed for MetaSound node class '%ls' of UObject class '%ls'", *GetOwningAssetName(), *ClassName);
	}
}
#endif // WITH_EDITORONLY_DATA

void FMetasoundAssetBase::OnNotifyBeginDestroy()
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	UObject* OwningAsset = GetOwningAsset();
	check(OwningAsset);

	// Unregistration of graph using local call is not necessary when cooking as deserialized objects are not mutable and, should they be
	// reloaded, omitting unregistration avoids potentially kicking off an invalid asynchronous task to unregister a non-existent runtime graph.
	if (Metasound::CanEverExecuteGraph())
	{
		UnregisterGraphWithFrontend();
	}
	else
	{
		if (GraphRegistryKey.IsValid())
		{
			FNodeClassRegistry::Get().UnregisterNode(GraphRegistryKey.NodeKey);
			GraphRegistryKey = { };
		}
	}

	if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
	{
		AssetManager->RemoveAsset(*OwningAsset);
	};
}

void FMetasoundAssetBase::UnregisterGraphWithFrontend()
{
	using namespace Metasound;
	using namespace Metasound::Frontend;
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSoundAssetBase::UnregisterGraphWithFrontend);

	check(IsInGameThread());
	checkf(Metasound::CanEverExecuteGraph(), TEXT("If execution is not supported, UnregisterNode must be called directly to avoid async attempt at destroying runtime graph that does not exist."));

	if (GraphRegistryKey.IsValid())
	{
		UObject* OwningAsset = GetOwningAsset();
		if (ensureAlways(OwningAsset))
		{
			const bool bSuccess = FNodeClassRegistry::Get().UnregisterGraph(GraphRegistryKey, OwningAsset);
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Verbose, "Failed to unregister node with key %ls for asset %ls. No registry entry exists with that key.", *GraphRegistryKey.ToString(), *GetOwningAssetName());
			}
		}

		GraphRegistryKey = { };
	}
}

bool FMetasoundAssetBase::IsInterfaceDeclared(const FMetasoundFrontendVersion& InVersion) const
{
	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = GetOwningAsset();
	check(DocInterface.GetObject());
	return DocInterface->GetConstDocument().Interfaces.Contains(InVersion);
}

#if WITH_EDITORONLY_DATA
bool FMetasoundAssetBase::VersionAsset(FMetaSoundFrontendDocumentBuilder& Builder)
{
	return false;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void FMetasoundAssetBase::CacheRegistryMetadata()
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSoundAssetBase::CacheRegistryMetadata);

	using namespace Metasound::Frontend;

	UObject* Owner = GetOwningAsset();
	check(Owner);
	FMetaSoundFrontendDocumentBuilder& DocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(Owner);
	DocBuilder.CacheRegistryMetadata();
}

FMetasoundFrontendDocumentModifyContext& FMetasoundAssetBase::GetModifyContext()
{
	// ModifyContext is now mutable to avoid mutations to it requiring access through
	// the deprecated Document controller causing the builder cache to get wiped unnecessarily.
	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = GetOwningAsset();
	check(DocInterface.GetObject());
	return DocInterface->GetConstDocument().Metadata.ModifyContext;
}

const FMetasoundFrontendDocumentModifyContext& FMetasoundAssetBase::GetConstModifyContext() const
{
	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = GetOwningAsset();
	check(DocInterface.GetObject());
	return DocInterface->GetConstDocument().Metadata.ModifyContext;
}

#endif // WITH_EDITOR

bool FMetasoundAssetBase::IsRegistered() const
{
	using namespace Metasound::Frontend;

	return GraphRegistryKey.IsValid();
}

bool FMetasoundAssetBase::IsReferencedAsset(const FMetasoundAssetBase& InAsset) const
{
	using namespace Metasound::Frontend;

	bool bIsReferenced = false;
	const FString InAssetKey = InAsset.GetGraphRegistryKey().NodeKey.ToString();
	AssetBasePrivate::DepthFirstTraversal(*this, [&](const FMetasoundAssetBase& ChildAsset)
	{
		TSet<const FMetasoundAssetBase*> Children;
		if (&ChildAsset == &InAsset)
		{
			bIsReferenced = true;
			return Children;
		}

		// It's possible the referenced asset has been unloaded, 
		// but is in the process of being reloaded, 
		// so the asset may not be registered in the asset manager yet
		// but the class key can still be checked
		const TSet<FString>& AssetClassKeys = ChildAsset.GetReferencedAssetClassKeys();
		if (AssetClassKeys.Contains(InAssetKey))
		{
			bIsReferenced = true;
			return Children;
		}

		TArray<FMetasoundAssetBase*> ChildRefs;
		ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(ChildAsset, ChildRefs));
		
		Algo::Transform(ChildRefs, Children, [](FMetasoundAssetBase* Child) { return Child; });
		return Children;
	});

	return bIsReferenced;
}

bool FMetasoundAssetBase::AddingReferenceCausesLoop(const FMetasoundAssetBase& InMetaSound) const
{
	using namespace Metasound::Frontend;

	bool bCausesLoop = false;
	const FMetasoundAssetBase* Parent = this;
	AssetBasePrivate::DepthFirstTraversal(InMetaSound, [&](const FMetasoundAssetBase& ChildAsset)
	{
		TSet<const FMetasoundAssetBase*> Children;
		if (Parent == &ChildAsset)
		{
			bCausesLoop = true;
			return Children;
		}

		TArray<FMetasoundAssetBase*> ChildRefs;
		ensureAlways(IMetaSoundAssetManager::GetChecked().TryLoadReferencedAssets(ChildAsset, ChildRefs));
		Algo::Transform(ChildRefs, Children, [](FMetasoundAssetBase* Child) { return Child; });
		return Children;
	});

	return bCausesLoop;
}

#if WITH_EDITOR
FText FMetasoundAssetBase::GetDisplayName(FString&& InTypeName) const
{
	using namespace Metasound::Frontend;

	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(GetOwningAsset());
	TConstStructView<FMetaSoundFrontendDocumentTemplate> Template = DocInterface->GetConstDocument().Template;
	if (Template.IsValid())
	{
		const UScriptStruct* Struct = Template.GetScriptStruct();
		check(Struct);
		return FText::Format(LOCTEXT("DocTemplateDisplayNameFormat", "{0} ({1})"), FText::FromString(MoveTemp(InTypeName)), Struct->GetDisplayNameText());
	}

	return FText::FromString(MoveTemp(InTypeName));

}
#endif // WITH_EDITOR

bool FMetasoundAssetBase::MarkMetasoundDocumentDirty() const
{
	if (const UObject* OwningAsset = GetOwningAsset())
	{
		return OwningAsset->MarkPackageDirty();
	}
	return false;
}

Metasound::Frontend::FDocumentHandle FMetasoundAssetBase::GetDocumentHandle(bool bReloadBuilder)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Metasound::Frontend::IDocumentController::CreateDocumentHandle(GetDocumentAccessPtr(bReloadBuilder), bReloadBuilder);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

Metasound::Frontend::FConstDocumentHandle FMetasoundAssetBase::GetDocumentHandle() const
{
	return Metasound::Frontend::IDocumentController::CreateDocumentHandle(GetDocumentConstAccessPtr());
}

Metasound::Frontend::FGraphHandle FMetasoundAssetBase::GetRootGraphHandle()
{
	return GetDocumentHandle()->GetRootGraph();
}

Metasound::Frontend::FConstGraphHandle FMetasoundAssetBase::GetRootGraphHandle() const
{
	return GetDocumentHandle()->GetRootGraph();
}

bool FMetasoundAssetBase::ImportFromJSON(const FString& InJSON)
{
	return false;
}

bool FMetasoundAssetBase::ImportFromJSONAsset(const FString& InAbsolutePath)
{
	return false;
}

const FMetasoundFrontendDocument& FMetasoundAssetBase::GetConstDocumentChecked() const
{
	const UObject* Owner = GetOwningAsset();
	check(Owner);
	TScriptInterface<const IMetaSoundDocumentInterface> DocInterface = Owner;
	return DocInterface->GetConstDocument();
}

FMetasoundFrontendDocument& FMetasoundAssetBase::GetDocumentChecked()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMetasoundFrontendDocument* Document = GetDocumentAccessPtr().Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	check(nullptr != Document);
	return *Document;
}

const Metasound::Frontend::FGraphRegistryKey& FMetasoundAssetBase::GetGraphRegistryKey() const
{
	return GraphRegistryKey;
}

FString FMetasoundAssetBase::GetOwningAssetName() const
{
	if (const UObject* OwningAsset = GetOwningAsset())
	{
		return OwningAsset->GetPathName();
	}
	return FString();
}

#if WITH_EDITOR
void FMetasoundAssetBase::RebuildReferencedAssetClasses()
{
	using namespace Metasound::Frontend;

	IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();
	AssetManager.AddAssetReferences(*this);
	TSet<IMetaSoundAssetManager::FAssetRef> ReferencedAssetClasses = AssetManager.GetReferencedAssets(*this);
	SetReferencedAssets(MoveTemp(ReferencedAssetClasses));
}
#endif // WITH_EDITOR

void FMetasoundAssetBase::RegisterAssetDependencies(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions& InRegistrationOptions)
{
	using namespace Metasound::Frontend;

	TArray<FMetasoundAssetBase*> References = GetReferencedAssets();
	for (FMetasoundAssetBase* Reference : References)
	{
		if (InRegistrationOptions.bForceReregister || !Reference->IsRegistered())
		{
			Reference->UpdateAndRegisterForExecution(InRegistrationOptions);
		}
	}
}

#if WITH_EDITORONLY_DATA
void FMetasoundAssetBase::UpdateAndRegisterReferencesForSerialization(const Metasound::Frontend::FMetaSoundAssetCookOptions& InCookOptions)
{
	using namespace Metasound::Frontend;

	TArray<FMetasoundAssetBase*> References = GetReferencedAssets();
	for (FMetasoundAssetBase* Reference : References)
	{
		if (!Reference->IsRegistered())
		{
			Reference->UpdateAndRegisterForSerialization(InCookOptions);
		}
	}
}

bool FMetasoundAssetBase::GetVersionedOnLoad() const
{
	return bVersionedOnLoad;
}

void FMetasoundAssetBase::ClearVersionedOnLoad()
{
	bVersionedOnLoad = false;
}

void FMetasoundAssetBase::SetVersionedOnLoad()
{
	bVersionedOnLoad = true;
}
#endif // WITH_EDITORONLY_DATA

FMetasoundAssetProxy::FMetasoundAssetProxy(const FParameters& InParams)
{
	Interfaces = InParams.Interfaces;
	GraphRegistryKey = InParams.GraphRegistryKey;
}

FMetasoundAssetProxy::FMetasoundAssetProxy(const FMetasoundAssetProxy& Other)
{
	Interfaces = Other.Interfaces;
	GraphRegistryKey = Other.GraphRegistryKey;
}

const Metasound::Frontend::FGraphRegistryKey& FMetasoundAssetProxy::GetGraphRegistryKey() const
{
	return GraphRegistryKey;
}

const Metasound::IGraph* FMetasoundAssetProxy::GetGraph() const
{
	using namespace Metasound::Frontend;
	if (GraphRegistryKey.IsValid())
	{
		return INodeClassRegistry::Get()->GetGraph(GraphRegistryKey).Get();
	}
	return nullptr;
}

const TSet<FMetasoundFrontendVersion>& FMetasoundAssetProxy::GetInterfaces() const
{
	return Interfaces;
}

#undef LOCTEXT_NAMESPACE // "MetaSound"
