// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAssetManager.h"

#include "Algo/Copy.h"
#include "AssetRegistry/AssetData.h"
#include "JsonObjectConverter.h"
#include "MetasoundAssetManagerTransactionPrivate.h"
#include "MetasoundAssetTagCollections.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendRegistryKey.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundAssetManager)


namespace Metasound::Frontend
{
	namespace AssetManagerPrivate
	{
		static TUniquePtr<IMetaSoundAssetManager> Instance;
	} // AssetManagerPrivate

	namespace AssetTagsPrivate
	{
		static const FLazyName AssetClassID = "AssetClassID";

#if WITH_EDITORONLY_DATA
		static const FLazyName AssetCollections = "AssetCollections";
		static const FLazyName DocumentVersion = "DocumentVersion";
		static const FLazyName IsPreset = "bIsPreset";
#endif // WITH_EDITORONLY_DATA

		template <typename TStructType>
		bool DeserializeTagFromJson(const FAssetData& InAssetData, FName TagName, TStructType& OutStruct)
		{
			FString TagString;
			const bool bTagDataFound = InAssetData.GetTagValue(TagName, TagString);
			if (bTagDataFound)
			{
				return FJsonObjectConverter::JsonObjectStringToUStruct(TagString, &OutStruct);
			}
			return false;
		};
	}
} // namespace Metasound::Frontend


FMetaSoundDocumentInfo::FMetaSoundDocumentInfo()
	: bIsPreset(0)
{
}

FMetaSoundDocumentInfo::FMetaSoundDocumentInfo(const IMetaSoundDocumentInterface& InDocInterface)
{
#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;

	const FMetasoundFrontendDocument& Document = InDocInterface.GetConstDocument();
	const FMetasoundFrontendClassMetadata& ClassMetadata = Document.RootGraph.Metadata;

	if (!IsRunningCookCommandlet())
	{
		bIsPreset = (uint8)(Document.Template.GetPtr<FMetaSoundFrontendPresetTemplate>() != nullptr);
		DocumentVersion = Document.Metadata.Version.Number;

		{
			IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();
			auto IsAssetReference = [&AssetManager](const FMetasoundFrontendClass& Class)
				{
					return AssetManager.IsAssetClass(Class.Metadata);
				};

			auto GetAssetKey = [](const FMetasoundFrontendClass& Class)
				{
					return FMetaSoundAssetKey(Class.Metadata);
				};
			Algo::TransformIf(Document.Dependencies, ReferencedAssetKeys, IsAssetReference, GetAssetKey);
		}
	}

#endif // WITH_EDITORONLY_DATA
}

FMetaSoundDocumentInfo::FMetaSoundDocumentInfo(const FAssetData& InAssetData, bool& bOutIsValid)
{
	bOutIsValid = true;

#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;

	if (!IsRunningCookCommandlet())
	{
		{
			FString DocVersionStr;
			bOutIsValid &= InAssetData.GetTagValue(AssetTagsPrivate::DocumentVersion, DocVersionStr);
			bOutIsValid &= FMetasoundFrontendVersionNumber::Parse(DocVersionStr, DocumentVersion);
		}

		{
			bool bTagIsPreset = false;
			bOutIsValid &= InAssetData.GetTagValue(AssetTagsPrivate::IsPreset, bTagIsPreset);
			bIsPreset = (uint8)bTagIsPreset;
		}

		{
			{
				FMetaSoundAssetTagCollections TagCollections;
				const bool bTagFound = AssetTagsPrivate::DeserializeTagFromJson(InAssetData, AssetTagsPrivate::AssetCollections, TagCollections);
				if (bTagFound)
				{
					ReferencedAssetKeys = MoveTemp(TagCollections.AssetKeys);
				}
				else
				{
					bOutIsValid = false;
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FMetaSoundDocumentInfo::ExportToContext(FAssetRegistryTagsContext& OutContext) const
{
	using namespace Metasound::Frontend;

	using FAssetRegistryTag = UObject::FAssetRegistryTag;

#if WITH_EDITORONLY_DATA
	OutContext.AddTag(FAssetRegistryTag(AssetTagsPrivate::DocumentVersion, DocumentVersion.ToString(), FAssetRegistryTag::TT_Alphabetical));
	OutContext.AddTag(FAssetRegistryTag(AssetTagsPrivate::IsPreset, FString(bIsPreset ? TEXT("1") : TEXT("0")), FAssetRegistryTag::TT_Numerical));

	auto SerializeTagToJson = []<typename TStructType>(const TStructType& InStruct)
	{
		FString JsonString;
		FJsonObjectConverter::UStructToJsonObjectString(InStruct, JsonString);
		return JsonString;
	};

	const FMetaSoundAssetTagCollections TagCollections { .AssetKeys = ReferencedAssetKeys };
	OutContext.AddTag(FAssetRegistryTag(AssetTagsPrivate::AssetCollections, SerializeTagToJson(TagCollections), FAssetRegistryTag::TT_Hidden));
#endif // WITH_EDITORONLY_DATA
}

namespace Metasound::Frontend
{
	FMetaSoundAssetClassInfo::FMetaSoundAssetClassInfo(const IMetaSoundDocumentInterface& InDocInterface)
		: FMetaSoundClassInfo(EMetasoundFrontendClassType::External)
	{
		InitFromDocument(InDocInterface);
	}

	FMetaSoundAssetClassInfo::FMetaSoundAssetClassInfo(const FAssetData& InAssetData)
		: FMetaSoundClassInfo(InAssetData)
	{
		using namespace Frontend;

		if (InAssetData.IsAssetLoaded())
		{
			UObject* MetaSound = InAssetData.GetAsset();
			TScriptInterface<IMetaSoundDocumentInterface> DocInterface(MetaSound);

			check(DocInterface.GetObject());
			InitFromDocument(*DocInterface.GetInterface());
			return;
		}

		FString AssetClassID;
		bool bSuccess = TryGetAssetClassTag(InAssetData, AssetClassID);
		{
			ClassName = FMetasoundFrontendClassName(FName(), FName(*AssetClassID));
		}

#if WITH_EDITORONLY_DATA
		DocInfo = FMetaSoundDocumentInfo(InAssetData, bSuccess);
#endif // WITH_EDITORONLY_DATA

		AssetPath = FTopLevelAssetPath(InAssetData.PackageName, InAssetData.AssetName);

		bIsValid = bIsValid & (int8)(bSuccess);
	}

	void FMetaSoundAssetClassInfo::ExportToContext(FAssetRegistryTagsContext& OutContext) const
	{
		using FAssetRegistryTag = UObject::FAssetRegistryTag;

		FMetaSoundClassInfo::ExportToContext(OutContext);

		// AssetClassID housed in ClassName, but ID and associated tag is
		// stored on this inheriting class, so this is serialized locally.
		OutContext.AddTag(FAssetRegistryTag(AssetTagsPrivate::AssetClassID, ClassName.Name.ToString(), FAssetRegistryTag::TT_Alphabetical));

#if WITH_EDITORONLY_DATA
		DocInfo.ExportToContext(OutContext);
#endif // WITH_EDITORONLY_DATA
	}

	void FMetaSoundAssetClassInfo::InitFromDocument(const IMetaSoundDocumentInterface& InDocInterface)
	{
		FMetaSoundClassInfo::InitFromDocument(InDocInterface);

		AssetPath = InDocInterface.GetAssetPathChecked();

#if WITH_EDITORONLY_DATA
		DocInfo = FMetaSoundDocumentInfo(InDocInterface);
#endif // WITH_EDITORONLY_DATA
	}

	bool FMetaSoundAssetClassInfo::TryGetAssetKey(const FAssetData& InAssetData, FMetaSoundAssetKey& OutKey)
	{
		if (InAssetData.IsAssetLoaded())
		{
			if (const UObject* Object = InAssetData.GetAsset())
			{
				TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(Object);
				if (DocInterface.GetObject())
				{
					OutKey = FMetaSoundAssetKey(DocInterface->GetConstDocument().RootGraph);
					return true;
				}
			}
		}

		bool bSuccess = true;
		{
			FString AssetClassID;
			bSuccess &= InAssetData.GetTagValue(AssetTagsPrivate::AssetClassID, AssetClassID);
			OutKey.ClassName = FMetasoundFrontendClassName(FName(), FName(*AssetClassID));
		}

		bSuccess = TryGetClassVersion(InAssetData, OutKey.Version);
		return bSuccess;
	}

	bool FMetaSoundAssetClassInfo::TryGetAssetClassName(const FAssetData& InAssetData, FMetasoundFrontendClassName& OutClassName)
	{
		if (InAssetData.IsAssetLoaded())
		{
			if (const UObject* Object = InAssetData.GetAsset())
			{
				TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(Object);
				if (DocInterface.GetObject())
				{
					OutClassName = DocInterface->GetConstDocument().RootGraph.Metadata.GetClassName();
					return true;
				}
			}
		}

		bool bSuccess = true;
		{
			FString AssetClassID;
			bSuccess &= TryGetAssetClassTag(InAssetData, AssetClassID);
			OutClassName = FMetasoundFrontendClassName(FName(), FName(*AssetClassID));
		}

		return bSuccess;
	}

	bool FMetaSoundAssetClassInfo::TryGetAssetClassTag(const FAssetData& InAssetData, FString& OutClassIDString)
	{
		return InAssetData.GetTagValue(AssetTagsPrivate::AssetClassID, OutClassIDString);
	}

	IMetaSoundAssetManager* IMetaSoundAssetManager::Get()
	{
		return AssetManagerPrivate::Instance.Get();
	}

	IMetaSoundAssetManager& IMetaSoundAssetManager::GetChecked()
	{
		check(AssetManagerPrivate::Instance.IsValid());
		return *AssetManagerPrivate::Instance;
	}

	void IMetaSoundAssetManager::Deinitialize()
	{
		if (AssetManagerPrivate::Instance.IsValid())
		{
			AssetManagerPrivate::Instance.Reset();
		}
	}

	void IMetaSoundAssetManager::Initialize(TUniquePtr<IMetaSoundAssetManager>&& InInterface)
	{
		check(!AssetManagerPrivate::Instance.IsValid());
		AssetManagerPrivate::Instance = MoveTemp(InInterface);
	}

	FMetaSoundAssetManagerBase::FMetaSoundAssetManagerBase()
		: Transactor(MakeUnique<FAssetManagerTransactor>())
	{
	}

	IMetaSoundAssetTransactor& FMetaSoundAssetManagerBase::GetTransactor()
	{
		check(Transactor);
		return *Transactor.Get();
	}
} // namespace Metasound::Frontend

FMetaSoundAssetKey::FMetaSoundAssetKey(const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InVersion)
	: ClassName(InClassName)
	, Version(InVersion)
{
}

FMetaSoundAssetKey::FMetaSoundAssetKey(const Metasound::Frontend::FNodeClassRegistryKey& RegKey)
{
	using namespace Metasound::Frontend;

	if (RegKey.Type != EMetasoundFrontendClassType::Invalid)
	{
		checkf(IsValidType(RegKey.Type), TEXT("Invalid ClassType '%s' for Registry Key"), LexToString(RegKey.Type));
		ClassName = RegKey.ClassName;
		Version = RegKey.Version;
	}
}

FMetaSoundAssetKey::FMetaSoundAssetKey(const FMetasoundFrontendClassMetadata& InMetadata)
{
	using namespace Metasound::Frontend;

	if (InMetadata.GetType() != EMetasoundFrontendClassType::Invalid)
	{
		checkf(IsValidType(InMetadata.GetType()), TEXT("Invalid ClassType '%s' for Registry Key"), LexToString(InMetadata.GetType()));
		ClassName = InMetadata.GetClassName();
		Version = InMetadata.GetVersion();
	}
}

const FMetaSoundAssetKey& FMetaSoundAssetKey::GetInvalid()
{
	static FMetaSoundAssetKey Invalid;
	return Invalid;
}

bool FMetaSoundAssetKey::IsValid() const
{
	return ClassName.IsValid() && Version.IsValid();
}

bool FMetaSoundAssetKey::IsValidType(EMetasoundFrontendClassType ClassType)
{
	switch (ClassType)
	{
		case EMetasoundFrontendClassType::External:
		case EMetasoundFrontendClassType::Graph:
			return true;

		case EMetasoundFrontendClassType::Input:
		case EMetasoundFrontendClassType::Literal:
		case EMetasoundFrontendClassType::Output:
		case EMetasoundFrontendClassType::Template:
		case EMetasoundFrontendClassType::Variable:
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
		case EMetasoundFrontendClassType::VariableAccessor:
		case EMetasoundFrontendClassType::VariableMutator:
		case EMetasoundFrontendClassType::Invalid:
		default:
			break;
	}

	return false;
}

FString FMetaSoundAssetKey::ToString() const
{
	TStringBuilder<128> KeyStringBuilder;
	KeyStringBuilder.Append(ClassName.GetFullName().ToString());
	KeyStringBuilder.AppendChar('_');
	KeyStringBuilder.Append(FString::FromInt(Version.Major));
	KeyStringBuilder.AppendChar('.');
	KeyStringBuilder.Append(FString::FromInt(Version.Minor));
	return KeyStringBuilder.ToString();
}
