// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAssetManagerTransaction.h"
#include "MetasoundAssetManagerTransactionPrivate.h"


namespace Metasound::Frontend
{
	FString FAssetManagerTransaction::LexToString(FAssetManagerTransaction::ETransactionType InTransactionType)
	{
		using namespace Metasound;

		switch (InTransactionType)
		{
		case ETransactionType::AssetRegistration:
		{
			return TEXT("Asset Registration");
		}

		case ETransactionType::AssetUnregistration:
		{
			return TEXT("Asset Unregistration");
		}

		case ETransactionType::Invalid:
		{
			return TEXT("Invalid");
		}

		default:
		{
			checkNoEntry();
			return TEXT("");
		}
		}
	}

	FAssetManagerTransaction::FAssetManagerTransaction(FAssetManagerTransaction::ETransactionType InType, const FMetaSoundAssetKey& InKey, FAssetManagerTransaction::FTimeType InTimestamp)
		: Type(InType)
		, Key(InKey)
		, Timestamp(InTimestamp)
	{
	}

	FAssetManagerTransaction::ETransactionType FAssetManagerTransaction::GetTransactionType() const
	{
		return Type;
	}

	FNodeClassRegistryKey FAssetManagerTransaction::GetNodeRegistryKey() const
	{
		return Key;
	}

	FAssetManagerTransaction::FTimeType FAssetManagerTransaction::GetTimestamp() const
	{
		return Timestamp;
	}

	FAssetManagerTransactor::FAssetManagerTransactor()
		: TransactionBuffer(MakeShared<FAssetManagerTransactionBuffer>())
	{
	}

	TUniquePtr<FAssetManagerTransactionStream> FAssetManagerTransactor::CreateTransactionStream()
	{
		return MakeUnique<FAssetManagerTransactionStream>(TransactionBuffer);
	}

	void FAssetManagerTransactor::RegisterAsset(FMetaSoundAssetKey Key)
	{
		const FAssetManagerTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
		TransactionBuffer->AddTransaction(FAssetManagerTransaction(FAssetManagerTransaction::ETransactionType::AssetRegistration, Key, Timestamp));
	}

	void FAssetManagerTransactor::UnregisterAsset(FMetaSoundAssetKey Key)
	{
		const FAssetManagerTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
		TransactionBuffer->AddTransaction(FAssetManagerTransaction(FAssetManagerTransaction::ETransactionType::AssetUnregistration, Key, Timestamp));
	}
} // namespace Metasound::Frontend
