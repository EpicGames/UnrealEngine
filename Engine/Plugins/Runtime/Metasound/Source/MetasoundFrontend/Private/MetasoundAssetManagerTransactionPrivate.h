// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundAssetKey.h"
#include "MetasoundAssetManagerTransaction.h"
#include "MetasoundFrontendRegistryTransaction.h"


namespace Metasound::Frontend
{
	using FAssetManagerTransactionBuffer = TTransactionBuffer<FAssetManagerTransaction>;
	using FAssetManagerTransactionStream = TTransactionStream<FAssetManagerTransaction>;

	class FAssetManagerTransactor : public IMetaSoundAssetTransactor
	{
	public:
		FAssetManagerTransactor();

		TUniquePtr<FAssetManagerTransactionStream> CreateTransactionStream();
		virtual void RegisterAsset(FMetaSoundAssetKey Key) override;
		virtual void UnregisterAsset(FMetaSoundAssetKey Key) override;

	private:
		TSharedRef<FAssetManagerTransactionBuffer> TransactionBuffer;
	};
} // namespace Metasound::Frontend