// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundAssetKey.h"
#include "UObject/Object.h"

#include "MetasoundAssetTagCollections.generated.h"


// Minimal Json-serializable arrays of asset tag data. Helper struct for asset
// tag serialization of collection properties.
USTRUCT()
struct FMetaSoundAssetTagCollections
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FMetaSoundAssetKey> AssetKeys;
#endif // WITH_EDITORONLY_DATA
};


// Minimal Json-serializable arrays of asset class tag data. Helper struct for asset
// tag serialization of collection properties.
USTRUCT()
struct FMetaSoundAssetTagClassCollections
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FMetasoundFrontendInterfaceMetadata> DefinedInterfaces;

	UPROPERTY()
	TArray<FMetasoundFrontendVersion> InheritedInterfaces;
#endif // WITH_EDITORONLY_DATA
};
