// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAutoConverterNode.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace AutoConverterNodePrivate
	{
		constexpr int32 AutoConverterNodeMajorVersion = 1;
		constexpr int32 AutoConverterNodeMinorVersion = 0;

		FVertexName GetInputName(const FConvertDataTypeInfo& InInfo)
		{
			return InInfo.FromDataTypeName;
		}

		FVertexName GetOutputName(const FConvertDataTypeInfo& InInfo)
		{
			return InInfo.ToDataTypeName;
		}

		FVertexInterface CreateVertexInterface(const FConvertDataTypeInfo& InInfo)
		{
			const FText InputDesc = METASOUND_LOCTEXT_FORMAT("AutoConvDisplayNamePatternFrom", "Input {0} value.", InInfo.FromDataTypeText);
			const FText OutputDesc = METASOUND_LOCTEXT_FORMAT("AutoConvDisplayNamePatternTo", "Output {0} value.", InInfo.ToDataTypeText);

			return FVertexInterface(
				FInputVertexInterface(
					FInputDataVertex(GetInputName(InInfo), InInfo.FromDataTypeName, FDataVertexMetadata{ InputDesc }, EVertexAccessType::Reference)
				),
				FOutputVertexInterface(
					FOutputDataVertex(GetOutputName(InInfo), InInfo.ToDataTypeName, FDataVertexMetadata{ OutputDesc }, EVertexAccessType::Reference)
				)
			);
		}

		FNodeClassName CreateAutoConverterNodeClassName(const FName& InFromDataTypeName, const FName& InToDataTypeName)
		{
			return FNodeClassName{ TEXT("Convert"), InToDataTypeName, InFromDataTypeName };
		}

		Frontend::FNodeClassRegistryKey CreateAutoConverterNodeClassRegistryKey(const FName& InFromDataType, const FName& InToDataType)
		{
			return Frontend::FNodeClassRegistryKey{EMetasoundFrontendClassType::External, CreateAutoConverterNodeClassName(InFromDataType, InToDataType), AutoConverterNodeMajorVersion, AutoConverterNodeMinorVersion};
		}

		FNodeClassMetadata CreateAutoConverterNodeMetadata(const FConvertDataTypeInfo& InInfo)
		{
			FNodeDisplayStyle DisplayStyle;
			DisplayStyle.bShowName = false;
			DisplayStyle.ImageName = TEXT("MetasoundEditor.Graph.Node.Conversion");
			DisplayStyle.bShowInputNames = false;
			DisplayStyle.bShowOutputNames = false;

			const FText FromTypeText = InInfo.FromDataTypeText;
			const FText ToTypeText = InInfo.ToDataTypeText;

			FNodeClassMetadata Info;
			Info.ClassName = CreateAutoConverterNodeClassName(InInfo.FromDataTypeName, InInfo.ToDataTypeName);
			Info.MajorVersion = AutoConverterNodeMajorVersion;
			Info.MinorVersion = AutoConverterNodeMinorVersion;
			Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_AutoConverterNodeDisplayNameFormat", "{0} to {1}", InInfo.FromDataTypeText, InInfo.ToDataTypeText);
			Info.Description = METASOUND_LOCTEXT_FORMAT("Metasound_AutoConverterNodeDescriptionNameFormat", "Converts from {0} to {1}.", InInfo.FromDataTypeText, InInfo.ToDataTypeText);
			Info.Author = PluginAuthor;
			Info.DisplayStyle = DisplayStyle;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = CreateVertexInterface(InInfo);

			Info.CategoryHierarchy.Emplace(NodeCategories::Conversions);
			if (InInfo.bIsFromEnum || InInfo.bIsToEnum)
			{
				Info.CategoryHierarchy.Emplace(NodeCategories::EnumConversions);
			}
			
			Info.Keywords =
			{
				METASOUND_LOCTEXT("MetasoundConvertKeyword", "Convert"),
				InInfo.FromDataTypeText, 
				InInfo.ToDataTypeText
			};

			return Info;
		}
	}
}

#undef LOCTEXT_NAMESPACE
