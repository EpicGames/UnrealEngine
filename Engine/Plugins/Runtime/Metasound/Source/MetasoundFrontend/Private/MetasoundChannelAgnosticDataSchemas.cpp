// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundChannelAgnosticDataSchemas.h"

#include "Containers/UnrealString.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontend/Private/MetasoundJsonBackend.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "StructDeserializer.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	static FAutoConsoleCommand DumpDataTypes(TEXT("au.metasound.dump_frontend_datatypes"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]
	{
		using namespace Metasound::Frontend;
		TArray<FName> Names;
		IDataTypeRegistry::Get().GetRegisteredDataTypeNames(Names);
		for (FName i : Names)
		{
			FDataTypeRegistryInfo Info;
			IDataTypeRegistry::Get().GetDataTypeInfo(i, Info);
			UE_LOGF(LogMetaSound, Display, "GetDataTypeInfo: %ls, bIsPolymorphic=%d, bIsAbstract=%d, Base=%ls",
				*i.ToString(), (int32)Info.bIsPolymorphic,(int32)Info.bIsAbstract, *Info.ParentDataTypeName.ToString()  );
		}
	}));

	static FAutoConsoleCommand DumpPolyTypes(TEXT("au.metasound.dump_poly_types"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]
	{
		using namespace Metasound;
	
		TArray<FPolyTypeInfo> All;
		FPolyRegistry::Get().GetAllRegisteredTypes(All);

		for (const FPolyTypeInfo& i : All)
		{
			UE_LOGF(LogMetaSound, Display, "PolyType: %ls, bIsPolymorphic=%d, Base=%ls",
				*i.TypeName.ToString(), (int32)i.bIsPolymorphic, *i.BaseTypeName.ToString());
		}
	}));

	// Brute this for now.
	TOptional<ESpeakerShortNames> LexFromString(const FString& InName)
	{
		for (int32 i = static_cast<int32>(ESpeakerShortNames::DefaultChannel); i < static_cast<int32>(ESpeakerShortNames::NumChannels); i++ )
		{
			const FString SpeakerName = LexToString(static_cast<ESpeakerShortNames>(i));
			if (InName.Equals(SpeakerName, ESearchCase::IgnoreCase))
			{
				return static_cast<ESpeakerShortNames>(i);
			}
		}
		// fail.
		return {};
	}

	static FString MakePrettyString(const TArray<Audio::FDiscreteChannelTypeFamily::FSpeaker>& InSpeakers)
	{
		return FString::JoinBy(InSpeakers,TEXT(", "), [](const Audio::FDiscreteChannelTypeFamily::FSpeaker InSpeaker) -> FString
		{
			return FString::Printf(TEXT("[Name=%s,Az=%2.2f, El=%2.2f]"), *InSpeaker.ID.ToString(), InSpeaker.AzimuthDegrees,
				InSpeaker.ElevationDegrees);
		});
	}

	// Metasound Discrete, ingests the Schema.
	class FMetasoundDiscreteChannelTypeFamily final : public Audio::FDiscreteChannelTypeFamily
	{
	public:
		using Super = FDiscreteChannelTypeFamily;
		static TArray<FSpeaker> MakeOrderFromSchema(const FCatDiscreteDataSchema& InSchema)
		{
			TArray<FSpeaker> Speakers;
			for (const FCatDiscreteChannel& i : InSchema.Speakers)
			{
				Speakers.Emplace(
					FSpeaker
					{
						.ID = i.ID,
						.AzimuthDegrees = i.AzimuthDegrees,
						.ElevationDegrees = i.ElevationDegrees,
						.bIsSpatialized = i.bIsSpatialized,
					});
			}
			
			// Make sure if we're using the "ChannelEnum" as the order, everything can resolve to an enum entry.
			auto NotConvertableToShortSpeakerName = [](const FSpeaker& i) -> bool { return !NameToShortSpeakerName(i.ID).IsSet(); };
			if (InSchema.OrderPolicy == ECatDiscreteOrderPolicy::ChannelEnum && 
				Speakers.FindByPredicate(NotConvertableToShortSpeakerName) == nullptr)
			{
				auto WavSortOrder = [](const FSpeaker& A,const FSpeaker& B) -> bool 
				{
					TOptional<ESpeakerShortNames> SpeakerA = NameToShortSpeakerName(A.ID);
					TOptional<ESpeakerShortNames> SpeakerB = NameToShortSpeakerName(B.ID);
					ensure(SpeakerA.IsSet());
					ensure(SpeakerB.IsSet());
					return static_cast<uint32>(*SpeakerA) < static_cast<uint32>(*SpeakerB);
				};
				Speakers.Sort(WavSortOrder);
			}
			else if (InSchema.OrderPolicy == ECatDiscreteOrderPolicy::Explicit)
			{	
				auto IndexOf = [&InSchema](const FName InName) -> int32 { return InSchema.Order.IndexOfByKey(InName); };
				auto ExplicitSortOrder = [&IndexOf](const FSpeaker& A,const FSpeaker& B) -> bool
					{
						return IndexOf(A.ID) < IndexOf(B.ID);
					};
				Speakers.Sort(ExplicitSortOrder);
			}
			else
			{
				checkNoEntry();
			}
			return Speakers;
		}

		FMetasoundDiscreteChannelTypeFamily(const FCatDiscreteDataSchema& InSchema, Audio::IChannelTypeRegistry& InRegistry)
			: Super(InSchema.Name, InRegistry.FindChannel(InSchema.BaseType), *InSchema.FriendlyName, MakeOrderFromSchema(InSchema), InSchema.bIsParentsDefault, InSchema.bIsAbstract)
		{
			UE_LOGF(LogMetaSound, Verbose, "Discrete: Unique=%ls\tNumChannels=%d\tParent=%ls\tFriendlyName=%ls\tDefault=%ls\tOrder=[%ls]\tAbstract=%ls",
				*InSchema.Name.ToString(),
				Order.Num(),
				*InSchema.BaseType.ToString(),
				*InSchema.FriendlyName,
				ToCStr(LexToString(InSchema.bIsParentsDefault)),
				*MakePrettyString(Order),
				ToCStr(LexToString(InSchema.bIsAbstract)));
		}
	};

	class FMetasoundSoundfieldChannelTypeFamily final : public Audio::FSoundfieldChannelTypeFamily
	{
	public:
		using Super = FSoundfieldChannelTypeFamily;
		
		FMetasoundSoundfieldChannelTypeFamily(const FCatSoundfieldDataSchema& InSchema, Audio::IChannelTypeRegistry& InRegistry)
			: Super(InSchema.Name,  InSchema.NumOrders, InRegistry.FindChannel(InSchema.BaseType), *InSchema.FriendlyName, InSchema.bIsParentsDefault, InSchema.bIsAbstract)
		{
			UE_LOGF(LogMetaSound, Verbose, "Soundfield: Unique=%ls\tParent=%ls\tFriendlyName=%ls\tOrder=%d\tDefault=%ls\tAbstract=%ls",
				*InSchema.Name.ToString(),
				*InSchema.BaseType.ToString(),
				*InSchema.FriendlyName,
				GetAmbisonicsOrder(),
				ToCStr(LexToString(InSchema.bIsParentsDefault)),
				ToCStr(LexToString(InSchema.bIsAbstract)));
		}
	};

	class FMetasoundCompositeChannelTypeFamily final : public Audio::FCompositeChannelTypeFamily
	{
	public:
		using Super = FCompositeChannelTypeFamily;

		FMetasoundCompositeChannelTypeFamily(const FCatCompositeDataSchema& InSchema, Audio::IChannelTypeRegistry& InRegistry)
			: Super(InSchema.Name,InRegistry.FindChannel(InSchema.BaseType), *InSchema.FriendlyName, InSchema.bIsParentsDefault)
		{
			UE_LOGF(LogMetaSound, Verbose, "Composite: Unique=%ls\tParent=%ls\tFriendlyName=%ls\tDefault=%ls\tAbstract=%ls",
				*InSchema.Name.ToString(),
				*InSchema.BaseType.ToString(),
				*InSchema.FriendlyName,
				ToCStr(LexToString(InSchema.bIsParentsDefault)),
				ToCStr(LexToString(InSchema.bIsAbstract)));
		}	
	};

	template <typename DataType, EVertexAccessType VertexAccess = EVertexAccessType::Reference>
	class TPolyInputNodeOperatorFactory : public IOperatorFactory
	{
		static constexpr bool bIsReferenceVertexAccess = VertexAccess == EVertexAccessType::Reference;
		static constexpr bool bIsValueVertexAccess = VertexAccess == EVertexAccessType::Value;

		static_assert(bIsValueVertexAccess || bIsReferenceVertexAccess, "Unsupported EVertexAccessType");

		// Choose which data reference type is created based on template parameters
		using FPassThroughDataReference = std::conditional_t<bIsReferenceVertexAccess, TDataReadReference<DataType>, TDataValueReference<DataType>>;

		// Return correct data reference type based on vertex access type for pass through scenario.
		FPassThroughDataReference CreatePassThroughDataReference(const FAnyDataReference& InRef)
		{
			if constexpr (bIsReferenceVertexAccess)
			{
				return InRef.GetDataReadReference<DataType>();
			}
			else if constexpr (bIsValueVertexAccess)
			{
				return InRef.GetDataValueReference<DataType>();
			}
			else
			{
				static_assert("Unsupported EVertexAccessType");
			}
		}

	public:
		explicit TPolyInputNodeOperatorFactory(const FVertexName& InVertexName)
			: VertexName(InVertexName)
		{
		}

		virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
		{
			using namespace MetasoundInputNodePrivate;

			if (const FAnyDataReference* Ref = InParams.InputData.FindDataReference(VertexName))
			{
				if constexpr (bIsReferenceVertexAccess)
				{
					if (EDataReferenceAccessType::Write == Ref->GetAccessType())
					{
						return MakeUnique<TNonOwningInputOperator<DataType, VertexAccess>>(VertexName, Ref->GetDataWriteReference<DataType>());
					}
				}
				// Pass through input value
				return MakeUnique<TPassThroughOperator<DataType, VertexAccess>>(VertexName, CreatePassThroughDataReference(*Ref));
			}
			else
			{
				// Owned input value
				return MakeUnique<TInputOperator<DataType, VertexAccess>>(VertexName, InParams.OperatorSettings, InParams.InputData);
			}
		}

	private:
		FVertexName VertexName;
	};

	// Polymorphic Specific INPUT NODE.
	template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
	class TPolyInputNode : public FInputNode
	{
		static constexpr bool bIsConstructorInput = VertexAccess == EVertexAccessType::Value;
		static constexpr bool bIsSupportedConstructorInput = TIsConstructorVertexSupported<DataType>::Value && bIsConstructorInput;
		static constexpr bool bIsReferenceInput = VertexAccess == EVertexAccessType::Reference;
		static constexpr bool bIsSupportedReferenceInput = TLiteralTraits<DataType>::bIsParsableFromAnyLiteralType && bIsReferenceInput;

		static constexpr bool bIsSupportedInput = bIsSupportedConstructorInput || bIsSupportedReferenceInput;

	public:
		// If true, this node can be instantiated by the Frontend.
		static constexpr bool bCanRegister = bIsSupportedInput;
		
		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InVertexName, const FName InConcreteTypeName)
		{
			return FInputNode::CreateNodeClassMetadata(InVertexName, InConcreteTypeName, VertexAccess);
		}


		explicit TPolyInputNode(FInputNodeConstructorParams&& InParams)
		:	FInputNode(MoveTemp(InParams), InParams.PolymorphicTypeName, VertexAccess, MakeShared<TPolyInputNodeOperatorFactory<DataType, VertexAccess>>(InParams.VertexName))
		{
		}

		explicit TPolyInputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		: FInputNode(MakeShared<TPolyInputNodeOperatorFactory<DataType, VertexAccess>>(InVertexName), MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		{
		}
	};

	template<typename SchemaType, typename CppChannelType, typename CppDataType>
	void AddCatDataFormat(const SchemaType& InSchema)
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		using FBaseType = CppDataType;

		struct FDataDrivenEntry : IDataTypeRegistryEntry 
		{
			static FDataTypeRegistryInfo MakeInfo(const SchemaType& InSchema)
			{
				FDataTypeRegistryInfo Info;

				Info.DataTypeName =  InSchema.Name; 

				// The preferred constructor argument type for creating instances of the data type.
				Info.PreferredLiteralType = ELiteralType::String;

				// Constructor argument support in TDataTypeLiteralFactory<TDataType>;
				Info.bIsParsable = true;
				Info.bIsDefaultParsable = false;
				Info.bIsBoolParsable = false;
				Info.bIsIntParsable = false;
				Info.bIsFloatParsable = false;
				Info.bIsStringParsable = true;
				Info.bIsProxyParsable = false;
				Info.bIsUniquePtrProxyParsable_DEPRECATED = false;
				Info.bIsDefaultArrayParsable = false;
				Info.bIsBoolArrayParsable = false;
				Info.bIsIntArrayParsable = false;
				Info.bIsFloatArrayParsable = false;
				Info.bIsStringArrayParsable = false;
				Info.bIsProxyArrayParsable = false;
				Info.bIsUniquePtrProxyArrayParsable_DEPRECATED = false;

				// Is a TEnum wrapped enum
				Info.bIsEnum = false;

				// Whether exact types are required
				Info.bIsExplicit = false;

				// Determines whether the type can be used with send/receive transmitters
				Info.bIsTransmittable = false;

				// Returns if DataType is a Variable type
				Info.bIsVariable = false;

				// Returns if DataType can be used for constructor vertices.
				Info.bIsConstructorType = false;

				// Returns if DataType represents an array type (ex. TArray<float>, TArray<int32>, etc.).
				Info.bIsArrayType = false;

				// Returns if DataType supports array parsing and passing array of base type to constructor.
				Info.bIsArrayParseable = false;

				// If provided in registration call, UClass this datatype was registered with.
				Info.ProxyGeneratorClass = nullptr;
			
				// If this type is polymorphic.
				Info.bIsPolymorphic = true;
				Info.bIsAbstract = InSchema.bIsAbstract;

				// If this type is polymorphic, what is its parent.
				Info.ParentDataTypeName = InSchema.BaseType;
			
				return Info;
			}

			void InitNodeClasses()
			{
				const FName DataTypeName = Info.DataTypeName; 
				const FName UnnamedVertex;

				if (Info.bIsParsable)
				{
					const FNodeClassMetadata InputMeta  = FInputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Reference);
					const FNodeClassMetadata OutputMeta = FOutputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Reference);
					const FNodeClassMetadata LiteralMeta = FLiteralNode::GetNodeMetadata(DataTypeName, EVertexAccessType::Reference);

					using namespace Metasound::VariableNames;

					// Variable.
					FNodeClassMetadata VarMeta = TVariableNode<FBaseType>::CreateNodeClassMetadata();                      
					VarMeta.ClassName = GetVariableNodeClassName(DataTypeName);                                           

					// Mutator.
					FNodeClassMetadata MutMeta = MetasoundVariableNodesPrivate::TVariableMutatorOperator<FBaseType>::GetNodeInfo();
					MutMeta.ClassName = GetVariableMutatorNodeClassName(DataTypeName);                                   

					// Variable accessor
					FNodeClassMetadata AccMeta = MetasoundVariableNodesPrivate::TVariableAccessorOperator<FBaseType>::GetNodeInfo();
					AccMeta.ClassName = GetVariableAccessorNodeClassName(DataTypeName);                                 

					// Deffered.
					FNodeClassMetadata DefMeta = MetasoundVariableNodesPrivate::TVariableDeferredAccessorOperator<FBaseType>::GetNodeInfo();
					DefMeta.ClassName = GetVariableDeferredAccessorNodeClassName(DataTypeName);
					InitNodeClassesBase(InputMeta, OutputMeta, LiteralMeta, VarMeta, MutMeta, AccMeta, DefMeta);
				}

				if (Info.bIsConstructorType)
				{
					InitNodeClassesBaseConstructor(
						FInputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Value),	
						FOutputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Value)
					);
				}
			}

			void InitRawAssignmentFunction()
			{
				if constexpr (TIsCopyable<CppDataType>::Value)
				{
					this->RawAssignmentFunction = [](const void* Src, void* Dest)
						{
							*(static_cast<FBaseType*>(Dest)) = *(static_cast<const FBaseType*>(Src));
						};
				}
			}

			void InitLiteralAssignmentFunction()
			{
				if constexpr (TIsCopyable<CppDataType>::Value)
				{
					if (Info.bIsParsable)
					{
						this->LiteralAssignmentFunction = [](const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral, const FAnyDataReference& OutDataRef)
							{
								*OutDataRef.GetWritableValue<FBaseType>() = TDataTypeLiteralFactory<FBaseType>::CreateExplicitArgs(InOperatorSettings, InLiteral);
							};
					}
				}
			}

			void InitPolymorphicInterface(const SchemaType& InSchema)
			{
				this->PolymorphicInterface = MakeShared<FPolymorphicDataTypeBase>(InSchema.Name, InSchema.BaseType);
			}
	

			FDataDrivenEntry(const SchemaType& InSchema)
				: Info(MakeInfo(InSchema))
			{
				InitNodeClasses();
				InitRawAssignmentFunction();
				InitLiteralAssignmentFunction();
				InitPolymorphicInterface(InSchema);
			}

			virtual ~FDataDrivenEntry() override = default;
		
			virtual TUniquePtr<IDataTypeRegistryEntry> Clone() const override
			{
				// The outer reference is cloned, this type is non-copyable.
				checkNoEntry();
				return {};
			}
		
			virtual const FDataTypeRegistryInfo& GetDataTypeInfo() const override
			{
				return Info;
			}
			virtual TSharedPtr<const IEnumDataTypeInterface> GetEnumInterface() const override
			{
				return {}; 
			}
			virtual TSharedPtr<IPolymorphicDataTypeInterface> GetPolymorphicInterface() const override
			{
				return PolymorphicInterface;;
			}

			virtual const FMetasoundFrontendClass& GetFrontendInputClass() const override
			{
				return InputClass;
			}

			virtual TSharedPtr<const Metasound::FNodeClassMetadata> GetInputClassMetadata() const override
			{
				return InputClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendConstructorInputClass() const override
			{
				return ConstructorInputClass;
			}

			virtual TSharedPtr<const Metasound::FNodeClassMetadata> GetConstructorInputClassMetadata() const override
			{
				return ConstructorInputClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendLiteralClass() const override
			{
				return LiteralClass;
			}

			virtual const FMetasoundFrontendClass& GetFrontendOutputClass() const override
			{
				return OutputClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetOutputClassMetadata() const override
			{
				return OutputClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendConstructorOutputClass() const override
			{
				return ConstructorOutputClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetConstructorOutputClassMetadata() const override
			{
				return ConstructorOutputClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendVariableClass() const override
			{
				return VariableClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetVariableClassMetadata() const override
			{
				return VariableClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendVariableMutatorClass() const override
			{
				return VariableMutatorClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetVariableMutatorClassMetadata() const override
			{
				return VariableMutatorClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendVariableAccessorClass() const override
			{
				return VariableAccessorClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetVariableAccessorClassMetadata() const override
			{
				return VariableAccessorClassMetadata;
			}

			virtual const FMetasoundFrontendClass& GetFrontendVariableDeferredAccessorClass() const override
			{
				return VariableDeferredAccessorClass;
			}

			virtual TSharedPtr<const FNodeClassMetadata> GetVariableDeferredAccessorClassMetadata() const override
			{
				return VariableDeferredAccessorClassMetadata;
			}

			virtual const IParameterAssignmentFunction& GetRawAssignmentFunction() const override
			{
				return RawAssignmentFunction;
			}

			virtual FLiteralAssignmentFunction GetLiteralAssignmentFunction() const override
			{
				return LiteralAssignmentFunction;
			}

			virtual TUniquePtr<INode> CreateOutputNode(FNodeData InNodeData) const override
			{
				if (OutputClassMetadata)
				{
					const FOutputVertexInterface& Outputs = InNodeData.Interface.GetOutputInterface();
					if (ensure(Outputs.Num() == 1))
					{
						const FVertexName& VertexName = Outputs.At(0).VertexName;
						return MakeUnique<FOutputNode>(VertexName, MoveTemp(InNodeData), OutputClassMetadata.ToSharedRef());
					}
				}

				return TUniquePtr<INode>(nullptr);
			}

			virtual TUniquePtr<INode> CreateConstructorOutputNode(FNodeData InNodeData) const override
			{
				if (ConstructorOutputClassMetadata)
				{
					const FOutputVertexInterface& Outputs = InNodeData.Interface.GetOutputInterface();
					if (ensure(Outputs.Num() == 1))
					{
						const FVertexName& VertexName = Outputs.At(0).VertexName;
						return MakeUnique<FOutputNode>(VertexName, MoveTemp(InNodeData), ConstructorOutputClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			FORCENOINLINE void InitNodeClassesBase(
				const FNodeClassMetadata& InInputClassMetadata,
				const FNodeClassMetadata& InOutputClassMetadata,
				const FNodeClassMetadata& InLiteralPrototypeMetadata,
				const FNodeClassMetadata& InVariableClassMetadata,
				const FNodeClassMetadata& InVariableMutatorClassMetadata,
				const FNodeClassMetadata& InVariableAccessorClassMetadata,
				const FNodeClassMetadata& InVariableDeferredAccessorClassMetadata
			)
			{
				this->InputClassMetadata = MakeShared<FNodeClassMetadata>(InInputClassMetadata);
				this->InputClass = GenerateClass(*this->InputClassMetadata, EMetasoundFrontendClassType::Input);

				this->OutputClassMetadata = MakeShared<Metasound::FNodeClassMetadata>(InOutputClassMetadata);
				this->OutputClass = GenerateClass(*this->OutputClassMetadata, EMetasoundFrontendClassType::Output);

				this->LiteralClass = GenerateClass(InLiteralPrototypeMetadata, EMetasoundFrontendClassType::Literal);

				this->VariableClassMetadata = MakeShared<FNodeClassMetadata>(InVariableClassMetadata);
				this->VariableClass = GenerateClass(*this->VariableClassMetadata, EMetasoundFrontendClassType::Variable);

				this->VariableMutatorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableMutatorClassMetadata);
				this->VariableMutatorClass = GenerateClass(*this->VariableMutatorClassMetadata, EMetasoundFrontendClassType::VariableMutator);

				this->VariableAccessorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableAccessorClassMetadata);
				this->VariableAccessorClass = GenerateClass(*this->VariableAccessorClassMetadata, EMetasoundFrontendClassType::VariableAccessor);

				this->VariableDeferredAccessorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableDeferredAccessorClassMetadata);
				this->VariableDeferredAccessorClass = GenerateClass(*this->VariableDeferredAccessorClassMetadata, EMetasoundFrontendClassType::VariableDeferredAccessor);
			}

			FORCENOINLINE void InitNodeClassesBaseConstructor(
				const FNodeClassMetadata& InConstructorInputClassMetadata,
				const FNodeClassMetadata& InConstructorOutputClassMetadata
			)
			{
				this->ConstructorInputClassMetadata = MakeShared<Metasound::FNodeClassMetadata>(InConstructorInputClassMetadata);
				this->ConstructorInputClass = Metasound::Frontend::GenerateClass(*this->ConstructorInputClassMetadata, EMetasoundFrontendClassType::Input);
				this->ConstructorOutputClassMetadata = MakeShared<Metasound::FNodeClassMetadata>(InConstructorOutputClassMetadata);
				this->ConstructorOutputClass = Metasound::Frontend::GenerateClass(*this->ConstructorOutputClassMetadata, EMetasoundFrontendClassType::Output);
			}


			virtual TUniquePtr<INode> CreateInputNode(Metasound::FNodeData InNodeData) const override
			{
				if (InputClassMetadata)
				{
					const FInputVertexInterface& Inputs = InNodeData.Interface.GetInputInterface();
					if (ensure(Inputs.Num() == 1))
					{
						const FVertexName& VertexName = Inputs.At(0).VertexName;
						return MakeUnique<TPolyInputNode<FBaseType, EVertexAccessType::Reference>>(VertexName, MoveTemp(InNodeData), InputClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			using TDataType = FBaseType;

			virtual TUniquePtr<INode> CreateConstructorInputNode(FNodeData InNodeData) const override
			{
				if (Info.bIsParsable && Info.bIsConstructorType)
				{
					if (ConstructorInputClassMetadata)
					{
						const FInputVertexInterface& Inputs = InNodeData.Interface.GetInputInterface();
						if (ensure(Inputs.Num() == 1))
						{
							const FVertexName& VertexName = Inputs.At(0).VertexName;
							return MakeUnique<TPolyInputNode<TDataType, EVertexAccessType::Value>>(
								VertexName, MoveTemp(InNodeData), ConstructorInputClassMetadata.ToSharedRef());
						}
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			virtual TUniquePtr<INode> CreateVariableNode(FLiteral InLiteral, FNodeData InNodeData) const override
			{
				if (Info.bIsParsable)
				{
					if (VariableClassMetadata)
					{
						return MakeUnique<TVariableNode<TDataType>>(MoveTemp(InLiteral), MoveTemp(InNodeData), VariableClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			virtual TUniquePtr<INode> CreateVariableMutatorNode(FNodeData InNodeData) const override
			{
				if (Info.bIsParsable)
				{
					if (VariableMutatorClassMetadata)
					{
						return MakeUnique<TVariableMutatorNode<TDataType>>(MoveTemp(InNodeData), VariableMutatorClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			virtual TUniquePtr<INode> CreateVariableAccessorNode(FNodeData InNodeData) const override
			{
				if (Info.bIsParsable)
				{
					if (VariableAccessorClassMetadata)
					{
						return MakeUnique<TVariableAccessorNode<TDataType>>(MoveTemp(InNodeData), VariableAccessorClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			virtual TUniquePtr<INode> CreateVariableDeferredAccessorNode(FNodeData InNodeData) const override
			{
				if (Info.bIsParsable)
				{
					if (VariableDeferredAccessorClassMetadata)
					{
						return MakeUnique<TVariableDeferredAccessorNode<TDataType>>(MoveTemp(InNodeData), VariableDeferredAccessorClassMetadata.ToSharedRef());
					}
				}
				return TUniquePtr<INode>(nullptr);
			}

			virtual TOptional<FAnyDataReference> CreateDataReference(EDataReferenceAccessType InAccessType, const FLiteral& InLiteral,
																	 const FOperatorSettings& InOperatorSettings) const override
			{
				using enum EDataReferenceAccessType;;
				switch (InAccessType)
				{
				case Read: return  FAnyDataReference(TDataReadReference<CppDataType>::CreateNew(InOperatorSettings, Info.DataTypeName));
				case Write: return FAnyDataReference(TDataWriteReference<CppDataType>::CreateNew(InOperatorSettings, Info.DataTypeName));
				case Value: return FAnyDataReference(TDataValueReference<CppDataType>::CreateNew(InOperatorSettings, Info.DataTypeName));
				default:
					checkNoEntry();
				}
				return {};
			}


			virtual TSharedPtr<Audio::IProxyData> CreateProxy(UObject* InObject) const override { return {}; }
			virtual TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const Metasound::FOperatorSettings&) const override { return {}; }

			// Stubbed deprecated methods still required by interface
			virtual TUniquePtr<INode> CreateLiteralNode(Metasound::FLiteralNodeConstructorParams&&) const override { return {}; }
			virtual TUniquePtr<INode> CreateReceiveNode(const Metasound::FNodeInitData&) const override { return {}; }

			FDataTypeRegistryInfo Info;
			FMetasoundFrontendClass InputClass;
			TSharedPtr<const FNodeClassMetadata> InputClassMetadata;
			FMetasoundFrontendClass ConstructorInputClass;
			TSharedPtr<const FNodeClassMetadata> ConstructorInputClassMetadata;
			FMetasoundFrontendClass OutputClass;
			TSharedPtr<const FNodeClassMetadata> OutputClassMetadata;
			FMetasoundFrontendClass ConstructorOutputClass;
			TSharedPtr<const FNodeClassMetadata> ConstructorOutputClassMetadata;

			FMetasoundFrontendClass LiteralClass;

			FMetasoundFrontendClass VariableClass;
			TSharedPtr<const FNodeClassMetadata> VariableClassMetadata;
			FMetasoundFrontendClass VariableMutatorClass;
			TSharedPtr<const FNodeClassMetadata> VariableMutatorClassMetadata;
			FMetasoundFrontendClass VariableAccessorClass;
			TSharedPtr<const FNodeClassMetadata> VariableAccessorClassMetadata;
			FMetasoundFrontendClass VariableDeferredAccessorClass;
			TSharedPtr<const FNodeClassMetadata> VariableDeferredAccessorClassMetadata;
			TSharedPtr<IEnumDataTypeInterface> EnumInterface;
			TSharedPtr<IPolymorphicDataTypeInterface> PolymorphicInterface;
			IParameterAssignmentFunction RawAssignmentFunction;
			FLiteralAssignmentFunction LiteralAssignmentFunction = nullptr;
		};

		ensure(Audio::GetChannelRegistry().RegisterType(InSchema.Name, MakeUnique<CppChannelType>(InSchema, Audio::GetChannelRegistry())));		
		ensure(IDataTypeRegistry::Get().RegisterDataType(MakeUnique<FDataDrivenEntry>(InSchema)));
	}

	template<typename SchemaType>
	bool BuildSchemaLoadOrder(const TMap<FName, SchemaType>& InSchemas, 
		TArray<FName>& OutLoadOrder, FName& OutProblemNode, FName& OutProblemParent, TArray<FName>& OutCyclicNodes)
	{
		OutLoadOrder.Reset();
		OutCyclicNodes.Reset();
		OutProblemNode = {};
		OutProblemParent = {};
		
		TMap<FName, TArray<FName>> ChildrenMap;
		TMap<FName, int32> Degree;
		ChildrenMap.Reserve(InSchemas.Num());
		Degree.Reserve(InSchemas.Num());
		
		for (auto i : InSchemas)
		{
			Degree.Add(i.Key, 0);
			ChildrenMap.FindOrAdd(i.Key);
		}
		for (auto i : InSchemas)
		{
			const FName Child = i.Key;
			const FName Parent = i.Value.BaseType;

			// No parent?
			if (Parent.IsNone())
			{
				continue;
			}
			// In Loading Q?
			else if (InSchemas.Contains(Parent))
			{
				ChildrenMap.FindOrAdd(Parent).Add(Child);
				++Degree.FindChecked(Child);
			}
			// Already registered as a type?
			else if (!Audio::GetChannelRegistry().FindChannel(Parent))
			{
				OutProblemNode = Child;
				OutProblemParent = Parent;
				return false;  // fail.
			}
		}

		TArray<FName> Ready;
		Ready.Reserve(InSchemas.Num());
		for (auto i : Degree)
		{
			// Add everything with 
			if (i.Value == 0)
			{
				Ready.Add(i.Key);
			}
		}

		while (Ready.Num() > 0)
		{
			const FName Current = Ready.Pop();
			OutLoadOrder.Add(Current);
			if (const TArray<FName>* Children  = ChildrenMap.Find(Current))
			{
				for (const FName Child : *Children)
				{
					if (int32& Deg = Degree.FindChecked(Child); !--Deg)
					{
						Ready.Add(Child);
					}
				}
			}
		}

		if (OutLoadOrder.Num() != InSchemas.Num())
		{
			for (auto i : Degree)
			{
				if (i.Value > 0)
				{
					OutCyclicNodes.Add(i.Key);
				}
			}
			
			// Cyclic, fail.
			return false;
		}
		
		// Success.
		return true;
	}
	
	template<typename SchemaType, typename CppChannelType, typename CppDataType>
	TArray<FName> LoadSchemas(const FString& Path)
	{
		TArray<FString> SchemasFiles;
		IFileManager::Get().FindFiles(SchemasFiles, *Path, TEXT(".json"));
		
		TMap<FName, SchemaType> Schemas;
		for (int32 i = 0; i < SchemasFiles.Num(); i++)
		{
			const FString File = Path / SchemasFiles[i];
			if (TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*File)))
			{
				TJsonStructDeserializerBackend<DefaultCharType> Backend(*Reader);
				SchemaType Schema;
				FStructDeserializer::Deserialize(Schema, Backend);
				if (ensureMsgf(Schema.FamilyType == CppChannelType::GetFamilyTypeName(),
					TEXT("Failed load schema. File=%s, LastError=%s"), *File, *Backend.GetLastErrorMessage()))
				{
					Schema.Name = AddCatNamespaceToName(Schema.Name);
					Schema.BaseType = AddCatNamespaceToName(Schema.BaseType);
					ensure(Schemas.Find(Schema.Name) == nullptr);
					Schemas.Add(Schema.Name, Schema);
				}
			}
		}
		
		TArray<FName> Order;
		TArray<FName> CyclicNodes;
		if (FName ProblemNode, ProblemParent; !BuildSchemaLoadOrder(Schemas, Order, ProblemNode, ProblemParent, CyclicNodes))
		{
#if !NO_LOGGING
			UE_CLOGF(!ProblemNode.IsNone(), LogMetaSound, Display, "LoadChannelAgosticDataSchemas: Error failed to find parent '%ls' in schema '%ls'", 
				*ProblemParent.ToString(), *ProblemNode.ToString());	
			
			// Log each cyclic.
			if (CyclicNodes.Num() > 0)
			{
				auto WalkCycle = [&Schemas](FName Node) -> TArray<FName>
				{
					TArray<FName> Cycles;
					TSet<FName> Seen;
					while (!Seen.Contains(Node))
					{
						Cycles.Add(Node); 
						Seen.Add(Node);
						Node = Schemas[Node].BaseType;
					}
					return Cycles;
				};
				for (FName i : CyclicNodes)
				{
					TArray<FName> Cyclic = WalkCycle(i);
					UE_LOGF(LogMetaSound, Error, "LoadChannelAgosticDataSchemas: Found cyclic loop in data schema '%ls'", 
						*FString::JoinBy(Cyclic, TEXT(" -> "), [](const FName Name){ return Name.ToString(); }));
				}
			}
#endif // !NO_LOGGING

			return {};
		}
		for (const FName i : Order)
		{
			AddCatDataFormat<SchemaType,CppChannelType, CppDataType>(*Schemas.Find(i));
		}

		return Order;
	}
	
	TArray<FName> LoadChannelAgnosticDataSchemas(const FString& InContentPath)
	{
		TArray<FName> LoadedSchemas;
		const FString SchemaPath = InContentPath / TEXT("Schemas") / TEXT("ChannelAgnostic");
		LoadedSchemas.Append(LoadSchemas<FCatDiscreteDataSchema,		FMetasoundDiscreteChannelTypeFamily,	FDiscreteChannelAgnosticType>(		SchemaPath / TEXT("Discrete")));
		LoadedSchemas.Append(LoadSchemas<FCatSoundfieldDataSchema,	FMetasoundSoundfieldChannelTypeFamily,	FSoundfieldChannelAgnosticType>(	SchemaPath / TEXT("Soundfield")));
		LoadedSchemas.Append(LoadSchemas<FCatCompositeDataSchema,	FMetasoundCompositeChannelTypeFamily,	FCompositeChannelAgnosticType>(		SchemaPath / TEXT("Composite")));
		UE_LOGF(LogMetaSound, Display, "LoadChannelAgosticDataSchemas: Loaded %d schemas from [%ls]",LoadedSchemas.Num(), *SchemaPath);	
		
		return LoadedSchemas;
	}

	void UnloadChannelAgnosticDataSchemas(const TArray<FName>& InSchemas)
	{
		for (const FName i : InSchemas)
		{
			Frontend::IDataTypeRegistry::Get().UnregisterDataType(i);
			Audio::GetChannelRegistry().UnregisterType(i);
		}
	}

	// Put these here for now.
	FName AddNamespaceToName(const FName InDatatype, const FName InNamespace)
	{
		if (InNamespace != InDatatype && !InDatatype.ToString().StartsWith(InNamespace.ToString()))
		{
			return *FString::Printf(TEXT("%s%s"), *InNamespace.ToString(), *InDatatype.ToString());
		}
		return InDatatype;
	}

	FName AddCatNamespaceToName(const FName InDatatype)
	{
		static const FName Cat(TEXT("Cat:"));
		return AddNamespaceToName(InDatatype, Cat);
	}
}

#undef LOCTEXT_NAMESPACE
