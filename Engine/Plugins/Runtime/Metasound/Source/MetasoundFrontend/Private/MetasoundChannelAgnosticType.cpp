// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundChannelAgnosticType.h"
#include "TypeFamily/TypeFamily.h"
#include "TypeFamily/ChannelTypeFamily.h"
#include "MetasoundPolymorphic.h"
#include "Interfaces/IPluginManager.h"

namespace Metasound
{
	namespace ChannelAgnosticTypePrivate
	{
		static void EnsureDefaultTypesRegistered();
		
		static const FName NAME_Mono(TEXT("Cat:Mono"));
		static const FName NAME_Mono1Dot0(TEXT("Cat:Mono1Dot0"));
		static const FName NAME_Discrete(TEXT("Cat:Discrete"));
	}
	
	FName FChannelAgnosticType::GetCatDefaultCatFormat()
	{
		return FDiscreteChannelAgnosticType::GetDefaultCatFormat();
	}

	// Abstract factories.
	TSharedRef<FChannelAgnosticType, ESPMode::NotThreadSafe> FChannelAgnosticType::CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType)
	{
		// Clunky, but for now keep it simple.
		using namespace Audio;
		auto& Type = GetChannelRegistry().FindConcreteChannelChecked(InDerivedType);
		if (Type.Cast<FCompositeChannelTypeFamily>())
		{
			return FCompositeChannelAgnosticType::CreateNew(InSettings, InDerivedType);
		}
		if (Type.Cast<FSoundfieldChannelTypeFamily>())
		{
			return FSoundfieldChannelAgnosticType::CreateNew(InSettings, InDerivedType);
		}
		check(Type.Cast<FDiscreteChannelTypeFamily>());
		return FDiscreteChannelAgnosticType::CreateNew(InSettings, InDerivedType);
	}
	TSharedRef<FDiscreteChannelAgnosticType, ESPMode::NotThreadSafe> FDiscreteChannelAgnosticType::CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType)
	{
		return MakeShared<FDiscreteChannelAgnosticType, ESPMode::NotThreadSafe>(InSettings, InDerivedType);
	}
	TSharedRef<FCompositeChannelAgnosticType, ESPMode::NotThreadSafe> FCompositeChannelAgnosticType::CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType)
	{
		return MakeShared<FCompositeChannelAgnosticType, ESPMode::NotThreadSafe>(InSettings, InDerivedType); 	
	}
	TSharedRef<FSoundfieldChannelAgnosticType, ESPMode::NotThreadSafe> FSoundfieldChannelAgnosticType::CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType)
	{
		return MakeShared<FSoundfieldChannelAgnosticType, ESPMode::NotThreadSafe>(InSettings, InDerivedType); 	
	}		

	FName FSoundfieldChannelAgnosticType::GetDefaultCatFormat()
	{
		static const FName DefaultFormat(TEXT("Cat:AmbisonicsOrder2"));
		return DefaultFormat;
	}

	FName FCompositeChannelAgnosticType::GetDefaultCatFormat()
	{
		static const FName DefaultFormat(TEXT("Cat:CompositePlaceholder"));	
		return DefaultFormat;
	}
	
	FName FDiscreteChannelAgnosticType::GetDefaultCatFormat()
	{
		static const FName DefaultFormat(TEXT("Cat:Mono1Dot0"));
		if (Audio::GetChannelRegistry().FindConcreteChannel(DefaultFormat) == nullptr)
		{
			UE_LOGF(LogMetaSound, Warning, "The default channel type '%ls' is not registered. Calling EnsureDefaultTypesRegistered() ", *DefaultFormat.ToString());
			ChannelAgnosticTypePrivate::EnsureDefaultTypesRegistered();
		}
		return DefaultFormat;
	}

	// Factory helper.	
	TOptional<TDataWriteReference<FChannelAgnosticType>> FChannelAgnosticType::CreateWriteReference(const FName InDerivedType, const FOperatorSettings& InOpSettings)
	{
		using namespace Frontend;
		const FLiteral Literal(InDerivedType.ToString());
		if (FDataTypeRegistryInfo Info; IDataTypeRegistry::Get().GetDataTypeInfo(InDerivedType, Info) && Info.bIsAbstract && Info.bIsPolymorphic)
		{
			// Error: Trying to create an abstract type!
			return {};
		}
		const TOptional<FAnyDataReference> Ref = IDataTypeRegistry::Get().CreateDataReference(InDerivedType, EDataReferenceAccessType::Write, Literal, InOpSettings);
		if (Ref.IsSet())
		{
			return Ref->GetAs<TDataWriteReference<FChannelAgnosticType>>();
		}
		return {};
	}

	


	FChannelAgnosticType::FChannelAgnosticType(const FOperatorSettings& InSettings, const FName& InChannelTypeName)
		: FChannelAgnosticType(InSettings.GetNumFramesPerBlock(), InChannelTypeName)
	{}

	TOptional<TDataWriteReference<FDiscreteChannelAgnosticType>> FDiscreteChannelAgnosticType::CreateWriteReference(const FName InDerivedType, const FOperatorSettings& InOpSettings)
	{
		using namespace Frontend;
		const FLiteral Literal(InDerivedType.ToString());
		if (FDataTypeRegistryInfo Info; IDataTypeRegistry::Get().GetDataTypeInfo(InDerivedType, Info) && Info.bIsAbstract && Info.bIsPolymorphic)
		{
			// Error: Trying to create an abstract type!
			return {};
		}
		const TOptional<FAnyDataReference> Ref = IDataTypeRegistry::Get().CreateDataReference(InDerivedType, EDataReferenceAccessType::Write, Literal, InOpSettings);
		if (Ref.IsSet())
		{
			return Ref->GetAs<TDataWriteReference<FDiscreteChannelAgnosticType>>();
		}
		return {};
	}

	FChannelAgnosticType::FChannelAgnosticType(const int32 InNumFramesPerBlock, const FName& InChannelTypeName)
		: Super(Audio::GetChannelRegistry().FindConcreteChannelChecked(InChannelTypeName), InNumFramesPerBlock)
	{}
	
	static_assert(CHasPolymorphicTypeGetter<FChannelAgnosticType> && TPolymorphicTraits<FChannelAgnosticType>::bIsPolymorphic);
	static_assert(CHasFactoryFunction<FChannelAgnosticType, const FOperatorSettings&, const FName>);
	static_assert(CHasFactoryFunction<FDiscreteChannelAgnosticType, const FOperatorSettings&, const FName>);
	static_assert(CHasFactoryFunction<FCompositeChannelAgnosticType, const FOperatorSettings&, const FName>);
	static_assert(CHasFactoryFunction<FSoundfieldChannelAgnosticType, const FOperatorSettings&, const FName>);

	// Register.
	static_assert(CHasFactoryFunctionWithSettingsAndArgs<FChannelAgnosticType, const FName&>);

	static_assert(TEnableConstructorVertex<FChannelAgnosticType>::Value);
	static_assert(TLiteralTraits<FChannelAgnosticType>::bIsParsableFromAnyLiteralType);
	
	static_assert(::Metasound::TInputNode<FChannelAgnosticType, EVertexAccessType::Reference>::bCanRegister);
	static_assert(::Metasound::TInputNode<FChannelAgnosticType, EVertexAccessType::Reference>::bCanRegister);
	static_assert(::Metasound::Frontend::TMetasoundDataTypeRegistration<FChannelAgnosticType>::bCanRegister);
	
	// Registering of data type is performed in module startup because it should
	// only be registered if the MetaSoundExperimental plugin is enabled. 
	DEFINE_METASOUND_DATA_TYPE(FChannelAgnosticType, "Cat"); 
	REGISTER_METASOUND_POLY_TYPE(FChannelAgnosticType);

	static_assert(TEnableConstructorVertex<FSoundfieldChannelAgnosticType>::Value);
	static_assert(CHasFactoryFunctionWithSettings<FSoundfieldChannelAgnosticType>);
	static_assert(TLiteralTraits<FSoundfieldChannelAgnosticType>::bIsParsableFromAnyLiteralType);

	// Registering of data type is performed in module startup because it should
	// only be registered if the MetaSoundExperimental plugin is enabled. 
	DEFINE_METASOUND_DATA_TYPE(FCompositeChannelAgnosticType, "Cat:Composite");
	REGISTER_METASOUND_POLY_TYPE(FCompositeChannelAgnosticType);

	// Registering of data type is performed in module startup because it should
	// only be registered if the MetaSoundExperimental plugin is enabled. 
	DEFINE_METASOUND_DATA_TYPE(FDiscreteChannelAgnosticType, "Cat:Discrete");
	REGISTER_METASOUND_POLY_TYPE(FDiscreteChannelAgnosticType);

	// Registering of data type is performed in module startup because it should
	// only be registered if the MetaSoundExperimental plugin is enabled. 
	DEFINE_METASOUND_DATA_TYPE(FSoundfieldChannelAgnosticType, "Cat:Soundfield");
	REGISTER_METASOUND_POLY_TYPE(FSoundfieldChannelAgnosticType);
	
	namespace ChannelAgnosticTypePrivate 
	{
		void EnsureDefaultTypesRegistered()
		{
			using namespace Audio; 
			
			IChannelTypeRegistry& Registry = GetChannelRegistry();
		
			if (!Registry.FindChannel(NAME_Discrete))
			{
				RegisterCatBaseClassesWithChannelRegistry();
			}
					
			if (!Registry.FindChannel(NAME_Mono))
			{
				// Mono abstract.
				Registry.RegisterType(
					NAME_Mono,
					MakeUnique<FDiscreteChannelTypeFamily>(
						NAME_Mono,
						Registry.FindChannel(NAME_Discrete), 
						TEXT("Mono"),
						TArray<FDiscreteChannelTypeFamily::FSpeaker>(),
						true,		// Parents default
						true));		// Abstract
			}
			
			if (!Registry.FindChannel(NAME_Mono1Dot0))
			{
				// Mono (1.0)
				Registry.RegisterType(
					NAME_Mono1Dot0,
					MakeUnique<FDiscreteChannelTypeFamily>(
						NAME_Mono1Dot0,
						Registry.FindChannel(NAME_Mono), 
						TEXT("Mono (1.0)"),
						TArray
							{
								FDiscreteChannelTypeFamily::FSpeaker
								{
									.ID =TEXT("FL"),		// Note use of FL (but azimuth of center).
									.AzimuthDegrees = 0,
									.ElevationDegrees = 0,
									.bIsSpatialized = true
								}
							},
						true,			// Parents default.
						false));		// Abstract
			}
		}
	}
	
	bool IsExperimentalPluginEnabled()
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindEnabledPlugin(TEXT("MetasoundExperimental"));
		return Plugin.IsValid();
	}
}






