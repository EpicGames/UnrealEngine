// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundConverterNodeRegistrationMacro.h"

#include "MetasoundAutoConverterNode.h"
#include "MetasoundFrontendNodeClassRegistry.h"

namespace Metasound
{
	bool RegisterAutoConverterNode(const AutoConverterNodePrivate::FConvertDataTypeInfo& InConvertInfo)
	{
		using namespace AutoConverterNodePrivate;
		using namespace Frontend;

		FConverterNodeRegistryKey RegistryKey = { InConvertInfo.FromDataTypeName, InConvertInfo.ToDataTypeName };

		FConverterNodeInfo ConverterNodeInfo =
		{
			GetInputName(InConvertInfo),
			GetOutputName(InConvertInfo),
			CreateAutoConverterNodeClassRegistryKey(InConvertInfo.FromDataTypeName, InConvertInfo.ToDataTypeName)
		};

		return INodeClassRegistry::GetChecked().RegisterConversionNode(RegistryKey, ConverterNodeInfo);
	}
}
