// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDataTypeRegistrationMacro.h"

namespace Metasound::Frontend
{
	namespace MetasoundDataTypeRegistrationPrivate
	{
		void UnregisterSendAndReceiveNodes(const FName& InDataTypeName, const FModuleInfo& InModuleInfo)
		{
			UnregisterNode(SendNodePrivate::CreateNodeClassRegistryKey(InDataTypeName), InModuleInfo);
			UnregisterNode(ReceiveNodePrivate::CreateNodeClassRegistryKey(InDataTypeName), InModuleInfo);
		}

		void UnregisterAutoConverterNode(INodeClassRegistry& InRegistry, const FName& InFromDataTypeName, const FName& InToDataTypeName, const FModuleInfo& InModuleInfo)
		{
			const Metasound::Frontend::FNodeClassRegistryKey Key = AutoConverterNodePrivate::CreateAutoConverterNodeClassRegistryKey(InFromDataTypeName, InToDataTypeName);

			if (InRegistry.IsNodeRegistered(Key))
			{
				UnregisterNode(Key, InModuleInfo);
			}
			InRegistry.UnregisterConversionNode(FConverterNodeClassRegistryKey{InFromDataTypeName, InToDataTypeName}, Key);
		}

		void UnregisterAutoConverterNodes(const FName& InDataTypeName, const FModuleInfo& InModuleInfo)
		{
			INodeClassRegistry& Registry = INodeClassRegistry::GetChecked();
			// Conversions to this data type:
			UnregisterAutoConverterNode(Registry, GetMetasoundDataTypeName<bool>(), InDataTypeName, InModuleInfo);
			UnregisterAutoConverterNode(Registry, GetMetasoundDataTypeName<int32>(), InDataTypeName,  InModuleInfo);
			UnregisterAutoConverterNode(Registry, GetMetasoundDataTypeName<float>(), InDataTypeName, InModuleInfo);
			UnregisterAutoConverterNode(Registry, GetMetasoundDataTypeName<FString>(), InDataTypeName, InModuleInfo);

			// Conversions from this data type:
			UnregisterAutoConverterNode(Registry, InDataTypeName, GetMetasoundDataTypeName<bool>(), InModuleInfo);
			UnregisterAutoConverterNode(Registry, InDataTypeName, GetMetasoundDataTypeName<int32>(), InModuleInfo);
			UnregisterAutoConverterNode(Registry, InDataTypeName, GetMetasoundDataTypeName<float>(), InModuleInfo);
			UnregisterAutoConverterNode(Registry, InDataTypeName, GetMetasoundDataTypeName<FString>(), InModuleInfo);
		}

		// Base registry entry for any data type
		FDataTypeRegistryEntryBase::FDataTypeRegistryEntryBase(const Frontend::FDataTypeRegistryInfo& Info, const TSharedPtr<Frontend::IEnumDataTypeInterface>& EnumInterface)
			: Info(Info), EnumInterface(EnumInterface)
		{
		}

		FDataTypeRegistryEntryBase::~FDataTypeRegistryEntryBase() {}

		const Frontend::FDataTypeRegistryInfo& FDataTypeRegistryEntryBase::GetDataTypeInfo() const
		{
			return Info;
		}

		TSharedPtr<const Frontend::IEnumDataTypeInterface> FDataTypeRegistryEntryBase::GetEnumInterface() const
		{
			return EnumInterface;
		}

		TSharedPtr<IPolymorphicDataTypeInterface> FDataTypeRegistryEntryBase::GetPolymorphicInterface() const
		{
			return PolymorphicInterface;
		}
		
		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendInputClass() const
		{
			return InputClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetInputClassMetadata() const
		{
			return InputClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendConstructorInputClass() const
		{
			return ConstructorInputClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetConstructorInputClassMetadata() const
		{
			return ConstructorInputClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendLiteralClass() const
		{
			return LiteralClass;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendOutputClass() const
		{
			return OutputClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetOutputClassMetadata() const
		{
			return OutputClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendConstructorOutputClass() const
		{
			return ConstructorOutputClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetConstructorOutputClassMetadata() const
		{
			return ConstructorOutputClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendVariableClass() const
		{
			return VariableClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetVariableClassMetadata() const
		{
			return VariableClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendVariableMutatorClass() const
		{
			return VariableMutatorClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetVariableMutatorClassMetadata() const
		{
			return VariableMutatorClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendVariableAccessorClass() const
		{
			return VariableAccessorClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetVariableAccessorClassMetadata() const
		{
			return VariableAccessorClassMetadata;
		}

		const FMetasoundFrontendClass& FDataTypeRegistryEntryBase::GetFrontendVariableDeferredAccessorClass() const
		{
			return VariableDeferredAccessorClass;
		}

		TSharedPtr<const FNodeClassMetadata> FDataTypeRegistryEntryBase::GetVariableDeferredAccessorClassMetadata() const
		{
			return VariableDeferredAccessorClassMetadata;
		}

		const Frontend::IParameterAssignmentFunction& FDataTypeRegistryEntryBase::GetRawAssignmentFunction() const
		{
			return RawAssignmentFunction;
		}

		Frontend::FLiteralAssignmentFunction FDataTypeRegistryEntryBase::GetLiteralAssignmentFunction() const
		{
			return LiteralAssignmentFunction;
		}

		TUniquePtr<INode> FDataTypeRegistryEntryBase::CreateOutputNode(FNodeData InNodeData) const
		{
			if (OutputClassMetadata)
			{
				const FOutputVertexInterface& Outputs = InNodeData.Interface.GetOutputInterface();
				if (ensure(Outputs.Num() == 1))
				{
					const FVertexName& VertexName = Outputs.At(0).VertexName;
					return MakeUnique<FOutputNode>(VertexName, MoveTemp(InNodeData), OutputClassMetadata.ToSharedRef());
				}
			}

			return TUniquePtr<INode>(nullptr);
		}
			
		TUniquePtr<INode> FDataTypeRegistryEntryBase::CreateConstructorOutputNode(FNodeData InNodeData) const
		{
			if (ConstructorOutputClassMetadata)
			{
				const FOutputVertexInterface& Outputs = InNodeData.Interface.GetOutputInterface();
				if (ensure(Outputs.Num() == 1))
				{
					const FVertexName& VertexName = Outputs.At(0).VertexName;
					return MakeUnique<FOutputNode>(VertexName, MoveTemp(InNodeData), ConstructorOutputClassMetadata.ToSharedRef());
				}
			}
			return TUniquePtr<INode>(nullptr);
		}

		FORCENOINLINE void FDataTypeRegistryEntryBase::InitNodeClassesBase(
			const FNodeClassMetadata& InInputClassMetadata,
			const FNodeClassMetadata& InOutputClassMetadata,
			const FNodeClassMetadata& InLiteralPrototypeMetadata,
			const FNodeClassMetadata& InVariableClassMetadata,
			const FNodeClassMetadata& InVariableMutatorClassMetadata,
			const FNodeClassMetadata& InVariableAccessorClassMetadata,
			const FNodeClassMetadata& InVariableDeferredAccessorClassMetadata
		)
		{
			this->InputClassMetadata = MakeShared<FNodeClassMetadata>(InInputClassMetadata);
			this->InputClass = Metasound::Frontend::GenerateClass(*this->InputClassMetadata, EMetasoundFrontendClassType::Input);

			this->OutputClassMetadata = MakeShared<FNodeClassMetadata>(InOutputClassMetadata);
			this->OutputClass = Metasound::Frontend::GenerateClass(*this->OutputClassMetadata, EMetasoundFrontendClassType::Output);

			this->LiteralClass = Metasound::Frontend::GenerateClass(InLiteralPrototypeMetadata, EMetasoundFrontendClassType::Literal);

			this->VariableClassMetadata = MakeShared<FNodeClassMetadata>(InVariableClassMetadata);
			this->VariableClass = Metasound::Frontend::GenerateClass(*this->VariableClassMetadata, EMetasoundFrontendClassType::Variable);

			this->VariableMutatorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableMutatorClassMetadata);
			this->VariableMutatorClass = Metasound::Frontend::GenerateClass(*this->VariableMutatorClassMetadata, EMetasoundFrontendClassType::VariableMutator);

			this->VariableAccessorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableAccessorClassMetadata);
			this->VariableAccessorClass = Metasound::Frontend::GenerateClass(*this->VariableAccessorClassMetadata, EMetasoundFrontendClassType::VariableAccessor);

			this->VariableDeferredAccessorClassMetadata = MakeShared<FNodeClassMetadata>(InVariableDeferredAccessorClassMetadata);
			this->VariableDeferredAccessorClass = Metasound::Frontend::GenerateClass(*this->VariableDeferredAccessorClassMetadata, EMetasoundFrontendClassType::VariableDeferredAccessor);
		}

		FORCENOINLINE void FDataTypeRegistryEntryBase::InitNodeClassesBaseConstructor(
			const FNodeClassMetadata& InConstructorInputClassMetadata,
			const FNodeClassMetadata& InConstructorOutputClassMetadata
		)
		{
			this->ConstructorInputClassMetadata = MakeShared<FNodeClassMetadata>(InConstructorInputClassMetadata);
			this->ConstructorInputClass = Metasound::Frontend::GenerateClass(*this->ConstructorInputClassMetadata, EMetasoundFrontendClassType::Input);
			this->ConstructorOutputClassMetadata = MakeShared<FNodeClassMetadata>(InConstructorOutputClassMetadata);
			this->ConstructorOutputClass = Metasound::Frontend::GenerateClass(*this->ConstructorOutputClassMetadata, EMetasoundFrontendClassType::Output);
		}

		bool UnregisterDataTypeWithFrontendInternal(const FName& InDataTypeName, const FModuleInfo& InModuleInfo)
		{
			UnregisterSendAndReceiveNodes(InDataTypeName, InModuleInfo);
			UnregisterAutoConverterNodes(InDataTypeName, InModuleInfo);
			return Frontend::IDataTypeRegistry::Get().UnregisterDataType(InDataTypeName);
		}

	}
}
