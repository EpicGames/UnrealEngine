// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicInterfaceNodeConfiguration.h"

#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundVertex.h"

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundDynamicInterfaceNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FNodeClassRegistryKey NodeClassKey(InNodeClass.Metadata);

	// Primary path: look up FClassInterface from the registry
	FClassInterface ClassInterface;
	bool bFound = INodeClassRegistry::GetChecked().FindClassInterface(NodeClassKey, ClassInterface);

	if (!bFound)
	{
		UE_LOG(LogMetaSound, Error, TEXT("Failed to find interface for node class %s while attempting to configure interface."), *NodeClassKey.ToString());
		return {};
	}

	if (ClassInterface.ContainsSubInterfaces() || ClassInterface.ContainsVariants())
	{
		FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(SubInterfaceCounts, VariantSelections);
		return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(VertexInterface));
	}

	// If we get here, a FMetaSoundDynamicInterfaceNodeConfiguration is attached to a node
	// whose FClassInterface has no sub-interfaces or variants. This is a
	// configuration mismatch.
	UE_LOG(LogMetaSound, Warning, TEXT("FMetaSoundDynamicInterfaceNodeConfiguration attached to node class %s which has no sub-interfaces or variants. Configuration will have no effect."), *NodeClassKey.ToString());
	return {};
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundDynamicInterfaceNodeConfiguration::GetOperatorData() const
{
	// No additional operator data needed. Sub-interface layouts are carried
	// on the FVertexInterface produced by FGraphBuilder via FClassInterface.
	return nullptr;
}
