// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundFrontend.h"

#include "Delegates/IDelegateInstance.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundFrontendNodeClassRegistryPrivate.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendRegistryContainer.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundJsonBackend.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"
#include "Modules/ModuleManager.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateAudioAnalyzer.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Serialization/MemoryReader.h"
#include "Templates/UniquePtr.h"


namespace Metasound
{
	namespace Frontend
	{
		FMetasoundFrontendClass GenerateClass(const FNodeClassMetadata& InNodeMetadata, EMetasoundFrontendClassType ClassType)
		{
			FMetasoundFrontendClass ClassDescription;

			ClassDescription.Metadata = FMetasoundFrontendClassMetadata::GenerateClassMetadata(InNodeMetadata, ClassType);
			ClassDescription.SetDefaultInterface(FMetasoundFrontendClassInterface::GenerateClassInterface(InNodeMetadata.DefaultInterface.CreateVertexInterface()));
#if WITH_EDITORONLY_DATA
			ClassDescription.Style = FMetasoundFrontendClassStyle::GenerateClassStyle(InNodeMetadata.DisplayStyle);
#endif // WITH_EDITORONLY_DATA

			return ClassDescription;
		}

		FMetasoundFrontendClass GenerateClass(const FNodeRegistryKey& InKey)
		{
			FMetasoundFrontendClass OutClass;

			FMetasoundFrontendRegistryContainer* Registry = FMetasoundFrontendRegistryContainer::Get();

			if (ensure(nullptr != Registry))
			{
				bool bSuccess = Registry->FindFrontendClassFromRegistered(InKey, OutClass);
				ensureAlwaysMsgf(bSuccess, TEXT("Cannot generate description of unregistered node [RegistryKey:%s]"), *InKey.ToString());
			}

			return OutClass;
		}

		bool ImportJSONToMetasound(const FString& InJSON, FMetasoundFrontendDocument& OutMetasoundDocument)
		{
			TArray<uint8> ReadBuffer;
			ReadBuffer.SetNumUninitialized(InJSON.Len() * sizeof(ANSICHAR));
			FMemory::Memcpy(ReadBuffer.GetData(), StringCast<ANSICHAR>(*InJSON).Get(), InJSON.Len() * sizeof(ANSICHAR));
			FMemoryReader MemReader(ReadBuffer);

			TJsonStructDeserializerBackend<DefaultCharType> Backend(MemReader);
			bool DeserializeResult = FStructDeserializer::Deserialize(OutMetasoundDocument, Backend);

			MemReader.Close();
			return DeserializeResult && !MemReader.IsError();
		}

		bool ImportJSONAssetToMetasound(const FString& InPath, FMetasoundFrontendDocument& OutMetasoundDocument)
		{
			if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InPath)))
			{
				TJsonStructDeserializerBackend<DefaultCharType> Backend(*FileReader);
				bool DeserializeResult = FStructDeserializer::Deserialize(OutMetasoundDocument, Backend);

				FileReader->Close();
				return DeserializeResult && !FileReader->IsError();
			}

			return false;
		}

		// This method is here to support back compatability for node registration.
		// Originally, nodes did not take in a FVertexInterface on construction,
		// but with the introduction of node configuration in 5.6, they now do take
		// in a FVertexInterface. 
		FDataVertexMetadata CreateVertexMetadata(const FMetasoundFrontendClassVertex& InClassVertex)
		{
#if WITH_EDITORONLY_DATA
			return FDataVertexMetadata{ InClassVertex.Metadata.GetDescription(), InClassVertex.Metadata.GetDisplayName(), InClassVertex.Metadata.bIsAdvancedDisplay };
#else
			return {};
#endif
		}

		FVertexInterface CreateDefaultVertexInterfaceFromClassInternal(const FMetasoundFrontendClass& InNodeClass, const bool bCreateProxies, const FProxyDataCache* InProxyDataCache = nullptr)
		{
			FVertexInterface Interface;

			FInputVertexInterface& Inputs = Interface.GetInputInterface();
			for (const FMetasoundFrontendClassInput& ClassInput : InNodeClass.GetDefaultInterface().Inputs)
			{
				const FMetasoundFrontendLiteral* DefaultLiteral = ClassInput.FindConstDefault(DefaultPageID);
				if (DefaultLiteral)
				{
					EMetasoundFrontendLiteralType LiteralType = DefaultLiteral->GetType();
					if (!bCreateProxies &&
						((EMetasoundFrontendLiteralType::UObject == LiteralType) || (EMetasoundFrontendLiteralType::UObjectArray == LiteralType)))
					{
						UE_LOGF(LogMetaSound, Warning, "Ignoring default literal set on vertex %ls of node %ls. Please update construct of node to use FNodeData", *ClassInput.Name.ToString(), *InNodeClass.Metadata.GetClassName().ToString());
						DefaultLiteral = nullptr;
					}
				}

				if (DefaultLiteral)
				{
					const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();
					FLiteral Literal = DefaultLiteral->ToLiteral(ClassInput.TypeName, &DataTypeRegistry, InProxyDataCache);

					Inputs.Add(
						FInputDataVertex
						{
							ClassInput.Name,
							ClassInput.TypeName,
							CreateVertexMetadata(ClassInput),
							FrontendVertexAccessTypeToCoreVertexAccessType(ClassInput.AccessType),
							Literal
						});
				}
				else
				{
					// No default literal or proxy creation was skipped for object literals
					Inputs.Add(
						FInputDataVertex
						{
							ClassInput.Name,
							ClassInput.TypeName,
							CreateVertexMetadata(ClassInput),
							FrontendVertexAccessTypeToCoreVertexAccessType(ClassInput.AccessType)
						});
				}
			}

			FOutputVertexInterface& Outputs = Interface.GetOutputInterface();
			for (const FMetasoundFrontendClassOutput& ClassOutput : InNodeClass.GetDefaultInterface().Outputs)
			{
				Outputs.Add(
					FOutputDataVertex
					{
						ClassOutput.Name,
						ClassOutput.TypeName,
						CreateVertexMetadata(ClassOutput),
						FrontendVertexAccessTypeToCoreVertexAccessType(ClassOutput.AccessType)
					});
			}

			FEnvironmentVertexInterface& Environments = Interface.GetEnvironmentInterface();
			for (const FMetasoundFrontendClassEnvironmentVariable& ClassEnvironment : InNodeClass.GetDefaultInterface().Environment)
			{
				Environments.Add(FEnvironmentVertex{ ClassEnvironment.Name, FText::GetEmpty() });
			}

			return Interface;
		}

		FVertexInterface CreateDefaultVertexInterfaceFromClass(const FMetasoundFrontendClass& InNodeClass, const Metasound::Frontend::FProxyDataCache* InProxyDataCache)
		{
			checkf(InProxyDataCache || IsInGameThread(), TEXT("Vertex interface creation without proxy data cache must occur on game thread to safely create UObject proxies. Populate the proxy data cache on the game thread before calling this or use CreateVertexInterfaceFromClassNoProxy."))
				return CreateDefaultVertexInterfaceFromClassInternal(InNodeClass, /*bCreateProxies=*/true, InProxyDataCache);
		}

		FVertexInterface CreateDefaultVertexInterfaceFromClassNoProxy(const FMetasoundFrontendClass& InNodeClass)
		{
			return CreateDefaultVertexInterfaceFromClassInternal(InNodeClass, /*bCreateProxies=*/false);
		}
	}
}

class FMetasoundFrontendModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		RegisterNodeTemplate(MakeUnique<FAudioAnalyzerNodeTemplate>());
		RegisterNodeTemplate(MakeUnique<FInputNodeTemplate>());
		RegisterNodeTemplate(MakeUnique<FRerouteNodeTemplate>());
		METASOUND_REGISTER_ITEMS_IN_MODULE

#if WITH_EDITORONLY_DATA
		FVersioningManager::Initialize();
#endif // WITH_EDITORONLY_DATA

		FModuleManager::LoadModuleChecked<IModuleInterface>("AudioChannelAgnosticCore");
		OnPluginUnmountedHandle = IPluginManager::Get().OnPluginUnmounted().AddLambda([](IPlugin& Plugin)
		{
			FNodeClassRegistry::Get().NotifyPluginUnmounted(Plugin);
		});

		// Only register/unregister CAT types if the experimental plugin is enabled. 
		if (IsExperimentalPluginEnabled())
		{
			RegisterDataType<FChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			RegisterDataType<FCompositeChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			RegisterDataType<FDiscreteChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			RegisterDataType<FSoundfieldChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
		}
	}

	virtual void ShutdownModule() override
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		IPluginManager::Get().OnPluginUnmounted().Remove(OnPluginUnmountedHandle);

#if WITH_EDITORONLY_DATA
		FVersioningManager::Shutdown();
#endif // WITH_EDITORONLY_DATA

		METASOUND_UNREGISTER_ITEMS_IN_MODULE
		UnregisterNodeTemplate(FAudioAnalyzerNodeTemplate::ClassName, FAudioAnalyzerNodeTemplate::VersionNumber);
		UnregisterNodeTemplate(FInputNodeTemplate::ClassName, FInputNodeTemplate::VersionNumber);
		UnregisterNodeTemplate(FRerouteNodeTemplate::ClassName, FRerouteNodeTemplate::VersionNumber);

		// Only register/unregister CAT types if the experimental plugin is enabled. 
		if (IsExperimentalPluginEnabled())
		{
			UnregisterDataType<FChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			UnregisterDataType<FCompositeChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			UnregisterDataType<FDiscreteChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
			UnregisterDataType<FSoundfieldChannelAgnosticType>(METASOUND_GET_MODULE_LIST_INFO);
		}
	}

private:
	FDelegateHandle OnPluginUnmountedHandle;
};

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(FMetasoundFrontendModule, MetasoundFrontend);
