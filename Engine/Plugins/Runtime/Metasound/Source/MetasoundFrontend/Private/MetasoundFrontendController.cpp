// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendController.h"

#include "MetasoundFrontendInvalidController.h"
#include "MetasoundFrontendDocumentController.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"


namespace Metasound
{
	namespace Frontend
	{
		namespace ControllerPrivate
		{
			// This check for class name is a hack due to the fact that handles do not provide a builder
			// in order to do a proper look-up of a rerouted output's access/data type. Once calling systems of the
			// recurse functions utilizing this function are refactored to use the builder API, this can be removed.
			bool IsRerouteClass(const FMetasoundFrontendClassName& ClassName)
			{
				return ClassName == FRerouteNodeTemplate::ClassName || ClassName == FInputNodeTemplate::ClassName;
			}
		}

		FOutputHandle IOutputController::GetInvalidHandle()
		{
			static FOutputHandle Invalid = MakeShared<FInvalidOutputController>();
			return Invalid;
		}

		FInputHandle IInputController::GetInvalidHandle()
		{
			static FInputHandle Invalid = MakeShared<FInvalidInputController>();
			return Invalid;
		}

		FVariableHandle IVariableController::GetInvalidHandle()
		{
			static FVariableHandle Invalid = MakeShared<FInvalidVariableController>();
			return Invalid;
		}

		FNodeHandle INodeController::GetInvalidHandle()
		{
			static FNodeHandle Invalid = MakeShared<FInvalidNodeController>();
			return Invalid;
		}

		const FMetasoundFrontendGraphClassPresetOptions& IGraphController::GetGraphPresetOptions() const
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				return Invalid::GetInvalidGraphClassPresetOptions();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		void IGraphController::SetGraphPresetOptions(const FMetasoundFrontendGraphClassPresetOptions& InPresetOptions)
		{
		}

		const TSet<FName>& IGraphController::GetInputsInheritingDefault() const
		{
			return Invalid::GetInvalidNameSet();
		}

		bool IGraphController::SetInputInheritsDefault(FName InName, bool bDefaultIsInherited)
		{
			return false;
		}

		void IGraphController::SetInputsInheritingDefault(TSet<FName>&& InNames)
		{
		}

		FGraphHandle IGraphController::GetInvalidHandle()
		{
			static FGraphHandle Invalid = MakeShared<FInvalidGraphController>();
			return Invalid;
		}

		FDocumentHandle IDocumentController::GetInvalidHandle()
		{
			static FDocumentHandle Invalid = MakeShared<FInvalidDocumentController>();
			return Invalid;
		}
			
		FDocumentHandle IDocumentController::CreateDocumentHandle(FDocumentAccessPtr InDocument, bool bReloadBuilder)
		{
			// Create using standard document controller.
			return FDocumentController::CreateDocumentHandle(InDocument, bReloadBuilder);
		}

		FDocumentHandle IDocumentController::CreateDocumentHandle(FMetasoundFrontendDocument& InDocument)
		{
			return CreateDocumentHandle(MakeAccessPtr<FDocumentAccessPtr>(InDocument.AccessPoint, InDocument));
		}

		FConstDocumentHandle IDocumentController::CreateDocumentHandle(FConstDocumentAccessPtr InDocument)
		{
			// Create using standard document controller. 
			return FDocumentController::CreateDocumentHandle(InDocument);
		}

		FConstDocumentHandle IDocumentController::CreateDocumentHandle(const FMetasoundFrontendDocument& InDocument)
		{
			return CreateDocumentHandle(MakeAccessPtr<FConstDocumentAccessPtr>(InDocument.AccessPoint, InDocument));
		}

		FDocumentAccess IDocumentAccessor::GetSharedAccess(IDocumentAccessor& InDocumentAccessor)
		{
			return InDocumentAccessor.ShareAccess();
		}

		FConstDocumentAccess IDocumentAccessor::GetSharedAccess(const IDocumentAccessor& InDocumentAccessor)
		{
			return InDocumentAccessor.ShareAccess();
		}

		FConstOutputHandle FindReroutedOutput(FConstOutputHandle InOutputHandle)
		{
			if (InOutputHandle->IsValid())
			{
				FConstNodeHandle NodeHandle = InOutputHandle->GetOwningNode();
				if (NodeHandle->IsValid())
				{
					const FMetasoundFrontendClassName& ClassName = NodeHandle->GetClassMetadata().GetClassName();
					if (ControllerPrivate::IsRerouteClass(ClassName))
					{
						TArray<FConstInputHandle> Inputs = NodeHandle->GetConstInputs();
						if (!Inputs.IsEmpty())
						{
							FConstInputHandle RerouteInputHandle = Inputs.Last();
							if (RerouteInputHandle->IsValid())
							{
								FConstOutputHandle ConnectedOutputHandle = RerouteInputHandle->GetConnectedOutput();
								return FindReroutedOutput(ConnectedOutputHandle);
							}
						}
					}
				}
			}

			return InOutputHandle;
		}

		void FindReroutedInputs(FConstInputHandle InHandleToCheck, TArray<FConstInputHandle>& InOutInputHandles)
		{
			using namespace ControllerPrivate;

			if (InHandleToCheck->IsValid())
			{
				FConstNodeHandle NodeHandle = InHandleToCheck->GetOwningNode();
				if (NodeHandle->IsValid())
				{
					if (ControllerPrivate::IsRerouteClass(NodeHandle->GetClassMetadata().GetClassName()))
					{
						TArray<FConstOutputHandle> Outputs = NodeHandle->GetConstOutputs();
						for (FConstOutputHandle& OutputHandle : Outputs)
						{
							TArray<FConstInputHandle> LinkedInputs = OutputHandle->GetConstConnectedInputs();
							for (FConstInputHandle LinkedInput : LinkedInputs)
							{
								FindReroutedInputs(LinkedInput, InOutInputHandles);
							}
						}

						return;
					}

					InOutInputHandles.Add(InHandleToCheck);
				}

			}
		}

		void IterateReroutedInputs(FConstInputHandle InHandleToCheck, TFunctionRef<void(FConstInputHandle)> Func)
		{
			if (InHandleToCheck->IsValid())
			{
				FConstNodeHandle NodeHandle = InHandleToCheck->GetOwningNode();
				if (NodeHandle->IsValid())
				{
					if (ControllerPrivate::IsRerouteClass(NodeHandle->GetClassMetadata().GetClassName()))
					{
						TArray<FConstOutputHandle> Outputs = NodeHandle->GetConstOutputs();
						for (FConstOutputHandle& OutputHandle : Outputs)
						{
							TArray<FConstInputHandle> LinkedInputs = OutputHandle->GetConstConnectedInputs();
							for (FConstInputHandle LinkedInput : LinkedInputs)
							{
								IterateReroutedInputs(LinkedInput, Func);
							}
						}

						return;
					}

					Func(InHandleToCheck);
				}

			}
		}
	} // namespace Frontend
} // namespace Metasound
