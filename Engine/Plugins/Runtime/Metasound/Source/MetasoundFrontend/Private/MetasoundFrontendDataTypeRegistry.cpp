// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFrontendDataTypeRegistry.h"

#include "Containers/UnrealString.h"
#include "MetasoundLiteral.h"
#include "MetasoundTrace.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/App.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace MetasoundFrontendDataTypeRegistryPrivate
		{
			// Return the compatible literal with the most descriptive type.
			// TODO: Currently TIsParsable<> allows for implicit conversion of
			// constructor arguments of integral types which can cause some confusion
			// here when trying to match a literal type to a constructor. For example:
			//
			// struct FBoolConstructibleType
			// {
			// 	FBoolConstructibleType(bool InValue);
			// };
			//
			// static_assert(TIsParsable<FBoolConstructible, double>::Value); 
			//
			// Implicit conversions are currently allowed in TIsParsable because this
			// is perfectly legal syntax.
			//
			// double Value = 10.0;
			// FBoolConstructibleType BoolConstructible = Value;
			//
			// There are some tricks to possibly disable implicit conversions when
			// checking for specific constructors, but they are yet to be implemented 
			// and are untested. Here's the basic idea.
			//
			// template<DataType, DesiredIntegralArgType>
			// struct TOnlyConvertIfIsSame
			// {
			// 		// Implicit conversion only defined if types match.
			// 		template<typename SuppliedIntegralArgType, std::enable_if<std::is_same<std::decay<SuppliedIntegralArgType>::type, DesiredIntegralArgType>::value, int> = 0>
			// 		operator DesiredIntegralArgType()
			// 		{
			// 			return DesiredIntegralArgType{};
			// 		}
			// };
			//
			// static_assert(false == std::is_constructible<FBoolConstructibleType, TOnlyConvertIfSame<double>>::value);
			// static_assert(true == std::is_constructible<FBoolConstructibleType, TOnlyConvertIfSame<bool>>::value);
			ELiteralType GetMostDescriptiveLiteralForDataType(const FDataTypeRegistryInfo& InDataTypeInfo)
			{
				if (InDataTypeInfo.bIsProxyArrayParsable)
				{
					return ELiteralType::UObjectProxyArray;
				}
				else if (InDataTypeInfo.bIsProxyParsable)
				{
					return ELiteralType::UObjectProxy;
				}
				else if (InDataTypeInfo.bIsEnum && InDataTypeInfo.bIsIntParsable)
				{
					return ELiteralType::Integer;
				}
				else if (InDataTypeInfo.bIsStringArrayParsable)
				{
					return ELiteralType::StringArray;
				}
				else if (InDataTypeInfo.bIsFloatArrayParsable)
				{
					return ELiteralType::FloatArray;
				}
				else if (InDataTypeInfo.bIsIntArrayParsable)
				{
					return ELiteralType::IntegerArray;
				}
				else if (InDataTypeInfo.bIsBoolArrayParsable)
				{
					return ELiteralType::BooleanArray;
				}
				else if (InDataTypeInfo.bIsStringParsable)
				{
					return ELiteralType::String;
				}
				else if (InDataTypeInfo.bIsFloatParsable)
				{
					return ELiteralType::Float;
				}
				else if (InDataTypeInfo.bIsIntParsable)
				{
					return ELiteralType::Integer;
				}
				else if (InDataTypeInfo.bIsBoolParsable)
				{
					return ELiteralType::Boolean;
				}
				else if (InDataTypeInfo.bIsDefaultArrayParsable)
				{
					return ELiteralType::NoneArray; 
				}
				else if (InDataTypeInfo.bIsDefaultParsable)
				{
					return ELiteralType::None;
				}
				else
				{
					// if we ever hit this, something has gone wrong with the REGISTER_METASOUND_DATATYPE macro.
					// we should have failed to compile if any of these are false.
					checkNoEntry();
					return ELiteralType::Invalid;
				}
			}

			// Base class for INodeRegistryEntrys that come from an IDataTypeRegistryEntry
			static const FClassInterface EmptyClassInterface;

			class FDataTypeNodeRegistryEntry : public INodeClassRegistryEntry
			{
			public:
				FDataTypeNodeRegistryEntry() = default;

				virtual ~FDataTypeNodeRegistryEntry() = default;

				virtual const TSet<FMetasoundFrontendVersion>* GetImplementedInterfaces() const override
				{
					return nullptr;
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					return EmptyClassInterface;
				}

				virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const override
				{
					// By default, data type related nodes do not offer any extensions. 
					return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
				}
				
				virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override
				{
					// No node configuration supported for this node type, so only compatible if setting to invalid (null) configuration
					return !InNodeConfiguration.IsValid();
				}
			};

			// Node registry entry for input nodes created from a data type registry entry.
			class FInputNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FInputNodeRegistryEntry() = delete;

				FInputNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FInputNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendInputClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateInputNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetInputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetInputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for constructor input nodes created from a data type registry entry.
			class FConstructorInputNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FConstructorInputNodeRegistryEntry() = delete;

				FConstructorInputNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FConstructorInputNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendConstructorInputClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateConstructorInputNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetConstructorInputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetConstructorInputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for output nodes created from a data type registry entry.
			class FOutputNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FOutputNodeRegistryEntry() = delete;

				FOutputNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FOutputNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendOutputClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateOutputNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetOutputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetOutputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for constructor output nodes created from a data type registry entry.
			class FConstructorOutputNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FConstructorOutputNodeRegistryEntry() = delete;

				FConstructorOutputNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FConstructorOutputNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendConstructorOutputClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateConstructorOutputNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetConstructorOutputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetConstructorOutputClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};


			// Node registry entry for literal nodes created from a data type registry entry.
			class FLiteralNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FLiteralNodeRegistryEntry() = delete;

				FLiteralNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FLiteralNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendLiteralClass();
				}

				using FDataTypeNodeRegistryEntry::CreateNode;

				virtual TUniquePtr<INode> CreateNode(FNodeData) const override
				{
					// FLiteral node should not be used.  Also cannot create it
					// with the data in FNodeData because it's missing the default
					// literal.
					return nullptr;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};


			// Node registry entry for init variable nodes created from a data type registry entry.
			class FVariableNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FVariableNodeRegistryEntry() = delete;

				FVariableNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FVariableNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendVariableClass();
				}

				using FDataTypeNodeRegistryEntry::CreateNode;

				virtual TUniquePtr<INode> CreateNode(FNodeData) const override
				{
					// Cannot create variable node with FNodeData because it's missing
					// the default literal.
					return nullptr;
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for set variable nodes created from a data type registry entry.
			class FVariableMutatorNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FVariableMutatorNodeRegistryEntry() = delete;

				FVariableMutatorNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FVariableMutatorNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendVariableMutatorClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateVariableMutatorNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableMutatorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableMutatorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for get variable nodes created from a data type registry entry.
			class FVariableAccessorNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FVariableAccessorNodeRegistryEntry() = delete;

				FVariableAccessorNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FVariableAccessorNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendVariableAccessorClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateVariableAccessorNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableAccessorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}
				
				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableAccessorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			// Node registry entry for get delayed variable nodes created from a data type registry entry.
			class FVariableDeferredAccessorNodeRegistryEntry : public FDataTypeNodeRegistryEntry
			{
			public:
				FVariableDeferredAccessorNodeRegistryEntry() = delete;

				FVariableDeferredAccessorNodeRegistryEntry(const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>& InDataTypeEntry)
				: DataTypeEntry(InDataTypeEntry)
				{
				}

				virtual ~FVariableDeferredAccessorNodeRegistryEntry() = default;

				virtual const FMetasoundFrontendClass& GetFrontendClass() const override
				{
					return DataTypeEntry->GetFrontendVariableDeferredAccessorClass();
				}

				virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
				{
					return DataTypeEntry->CreateVariableDeferredAccessorNode(MoveTemp(InNodeData));
				}

				virtual FVertexInterface GetDefaultVertexInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableDeferredAccessorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return FVertexInterface(NodeClassMetadata->DefaultInterface);
					}
					return FVertexInterface();
				}

				virtual const FClassInterface& GetClassInterface() const override
				{
					TSharedPtr<const FNodeClassMetadata> NodeClassMetadata = DataTypeEntry->GetVariableDeferredAccessorClassMetadata();
					if (ensure(NodeClassMetadata))
					{
						return NodeClassMetadata->DefaultInterface;
					}
					return EmptyClassInterface;
				}

			private:
				
				TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> DataTypeEntry;
			};

			class FDataTypeRegistry : public IDataTypeRegistry
			{
			public:
				virtual ~FDataTypeRegistry() = default;

	
				virtual bool RegisterDataType(TUniquePtr<IDataTypeRegistryEntry>&& InEntry) override;
				virtual bool UnregisterDataType(const FName& InDataTypeName) override;

				virtual void GetRegisteredDataTypeNames(TArray<FName>& OutNames) const override;

				virtual const IDataTypeRegistryEntry* FindDataTypeRegistryEntry(const FName& InDataTypeName) const override;
				virtual bool GetDataTypeInfo(const UObject* InObject, FDataTypeRegistryInfo& OutInfo) const override;
				virtual bool GetDataTypeInfo(UClass* InClass, FDataTypeRegistryInfo& OutInfo) const override;
				virtual bool GetDataTypeInfo(const FName& InDataType, FDataTypeRegistryInfo& OutInfo) const override;

				virtual void IterateDataTypeInfo(TFunctionRef<void(const FDataTypeRegistryInfo&)> InFunction) const override;

				// Return the enum interface for a data type. If the data type does not have 
				// an enum interface, returns a nullptr.
				virtual TSharedPtr<const IEnumDataTypeInterface> GetEnumInterfaceForDataType(const FName& InDataType) const override;

				virtual ELiteralType GetDesiredLiteralType(const FName& InDataType) const override;

				virtual bool IsRegistered(const FName& InDataType) const override;

				virtual bool IsLiteralTypeSupported(const FName& InDataType, ELiteralType InLiteralType) const override;
				virtual bool IsLiteralTypeSupported(const FName& InDataType, EMetasoundFrontendLiteralType InLiteralType) const override;

				virtual UClass* GetUClassForDataType(const FName& InDataType) const override;

				virtual bool IsUObjectProxyFactory(UObject* InObject) const override;
				virtual TSharedPtr<Audio::IProxyData> CreateProxyFromUObject(const FName& InDataType, UObject* InObject) const override;
				virtual bool IsValidUObjectForDataType(const FName& InDataTypeName, const UObject* InUObject) const override;

				virtual FLiteral CreateDefaultLiteral(const FName& InDataType) const override;
				virtual FLiteral CreateLiteralFromUObject(const FName& InDataType, UObject* InObject) const override;
				virtual FLiteral CreateLiteralFromUObjectArray(const FName& InDataType, const TArray<UObject*>& InObjectArray) const override;

				virtual TOptional<FAnyDataReference> CreateDataReference(const FName& InDataType, EDataReferenceAccessType InAccessType, const FLiteral& InLiteral, const FOperatorSettings& InOperatorSettings) const override;
				virtual TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FName& InDataType, const FOperatorSettings& InOperatorSettings) const override;

				virtual const IParameterAssignmentFunction& GetRawAssignmentFunction(const FName& InDataType) const override;
				virtual FLiteralAssignmentFunction GetLiteralAssignmentFunction(const FName& InDataType) const override;

				virtual bool GetFrontendInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendConstructorInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendLiteralClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendConstructorOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendVariableClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendVariableMutatorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendVariableAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual bool GetFrontendVariableDeferredAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetInputClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetConstructorInputClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetOutputClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetConstructorOutputClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetVariableClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetVariableMutatorClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetVariableAccessorClassMetadata(const FName& InDataType) const override;
				virtual TSharedPtr<const FNodeClassMetadata> GetVariableDeferredAccessorClassMetadata(const FName& InDataType) const override;

				// Create a new instance of a C++ implemented node from the registry.
				virtual TUniquePtr<INode> CreateInputNode(const FName& InInputType, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateConstructorInputNode(const FName& InInputType, FNodeData InNodeData) const override;

				UE_DEPRECATED(5.6, "Direct creation of literal nodes will no longer be supported")
				virtual TUniquePtr<INode> CreateLiteralNode(const FName& InLiteralType, FLiteralNodeConstructorParams&& InParams) const override;

				UE_DEPRECATED(5.6, "Direct creation of receive nodes will no longer be supported")
				virtual TUniquePtr<INode> CreateReceiveNode(const FName& InDataTypeName, const FNodeInitData& InParams) const override;

				virtual TUniquePtr<INode> CreateOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateConstructorOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateVariableNode(const FName& InDataType, FLiteral InLiteral, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateVariableMutatorNode(const FName& InDataType, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateVariableAccessorNode(const FName& InDataType, FNodeData InNodeData) const override;
				virtual TUniquePtr<INode> CreateVariableDeferredAccessorNode(const FName& InDataType, FNodeData InNodeData) const override;

			private:

				const IDataTypeRegistryEntry* FindDataTypeEntry(const FName& InDataTypeName) const;

				TMap<FName, TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>> RegisteredDataTypes;

				// UObject type names to DataTypeNames
				TMap<const UClass*, FName> RegisteredObjectClasses;

				mutable FTransactionallySafeCriticalSection RegistryMapMutex;
				mutable FTransactionallySafeCriticalSection RegistryObjectMapMutex;
			};

			bool FDataTypeRegistry::RegisterDataType(TUniquePtr<IDataTypeRegistryEntry>&& InEntry)
			{
				METASOUND_LLM_SCOPE;

				if (InEntry.IsValid())
				{
					TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe> Entry(InEntry.Release());

					const FName Name = Entry->GetDataTypeInfo().DataTypeName;

					{
						UE::TScopeLock RegistryLock(RegistryMapMutex);
						UE::TScopeLock ObjectRegistryLock(RegistryObjectMapMutex);

						if (!ensureAlwaysMsgf(!RegisteredDataTypes.Contains(Name),
							TEXT("Name collision when trying to register Metasound Data Type [Name:%s]. DataType must have "
								"unique name and REGISTER_METASOUND_DATATYPE cannot be called in a public header."),
								*Name.ToString()))
						{
							return false;
						}

						const FDataTypeRegistryInfo& RegistryInfo = Entry->GetDataTypeInfo();
						if (const UClass* Class = RegistryInfo.ProxyGeneratorClass)
						{
							RegisteredObjectClasses.Add(Class, Name);
						}

						RegisteredDataTypes.Add(Name, Entry);
					}

					// Register nodes associated with data type.
					INodeClassRegistry* NodeRegistry = INodeClassRegistry::Get();
					if (ensure(nullptr != NodeRegistry))
					{
						if (Entry->GetDataTypeInfo().bIsParsable)
						{
							const bool bIsAbstract = Entry->GetDataTypeInfo().bIsAbstract;;
							
							NodeRegistry->RegisterNode(MakeUnique<FInputNodeRegistryEntry>(Entry));							
							NodeRegistry->RegisterNode(MakeUnique<FOutputNodeRegistryEntry>(Entry));

							if (!bIsAbstract)
							{
								NodeRegistry->RegisterNode(MakeUnique<FLiteralNodeRegistryEntry>(Entry));
								NodeRegistry->RegisterNode(MakeUnique<FVariableNodeRegistryEntry>(Entry));
								NodeRegistry->RegisterNode(MakeUnique<FVariableMutatorNodeRegistryEntry>(Entry));
								NodeRegistry->RegisterNode(MakeUnique<FVariableAccessorNodeRegistryEntry>(Entry));
								NodeRegistry->RegisterNode(MakeUnique<FVariableDeferredAccessorNodeRegistryEntry>(Entry));
							}

							if (Entry->GetDataTypeInfo().bIsConstructorType)
							{
								NodeRegistry->RegisterNode(MakeUnique<FConstructorInputNodeRegistryEntry>(Entry));
								NodeRegistry->RegisterNode(MakeUnique<FConstructorOutputNodeRegistryEntry>(Entry));
							}
						}
					}


					UE_LOGF(LogMetaSound, Verbose, "Registered Metasound Datatype [Name:%ls].", *Name.ToString());
					return true;
				}

				return false;
			}

			bool FDataTypeRegistry::UnregisterDataType(const FName& InDataTypeName)
			{
				METASOUND_LLM_SCOPE;

				bool bSuccess = false;
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeRegistryEntry(InDataTypeName))
				{
					bSuccess = true;
					const FDataTypeRegistryInfo& DataTypeInfo = Entry->GetDataTypeInfo();

					// Unregister nodes associated with data type.
					INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();
					if (DataTypeInfo.bIsParsable)
					{
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendInputClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendOutputClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendLiteralClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendVariableClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendVariableMutatorClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendVariableAccessorClass().Metadata});
						NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendVariableDeferredAccessorClass().Metadata});

						if (DataTypeInfo.bIsConstructorType)
						{
							NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendConstructorInputClass().Metadata});
							NodeRegistry.UnregisterNode(FNodeClassRegistryKey{Entry->GetFrontendConstructorOutputClass().Metadata});
						}
					}
					
					{
						UE::TScopeLock ObjectRegistryLock(RegistryObjectMapMutex);
						if (const UClass* Class = DataTypeInfo.ProxyGeneratorClass)
						{
							bSuccess = bSuccess && RegisteredObjectClasses.Remove(Class);
						}
					}
					
					{
						UE::TScopeLock RegistryLock(RegistryMapMutex);
						bSuccess = bSuccess && RegisteredDataTypes.Remove(InDataTypeName);
					}

				}

				UE_CLOGF(bSuccess, LogMetaSound, Verbose, "Unregistered Metasound Datatype [Name:%ls].", *InDataTypeName.ToString());
				UE_CLOGF(!bSuccess, LogMetaSound, Error, "Failed to unregistered Metasound Datatype [Name:%ls].", *InDataTypeName.ToString());

				return bSuccess;
			}

			void FDataTypeRegistry::GetRegisteredDataTypeNames(TArray<FName>& OutNames) const
			{
				UE::TScopeLock Lock(RegistryMapMutex);
				RegisteredDataTypes.GetKeys(OutNames);
			}

			const IDataTypeRegistryEntry* FDataTypeRegistry::FindDataTypeRegistryEntry(const FName& InDataTypeName) const
			{
				return FindDataTypeEntry(InDataTypeName);
			}

			bool FDataTypeRegistry::GetDataTypeInfo(const UObject* InObject, FDataTypeRegistryInfo& OutInfo) const
			{
				if (InObject)
				{
					UE::TScopeLock Lock(RegistryObjectMapMutex);
					for (UClass* Class = InObject->GetClass(); Class != UObject::StaticClass(); Class = Class->GetSuperClass())
					{
						if (const FName* DataTypeName = RegisteredObjectClasses.Find(Class))
						{
							if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(*DataTypeName))
							{
								const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();
								if (Info.bIsExplicit && Class != InObject->GetClass())
								{
									return false;
								}
								OutInfo = Info;
								return true;
							}
						}
					}
				}

				return false;
			}
			
			bool FDataTypeRegistry::GetDataTypeInfo(UClass* InClass, FDataTypeRegistryInfo& OutInfo) const
			{
				if (InClass)
				{
					UE::TScopeLock Lock(RegistryObjectMapMutex);
					for (UClass* Class = InClass; Class != UObject::StaticClass(); Class = Class->GetSuperClass())
					{
						if (const FName* DataTypeName = RegisteredObjectClasses.Find(Class))
						{
							if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(*DataTypeName))
							{
								const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();
								if (Info.bIsExplicit && Class != InClass)
								{
									return false;
								}
								OutInfo = Info;
								return true;
							}
						}
					}
				}
				return false;
			}

			bool FDataTypeRegistry::GetDataTypeInfo(const FName& InDataType, FDataTypeRegistryInfo& OutInfo) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutInfo = Entry->GetDataTypeInfo();
					return true;
				}
				return false;
			}

			void FDataTypeRegistry::IterateDataTypeInfo(TFunctionRef<void(const FDataTypeRegistryInfo&)> InFunction) const
			{
				UE::TScopeLock Lock(RegistryMapMutex);
				for (const TPair<FName, TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>>& Entry : RegisteredDataTypes)
				{
					InFunction(Entry.Value->GetDataTypeInfo());
				}
			}

			bool FDataTypeRegistry::IsRegistered(const FName& InDataType) const
			{
				UE::TScopeLock Lock(RegistryMapMutex);
				return RegisteredDataTypes.Contains(InDataType);
			}

			// Return the enum interface for a data type. If the data type does not have 
			// an enum interface, returns a nullptr.
			TSharedPtr<const IEnumDataTypeInterface> FDataTypeRegistry::GetEnumInterfaceForDataType(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetEnumInterface();
				}
				return nullptr;
			}

			ELiteralType FDataTypeRegistry::GetDesiredLiteralType(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();

					// If there's a designated preferred literal type for this datatype, use that.
					if (Info.PreferredLiteralType != Metasound::ELiteralType::Invalid)
					{
						return Info.PreferredLiteralType;
					}

					// Otherwise, we opt for the highest precision construction option available.
					return MetasoundFrontendDataTypeRegistryPrivate::GetMostDescriptiveLiteralForDataType(Info);
				}
				return Metasound::ELiteralType::Invalid;
			}

			bool FDataTypeRegistry::IsLiteralTypeSupported(const FName& InDataType, ELiteralType InLiteralType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();

					switch (InLiteralType)
					{
						case Metasound::ELiteralType::Boolean:
						{
							return Info.bIsBoolParsable;
						}
						case Metasound::ELiteralType::BooleanArray:
						{
							return Info.bIsBoolArrayParsable;
						}

						case Metasound::ELiteralType::Integer:
						{
							return Info.bIsIntParsable;
						}
						case Metasound::ELiteralType::IntegerArray:
						{
							return Info.bIsIntArrayParsable;
						}

						case Metasound::ELiteralType::Float:
						{
							return Info.bIsFloatParsable;
						}
						case Metasound::ELiteralType::FloatArray:
						{
							return Info.bIsFloatArrayParsable;
						}

						case Metasound::ELiteralType::String:
						{
							return Info.bIsStringParsable;
						}
						case Metasound::ELiteralType::StringArray:
						{
							return Info.bIsStringArrayParsable;
						}

						case Metasound::ELiteralType::UObjectProxy:
						{
							return Info.bIsProxyParsable || Info.bIsUniquePtrProxyParsable_DEPRECATED;
						}
						case Metasound::ELiteralType::UObjectProxyArray:
						{
							return Info.bIsProxyArrayParsable || Info.bIsUniquePtrProxyArrayParsable_DEPRECATED;
						}

						case Metasound::ELiteralType::None:
						{
							return Info.bIsDefaultParsable;
						}
						case Metasound::ELiteralType::NoneArray:
						{
							return Info.bIsDefaultArrayParsable;
						}

						case Metasound::ELiteralType::Invalid:
						default:
						{
							static_assert(static_cast<int32>(Metasound::ELiteralType::COUNT) == 13, "Possible missing case coverage for ELiteralType");
							return false;
						}
					}
				}

				return false;
			}

			bool FDataTypeRegistry::IsLiteralTypeSupported(const FName& InDataType, EMetasoundFrontendLiteralType InLiteralType) const
			{
				return IsLiteralTypeSupported(InDataType, GetMetasoundLiteralType(InLiteralType));
			}

			UClass* FDataTypeRegistry::GetUClassForDataType(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetDataTypeInfo().ProxyGeneratorClass;
				}

				return nullptr;
			}

			FLiteral FDataTypeRegistry::CreateDefaultLiteral(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();
					if (Info.bIsEnum)
					{
						if (TSharedPtr<const IEnumDataTypeInterface> EnumInterface = Entry->GetEnumInterface())
						{
							return FLiteral(EnumInterface->GetDefaultValue());
						}
					}
					return FLiteral::GetDefaultForType(Info.PreferredLiteralType);
				}
				return FLiteral::CreateInvalid();
			}

			bool FDataTypeRegistry::IsUObjectProxyFactory(UObject* InObject) const
			{
				if (!InObject)
				{
					return false;
				}

				UE::TScopeLock Lock(RegistryObjectMapMutex);
				UClass* ObjectClass = InObject->GetClass();
				while (ObjectClass != UObject::StaticClass())
				{
					if (const FName* DataTypeName = RegisteredObjectClasses.Find(ObjectClass))
					{
						if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(*DataTypeName))
						{
							const FDataTypeRegistryInfo& Info = Entry->GetDataTypeInfo();
							return !Info.bIsExplicit || ObjectClass == InObject->GetClass();
						}
					}

					ObjectClass = ObjectClass->GetSuperClass();
				}

				return false;
			}

			TSharedPtr<Audio::IProxyData> FDataTypeRegistry::CreateProxyFromUObject(const FName& InDataType, UObject* InObject) const
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Metasound::FDataTypeRegistry::CreateProxyFromUObject Type:%s"), *InDataType.ToString()));

				TSharedPtr<Audio::IProxyData> ProxyPtr;

				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					ProxyPtr = Entry->CreateProxy(InObject);
					if (ProxyPtr)
					{
						UE_LOGF(LogMetaSound, VeryVerbose, "Created UObject proxy for '%ls'.", *InObject->GetName());
					}
					else
					{
						if (InObject && FApp::CanEverRenderAudio())
						{
							UE_LOGF(LogMetaSound, Error, "Failed to create a valid proxy from UObject '%ls'.", *InObject->GetName());
						}
					}
				}

				return ProxyPtr;
			}

			bool FDataTypeRegistry::IsValidUObjectForDataType(const FName& InDataTypeName, const UObject* InUObject) const
			{
				FDataTypeRegistryInfo DataTypeInfo;
				if (GetDataTypeInfo(InDataTypeName, DataTypeInfo))
				{
					if (DataTypeInfo.bIsProxyParsable || DataTypeInfo.bIsUniquePtrProxyParsable_DEPRECATED)
					{
						if (InUObject)
						{
							UE::TScopeLock ObjectRegistryLock(RegistryObjectMapMutex);
							UE::TScopeLock RegistryLock(RegistryMapMutex);

							const UClass* ObjectClass = InUObject->GetClass();
							while (ObjectClass != UObject::StaticClass())
							{
								if (const FName* DataTypeSupportedByUObject = RegisteredObjectClasses.Find(ObjectClass))
								{
									// If this is the specified data type and it corresponds to the given object, skip finding its registry entry.
									if (*DataTypeSupportedByUObject == InDataTypeName && ObjectClass == InUObject->GetClass())
									{
										return true;
									}

									// Find the object's data type's registry entry to determine if it must match the given object.
									if (const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>* Entry = RegisteredDataTypes.Find(*DataTypeSupportedByUObject))
									{
										const FDataTypeRegistryInfo& Info = (*Entry)->GetDataTypeInfo();
										if (Info.bIsExplicit)
										{
											return false;
										}
									}

									// If this is the specified data type, then the object is valid.
									if (*DataTypeSupportedByUObject == InDataTypeName)
									{
										return true;
									}
								}

								ObjectClass = ObjectClass->GetSuperClass();
							}
						}
						else
						{
							return true;
						}
					}
				}

				return false;
			}

			FLiteral FDataTypeRegistry::CreateLiteralFromUObject(const FName& InDataType, UObject* InObject) const
			{
				TSharedPtr<Audio::IProxyData> ProxyPtr = CreateProxyFromUObject(InDataType, InObject);
				return Metasound::FLiteral(MoveTemp(ProxyPtr));
			}

			FLiteral FDataTypeRegistry::CreateLiteralFromUObjectArray(const FName& InDataType, const TArray<UObject*>& InObjectArray) const
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Metasound::FDataTypeRegistry::CreateLiteralFromUObjectArray Type:%s"), *InDataType.ToString()));

				TArray<TSharedPtr<Audio::IProxyData>> ProxyArray;
				const IDataTypeRegistryEntry* DataTypeEntry = FindDataTypeEntry(InDataType);
				if (!DataTypeEntry)
				{
					UE_LOGF(LogMetaSound, Error, "Failed to create a valid proxy from Array DataType '%ls': Type is not registered.", *InDataType.ToString());
					return Metasound::FLiteral(MoveTemp(ProxyArray));
				}

				const FDataTypeRegistryInfo& DataTypeInfo = DataTypeEntry->GetDataTypeInfo();

				const bool bIsProxyArrayParseable = DataTypeInfo.bIsProxyArrayParsable || DataTypeInfo.bIsUniquePtrProxyArrayParsable_DEPRECATED;

				if (!bIsProxyArrayParseable)
				{
					UE_LOGF(LogMetaSound, Error, "Failed to create a valid proxy from DataType '%ls': Type is not proxy parseable.", *InDataType.ToString());
					return Metasound::FLiteral(MoveTemp(ProxyArray));
				}

				const FName ElementDataType = CreateElementTypeNameFromArrayTypeName(InDataType);
				const IDataTypeRegistryEntry* ElementEntry = FindDataTypeEntry(ElementDataType);
				if (!ElementEntry)
				{
					UE_LOGF(LogMetaSound, Error, "Failed to create a valid proxy from DataType '%ls': ElementType '%ls' is not registered.", *InDataType.ToString(), *ElementDataType.ToString());
					return Metasound::FLiteral(MoveTemp(ProxyArray));
				}

				for (UObject* InObject : InObjectArray)
				{
					TSharedPtr<Audio::IProxyData> ProxyPtr = CreateProxyFromUObject(ElementDataType, InObject);
					ProxyPtr = ElementEntry->CreateProxy(InObject);
					if (!ProxyPtr && InObject && FApp::CanEverRenderAudio())
					{
						UE_LOGF(LogMetaSound, Error, "Failed to create a valid proxy from UObject '%ls'.", *InObject->GetName());
					}

					ProxyArray.Emplace(MoveTemp(ProxyPtr));
				}

				return Metasound::FLiteral(MoveTemp(ProxyArray));
			}

			TOptional<FAnyDataReference> FDataTypeRegistry::CreateDataReference(const FName& InDataType, EDataReferenceAccessType InAccessType, const FLiteral& InLiteral, const FOperatorSettings& InOperatorSettings) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateDataReference(InAccessType, InLiteral, InOperatorSettings);
				}
				return TOptional<FAnyDataReference>();
			}

			TSharedPtr<IDataChannel, ESPMode::ThreadSafe> FDataTypeRegistry::CreateDataChannel(const FName& InDataType, const FOperatorSettings& InOperatorSettings) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateDataChannel(InOperatorSettings);
				}
				return nullptr;
			}

			const IParameterAssignmentFunction& FDataTypeRegistry::GetRawAssignmentFunction(const FName& InDataType) const
			{
				static IParameterAssignmentFunction NoOp;
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetRawAssignmentFunction();
				}
				return NoOp;
			}

			FLiteralAssignmentFunction FDataTypeRegistry::GetLiteralAssignmentFunction(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetLiteralAssignmentFunction();
				}
				return nullptr;
			}

			bool FDataTypeRegistry::GetFrontendInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendInputClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendConstructorInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendConstructorInputClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendLiteralClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendLiteralClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendOutputClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendConstructorOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendConstructorOutputClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendVariableClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendVariableClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendVariableMutatorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const

			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendVariableMutatorClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendVariableAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const

			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendVariableAccessorClass();
					return true;
				}
				return false;
			}

			bool FDataTypeRegistry::GetFrontendVariableDeferredAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const

			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					OutClass = Entry->GetFrontendVariableDeferredAccessorClass();
					return true;
				}
				return false;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetInputClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetInputClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetConstructorInputClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetConstructorInputClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetOutputClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetOutputClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetConstructorOutputClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetConstructorOutputClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetVariableClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetVariableClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetVariableMutatorClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetVariableMutatorClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetVariableAccessorClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetVariableAccessorClassMetadata();
				}
				return nullptr;
			}

			TSharedPtr<const FNodeClassMetadata> FDataTypeRegistry::GetVariableDeferredAccessorClassMetadata(const FName& InDataType) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->GetVariableDeferredAccessorClassMetadata();
				}
				return nullptr;
			}


			TUniquePtr<INode> FDataTypeRegistry::CreateInputNode(const FName& InDataType, FNodeData InNodeData) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateInputNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateConstructorInputNode(const FName& InDataType, FNodeData InNodeData) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateConstructorInputNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateLiteralNode(const FName& InDataType, FLiteralNodeConstructorParams&& InParams) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return Entry->CreateLiteralNode(MoveTemp(InParams));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateReceiveNode(const FName& InDataType, const FNodeInitData& InParams) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return Entry->CreateReceiveNode(InParams);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataTypeName))
				{
					return Entry->CreateOutputNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateConstructorOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataTypeName))
				{
					return Entry->CreateConstructorOutputNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateVariableNode(const FName& InDataType, FLiteral InLiteral, FNodeData InNodeData) const
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateVariableNode(MoveTemp(InLiteral), MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateVariableMutatorNode(const FName& InDataType, FNodeData InNodeData) const 
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateVariableMutatorNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateVariableAccessorNode(const FName& InDataType, FNodeData InNodeData) const 
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateVariableAccessorNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			TUniquePtr<INode> FDataTypeRegistry::CreateVariableDeferredAccessorNode(const FName& InDataType, FNodeData InNodeData) const 
			{
				if (const IDataTypeRegistryEntry* Entry = FindDataTypeEntry(InDataType))
				{
					return Entry->CreateVariableDeferredAccessorNode(MoveTemp(InNodeData));
				}
				return nullptr;
			}

			const IDataTypeRegistryEntry* FDataTypeRegistry::FindDataTypeEntry(const FName& InDataTypeName) const
			{
				UE::TScopeLock Lock(RegistryMapMutex);
				{
					const TSharedRef<IDataTypeRegistryEntry, ESPMode::ThreadSafe>* Entry = RegisteredDataTypes.Find(InDataTypeName);

					if (nullptr != Entry)
					{
						return &Entry->Get();
					}
					else
					{
						UE_LOGF(LogMetaSound, Error, "Data type is not registered [Name:%ls]", *InDataTypeName.ToString());
					}
				}

				return nullptr;
			}
		}

		void FPolymorphicDataTypeBase::ResolveParent(const IDataTypeRegistry& InRegistry) const
		{
			static const FName VoidString = TEXT("void");
			if (!ParentDataType.IsValid() && !ParentTypeName.IsNone() && ParentTypeName != VoidString)
			{
				if (const IDataTypeRegistryEntry* Entry = InRegistry.FindDataTypeRegistryEntry(ParentTypeName))
				{
					PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS					
					ParentDataType = Entry->GetPolymorphicInterface();
					PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS					
				}
				UE_CLOGF(!ParentDataType.IsValid(), LogMetaSound, Warning, "Failed to resolve Polymorphic Type '%ls' Parent Type '%ls'",
					*DataTypeName.ToString(), *ParentTypeName.ToString());
			}
		}

		FPolymorphicDataTypeBase::FPolymorphicDataTypeBase(const FName InDataTypeName, const FName InParentName, const bool InbIsAbstract)
			: DataTypeName(InDataTypeName), ParentTypeName(InParentName), bIsAbstract(InbIsAbstract)
		{
			UE_LOGF(LogMetaSound, Verbose, "PolymorphicInterface Type=%ls, Base=%ls, bIsAbstract=%d",
				*InDataTypeName.ToString(), *ParentTypeName.ToString(), (int32)bIsAbstract);
			
			// Register with Core Registry.
			PRAGMA_DISABLE_INTERNAL_WARNINGS
			FPolyRegistry::Get().Register(
				{
					.TypeName = InDataTypeName,
					.BaseTypeName = InParentName,
					.bIsPolymorphic = true
					}
				);
			PRAGMA_ENABLE_INTERNAL_WARNINGS			
		}

		bool FPolymorphicDataTypeBase::IsA(const FName InDataTypeName, const IDataTypeRegistry& InRegistry) const
		{
			if (DataTypeName == InDataTypeName)
			{
				return true;
			}

			// Resolve.
			ResolveParent(InRegistry);
		
			if (const TSharedPtr<const IPolymorphicDataTypeInterface> Parent = ParentDataType.Pin())
			{
				PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
				return Parent->IsA(InDataTypeName);
				PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS				
			}
			
			return false;
		}

		IDataTypeRegistry& IDataTypeRegistry::Get()
		{
			static MetasoundFrontendDataTypeRegistryPrivate::FDataTypeRegistry Registry;
			return Registry;
		}

		void CreateDefaultsInternal(const FOperatorSettings& InOperatorSettings, bool bInOverrideExistingData, FInputVertexInterfaceData& OutVertexData)
		{
			using namespace VertexDataPrivate;

			auto VertexAccessTypeToDataReferenceAccessType = [](EVertexAccessType InVertexAccessType) -> EDataReferenceAccessType
			{
				switch(InVertexAccessType)
				{
					case EVertexAccessType::Value:
						return EDataReferenceAccessType::Value;

					case EVertexAccessType::Reference:
					default:
						return EDataReferenceAccessType::Write;
				}
			};

			IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

			for (const FInputBinding& Binding : OutVertexData)
			{
				if (!bInOverrideExistingData && Binding.IsBound())
				{
					// Do not create defaults if data is already set. 
					continue;
				}

				// Attempt to create default data reference from the literal stored
				// on the input vertex.
				const FInputDataVertex& InputVertex = Binding.GetVertex();
				EDataReferenceAccessType AccessType = VertexAccessTypeToDataReferenceAccessType(InputVertex.AccessType);

				if (const IDataTypeRegistryEntry* Entry = DataTypeRegistry.FindDataTypeRegistryEntry(InputVertex.DataTypeName))
				{
					TOptional<FAnyDataReference> DataRef = Entry->CreateDataReference(AccessType, InputVertex.GetDefaultLiteral(), InOperatorSettings);

					if (DataRef.IsSet())
					{
						// Set as vertex data reference.
						OutVertexData.SetVertex(InputVertex.VertexName, *DataRef);
					}
					else
					{
						const FDataTypeRegistryInfo& DataTypeInfo = Entry->GetDataTypeInfo();
						if (DataTypeInfo.bIsParsable)
						{
							// All parsable inputs should have creatable defaults.
							UE_LOGF(LogMetaSound, Warning, "Failed to create default data reference for vertex %ls of data type %ls using constructor argument %ls", *InputVertex.VertexName.ToString(), *InputVertex.DataTypeName.ToString(), *::LexToString(InputVertex.GetDefaultLiteral()));
						}
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to create default data reference for vertex %ls of data type %ls because data type is not registered. Please ensure that the plugin which registers the data type is loaded.", *InputVertex.VertexName.ToString(), *InputVertex.DataTypeName.ToString());
				}
			}
		}

		void CreateDefaultsIfNotBound(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData)
		{
			constexpr bool bOverrideExistingData = false;
			CreateDefaultsInternal(InOperatorSettings, bOverrideExistingData, OutVertexData);
		}

		void CreateDefaults(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData)
		{
			constexpr bool bOverrideExistingData = true;
			CreateDefaultsInternal(InOperatorSettings, bOverrideExistingData, OutVertexData);
		}

	}
}
