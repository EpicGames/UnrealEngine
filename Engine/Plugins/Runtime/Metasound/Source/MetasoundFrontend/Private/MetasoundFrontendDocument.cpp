// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocument.h"

#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "CoreGlobals.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Logging/LogMacros.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundLog.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundVertex.h"

#if WITH_EDITORONLY_DATA
#include "Internationalization/Internationalization.h"
#include "Types/SlateVector2.h"
#endif // WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendDocument)

namespace Metasound
{
	const FGuid FrontendInvalidID = FGuid();

	namespace Frontend
	{
		int32 MetaSoundAutoUpdateNativeClassesOfEqualVersionCVar = 1;
		FAutoConsoleVariableRef CVarMetaSoundAutoUpdateNativeClass(
			TEXT("au.MetaSound.AutoUpdate.NativeClassesOfEqualVersion"),
			MetaSoundAutoUpdateNativeClassesOfEqualVersionCVar,
			TEXT("If true, node references to native classes that share a version number will attempt to auto-update if the interface is different, which results in slower graph load times.\n")
			TEXT("0: Don't auto-update native classes of the same version with interface discrepancies, !0: Auto-update native classes of the same version with interface discrepancies (default)"),
			ECVF_Default);

		namespace DocumentPrivate
		{
			template <typename ResolveType>
			FGuid ResolveTargetPageID(const ResolveType& InToResolve)
			{
				// Registry is not available in tests, so for now resolution is considered successful at this level
				// if registry is not initialized and providing a resolved page ID. TODO: Add a test implementation
				// that returns the default page (or whatever page behavior is desired for testing).
				if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// This deprecated function can be removed when the deprecated versions of ForEachLiteral are removed. 
					return IDocumentBuilderRegistry::GetChecked().ResolveTargetPageID(InToResolve);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}

				return Frontend::DefaultPageID;
			}
		} // namespace DocumentPrivate

#if WITH_EDITORONLY_DATA
		const FText DefaultPageDisplayName = NSLOCTEXT("MetasoundFrontend", "DefaultPageDisplayName", "Default");
#endif // WITH_EDITORONLY_DATA

		namespace DisplayStyle
		{
			namespace EdgeAnimation
			{
				const FLinearColor DefaultColor = FLinearColor::Transparent;
			}

			namespace NodeLayout
			{
				const FVector2D DefaultOffsetX { 300.0f, 0.0f };
				const FVector2D DefaultOffsetY { 0.0f, 120.0f };
			} // namespace NodeLayout
		} // namespace DisplayStyle

		namespace ClassTypePrivate
		{
			static const FString External = TEXT("External");
			static const FString Graph = TEXT("Graph");
			static const FString Input = TEXT("Input");
			static const FString Output = TEXT("Output");
			static const FString Literal = TEXT("Literal");
			static const FString Variable = TEXT("Variable");
			static const FString VariableDeferredAccessor = TEXT("Variable (Deferred Accessor)");
			static const FString VariableAccessor = TEXT("Variable (Accessor)");
			static const FString VariableMutator = TEXT("Variable (Mutator)");
			static const FString Template = TEXT("Template");
			static const FString Invalid = TEXT("Invalid");

			static const TSortedMap<FString, EMetasoundFrontendClassType, TInlineAllocator<32>> ClassTypeCStringToEnum = 
			{
				{External, EMetasoundFrontendClassType::External},
				{Graph, EMetasoundFrontendClassType::Graph},
				{Input, EMetasoundFrontendClassType::Input},
				{Output, EMetasoundFrontendClassType::Output},
				{Literal, EMetasoundFrontendClassType::Literal},
				{Variable, EMetasoundFrontendClassType::Variable},
				{VariableDeferredAccessor, EMetasoundFrontendClassType::VariableDeferredAccessor},
				{VariableAccessor, EMetasoundFrontendClassType::VariableAccessor},
				{VariableMutator, EMetasoundFrontendClassType::VariableMutator},
				{Template, EMetasoundFrontendClassType::Template},
				{Invalid, EMetasoundFrontendClassType::Invalid}
			};
		}

		EMetasoundFrontendVertexAccessType CoreVertexAccessTypeToFrontendVertexAccessType(Metasound::EVertexAccessType InAccessType)
		{
			switch (InAccessType)
			{
				case EVertexAccessType::Value:
					return EMetasoundFrontendVertexAccessType::Value;

				case EVertexAccessType::Reference:
				default:
					return EMetasoundFrontendVertexAccessType::Reference;
			}
		}

		EVertexAccessType FrontendVertexAccessTypeToCoreVertexAccessType(EMetasoundFrontendVertexAccessType InAccessType)
		{
			switch (InAccessType)
			{
				case EMetasoundFrontendVertexAccessType::Value:
					return EVertexAccessType::Value;

				case EMetasoundFrontendVertexAccessType::Reference:
				default:
					return EVertexAccessType::Reference;
			}
		}
	} // namespace Frontend

	namespace DocumentPrivate
	{
		/*
		 * Sets an array to a given array and updates the change ID if the array changed.
		 * @returns true if value changed, false if not.
		 */
		template <typename TElementType>
		bool SetWithChangeID(const TElementType& InNewValue, TElementType& OutValue, FGuid& OutChangeID)
		{
			if (OutValue != InNewValue)
			{
				OutValue = InNewValue;
				OutChangeID = FGuid::NewGuid();
				return true;
			}

			return false;
		}

		/* Array Text specialization as FText does not implement == nor does it support IsBytewiseComparable */
		template <>
		bool SetWithChangeID<TArray<FText>>(const TArray<FText>& InNewArray, TArray<FText>& OutArray, FGuid& OutChangeID)
		{
			bool bIsEqual = OutArray.Num() == InNewArray.Num();
			if (bIsEqual)
			{
				for (int32 i = 0; i < InNewArray.Num(); ++i)
				{
					bIsEqual &= InNewArray[i].IdenticalTo(OutArray[i]);
				}
			}

			if (!bIsEqual)
			{
				OutArray = InNewArray;
				OutChangeID = FGuid::NewGuid();
			}

			return !bIsEqual;
		}

		/* Text specialization as FText does not implement == nor does it support IsBytewiseComparable */
		template <>
		bool SetWithChangeID<FText>(const FText& InNewText, FText& OutText, FGuid& OutChangeID)
		{
			if (!InNewText.IdenticalTo(OutText))
			{
				OutText = InNewText;
				OutChangeID = FGuid::NewGuid();
				return true;
			}

			return false;
		}


		FName ResolveMemberDataType(FName DataType, EAudioParameterType ParamType)
		{
			if (!DataType.IsNone())
			{
				const bool bIsRegisteredType = Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(DataType);
				if (ensureAlwaysMsgf(bIsRegisteredType, TEXT("Attempting to register Interface member with unregistered DataType '%s'."), *DataType.ToString()))
				{
					return DataType;
				}
			}

			return Frontend::ConvertParameterToDataType(ParamType);
		};
	} // namespace DocumentPrivate
} // namespace Metasound


#if WITH_EDITORONLY_DATA
void FMetasoundFrontendDocumentModifyContext::ClearDocumentModified()
{
	bDocumentModified = false;
}

bool FMetasoundFrontendDocumentModifyContext::GetDocumentModified() const
{
	return bDocumentModified;
}

bool FMetasoundFrontendDocumentModifyContext::GetForceRefreshViews() const
{
	return bForceRefreshViews;
}

const TSet<FName>& FMetasoundFrontendDocumentModifyContext::GetInterfacesModified() const
{
	return InterfacesModified;
}

const TSet<FGuid>& FMetasoundFrontendDocumentModifyContext::GetMemberIDsModified() const
{
	return MemberIDsModified;
}

const TSet<FGuid>& FMetasoundFrontendDocumentModifyContext::GetNodeIDsModified() const
{
	return NodeIDsModified;
}

void FMetasoundFrontendDocumentModifyContext::Reset()
{
	bDocumentModified = false;
	bForceRefreshViews = false;
	InterfacesModified.Empty();
	MemberIDsModified.Empty();
	NodeIDsModified.Empty();
}

void FMetasoundFrontendDocumentModifyContext::SetDocumentModified()
{
	bDocumentModified = true;
}

void FMetasoundFrontendDocumentModifyContext::SetForceRefreshViews()
{
	bDocumentModified = true;
	bForceRefreshViews = true;
}

void FMetasoundFrontendDocumentModifyContext::AddInterfaceModified(FName InInterfaceModified)
{
	bDocumentModified = true;
	InterfacesModified.Add(InInterfaceModified);
}

void FMetasoundFrontendDocumentModifyContext::AddInterfacesModified(const TSet<FName>& InInterfacesModified)
{
	bDocumentModified = true;
	InterfacesModified.Append(InInterfacesModified);
}

void FMetasoundFrontendDocumentModifyContext::AddMemberIDModified(const FGuid& InMemberIDModified)
{
	bDocumentModified = true;
	MemberIDsModified.Add(InMemberIDModified);
}

void FMetasoundFrontendDocumentModifyContext::AddMemberIDsModified(const TSet<FGuid>& InMemberIDsModified)
{
	bDocumentModified = true;
	MemberIDsModified.Append(InMemberIDsModified);
}

void FMetasoundFrontendDocumentModifyContext::AddNodeIDModified(const FGuid& InNodeIDModified)
{
	bDocumentModified = true;
	NodeIDsModified.Add(InNodeIDModified);
}

void FMetasoundFrontendDocumentModifyContext::AddNodeIDsModified(const TSet<FGuid>& InNodesModified)
{
	bDocumentModified = true;
	NodeIDsModified.Append(InNodesModified);
}
#endif // WITH_EDITORONLY_DATA

FString LexToString(const EMetasoundFrontendClassAccessFlags& InFlags)
{
	switch (InFlags)
	{
		case EMetasoundFrontendClassAccessFlags::Deprecated:
			return TEXT("Deprecated");

		case EMetasoundFrontendClassAccessFlags::Referenceable:
			return TEXT("Referenceable");

		default:
			return TEXT("Unset");
	}
}

FMetasoundCommentNodeIntVector::FMetasoundCommentNodeIntVector(const FIntVector2& InValue)
	: FIntVector2(InValue)
{
}

FMetasoundCommentNodeIntVector::FMetasoundCommentNodeIntVector(const FVector2f& InValue)
	: FIntVector2(static_cast<int32>(InValue.X), static_cast<int32>(InValue.Y))
{
}

FMetasoundCommentNodeIntVector::FMetasoundCommentNodeIntVector(const FVector2d& InValue)
	: FIntVector2(static_cast<int32>(InValue.X), static_cast<int32>(InValue.Y))
{
}

#if WITH_EDITORONLY_DATA
FMetasoundCommentNodeIntVector::FMetasoundCommentNodeIntVector(const FDeprecateSlateVector2D& InValue)
	: FIntVector2(static_cast<int32>(InValue.X), static_cast<int32>(InValue.Y))
{
}
#endif // WITH_EDITORONLY_DATA

FMetasoundCommentNodeIntVector& FMetasoundCommentNodeIntVector::operator=(const FVector2f& InValue)
{
	X = static_cast<int32>(InValue.X);
	Y = static_cast<int32>(InValue.Y);
	return *this;
}

FMetasoundCommentNodeIntVector& FMetasoundCommentNodeIntVector::operator=(const FVector2d& InValue)
{
	X = static_cast<int32>(InValue.X);
	Y = static_cast<int32>(InValue.Y);
	return *this;
}

FMetasoundCommentNodeIntVector& FMetasoundCommentNodeIntVector::operator=(const FIntVector2& InValue)
{
	X = InValue.X;
	Y = InValue.Y;
	return *this;
}

#if WITH_EDITORONLY_DATA
FMetasoundCommentNodeIntVector& FMetasoundCommentNodeIntVector::operator=(const FDeprecateSlateVector2D& InValue)
{
	X = InValue.X;
	Y = InValue.Y;
	return *this;
}
#endif // WITH_EDITORONLY_DATA

bool FMetasoundCommentNodeIntVector::Serialize(FStructuredArchive::FSlot Slot)
{
	Slot << static_cast<FIntVector2&>(*this);
	return true;
}

bool FMetasoundCommentNodeIntVector::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
#if WITH_EDITORONLY_DATA
	if (const UScriptStruct* DeprecatedSlateType = FDeprecateSlateVector2D::StaticStruct())
	{
		if (Tag.GetType().IsStruct(DeprecatedSlateType->GetFName()))
		{
			FDeprecateSlateVector2D OldVector;
			Slot << OldVector;
			X = static_cast<int32>(OldVector.X);
			Y = static_cast<int32>(OldVector.Y);

			UE_LOGF(LogMetaSound, VeryVerbose, "FMetasoundCommentNodeIntVector::SerializeFromMismatchedTag - DeprecateSlateVector2D Type found: Resolving Mismatch");
			return true;
		}
	}
#endif // WITH_EDITORONLY_DATA

	const bool bIsCookCommandlet = IsRunningCookCommandlet();
	if (Tag.GetType().IsStruct("DeprecateSlateVector2D"))
	{
		// Missing type, don't care about visualizing this data.
		// Ignore the serialization.
		X = 0;
		Y = 0;

		UE_LOGF(LogMetaSound, VeryVerbose, "FMetasoundCommentNodeIntVector::SerializeFromMismatchedTag - DeprecateSlateVector2D Type not loaded: Ignoring Mismatch");
		return bIsCookCommandlet;
	}

	if (Tag.GetType().IsStruct(NAME_Vector2d))
	{
		FVector2d OldVector;
		Slot << OldVector;
		X = static_cast<int32>(OldVector.X);
		Y = static_cast<int32>(OldVector.Y);
		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_Vector2f))
	{
		FVector2f OldVector;
		Slot << OldVector;
		X = static_cast<int32>(OldVector.X);
		Y = static_cast<int32>(OldVector.Y);
		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_Vector2D))
	{
		if (Slot.GetUnderlyingArchive().UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
		{
			FVector2f OldVector;
			Slot << OldVector;
			X = static_cast<int32>(OldVector.X);
			Y = static_cast<int32>(OldVector.Y);
			return true;
		}
		else
		{
			FVector2d OldVector;
			Slot << OldVector;
			X = static_cast<int32>(OldVector.X);
			Y = static_cast<int32>(OldVector.Y);
			return true;
		}
	}

	// Hack for running cook commandlet, where we don't really care
	// if it fails as cooked comment content will never be visible,
	// so don't bother reporting if old data was not translated.
	if (bIsCookCommandlet)
	{
		UE_LOGF(LogMetaSound, VeryVerbose, "FMetasoundCommentNodeIntVector::SerializeFromMismatchedTag - Did not resolve. Ignoring value (cooking content, value not necessary).");
	}
	else
	{
		UE_LOGF(LogMetaSound, VeryVerbose, "FMetasoundCommentNodeIntVector::SerializeFromMismatchedTag - Did not resolve. Returning failure state.");
	}

	return bIsCookCommandlet;
}

const FMetasoundFrontendVersionNumber& FMetasoundFrontendVersionNumber::GetInvalid()
{
	static const FMetasoundFrontendVersionNumber Invalid{ 0, 0 };
	return Invalid;
}

bool FMetasoundFrontendVersionNumber::IsValid() const
{
	return *this != GetInvalid();
}

bool FMetasoundFrontendVersionNumber::Parse(const FString& InString, FMetasoundFrontendVersionNumber& OutVersionNumber)
{
	if (!InString.StartsWith(TEXT("v")))
	{
		return false;
	}

	TArray<FString> Parts;
	InString.ParseIntoArray(Parts, TEXT("."));
	if (Parts.Num() != 2)
	{
		return false;
	}

	Parts[0].RightChopInline(1); // Remove 'v'

	OutVersionNumber.Major = FCString::Atoi(*Parts[0]);
	OutVersionNumber.Minor = FCString::Atoi(*Parts[1]);
	return true;
}

Audio::FParameterInterface::FVersion FMetasoundFrontendVersionNumber::ToInterfaceVersion() const
{
	return Audio::FParameterInterface::FVersion{ Major, Minor };
}

FString FMetasoundFrontendVersionNumber::ToString() const
{
	return FString::Format(TEXT("v{0}.{1}"), { Major, Minor });
}

FMetasoundFrontendNodeInterface::FMetasoundFrontendNodeInterface(const FMetasoundFrontendClassInterface& InClassInterface)
{
	for (const FMetasoundFrontendClassInput& Input : InClassInterface.Inputs)
	{
		Inputs.Add(Input);
	}

	for (const FMetasoundFrontendClassOutput& Output : InClassInterface.Outputs)
	{
		Outputs.Add(Output);
	}

	for (const FMetasoundFrontendClassEnvironmentVariable& EnvVar : InClassInterface.Environment)
	{
		FMetasoundFrontendVertex EnvVertex;
		EnvVertex.Name = EnvVar.Name;
		EnvVertex.TypeName = EnvVar.TypeName;

		Environment.Add(MoveTemp(EnvVertex));
	}
}

bool FMetasoundFrontendNodeInterface::Update(const FMetasoundFrontendClassInterface& InClassInterface)
{
	auto NoOp = [](const FMetasoundFrontendVertex&)->void {};

	return Update(InClassInterface, NoOp, NoOp);
}

bool FMetasoundFrontendNodeInterface::Update(const FMetasoundFrontendClassInterface& InClassInterface, TFunctionRef<void(const FMetasoundFrontendVertex&)> OnPreRemoveInput, TFunctionRef<void(const FMetasoundFrontendVertex&)> OnPreRemoveOutput)
{
	bool bInterfaceUpdated = false;
	TArray<FMetasoundFrontendVertex> UnmatchedVertices;

	struct FFindMatchingVertex
	{
		const FMetasoundFrontendClassVertex& ClassVertex;
		bool operator()(const FMetasoundFrontendVertex& InNodeVertex)
		{
			return (ClassVertex.Name == InNodeVertex.Name) && (ClassVertex.TypeName == InNodeVertex.TypeName);
		}
	};

	// Update node inputs
	Algo::Transform(Inputs, UnmatchedVertices, [](const FMetasoundFrontendVertex& Vertex) { return Vertex; });
	for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
	{
		if (const int32 Index = UnmatchedVertices.IndexOfByPredicate(FFindMatchingVertex{ClassInput}); Index != INDEX_NONE)
		{
			// Update the node vertex with anything new from the class vertex
			UnmatchedVertices.RemoveAt(Index, 1, EAllowShrinking::No);
		}
		else
		{
			// Add class input to node inputs
			Inputs.Add(ClassInput);
			bInterfaceUpdated |= true;
		}
	}

	// Remove any inputs that did not exist in the class interface
	for (const FMetasoundFrontendVertex& UnmatchedInput : UnmatchedVertices)
	{
		// Allow outside systems to react before removing the unmatched node inputs
		OnPreRemoveInput(UnmatchedInput);
		if (const int32 Index = Inputs.IndexOfByPredicate([UnmatchedInput](const FMetasoundFrontendVertex& i) { return i.Name == UnmatchedInput.Name && i.TypeName == UnmatchedInput.TypeName; } ); Index != INDEX_NONE)
		{
			// Update the node vertex with anything new from the class vertex
			Inputs.RemoveAt(Index, 1, EAllowShrinking::No);
		}	
		bInterfaceUpdated |= true;
	}

	// Update node outputs
	UnmatchedVertices.Reset();
	Algo::Transform(Outputs, UnmatchedVertices, [](const FMetasoundFrontendVertex& Vertex) { return Vertex; });
	for (const FMetasoundFrontendClassOutput& ClassOutput : InClassInterface.Outputs)
	{
		if (const int32 Index = UnmatchedVertices.IndexOfByPredicate(FFindMatchingVertex{ClassOutput}); Index != INDEX_NONE)
		{
			// Update the node vertex with anything new from the class vertex
			UnmatchedVertices.RemoveAt(Index, 1, EAllowShrinking::No);
		}
		else
		{
			// Add class input to node inputs
			Outputs.Add(ClassOutput);
			bInterfaceUpdated |= true;
		}
	}

	// Remove any outputs that did not exist in the class interface
	for (const FMetasoundFrontendVertex& UnmatchedOutput : UnmatchedVertices)
	{
		// Allow outside systems to react before removing the unmatched node outputs
		OnPreRemoveOutput(UnmatchedOutput);

		if (const int32 Index = Outputs.IndexOfByPredicate([&UnmatchedOutput](const FMetasoundFrontendVertex& i) { return i.Name == UnmatchedOutput.Name && i.TypeName == UnmatchedOutput.TypeName; } ); Index != INDEX_NONE)
		{
			// Update the node vertex with anything new from the class vertex
			Outputs.RemoveAt(Index, 1, EAllowShrinking::No);
		}	
		bInterfaceUpdated |= true;
	}

	// If the interface was updated, make sure the node's interface is sorted
	// the same as the class's interface
	if (bInterfaceUpdated)
	{
		// Sort the input interface
		check(InClassInterface.Inputs.Num() == Inputs.Num());
		for (int32 ClassInputIndex = 0; ClassInputIndex < InClassInterface.Inputs.Num(); ClassInputIndex++)
		{
			int32 NodeInputIndex = Inputs.IndexOfByPredicate(FFindMatchingVertex{InClassInterface.Inputs[ClassInputIndex]});
			check(NodeInputIndex != INDEX_NONE); // The matching node inputs should exist by now.
			if (NodeInputIndex > ClassInputIndex)
			{
				Inputs.Swap(NodeInputIndex, ClassInputIndex);
			}
			else
			{
				// This sort algorithm should never have NodeInputIndex < ClassInputIndex
				check(NodeInputIndex == ClassInputIndex);
			}
		}

		// Sort the output interface
		check(InClassInterface.Outputs.Num() == Outputs.Num());
		for (int32 ClassOutputIndex = 0; ClassOutputIndex < InClassInterface.Outputs.Num(); ClassOutputIndex++)
		{
			int32 NodeOutputIndex = Outputs.IndexOfByPredicate(FFindMatchingVertex{InClassInterface.Outputs[ClassOutputIndex]});
			check(NodeOutputIndex != INDEX_NONE); // The matching node inputs should exist by now.
			if (NodeOutputIndex > ClassOutputIndex)
			{
				Outputs.Swap(NodeOutputIndex, ClassOutputIndex);
			}
			else
			{
				// This sort algorithm should never have NodeOutputIndex < ClassOutputIndex
				check(NodeOutputIndex == ClassOutputIndex);
			}
		}
	}
	
	return bInterfaceUpdated;
}

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundFrontendNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	// By default, node configurations do not override the class interface. 
	return TInstancedStruct<FMetasoundFrontendClassInterface>{};
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundFrontendNodeConfiguration::GetOperatorData() const
{
	return {};
}

FMetasoundFrontendNode::FMetasoundFrontendNode(const FMetasoundFrontendClass& InClass)
: FMetasoundFrontendNode(InClass, TInstancedStruct<FMetaSoundFrontendNodeConfiguration>())
{
}

FMetasoundFrontendNode::FMetasoundFrontendNode(const FMetasoundFrontendClass& InClass, TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfiguration)
: ClassID(InClass.ID)
, Name(InClass.Metadata.GetClassName().Name.ToString())
, Configuration(MoveTemp(InConfiguration))
{
	const FMetasoundFrontendClassInterface* ClassInterfacePtr = nullptr;

	// Determine whether to initialize the node interface with the class's default interface or
	// an override
	if (const FMetaSoundFrontendNodeConfiguration* ConfigurationPtr = Configuration.GetPtr())
	{
		ClassInterfaceOverride = ConfigurationPtr->OverrideDefaultInterface(InClass);
		ClassInterfacePtr = ClassInterfaceOverride.GetPtr();
	}

	if (ClassInterfacePtr == nullptr)
	{
		ClassInterfacePtr = &InClass.GetDefaultInterface();
	}

	Interface = FMetasoundFrontendNodeInterface{*ClassInterfacePtr};
}

FString FMetasoundFrontendVersion::ToString() const
{
	return FString::Format(TEXT("{0} {1}"), { Name.ToString(), Number.ToString() });
}

bool FMetasoundFrontendVersion::IsValid() const
{
	return Number != GetInvalid().Number && Name != GetInvalid().Name;
}

const FMetasoundFrontendVersion& FMetasoundFrontendVersion::GetInvalid()
{
	static const FMetasoundFrontendVersion InvalidVersion { FName(), FMetasoundFrontendVersionNumber::GetInvalid() };
	return InvalidVersion;
}

bool FMetasoundFrontendVertex::IsFunctionalEquivalent(const FMetasoundFrontendVertex& InLHS, const FMetasoundFrontendVertex& InRHS)
{
	return (InLHS.Name == InRHS.Name) && (InLHS.TypeName == InRHS.TypeName);
}

bool operator==(const FMetasoundFrontendVertex& InLHS, const FMetasoundFrontendVertex& InRHS)
{
	return (InLHS.Name == InRHS.Name) && (InLHS.TypeName == InRHS.TypeName) && (InLHS.VertexID == InRHS.VertexID);
}

void FMetasoundFrontendClassVertex::SplitName(FName& OutNamespace, FName& OutParameterName) const
{
	Audio::FParameterPath::SplitName(Name, OutNamespace, OutParameterName);
}

bool FMetasoundFrontendClassVertex::IsFunctionalEquivalent(const FMetasoundFrontendClassVertex& InLHS, const FMetasoundFrontendClassVertex& InRHS)
{
	bool bEquivalentAdvancedDisplay = true;
#if WITH_EDITORONLY_DATA
	bEquivalentAdvancedDisplay = InLHS.Metadata.bIsAdvancedDisplay == InRHS.Metadata.bIsAdvancedDisplay;
#endif // WITH_EDITORONLY_DATA

	return FMetasoundFrontendVertex::IsFunctionalEquivalent(InLHS, InRHS) && (InLHS.AccessType == InRHS.AccessType) && bEquivalentAdvancedDisplay;
}

bool FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(EMetasoundFrontendVertexAccessType InFromType, EMetasoundFrontendVertexAccessType InToType)
{
	// Reroute nodes can have undefined access type, so if either is unset, then connection is valid.
	if (EMetasoundFrontendVertexAccessType::Unset != InFromType && EMetasoundFrontendVertexAccessType::Unset != InToType)
	{
		if (EMetasoundFrontendVertexAccessType::Value == InToType)
		{
			// If the input vertex accesses by "Value" then the output vertex 
			// must also access by "Value" to enforce unexpected consequences 
			// of connecting data which varies over time to an input which only
			// evaluates the data during operator initialization.
			return (EMetasoundFrontendVertexAccessType::Value == InFromType);
		}
	}

	return true;
}

FMetasoundFrontendInterfaceUClassOptions::FMetasoundFrontendInterfaceUClassOptions(const Audio::FParameterInterface::FClassOptions& InOptions)
	: ClassPath(InOptions.ClassPath)
	, bIsModifiable(InOptions.bIsModifiable)
	, bIsDefault(InOptions.bIsDefault)
{
}

FMetasoundFrontendInterfaceUClassOptions::FMetasoundFrontendInterfaceUClassOptions(const FTopLevelAssetPath& InClassPath, bool bInIsModifiable, bool bInIsDefault)
	: ClassPath(InClassPath)
	, bIsModifiable(bInIsModifiable)
	, bIsDefault(bInIsDefault)
{
}

FMetasoundFrontendInterface::FMetasoundFrontendInterface(Audio::FParameterInterfacePtr InInterface)
{
	using namespace Metasound::DocumentPrivate;
	using namespace Metasound::Frontend;

	Metadata.Version = { InInterface->GetName(), FMetasoundFrontendVersionNumber { InInterface->GetVersion().Major, InInterface->GetVersion().Minor } };
#if WITH_EDITORONLY_DATA
	Metadata.Description = InInterface->GetDescription();
#endif
	
	// Transfer all input data from AudioExtension interface struct to FrontendInterface
	Algo::Transform(InInterface->GetInputs(), Inputs, [this](const Audio::FParameterInterface::FInput& Input)
	{
#if WITH_EDITOR
		AddSortOrderToInputStyle(Input.SortOrderIndex);

		// Setup required inputs by telling the style that the input is required
		// This will later be validated against.
		if (!Input.RequiredText.IsEmpty())
		{
			AddRequiredInputToStyle(Input.InitValue.ParamName, Input.RequiredText);
		}
#endif // WITH_EDITOR

		return FMetasoundFrontendClassInput(Input);
	});

	// Transfer all output data from AudioExtension interface struct to FrontendInterface
	Algo::Transform(InInterface->GetOutputs(), Outputs, [this](const Audio::FParameterInterface::FOutput& Output)
	{
#if WITH_EDITOR
		AddSortOrderToOutputStyle(Output.SortOrderIndex);

		// Setup required outputs by telling the style that the output is required.  This will later be validated against.
		if (!Output.RequiredText.IsEmpty())
		{
			AddRequiredOutputToStyle(Output.ParamName, Output.RequiredText);
		}
#endif // WITH_EDITOR

		return FMetasoundFrontendClassOutput(Output);
	});

	// Transfer all environment variables from AudioExtension interface struct to FrontendInterface
	Algo::Transform(InInterface->GetEnvironment(), Environment, [](const Audio::FParameterInterface::FEnvironmentVariable& Variable)
	{
		return FMetasoundFrontendClassEnvironmentVariable(Variable);
	});

	// Transfer all class options from AudioExtension interface struct to FrontendInterface
	Algo::Transform(InInterface->GetUClassOptions(), Metadata.UClassOptions, [](const Audio::FParameterInterface::FClassOptions& Options)
	{
		return FMetasoundFrontendInterfaceUClassOptions(Options);
	});
}

#if WITH_EDITORONLY_DATA
const FMetasoundFrontendInterfaceUClassOptions* FMetasoundFrontendInterface::FindClassOptions(const FTopLevelAssetPath& InClassPath) const
{
	auto FindClassOptionsPredicate = [&InClassPath](const FMetasoundFrontendInterfaceUClassOptions& Options) { return Options.ClassPath == InClassPath; };
	return UClassOptions.FindByPredicate(FindClassOptionsPredicate);
}
#endif // WITH_EDITORONLY_DATA

const FMetasoundFrontendClassName FMetasoundFrontendClassName::InvalidClassName;

FMetasoundFrontendClassName::FMetasoundFrontendClassName(const FName& InNamespace, const FName& InName)
	: Namespace(InNamespace)
	, Name(InName)
{
}

FMetasoundFrontendClassName::FMetasoundFrontendClassName(const FName& InNamespace, const FName& InName, const FName& InVariant)
: Namespace(InNamespace)
, Name(InName)
, Variant(InVariant)
{
}

FMetasoundFrontendClassName::FMetasoundFrontendClassName(const Metasound::FNodeClassName& InName)
: FMetasoundFrontendClassName(InName.GetNamespace(), InName.GetName(), InName.GetVariant())
{
}

FName FMetasoundFrontendClassName::GetScopedName() const
{
	return Metasound::FNodeClassName::FormatScopedName(Namespace, Name);
}

FName FMetasoundFrontendClassName::GetFullName() const
{
	return Metasound::FNodeClassName::FormatFullName(Namespace, Name, Variant);
}

bool FMetasoundFrontendClassName::IsValid() const
{
	return *this != InvalidClassName;
}

// Returns NodeClassName version of full name
Metasound::FNodeClassName FMetasoundFrontendClassName::ToNodeClassName() const
{
	return { Namespace, Name, Variant };
}

FString FMetasoundFrontendClassName::ToString() const
{
	FNameBuilder NameBuilder;
	ToString(NameBuilder);
	return *NameBuilder;
}

void FMetasoundFrontendClassName::ToString(FNameBuilder& NameBuilder) const
{
	Metasound::FNodeClassName::FormatFullName(NameBuilder, Namespace, Name, Variant);
}

bool FMetasoundFrontendClassName::Parse(const FString& InClassName, FMetasoundFrontendClassName& OutClassName)
{
	OutClassName = { };
	TArray<FString> Tokens;
	InClassName.ParseIntoArray(Tokens, TEXT("."));

	// Name is required, which in turn requires at least "None" is serialized for the namespace
	if (Tokens.Num() < 2)
	{
		return false;
	}

	OutClassName.Namespace = FName(*Tokens[0]);
	OutClassName.Name = FName(*Tokens[1]);

	// Variant is optional
	if (Tokens.Num() > 2)
	{
		OutClassName.Variant = FName(*Tokens[2]);
	}

	return true;
}

FMetasoundFrontendClassInterface FMetasoundFrontendClassInterface::GenerateClassInterface(const Metasound::FVertexInterface& InVertexInterface)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FMetasoundFrontendClassInterface ClassInterface;

	// Copy over inputs
	{
		const FInputVertexInterface& InputInterface = InVertexInterface.GetInputInterface();

#if WITH_EDITOR
		FMetasoundFrontendInterfaceStyle InputStyle;
#endif // WITH_EDITOR

		// Reserve memory to minimize memory use in ClassInterface.Inputs array.
		ClassInterface.Inputs.Reserve(InputInterface.Num());


		for (const FInputDataVertex& InputVertex : InputInterface)
		{
			FMetasoundFrontendClassInput ClassInput;
			ClassInput.Name = InputVertex.VertexName;
			ClassInput.TypeName = InputVertex.DataTypeName;
			ClassInput.AccessType = CoreVertexAccessTypeToFrontendVertexAccessType(InputVertex.AccessType);
			ClassInput.VertexID = FClassIDGenerator::Get().CreateInputID(ClassInput);

#if WITH_EDITOR
			const FDataVertexMetadata& VertexMetadata = InputVertex.Metadata;

			ClassInput.Metadata.SetSerializeText(false);
			ClassInput.Metadata.SetDisplayName(VertexMetadata.DisplayName);
			ClassInput.Metadata.SetDescription(VertexMetadata.Description);
			ClassInput.Metadata.bIsAdvancedDisplay = VertexMetadata.bIsAdvancedDisplay;

			// Advanced display items are pushed to bottom of sort order
			ClassInput.Metadata.SortOrderIndex = InputInterface.GetSortOrderIndex(InputVertex.VertexName);
			if (ClassInput.Metadata.bIsAdvancedDisplay)
			{
				ClassInput.Metadata.SortOrderIndex += InputInterface.Num();
			}
			InputStyle.DefaultSortOrder.Add(ClassInput.Metadata.SortOrderIndex);
#endif // WITH_EDITOR

			FLiteral DefaultLiteral = InputVertex.GetDefaultLiteral();
			if (DefaultLiteral.GetType() != ELiteralType::Invalid)
			{
				ClassInput.InitDefault().SetFromLiteral(DefaultLiteral);
			}

			ClassInterface.Inputs.Add(MoveTemp(ClassInput));
		}

#if WITH_EDITOR
		// Must set via direct accessor to avoid updating the change GUID
		// (All instances of this generation call should be done for code
		// defined classes only, which do not currently create a persistent
		// change hash between builds and leave the guid 0'ed).
		ClassInterface.InputStyle = InputStyle;
#endif // WITH_EDITOR
	}

	// Copy over outputs
	{
		const FOutputVertexInterface& OutputInterface = InVertexInterface.GetOutputInterface();

#if WITH_EDITOR
		FMetasoundFrontendInterfaceStyle OutputStyle;
#endif // WITH_EDITOR

		// Reserve memory to minimize memory use in ClassInterface.Outputs array.
		ClassInterface.Outputs.Reserve(OutputInterface.Num());

		for (const FOutputDataVertex& OutputVertex: OutputInterface)
		{
			FMetasoundFrontendClassOutput ClassOutput;

			ClassOutput.Name = OutputVertex.VertexName;
			ClassOutput.TypeName = OutputVertex.DataTypeName;
			ClassOutput.AccessType = CoreVertexAccessTypeToFrontendVertexAccessType(OutputVertex.AccessType);
			ClassOutput.VertexID = FClassIDGenerator::Get().CreateOutputID(ClassOutput);
#if WITH_EDITOR
			const FDataVertexMetadata& VertexMetadata = OutputVertex.Metadata;

			ClassOutput.Metadata.SetSerializeText(false);
			ClassOutput.Metadata.SetDisplayName(VertexMetadata.DisplayName);
			ClassOutput.Metadata.SetDescription(VertexMetadata.Description);
			ClassOutput.Metadata.bIsAdvancedDisplay = VertexMetadata.bIsAdvancedDisplay;

			// Advanced display items are pushed to bottom below non-advanced
			ClassOutput.Metadata.SortOrderIndex = OutputInterface.GetSortOrderIndex(OutputVertex.VertexName);
			if (ClassOutput.Metadata.bIsAdvancedDisplay)
			{
				ClassOutput.Metadata.SortOrderIndex += OutputInterface.Num();
			}
			OutputStyle.DefaultSortOrder.Add(ClassOutput.Metadata.SortOrderIndex);
#endif // WITH_EDITOR

			ClassInterface.Outputs.Add(MoveTemp(ClassOutput));
		}

#if WITH_EDITOR
		// Must set via direct accessor to avoid updating the change GUID
		// (All instances of this generation call should be done for code
		// defined classes only, which do not currently create a persistent
		// change hash between builds and leave the guid 0'ed).
		ClassInterface.OutputStyle = MoveTemp(OutputStyle);
#endif // WITH_EDITOR
	}

	// Reserve size to minimize memory use in ClassInterface.Environment array
	ClassInterface.Environment.Reserve(InVertexInterface.GetEnvironmentInterface().Num());

	for (const FEnvironmentVertex& EnvVertex : InVertexInterface.GetEnvironmentInterface())
	{
		FMetasoundFrontendClassEnvironmentVariable EnvVar;

		EnvVar.Name = EnvVertex.VertexName;
		EnvVar.bIsRequired = true;

		ClassInterface.Environment.Add(EnvVar);
	}

	return ClassInterface;
}

void FMetasoundFrontendClassMetadata::AddAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags)
{
	AccessFlags |= static_cast<uint16>(InAccessFlags);
}

void FMetasoundFrontendClassMetadata::ClearAccessFlags()
{
	AccessFlags = static_cast<uint16>(EMetasoundFrontendClassAccessFlags::None);
}

EMetasoundFrontendClassType FMetasoundFrontendClassMetadata::GetType() const
{
	return Type;
}

const FMetasoundFrontendVersionNumber& FMetasoundFrontendClassMetadata::GetVersion() const
{
	return Version;
}

EMetasoundFrontendClassAccessFlags FMetasoundFrontendClassMetadata::GetAccessFlags() const
{
	return static_cast<EMetasoundFrontendClassAccessFlags>(AccessFlags);
}

void FMetasoundFrontendClassMetadata::RemoveAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags)
{
	AccessFlags &= ~static_cast<uint16>(InAccessFlags);
}

void FMetasoundFrontendClassMetadata::SetAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags)
{
	AccessFlags = static_cast<uint16>(InAccessFlags);
}

#if WITH_EDITOR
void FMetasoundFrontendClassMetadata::SetAuthor(const FString& InAuthor)
{
	using namespace Metasound::DocumentPrivate;

	SetWithChangeID(InAuthor, Author, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy)
{
	using namespace Metasound::DocumentPrivate;

	TArray<FText>& TextToSet = bSerializeText ? CategoryHierarchy : CategoryHierarchyTransient;
	SetWithChangeID(InCategoryHierarchy, TextToSet, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetKeywords(const TArray<FText>& InKeywords)
{
	using namespace Metasound::DocumentPrivate;
	TArray<FText>& TextToSet = bSerializeText ? Keywords : KeywordsTransient;
	SetWithChangeID(InKeywords, TextToSet, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetDescription(const FText& InDescription)
{
	using namespace Metasound::DocumentPrivate;

	FText& TextToSet = bSerializeText ? Description : DescriptionTransient;
	SetWithChangeID(InDescription, TextToSet, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetDisplayName(const FText& InDisplayName)
{
	using namespace Metasound::DocumentPrivate;

	FText& TextToSet = bSerializeText ? DisplayName : DisplayNameTransient;
	SetWithChangeID(InDisplayName, TextToSet, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetIsDeprecated(bool bInIsDeprecated)
{
	using namespace Metasound::DocumentPrivate;
	SetWithChangeID(bInIsDeprecated, bIsDeprecated, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetPromptIfMissing(const FText& InPromptIfMissing)
{
	using namespace Metasound::DocumentPrivate;
	SetWithChangeID(InPromptIfMissing, PromptIfMissingTransient, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetSerializeText(bool bInSerializeText)
{
	if (bSerializeText)
	{
		if (!bInSerializeText)
		{
			DescriptionTransient = Description;
			DisplayNameTransient = DisplayName;

			Description = { };
			DisplayName = { };

			KeywordsTransient = MoveTemp(Keywords);
			CategoryHierarchyTransient = MoveTemp(CategoryHierarchy);
		}
	}
	else
	{
		if (bInSerializeText)
		{
			Description = DescriptionTransient;
			DisplayName = DisplayNameTransient;

			DescriptionTransient = { };
			DisplayNameTransient = { };

			Keywords = MoveTemp(KeywordsTransient);
			CategoryHierarchy = MoveTemp(CategoryHierarchyTransient);
		}
	}

	bSerializeText = bInSerializeText;
}
#endif // WITH_EDITOR

void FMetasoundFrontendClassMetadata::SetVersion(const FMetasoundFrontendVersionNumber& InVersion)
{
	using namespace Metasound::DocumentPrivate;
	SetWithChangeID(InVersion, Version, ChangeID);
}

void FMetasoundFrontendClassMetadata::SetClassName(const FMetasoundFrontendClassName& InClassName)
{
	using namespace Metasound::DocumentPrivate;
	SetWithChangeID(InClassName, ClassName, ChangeID);
}

void FMetasoundFrontendClass::SetDefaultInterface(const FMetasoundFrontendClassInterface& InInterface)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Interface = InInterface;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMetasoundFrontendClassInterface& FMetasoundFrontendClass::GetDefaultInterface()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Interface;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FMetasoundFrontendClassInterface& FMetasoundFrontendClass::GetDefaultInterface() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Interface;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FMetasoundFrontendClassInterface& FMetasoundFrontendClass::GetInterfaceForNode(const FMetasoundFrontendNode& InNode) const
{
	if (const FMetasoundFrontendClassInterface* InterfaceOverride = InNode.ClassInterfaceOverride.GetPtr())
	{
#if !UE_BUILD_SHIPPING
		if (Metadata.GetType() != EMetasoundFrontendClassType::External)
		{
			UE_LOGF(LogMetaSound, Error, "Found class interface override on class %ls. Class interface overrides are intended to be used on external nodes", *Metadata.GetClassName().ToString());
		}
#endif // !UE_BUILD_SHIPPING
		return *InterfaceOverride;
	}

	return GetDefaultInterface();
}

#if WITH_EDITOR
bool FMetasoundFrontendClass::CacheGraphDependencyMetadataFromRegistry(FMetasoundFrontendClass& InOutDependency)
{
	using namespace Metasound::Frontend;

	const FNodeRegistryKey Key = FNodeRegistryKey(InOutDependency.Metadata);
	FMetasoundFrontendClass RegistryClass;

	INodeClassRegistry* Registry = INodeClassRegistry::Get();
	if (ensure(Registry))
	{
		if (Registry->FindFrontendClassFromRegistered(Key, RegistryClass))
		{
			InOutDependency.Metadata = RegistryClass.Metadata;
			InOutDependency.Style = RegistryClass.Style;

			using FNameTypeKey = TPair<FName, FName>;
			using FVertexMetadataMap = TMap<FNameTypeKey, const FMetasoundFrontendVertexMetadata*>;
			auto MakePairFromVertex = [](const FMetasoundFrontendClassVertex& InVertex)
			{
				const FNameTypeKey Key(InVertex.Name, InVertex.TypeName);
				return TPair<FNameTypeKey, const FMetasoundFrontendVertexMetadata*> { Key, &InVertex.Metadata };
			};

			auto AddRegistryVertexMetadata = [](const FVertexMetadataMap& InInterfaceMembers, FMetasoundFrontendClassVertex& OutVertex, FMetasoundFrontendInterfaceStyle& OutNewStyle)
			{
				const FNameTypeKey Key(OutVertex.Name, OutVertex.TypeName);
				if (const FMetasoundFrontendVertexMetadata* RegVertex = InInterfaceMembers.FindRef(Key))
				{
					OutVertex.Metadata = *RegVertex;
					OutVertex.Metadata.SetSerializeText(false);
				}
				OutNewStyle.DefaultSortOrder.Add(OutVertex.Metadata.SortOrderIndex);
			};

			FMetasoundFrontendInterfaceStyle InputStyle;
			FVertexMetadataMap InputMembers;
			Algo::Transform(RegistryClass.GetDefaultInterface().Inputs, InputMembers, [&](const FMetasoundFrontendClassInput& Input) { return MakePairFromVertex(Input); });
			Algo::ForEach(InOutDependency.GetDefaultInterface().Inputs, [&](FMetasoundFrontendClassInput& Input)
			{
				AddRegistryVertexMetadata(InputMembers, Input, InputStyle);
			});
			InOutDependency.GetDefaultInterface().SetInputStyle(MoveTemp(InputStyle));

			FMetasoundFrontendInterfaceStyle OutputStyle;
			FVertexMetadataMap OutputMembers;
			Algo::Transform(RegistryClass.GetDefaultInterface().Outputs, OutputMembers, [&](const FMetasoundFrontendClassOutput& Output) { return MakePairFromVertex(Output); });
			Algo::ForEach(InOutDependency.GetDefaultInterface().Outputs, [&](FMetasoundFrontendClassOutput& Output)
			{
				AddRegistryVertexMetadata(OutputMembers, Output, OutputStyle);
			});
			InOutDependency.GetDefaultInterface().SetOutputStyle(MoveTemp(OutputStyle));

			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
FMetasoundFrontendClassStyle FMetasoundFrontendClassStyle::GenerateClassStyle(const Metasound::FNodeDisplayStyle& InNodeDisplayStyle)
{
	FMetasoundFrontendClassStyle Style;

	Style.Display.bShowName = InNodeDisplayStyle.bShowName;
	Style.Display.bShowInputNames = InNodeDisplayStyle.bShowInputNames;
	Style.Display.bShowOutputNames = InNodeDisplayStyle.bShowOutputNames;
	Style.Display.ImageName = InNodeDisplayStyle.ImageName;
	Style.Display.StyleSet = InNodeDisplayStyle.StyleSet;
	Style.Display.Icon = InNodeDisplayStyle.Icon;
	Style.Display.Color = InNodeDisplayStyle.Color;

	return Style;
}
#endif // WITH_EDITORONLY_DATA

FMetasoundFrontendClassMetadata::FMetasoundFrontendClassMetadata()
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: bAutoUpdateManagesInterface(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
{
}

FMetasoundFrontendClassMetadata FMetasoundFrontendClassMetadata::GenerateClassMetadata(const Metasound::FNodeClassMetadata& InNodeClassMetadata, EMetasoundFrontendClassType InType)
{
	FMetasoundFrontendClassMetadata NewMetadata;

	NewMetadata.Type = InType;

	NewMetadata.ClassName = InNodeClassMetadata.ClassName;
	NewMetadata.Version = { InNodeClassMetadata.MajorVersion, InNodeClassMetadata.MinorVersion };

#if WITH_EDITOR
	NewMetadata.SetSerializeText(false);
	NewMetadata.SetDisplayName(InNodeClassMetadata.DisplayName);
	NewMetadata.SetDescription(InNodeClassMetadata.Description);
	NewMetadata.SetPromptIfMissing(InNodeClassMetadata.PromptIfMissing);
	NewMetadata.SetAuthor(InNodeClassMetadata.Author);
	NewMetadata.SetKeywords(InNodeClassMetadata.Keywords);
	NewMetadata.SetCategoryHierarchy(InNodeClassMetadata.CategoryHierarchy);
	NewMetadata.AddAccessFlags(static_cast<EMetasoundFrontendClassAccessFlags>(InNodeClassMetadata.AccessFlags));
#endif // WITH_EDITOR

	return NewMetadata;
}

FMetasoundFrontendClassInputDefault::FMetasoundFrontendClassInputDefault(FMetasoundFrontendLiteral InLiteral)
	: Literal(InLiteral)
	, PageID(Metasound::Frontend::DefaultPageID)
{
}

FMetasoundFrontendClassInputDefault::FMetasoundFrontendClassInputDefault(const FGuid& InPageID, FMetasoundFrontendLiteral InLiteral)
	: Literal(MoveTemp(InLiteral))
	, PageID(InPageID)
{
}

FMetasoundFrontendClassInputDefault::FMetasoundFrontendClassInputDefault(const FAudioParameter& InParameter)
	: Literal(InParameter)
{
}

bool FMetasoundFrontendClassInputDefault::IsFunctionalEquivalent(const FMetasoundFrontendClassInputDefault& InLHS, const FMetasoundFrontendClassInputDefault& InRHS)
{
	return InLHS == InRHS;
}

bool operator==(const FMetasoundFrontendClassInputDefault& InLHS, const FMetasoundFrontendClassInputDefault& InRHS)
{
	if (InLHS.PageID != InRHS.PageID)
	{
		return false;
	}

	return InLHS.Literal.IsEqual(InRHS.Literal);
}

FMetasoundFrontendClassInput::FMetasoundFrontendClassInput(const FMetasoundFrontendClassVertex& InOther)
:	FMetasoundFrontendClassVertex(InOther)
{
	using namespace Metasound;

	const ELiteralType LiteralType = Frontend::IDataTypeRegistry::Get().GetDesiredLiteralType(InOther.TypeName);
	const EMetasoundFrontendLiteralType DefaultType = Frontend::GetMetasoundFrontendLiteralType(LiteralType);
	InitDefault().SetType(DefaultType);
}

FMetasoundFrontendClassInput::FMetasoundFrontendClassInput(const Audio::FParameterInterface::FInput& InInput)
{
	using namespace Metasound;

	Name = InInput.InitValue.ParamName;
	InitDefault(FMetasoundFrontendLiteral(InInput.InitValue));
	TypeName = DocumentPrivate::ResolveMemberDataType(InInput.DataType, InInput.InitValue.ParamType);
	VertexID = Frontend::FClassIDGenerator::Get().CreateInputID(InInput);

#if WITH_EDITOR
	// Interfaces should never serialize text to avoid desync between
	// copied versions serialized in assets and those defined in code.
	Metadata.SetSerializeText(false);
	Metadata.SetDisplayName(InInput.DisplayName);
	Metadata.SetDescription(InInput.Description);
	Metadata.SortOrderIndex = InInput.SortOrderIndex;
#endif // WITH_EDITOR
}

bool FMetasoundFrontendClassInput::IsFunctionalEquivalent(const FMetasoundFrontendClassInput& InLHS, const FMetasoundFrontendClassInput& InRHS)
{
	if (!FMetasoundFrontendClassVertex::IsFunctionalEquivalent(InLHS, InRHS))
	{
		return false;
	}

	const TArray<FMetasoundFrontendClassInputDefault>& LHSDefaults = InLHS.GetDefaults();
	const TArray<FMetasoundFrontendClassInputDefault>& RHSDefaults = InRHS.GetDefaults();
	if (LHSDefaults.Num() != RHSDefaults.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < LHSDefaults.Num(); ++Index)
	{
		if (!FMetasoundFrontendClassInputDefault::IsFunctionalEquivalent(LHSDefaults[Index], RHSDefaults[Index]))
		{
			return false;
		}
	}

	return true;
}

FMetasoundFrontendLiteral& FMetasoundFrontendClassInput::AddDefault(const FGuid& InPageID)
{
	checkf(!ContainsDefault(InPageID), TEXT("Page default with given ID already exists"));
	return Defaults.Add_GetRef(FMetasoundFrontendClassInputDefault { InPageID }).Literal;
}

bool FMetasoundFrontendClassInput::ContainsDefault(const FGuid& InPageID) const
{
	auto IsPage = [&InPageID](const FMetasoundFrontendClassInputDefault& Default) { return Default.PageID == InPageID; };
	return Defaults.ContainsByPredicate(IsPage);
}

const FMetasoundFrontendLiteral* FMetasoundFrontendClassInput::FindConstDefault(const FGuid& InPageID) const
{
	auto IsPage = [&InPageID](FMetasoundFrontendClassInputDefault& Default) { return Default.PageID == InPageID; };
	if (const FMetasoundFrontendClassInputDefault* Default = Defaults.FindByPredicate(IsPage))
	{
		return &Default->Literal;
	}

	return nullptr;
}

const FMetasoundFrontendLiteral& FMetasoundFrontendClassInput::FindConstDefaultChecked(const FGuid& InPageID) const
{
	const FMetasoundFrontendLiteral* Literal = FindConstDefault(InPageID);
	check(Literal);
	return *Literal;
}

FMetasoundFrontendLiteral* FMetasoundFrontendClassInput::FindDefault(const FGuid& InPageID)
{
	auto IsPage = [&InPageID](FMetasoundFrontendClassInputDefault& Default) { return Default.PageID == InPageID; };
	if (FMetasoundFrontendClassInputDefault* Default = Defaults.FindByPredicate(IsPage))
	{
		return &Default->Literal;
	}

	return nullptr;
}

FMetasoundFrontendLiteral& FMetasoundFrontendClassInput::FindDefaultChecked(const FGuid& InPageID)
{
	FMetasoundFrontendLiteral* Literal = FindDefault(InPageID);
	check(Literal);
	return *Literal;
}

const TArray<FMetasoundFrontendClassInputDefault>& FMetasoundFrontendClassInput::GetDefaults() const
{
	return Defaults;
}

FMetasoundFrontendLiteral& FMetasoundFrontendClassInput::InitDefault()
{
	using namespace Metasound::Frontend;

	checkf(Defaults.IsEmpty(), TEXT("Default(s) already initialized"));
	FMetasoundFrontendLiteral& NewLiteral = Defaults.Add_GetRef(FMetasoundFrontendClassInputDefault
	{
		::Metasound::Frontend::DefaultPageID
	}).Literal;

	if (IDataTypeRegistry::Get().IsRegistered(TypeName))
	{
		NewLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(TypeName));
	}
	return NewLiteral;
}

void FMetasoundFrontendClassInput::InitDefault(FMetasoundFrontendLiteral InitLiteral)
{
	checkf(Defaults.IsEmpty(), TEXT("Default(s) already initialized"));
	Defaults.Add_GetRef(FMetasoundFrontendClassInputDefault{ Metasound::Frontend::DefaultPageID }).Literal = MoveTemp(InitLiteral);
}

void FMetasoundFrontendClassInput::IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral&)> IterFunc)
{
	for (FMetasoundFrontendClassInputDefault& Default : Defaults)
	{
		IterFunc(Default.PageID, Default.Literal);
	}
}

void FMetasoundFrontendClassInput::IterateDefaults(TFunctionRef<void(const FGuid&, const FMetasoundFrontendLiteral&)> IterFunc) const
{
	for (const FMetasoundFrontendClassInputDefault& Default : Defaults)
	{
		IterFunc(Default.PageID, Default.Literal);
	}
}

bool FMetasoundFrontendClassInput::RemoveDefault(const FGuid& InPageID)
{
	auto IsPage = [&InPageID](const FMetasoundFrontendClassInputDefault& Default) { return Default.PageID == InPageID; };
	return Defaults.RemoveAllSwap(IsPage) > 0;
}

void FMetasoundFrontendClassInput::ResetDefaults(bool bInitializeDefaultPage)
{
	using namespace Metasound::Frontend;

	Defaults.Reset();
	if (bInitializeDefaultPage)
	{
		InitDefault();
		Defaults.Shrink();
	}
}

void FMetasoundFrontendClassInput::SetDefaults(TArray<FMetasoundFrontendClassInputDefault> InputDefaults)
{
#if DO_CHECK
	auto IsDefaultPageID = [](const FMetasoundFrontendClassInputDefault& Default) { return Default.PageID == Metasound::Frontend::DefaultPageID; };
	check(InputDefaults.ContainsByPredicate(IsDefaultPageID));
#endif // DO_CHECK

	Defaults = MoveTemp(InputDefaults);
}

FMetasoundFrontendClassVariable::FMetasoundFrontendClassVariable(const FMetasoundFrontendClassVertex& InOther)
	: FMetasoundFrontendClassVertex(InOther)
{
	using namespace Metasound::Frontend;

	EMetasoundFrontendLiteralType DefaultType = GetMetasoundFrontendLiteralType(IDataTypeRegistry::Get().GetDesiredLiteralType(InOther.TypeName));

	DefaultLiteral.SetType(DefaultType);
}

FMetasoundFrontendClassOutput::FMetasoundFrontendClassOutput(const Audio::FParameterInterface::FOutput& Output)
{
	using namespace Metasound::Frontend;

	Name = Output.ParamName;
	TypeName = Metasound::DocumentPrivate::ResolveMemberDataType(Output.DataType, Output.ParamType);
	VertexID = FClassIDGenerator::Get().CreateOutputID(Output);

#if WITH_EDITOR
	// Interfaces should never serialize text to avoid desync between
	// copied versions serialized in assets and those defined in code.
	Metadata.SetSerializeText(false);
	Metadata.SetDisplayName(Output.DisplayName);
	Metadata.SetDescription(Output.Description);
	Metadata.SortOrderIndex = Output.SortOrderIndex;
#endif // WITH_EDITOR
}

FMetasoundFrontendClassOutput::FMetasoundFrontendClassOutput(const FMetasoundFrontendClassVertex& InOther)
	: FMetasoundFrontendClassVertex(InOther)
{
}

FMetasoundFrontendClassEnvironmentVariable::FMetasoundFrontendClassEnvironmentVariable(const Audio::FParameterInterface::FEnvironmentVariable& InVariable)
	: Name(InVariable.ParamName)
	// Disabled as it isn't used to infer type when getting/setting at a lower level.
	// TODO: Either remove type info for environment variables all together or enforce type.
	// , TypeName(Metasound::Frontend::DocumentPrivate::ResolveMemberDataType(Environment.DataType, Environment.ParamType))
{
}

#if WITH_EDITORONLY_DATA
void FMetasoundFrontendInterfaceStyle::SortVertices(TArray<const FMetasoundFrontendVertex*>& OutVertices, TFunctionRef<FText(const FMetasoundFrontendVertex&)> InGetDisplayNamePredicate) const
{
	TMap<FGuid, int32> NodeIDToSortIndex;
	int32 HighestSortOrder = TNumericLimits<int32>::Min();
	for (int32 i = 0; i < OutVertices.Num(); ++i)
	{
		const FGuid& VertexID = OutVertices[i]->VertexID;
		int32 SortIndex = 0;
		if (DefaultSortOrder.IsValidIndex(i))
		{
			SortIndex = DefaultSortOrder[i];
			HighestSortOrder = FMath::Max(SortIndex, HighestSortOrder);
		}
		else
		{
			SortIndex = ++HighestSortOrder;
		}
		NodeIDToSortIndex.Add(VertexID, SortIndex);
	}

	Algo::Sort(OutVertices, [&](const FMetasoundFrontendVertex* VertexA, const FMetasoundFrontendVertex* VertexB)
	{
		const FGuid VertexAID = VertexA->VertexID;
		const FGuid VertexBID = VertexB->VertexID;
		const int32 AID = NodeIDToSortIndex[VertexAID];
		const int32 BID = NodeIDToSortIndex[VertexBID];

		// If IDs are equal, sort alphabetically using provided name predicate
		if (AID == BID)
		{
			return InGetDisplayNamePredicate(*VertexA).CompareTo(InGetDisplayNamePredicate(*VertexB)) < 0;
		}
		return AID < BID;
	});
}
#endif // WITH_EDITORONLY_DATA

FMetasoundFrontendGraphClass::FMetasoundFrontendGraphClass()
{
	Metadata.SetType(EMetasoundFrontendClassType::Graph);
}

FMetasoundFrontendGraphClass::~FMetasoundFrontendGraphClass()
{
}

#if WITH_EDITORONLY_DATA
const FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::AddGraphPage(const FGuid& InPageID, bool bDuplicateLastGraph, bool bSetAsBuildGraph)
{
	checkf(!ContainsGraphPage(InPageID), TEXT("Cannot add new graph page with existing PageID"));

	FMetasoundFrontendGraph* NewGraph = nullptr;
	if (bDuplicateLastGraph)
	{
		checkf(!PagedGraphs.IsEmpty(), TEXT("Cannot duplicate graph. No graph to duplicate"));
		FMetasoundFrontendGraph ToDuplicate = PagedGraphs.Last();
		NewGraph = &PagedGraphs.Add_GetRef(MoveTemp(ToDuplicate));
	}
	else
	{
		NewGraph = &PagedGraphs.AddDefaulted_GetRef();
	}

	NewGraph->PageID = InPageID;
	return *NewGraph;
}
#endif // WITH_EDITORONLY_DATA

bool FMetasoundFrontendGraphClass::ContainsGraphPage(const FGuid& InPageID) const
{
	auto MatchesPageID = [&InPageID](const FMetasoundFrontendGraph& Iter) { return Iter.PageID == InPageID; };
	return PagedGraphs.ContainsByPredicate(MatchesPageID);
}

#if WITH_EDITORONLY_DATA
bool FMetasoundFrontendGraphClass::RemoveGraphPage(const FGuid& InPageID, FGuid* OutAdjacentPageID)
{
	for (int32 Index = 0; Index < PagedGraphs.Num(); ++Index)
	{
		if (PagedGraphs[Index].PageID == InPageID)
		{
			PagedGraphs.RemoveAtSwap(Index, EAllowShrinking::Yes);

			if (OutAdjacentPageID)
			{
				if (Index > 0)
				{
					*OutAdjacentPageID = PagedGraphs[Index - 1].PageID;
				}
				else if (Index < PagedGraphs.Num())
				{
					*OutAdjacentPageID = PagedGraphs[0].PageID;
				}

			}

			return true;
		}
	}

	if (OutAdjacentPageID)
	{
		*OutAdjacentPageID = Metasound::Frontend::DefaultPageID;
	}

	return false;
}

void FMetasoundFrontendGraphClass::ResetGraphPages(bool bClearDefaultGraph)
{
	PagedGraphs.RemoveAllSwap([](const FMetasoundFrontendGraph& PageGraph)
	{
		return PageGraph.PageID != Metasound::Frontend::DefaultPageID;
	}, EAllowShrinking::Yes);

	if (bClearDefaultGraph)
	{
		IterateGraphPages([](FMetasoundFrontendGraph& PageGraph)
		{
			PageGraph.Nodes.Empty();
			PageGraph.Edges.Empty();
			PageGraph.Variables.Empty();
			PageGraph.Style = { };
		});
	}
}
#endif // WITH_EDITORONLY_DATA

FMetasoundFrontendGraph* FMetasoundFrontendGraphClass::FindGraph(const FGuid& InPageID)
{
	auto MatchesPageID = [this, &InPageID](const FMetasoundFrontendGraph& Iter) { return Iter.PageID == InPageID; };
	FMetasoundFrontendGraph* PageGraph = PagedGraphs.FindByPredicate(MatchesPageID);
	return PageGraph;
}

FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::FindGraphChecked(const FGuid& InPageID)
{
	FMetasoundFrontendGraph* FoundGraph = FindGraph(InPageID);
	check(FoundGraph);
	return *FoundGraph;
}

const FMetasoundFrontendGraph* FMetasoundFrontendGraphClass::FindConstGraph(const FGuid& InPageID) const
{
	auto MatchesPageID = [this, &InPageID](const FMetasoundFrontendGraph& Iter) { return Iter.PageID == InPageID; };
	const FMetasoundFrontendGraph* PageGraph = PagedGraphs.FindByPredicate(MatchesPageID);
	return PageGraph;
}

const FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::FindConstGraphChecked(const FGuid& InPageID) const
{
	const FMetasoundFrontendGraph* FoundGraph = FindConstGraph(InPageID);
	check(FoundGraph);
	return *FoundGraph;
}

FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::GetDefaultGraph()
{
	return FindGraphChecked(Metasound::Frontend::DefaultPageID);
}

const FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::GetConstDefaultGraph() const
{
	return FindConstGraphChecked(Metasound::Frontend::DefaultPageID);
}

FMetasoundFrontendGraph& FMetasoundFrontendGraphClass::InitDefaultGraphPage()
{
	checkf(PagedGraphs.IsEmpty(), TEXT("Attempting to initialize default page for graph class with existing graph implementation"));
	FMetasoundFrontendGraph& NewGraph = PagedGraphs.AddDefaulted_GetRef();
	NewGraph.PageID = Metasound::Frontend::DefaultPageID;
	return NewGraph;
}

void FMetasoundFrontendGraphClass::IterateGraphPages(TFunctionRef<void(FMetasoundFrontendGraph&)> IterFunc)
{
	for (FMetasoundFrontendGraph& Iter : PagedGraphs)
	{
		IterFunc(Iter);
	}
}

void FMetasoundFrontendGraphClass::IterateGraphPages(TFunctionRef<void(const FMetasoundFrontendGraph&)> IterFunc) const
{
	for (const FMetasoundFrontendGraph& Iter : PagedGraphs)
	{
		IterFunc(Iter);
	}
}

#if WITH_EDITORONLY_DATA
bool FMetasoundFrontendDocument::MigrateProperties()
{
	bool bMigrated = false;
	if (ArchetypeVersion.IsValid())
	{
		bMigrated = true;
		Interfaces.Add(ArchetypeVersion);
		ArchetypeVersion = FMetasoundFrontendVersion::GetInvalid();
	}

	if (!InterfaceVersions.IsEmpty())
	{
		bMigrated = true;
		Interfaces.Append(InterfaceVersions);
		InterfaceVersions.Reset();
	}

	for (TPair<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>& Pair : Metadata.MemberMetadata)
	{
		if (UMetaSoundFrontendMemberMetadata* MemberMetadata = Pair.Value.Get())
		{
			const bool bMigratedMetadata = MemberMetadata->MigrateProperties();
			if (bMigratedMetadata)
			{
				bMigrated = true;
			}
		}
	}
	return bMigrated;
}
#endif // WITH_EDITORONLY_DATA

void FMetasoundFrontendGraphClass::ResetGraphs()
{
	PagedGraphs.Empty();
}

#if WITH_EDITORONLY_DATA
TArray<FMetasoundFrontendGraph>& FMetasoundFrontendGraphClass::IPropertyVersionTransform::GetPagesUnsafe(FMetasoundFrontendGraphClass& GraphClass)
{
	return GraphClass.PagedGraphs;
}

FMetasoundFrontendVersionNumber FMetasoundFrontendDocument::GetMaxVersion()
{
	using namespace Metasound::Frontend;
	return FVersioningManager::GetMaxDocumentVersion();
}
#endif // WITH_EDITORONLY_DATA

FMetasoundFrontendDocument::FMetasoundFrontendDocument()
{
	RootGraph.Metadata.SetType(EMetasoundFrontendClassType::Graph);

#if WITH_EDITORONLY_DATA
	ArchetypeVersion = FMetasoundFrontendVersion::GetInvalid();
#endif // WITH_EDITORONLY_DATA
}

const TCHAR* LexToString(EMetasoundFrontendClassType InClassType)
{
	using namespace Metasound::Frontend;

	switch (InClassType)
	{
		case EMetasoundFrontendClassType::External:
			return *ClassTypePrivate::External;
		case EMetasoundFrontendClassType::Graph:
			return *ClassTypePrivate::Graph;
		case EMetasoundFrontendClassType::Input:
			return *ClassTypePrivate::Input;
		case EMetasoundFrontendClassType::Output:
			return *ClassTypePrivate::Output;
		case EMetasoundFrontendClassType::Literal:
			return *ClassTypePrivate::Literal;
		case EMetasoundFrontendClassType::Variable:
			return *ClassTypePrivate::Variable;
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
			return *ClassTypePrivate::VariableDeferredAccessor;
		case EMetasoundFrontendClassType::VariableAccessor:
			return *ClassTypePrivate::VariableAccessor;
		case EMetasoundFrontendClassType::VariableMutator:
			return *ClassTypePrivate::VariableMutator;
		case EMetasoundFrontendClassType::Template:
			return *ClassTypePrivate::Template;
		case EMetasoundFrontendClassType::Invalid:
			return *ClassTypePrivate::Invalid;
		default:
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missed EMetasoundFrontendClassType case coverage");
			return nullptr;
	}
}

const TCHAR* LexToString(EMetasoundFrontendVertexAccessType InVertexAccess)
{
	switch (InVertexAccess)
	{
		case EMetasoundFrontendVertexAccessType::Value:
			return TEXT("Value");
			
		case EMetasoundFrontendVertexAccessType::Reference:
			return TEXT("Reference");

		case EMetasoundFrontendVertexAccessType::Unset:
		default:
			return TEXT("Unset");
	}
}

namespace Metasound::Frontend
{
	bool StringToClassType(const FString& InString, EMetasoundFrontendClassType& OutClassType)
	{
		if (const EMetasoundFrontendClassType* FoundClassType = ClassTypePrivate::ClassTypeCStringToEnum.Find(*InString))
		{
			OutClassType = *FoundClassType;
		}
		else
		{
			OutClassType = EMetasoundFrontendClassType::Invalid;
		}
		
		return OutClassType != EMetasoundFrontendClassType::Invalid;
	}

	void ForEachLiteral(const FMetasoundFrontendDocument& InDoc, FForEachLiteralFunctionRef OnLiteral)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ForEachLiteral(InDoc.RootGraph, OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const FMetasoundFrontendGraphClass& GraphClass : InDoc.Subgraphs)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ForEachLiteral(GraphClass, OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		for (const FMetasoundFrontendClass& Dependency : InDoc.Dependencies)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ForEachLiteral(Dependency, OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	void ForEachLiteral(const FMetasoundFrontendGraphClass& InGraphClass, FForEachLiteralFunctionRef OnLiteral)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ForEachLiteral(static_cast<const FMetasoundFrontendClass&>(InGraphClass), OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const FGuid PageID = DocumentPrivate::ResolveTargetPageID(InGraphClass);
		const FMetasoundFrontendGraph& Graph = InGraphClass.FindConstGraphChecked(PageID);
		for (const FMetasoundFrontendNode& Node : Graph.Nodes)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ForEachLiteral(Node, OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		for (const FMetasoundFrontendVariable& Variable : Graph.Variables)
		{
			OnLiteral(Variable.TypeName, Variable.Literal);
		}
	}

	void ForEachLiteral(const FMetasoundFrontendClass& InClass, FForEachLiteralFunctionRef OnLiteral)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ForEachLiteral(InClass.GetDefaultInterface(), OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void ForEachLiteral(const FMetasoundFrontendClassInterface& InClassInterface, FForEachLiteralFunctionRef OnLiteral)
	{
		for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
		{
			const FGuid PageID = DocumentPrivate::ResolveTargetPageID(ClassInput);
			const FMetasoundFrontendLiteral& DefaultLiteral = ClassInput.FindConstDefaultChecked(PageID);
			OnLiteral(ClassInput.TypeName, DefaultLiteral);
		}
	}

	void ForEachLiteral(const FMetasoundFrontendNode& InNode, FForEachLiteralFunctionRef OnLiteral)
	{
		for (const FMetasoundFrontendVertexLiteral& VertexLiteral : InNode.InputLiterals)
		{
			auto HasEqualVertexID = [&VertexLiteral](const FMetasoundFrontendVertex& InVertex) -> bool
			{ 
				return InVertex.VertexID == VertexLiteral.VertexID; 
			};

			if (const FMetasoundFrontendVertex* InputVertex = InNode.Interface.Inputs.FindByPredicate(HasEqualVertexID))
			{
				OnLiteral(InputVertex->TypeName, VertexLiteral.Value);
			}
		}

		if (InNode.ClassInterfaceOverride.IsValid())
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ForEachLiteral(InNode.ClassInterfaceOverride.Get(), OnLiteral);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}




	namespace DocumentPrivate
	{
	}

	void ForEachLiteral(const FMetasoundFrontendDocument& InDoc, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder)
	{
		ForEachLiteral(InDoc.RootGraph, OnLiteral, InPageOrder);

		for (const FMetasoundFrontendGraphClass& GraphClass : InDoc.Subgraphs)
		{
			ForEachLiteral(GraphClass, OnLiteral, InPageOrder);
		}

		for (const FMetasoundFrontendClass& Dependency : InDoc.Dependencies)
		{
			ForEachLiteral(Dependency, OnLiteral, InPageOrder);
		}
	}

	void ForEachLiteral(const FMetasoundFrontendGraphClass& InGraphClass, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder)
	{
		ForEachLiteral(static_cast<const FMetasoundFrontendClass&>(InGraphClass), OnLiteral, InPageOrder);

		const FMetasoundFrontendGraph* Graph = FindPreferredPage<FMetasoundFrontendGraph>(InGraphClass.GetConstGraphPages(), InPageOrder);
		if (Graph)
		{
			for (const FMetasoundFrontendNode& Node : Graph->Nodes)
			{
				ForEachLiteral(Node, OnLiteral, InPageOrder);
			}

			for (const FMetasoundFrontendVariable& Variable : Graph->Variables)
			{
				OnLiteral(Variable.TypeName, Variable.Literal);
			}
		}
	}

	void ForEachLiteral(const FMetasoundFrontendClass& InClass, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder)
	{
		ForEachLiteral(InClass.GetDefaultInterface(), OnLiteral, InPageOrder);
	}

	void ForEachLiteral(const FMetasoundFrontendClassInterface& InClassInterface, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder)
	{
		for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
		{
			const FMetasoundFrontendClassInputDefault* DefaultInput = FindPreferredPage<FMetasoundFrontendClassInputDefault>(ClassInput.GetDefaults(), InPageOrder);
			if (DefaultInput)
			{
				OnLiteral(ClassInput.TypeName, DefaultInput->Literal);
			}
		}
	}

	void ForEachLiteral(const FMetasoundFrontendNode& InNode, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder)
	{
		for (const FMetasoundFrontendVertexLiteral& VertexLiteral : InNode.InputLiterals)
		{
			auto HasEqualVertexID = [&VertexLiteral](const FMetasoundFrontendVertex& InVertex) -> bool
			{ 
				return InVertex.VertexID == VertexLiteral.VertexID; 
			};

			if (const FMetasoundFrontendVertex* InputVertex = InNode.Interface.Inputs.FindByPredicate(HasEqualVertexID))
			{
				OnLiteral(InputVertex->TypeName, VertexLiteral.Value);
			}
		}

		if (InNode.ClassInterfaceOverride.IsValid())
		{
			ForEachLiteral(InNode.ClassInterfaceOverride.Get(), OnLiteral, InPageOrder);
		}
	}
} // namespace Metasound::Frontend
