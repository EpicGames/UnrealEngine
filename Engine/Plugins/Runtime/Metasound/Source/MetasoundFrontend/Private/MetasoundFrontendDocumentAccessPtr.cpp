// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocumentAccessPtr.h"

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendDocumentBuilder.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace BuilderPrivate
		{
			FMetasoundFrontendGraph& FindBuildGraphChecked(FMetasoundFrontendGraphClass& InGraphClass)
			{
				if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
				{
					if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(InGraphClass.Metadata.GetClassName(), { }))
					{
						// Const cast and not exposed to public API to dissuade from direct manipulation of graphs as this can corrupt the internal document cache
						return const_cast<FMetasoundFrontendGraph&>(Builder->FindConstBuildGraphChecked());
					}
				}

				return InGraphClass.GetDefaultGraph();
			}

			const FMetasoundFrontendGraph& FindConstBuildGraphChecked(const FMetasoundFrontendGraphClass& InGraphClass)
			{
				if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
				{
					if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(InGraphClass.Metadata.GetClassName(), { }))
					{
						return Builder->FindConstBuildGraphChecked();
					}
				}

				return InGraphClass.GetConstDefaultGraph();
			}
		}

		namespace MetasoundFrontendDocumentAccessPtrPrivate
		{
			/** TFindInArray is a callable object for finding child objects with a parent 
			 * objects member array. When searching for a child object, it caches the index
			 * of the child object and will test that index first on subsequent searches.
			 *
			 * @tparam ParentType - The type of the parent object.
			 * @tparam ChildType - The type of the child object.
			 * @tparam ArrayAccessType - The type of callable object used to access the parent's array.
			 * @tparam TestType - The type of callable objet used to test for finding the desired child instance.
			 */
			template<typename ParentType, typename ChildType, typename ArrayAccessType, typename TestType>
			class TFindInArray
			{
			public:	
				/** Create a TFindInArray
				 *
				 * @param InAccessArray - A callable object which accepts a reference to the parent
				 *                        object and returns the array containing child objects.
				 * @param InTest - A callable object which accepts a const reference to a child object
				 *                 and returns true if it is the desired child object.
				 */
				TFindInArray(ArrayAccessType InAccessArray, TestType InTest)
				: AccessArray(InAccessArray)
				, Test(InTest)
				{
				}

				TFindInArray(const TFindInArray&) = default;
				TFindInArray& operator=(const TFindInArray&) = default;
				TFindInArray(TFindInArray&&) = default;
				TFindInArray& operator=(TFindInArray&&) = default;

				/** Find child object within parent object.
				 *
				 * @param InParent - The parent object containing the desired child object.
				 *
				 * @return If found, A pointer to the child object. Otherwise a nullptr.
				 */
				ChildType* operator()(ParentType& InParent) const
				{
					auto& Array = AccessArray(InParent);

					if (INDEX_NONE != CachedIndex)
					{
						if (CachedIndex < Array.Num())
						{
							if (Test(Array[CachedIndex]))
							{
								return &Array[CachedIndex];
							}
						}
					}

					CachedIndex = Array.IndexOfByPredicate(Test);
					if (INDEX_NONE != CachedIndex)
					{
						return &Array[CachedIndex];
					}

					return nullptr;
				}

			private:

				mutable int32 CachedIndex = INDEX_NONE;
				ArrayAccessType AccessArray;
				TestType Test;
			};

			/** TFindClassInDocument is a callable object for finding a class within
			 * a document. It searches both the document's Dependencies and Subgraphs.
			 * When searching for a class, it caches the location of the class and 
			 * will test that location first on subsequent searches.
			 * 
			 * @tparam ClassType type of class to tind.
			 * @tparam TestType - The type of callable objet used to test for finding the desired class.
			 */
			template<typename ClassType, typename TestType>
			struct TFindClassInDocument
			{
			public:
				/** Create a TFindClassInDocument
				 *
				 * @param InTest - A callable object which accepts a const reference to a FMetasoundFrontendClass
				 *                 and returns true if it is the desired class and false otherwise.
				 */
				TFindClassInDocument(TestType InTest)
				: Test(InTest)
				{
				}

				/** Find class in document.
				 *
				 * @tparam DocumentType - Type of document to search.
				 *
				 * @param InDoc - The document to search.
				 * 
				 * @return If found, a pointer to the desired class. Otherwise, a nullptr.
				 */
				template<typename DocumentType>
				ClassType*  operator()(DocumentType& InDoc) const
				{
					// Search in cached location.
					if (ClassType* FoundClass = FindClassInCachedLocation<DocumentType>(InDoc))
					{
						return FoundClass;
					}

					// Search in dependencies
					CachedIndex = InDoc.Dependencies.IndexOfByPredicate(Test);
					if (INDEX_NONE != CachedIndex)
					{
						CachedSource = ESource::Dependencies;
						return &InDoc.Dependencies[CachedIndex];
					}

					// Search in subgraphs.
					CachedIndex = InDoc.Subgraphs.IndexOfByPredicate(Test);
					if (INDEX_NONE != CachedIndex)
					{
						CachedSource = ESource::Subgraphs;
						return &InDoc.Subgraphs[CachedIndex];
					}

					// Could not find class.
					CachedSource = ESource::None;
					return nullptr;
				}

			private:

				template<typename DocumentType>
				ClassType*  FindClassInCachedLocation(DocumentType& InDoc) const
				{
					if (CachedIndex != INDEX_NONE)
					{
						switch (CachedSource)
						{
						case ESource::Dependencies:
							{
								if (CachedIndex < InDoc.Dependencies.Num())
								{
									if (Test(InDoc.Dependencies[CachedIndex]))
									{
										return &InDoc.Dependencies[CachedIndex];
									}
								}
							}
							break;

						case ESource::Subgraphs:
							{
								if (CachedIndex < InDoc.Subgraphs.Num())
								{
									if (Test(InDoc.Subgraphs[CachedIndex]))
									{
										return &InDoc.Subgraphs[CachedIndex];
									}
								}
							}
							break;
						default:
						break;
						}
					}

					return nullptr;
				}

				enum class ESource : uint8 { None, Dependencies, Subgraphs };

				mutable int32 CachedIndex = INDEX_NONE;
				mutable ESource CachedSource = ESource::None;
				TestType Test;
			};

			/** A callable object for accessing arrays of inputs from a node. */
			struct FGetNodeInputArray 
			{
			 	TArray<FMetasoundFrontendVertex>& operator()(FMetasoundFrontendNode& InNode) const { return InNode.Interface.Inputs; }
			 	const TArray<FMetasoundFrontendVertex>& operator()(const FMetasoundFrontendNode& InNode) const { return InNode.Interface.Inputs; }
			};

			/** A callable object for accessing arrays of outputs from a node. */
			struct FGetNodeOutputArray 
			{
			 	TArray<FMetasoundFrontendVertex>& operator()(FMetasoundFrontendNode& InNode) const { return InNode.Interface.Outputs; }
			 	const TArray<FMetasoundFrontendVertex>& operator()(const FMetasoundFrontendNode& InNode) const { return InNode.Interface.Outputs; }
			};

			/** A callable object for accessing arrays of inputs from a class. */
			struct FGetClassInputArray 
			{
			 	TArray<FMetasoundFrontendClassInput>& operator()(FMetasoundFrontendClass& InClass) const { return InClass.GetDefaultInterface().Inputs; }
			 	const TArray<FMetasoundFrontendClassInput>& operator()(const FMetasoundFrontendClass& InClass) const { return InClass.GetDefaultInterface().Inputs; }
			};

			/** A callable object for accessing arrays of outputs from a class. */
			struct FGetClassOutputArray 
			{
			 	TArray<FMetasoundFrontendClassOutput>& operator()(FMetasoundFrontendClass& InClass) const { return InClass.GetDefaultInterface().Outputs; }
			 	const TArray<FMetasoundFrontendClassOutput>& operator()(const FMetasoundFrontendClass& InClass) const { return InClass.GetDefaultInterface().Outputs; }
			};

			/** A callable object for accessing arrays of nodes from a graph class. */
			struct FGetGraphClassNodes
			{
			 	TArray<FMetasoundFrontendNode>& operator()(FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindBuildGraphChecked(InGraphClass).Nodes; }
			 	const TArray<FMetasoundFrontendNode>& operator()(const FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindConstBuildGraphChecked(InGraphClass).Nodes; }
			};

			/** A callable object for accessing arrays of variables from a graph class. */
			/*
			struct FGetGraphClassVariables
			{
			 	TMap<FGuid, FMetasoundFrontendVariable>& operator()(FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindBuildGraphChecked(InGraphClass).Variables; }
			 	const TMap<FGuid, FMetasoundFrontendVariable>& operator()(const FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindConstBuildGraphChecked(InGraphClass).Variables; }
			};
			*/
			struct FGetGraphClassVariables
			{
			 	TArray<FMetasoundFrontendVariable>& operator()(FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindBuildGraphChecked(InGraphClass).Variables; }
			 	const TArray<FMetasoundFrontendVariable>& operator()(const FMetasoundFrontendGraphClass& InGraphClass) const { return BuilderPrivate::FindConstBuildGraphChecked(InGraphClass).Variables; }
			};

			/** A callable object for accessing arrays of subgraphs from a document. */
			struct FGetDocumentSubgraphs
			{
			 	TArray<FMetasoundFrontendGraphClass>& operator()(FMetasoundFrontendDocument& InDocument) const { return InDocument.Subgraphs; }
			 	const TArray<FMetasoundFrontendGraphClass>& operator()(const FMetasoundFrontendDocument& InDocument) const { return InDocument.Subgraphs; }
			};

			/** A callable object for accessing arrays of dependencies from a document. */
			struct FGetDocumentDependencies
			{
			 	TArray<FMetasoundFrontendClass>& operator()(FMetasoundFrontendDocument& InDocument) const { return InDocument.Dependencies; }
			 	const TArray<FMetasoundFrontendClass>& operator()(const FMetasoundFrontendDocument& InDocument) const { return InDocument.Dependencies; }
			};

			/** A callable object for testing whether a vertex contains a matching name. */
			struct FIsVertexWithName
			{
				FIsVertexWithName(const FVertexName& InName)
				: Name(InName)
				{
				}

				bool operator() (const FMetasoundFrontendVertex& InVertex) const
				{
					return InVertex.Name == Name;
				}

			private:
				FVertexName Name;
			};

			/** A callable object for testing whether a vertex contains a matching ID. */
			struct FIsVertexWithID
			{
				FIsVertexWithID(const FGuid& InID)
				: ID(InID)
				{
				}

				bool operator() (const FMetasoundFrontendVertex& InVertex) const
				{
					return InVertex.VertexID == ID;
				}

			private:
				FGuid ID;
			};

			/** A callable object for testing whether a class input contains a matching node ID. */
			struct FIsClassInputWithNodeID
			{
				FIsClassInputWithNodeID(const FGuid& InNodeID)
				: NodeID(InNodeID)
				{
				}

				bool operator() (const FMetasoundFrontendClassInput& InClassInput) const { return InClassInput.NodeID == NodeID; }
			private:
				FGuid NodeID;
			};

			/** A callable object for testing whether a class output contains a matching node ID. */
			struct FIsClassOutputWithNodeID
			{
				FIsClassOutputWithNodeID(const FGuid& InNodeID)
				: NodeID(InNodeID)
				{
				}

				bool operator() (const FMetasoundFrontendClassOutput& InClassOutput) const
				{
					return InClassOutput.NodeID == NodeID;
				}

			private:
				FGuid NodeID;
			};

			/** A callable object for testing whether a variable contains a matching ID. */
			struct FIsVariableWithID
			{
				FIsVariableWithID(const FGuid& InVariableID)
				: VariableID(InVariableID)
				{
				}

				bool operator() (const FMetasoundFrontendVariable& InVariable) const
				{
					return InVariable.ID == VariableID;
				}

			private:
				FGuid VariableID;
			};

			/** A callable object for testing whether a node contains a matching ID. */
			struct FIsNodeWithID
			{
				FIsNodeWithID(const FGuid& InNodeID)
				: NodeID(InNodeID)
				{
				}

				bool operator() (const FMetasoundFrontendNode& InNode) const
				{
					return InNode.GetID() == NodeID;
				}

			private:
				FGuid NodeID;
			};

			/** A callable object for testing whether a class contains a matching ID. */
			struct FIsClassWithID
			{
				FIsClassWithID(const FGuid& InClassID)
				: ClassID(InClassID)
				{
				}

				bool operator() (const FMetasoundFrontendClass& InClass) const
				{
					return InClass.ID == ClassID;
				}

			private:
				FGuid ClassID;
			};

			/** A callable object for testing whether a class contains a matching Metadata. */
			class FIsClassWithMetadata
			{
			public:
				FIsClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata)
				: Metadata(InMetadata)
				{
				}

				bool operator()(const FMetasoundFrontendClass& InClass) const
				{ 
					return NodeRegistryKey::IsEqual(Metadata, InClass.Metadata);
				};
			private:
				FMetasoundFrontendClassMetadata Metadata;
			};

			/** A callable object for testing whether a class contains a matching registry key. */
			class FIsClassWithNodeRegistryKey
			{
			public:
				FIsClassWithNodeRegistryKey(const FNodeRegistryKey& InRegistryKey)
				: RegistryKey(InRegistryKey)
				{
				}

				bool operator()(const FMetasoundFrontendClass& InClass) const
				{
					return RegistryKey == FNodeRegistryKey(InClass.Metadata);
				};

				bool operator()(const FMetasoundFrontendGraphClass& InGraphClass) const
				{
					return RegistryKey == FNodeRegistryKey(InGraphClass);
				};
			private:
				FNodeRegistryKey RegistryKey;
			};
		}

		FVertexAccessPtr FNodeAccessPtr::GetInputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendNode, FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FVertexAccessPtr>(Finder);
		}

		FVertexAccessPtr FNodeAccessPtr::GetInputWithVertexID(const FGuid& InID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendNode, FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FVertexAccessPtr>(Finder);
		}

		FVertexAccessPtr FNodeAccessPtr::GetOutputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendNode, FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FVertexAccessPtr>(Finder);
		}

		FVertexAccessPtr FNodeAccessPtr::GetOutputWithVertexID(const FGuid& InID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendNode, FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FNodeAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FNodeAccessPtr::GetInputWithVertexID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FNodeAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FNodeAccessPtr::GetOutputWithVertexID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FConstNodeAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FConstNodeAccessPtr::GetInputWithVertexID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeInputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeInputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FConstNodeAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		FConstVertexAccessPtr FConstNodeAccessPtr::GetOutputWithVertexID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendNode, const FMetasoundFrontendVertex, FGetNodeOutputArray, FIsVertexWithID>;
			FFinder Finder{FGetNodeOutputArray(), FIsVertexWithID(InID)};

			return GetMemberAccessPtr<FConstVertexAccessPtr>(Finder);
		}

		// FConstClassAccessPtr
		FClassInputAccessPtr FClassAccessPtr::GetInputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendClass, FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FClassInputAccessPtr>(Finder);
		}

		FClassOutputAccessPtr FClassAccessPtr::GetOutputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendClass, FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder { FGetClassOutputArray(), FIsVertexWithName(InName) };

			return GetMemberAccessPtr<FClassOutputAccessPtr>(Finder);
		}

		FConstClassInputAccessPtr FClassAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FClassAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		// FConstClassAccessPtr
		FConstClassInputAccessPtr FConstClassAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FConstClassAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		// FGraphClassAccessPtr
		FClassInputAccessPtr FGraphClassAccessPtr::GetInputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendClass, FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FClassInputAccessPtr>(Finder);
		}

		FClassInputAccessPtr FGraphClassAccessPtr::GetInputWithNodeID(const FGuid& InNodeID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendGraphClass, FMetasoundFrontendClassInput, FGetClassInputArray, FIsClassInputWithNodeID>;
			FFinder Finder{FGetClassInputArray(), FIsClassInputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FClassInputAccessPtr>(Finder);
		}

		FClassOutputAccessPtr FGraphClassAccessPtr::GetOutputWithName(const FVertexName& InName)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendClass, FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FClassOutputAccessPtr>(Finder);
		}

		FClassOutputAccessPtr FGraphClassAccessPtr::GetOutputWithNodeID(const FGuid& InNodeID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendGraphClass, FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsClassOutputWithNodeID>;
			FFinder Finder{FGetClassOutputArray(), FIsClassOutputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FClassOutputAccessPtr>(Finder);
		}

		FVariableAccessPtr FGraphClassAccessPtr::GetVariableWithID(const FGuid& InID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendGraphClass, FMetasoundFrontendVariable, FGetGraphClassVariables, FIsVariableWithID>;
			FFinder Finder{FGetGraphClassVariables(), FIsVariableWithID(InID)};

			return GetMemberAccessPtr<FVariableAccessPtr>(Finder);
		}

		FNodeAccessPtr FGraphClassAccessPtr::GetNodeWithNodeID(const FGuid& InNodeID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendGraphClass, FMetasoundFrontendNode, FGetGraphClassNodes, FIsNodeWithID>;
			FFinder Finder{FGetGraphClassNodes(), FIsNodeWithID(InNodeID)};

			return GetMemberAccessPtr<FNodeAccessPtr>(Finder);
		}

		FGraphAccessPtr FGraphClassAccessPtr::GetGraph()
		{
			auto FindGraph = [](FMetasoundFrontendGraphClass& InGraphClass) -> FMetasoundFrontendGraph* 
			{ 
				return &BuilderPrivate::FindBuildGraphChecked(InGraphClass); 
			};

			return GetMemberAccessPtr<FGraphAccessPtr>(FindGraph);
		}


		FConstClassInputAccessPtr FGraphClassAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}


		FConstClassInputAccessPtr FGraphClassAccessPtr::GetInputWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsClassInputWithNodeID>;
			FFinder Finder{FGetClassInputArray(), FIsClassInputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FGraphClassAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FGraphClassAccessPtr::GetOutputWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsClassOutputWithNodeID>;
			FFinder Finder{FGetClassOutputArray(), FIsClassOutputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		FConstVariableAccessPtr FGraphClassAccessPtr::GetVariableWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendVariable, FGetGraphClassVariables, FIsVariableWithID>;
			FFinder Finder{FGetGraphClassVariables(), FIsVariableWithID(InID)};

			return GetMemberAccessPtr<FConstVariableAccessPtr>(Finder);
		}

		FConstNodeAccessPtr FGraphClassAccessPtr::GetNodeWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendNode, FGetGraphClassNodes, FIsNodeWithID>;
			FFinder Finder{FGetGraphClassNodes(), FIsNodeWithID(InNodeID)};

			return GetMemberAccessPtr<FConstNodeAccessPtr>(Finder);
		}

		FConstGraphAccessPtr FGraphClassAccessPtr::GetGraph() const
		{
			auto FindGraph = [](const FMetasoundFrontendGraphClass& InGraphClass) -> const FMetasoundFrontendGraph* 
			{ 
				return &BuilderPrivate::FindConstBuildGraphChecked(InGraphClass); 
			};

			return GetMemberAccessPtr<FConstGraphAccessPtr>(FindGraph);
		}

		// FConstGraphClassAccessPtr
		FConstClassInputAccessPtr FConstGraphClassAccessPtr::GetInputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassInputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}


		FConstClassInputAccessPtr FConstGraphClassAccessPtr::GetInputWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendClassInput, FGetClassInputArray, FIsClassInputWithNodeID>;
			FFinder Finder{FGetClassInputArray(), FIsClassInputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FConstClassInputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FConstGraphClassAccessPtr::GetOutputWithName(const FVertexName& InName) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsVertexWithName>;
			FFinder Finder{FGetClassOutputArray(), FIsVertexWithName(InName)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		FConstClassOutputAccessPtr FConstGraphClassAccessPtr::GetOutputWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendClassOutput, FGetClassOutputArray, FIsClassOutputWithNodeID>;
			FFinder Finder{FGetClassOutputArray(), FIsClassOutputWithNodeID(InNodeID)};

			return GetMemberAccessPtr<FConstClassOutputAccessPtr>(Finder);
		}

		FConstVariableAccessPtr FConstGraphClassAccessPtr::GetVariableWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendVariable, FGetGraphClassVariables, FIsVariableWithID>;
			FFinder Finder{FGetGraphClassVariables(), FIsVariableWithID(InID)};

			return GetMemberAccessPtr<FConstVariableAccessPtr>(Finder);
		}

		FConstNodeAccessPtr FConstGraphClassAccessPtr::GetNodeWithNodeID(const FGuid& InNodeID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendGraphClass, const FMetasoundFrontendNode, FGetGraphClassNodes, FIsNodeWithID>;
			FFinder Finder{FGetGraphClassNodes(), FIsNodeWithID(InNodeID)};

			return GetMemberAccessPtr<FConstNodeAccessPtr>(Finder);
		}

		FConstGraphAccessPtr FConstGraphClassAccessPtr::GetGraph() const
		{
			auto FindGraph = [](const FMetasoundFrontendGraphClass& InGraphClass) -> const FMetasoundFrontendGraph* 
			{ 
				return &BuilderPrivate::FindConstBuildGraphChecked(InGraphClass); 
			};

			return GetMemberAccessPtr<FConstGraphAccessPtr>(FindGraph);
		}

		// FDocumentAccessPtr
		FGraphClassAccessPtr FDocumentAccessPtr::GetRootGraph()
		{
			return GetMemberAccessPtr<FGraphClassAccessPtr>([](FMetasoundFrontendDocument& InDoc) { return &InDoc.RootGraph; });
		}

		FGraphClassAccessPtr FDocumentAccessPtr::GetSubgraphWithID(const FGuid& InID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendDocument, FMetasoundFrontendGraphClass, FGetDocumentSubgraphs, FIsClassWithID>;
			FFinder Finder{FGetDocumentSubgraphs(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FGraphClassAccessPtr>(Finder);
		}

		FClassAccessPtr FDocumentAccessPtr::GetDependencyWithID(const FGuid& InID)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<FMetasoundFrontendDocument, FMetasoundFrontendClass, FGetDocumentDependencies, FIsClassWithID>;
			FFinder Finder{FGetDocumentDependencies(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FClassAccessPtr>(Finder);
		}


		FClassAccessPtr FDocumentAccessPtr::GetClassWithID(const FGuid& InID) 
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<FMetasoundFrontendClass, FIsClassWithID>;
			FFinder Finder(InID);

			return GetMemberAccessPtr<FClassAccessPtr>(Finder);
		}

		FClassAccessPtr FDocumentAccessPtr::GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<FMetasoundFrontendClass, FIsClassWithMetadata>;
			FFinder Finder(InMetadata);

			return GetMemberAccessPtr<FClassAccessPtr>(Finder);
		}

		FClassAccessPtr FDocumentAccessPtr::GetClassWithInfo(const FNodeClassInfo& InInfo)
		{
			return { };
		}

		FClassAccessPtr FDocumentAccessPtr::GetClassWithRegistryKey(const FNodeRegistryKey& InKey)
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<FMetasoundFrontendClass, FIsClassWithNodeRegistryKey>;
			FFinder Finder(InKey);

			return GetMemberAccessPtr<FClassAccessPtr>(Finder);
		}

		FConstGraphClassAccessPtr FDocumentAccessPtr::GetRootGraph() const
		{
			return GetMemberAccessPtr<FConstGraphClassAccessPtr>([](const FMetasoundFrontendDocument& InDoc) { return &InDoc.RootGraph; });
		}

		FConstGraphClassAccessPtr FDocumentAccessPtr::GetSubgraphWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendDocument, const FMetasoundFrontendGraphClass, FGetDocumentSubgraphs, FIsClassWithID>;
			FFinder Finder{FGetDocumentSubgraphs(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FConstGraphClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FDocumentAccessPtr::GetDependencyWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendDocument, const FMetasoundFrontendClass, FGetDocumentDependencies, FIsClassWithID>;
			FFinder Finder{FGetDocumentDependencies(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}


		FConstClassAccessPtr FDocumentAccessPtr::GetClassWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithID>;
			FFinder Finder(InID);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FDocumentAccessPtr::GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithMetadata>;
			FFinder Finder(InMetadata);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FDocumentAccessPtr::GetClassWithInfo(const FNodeClassInfo& InInfo) const
		{
			return { };
		}

		FConstClassAccessPtr FDocumentAccessPtr::GetClassWithRegistryKey(const FNodeRegistryKey& InKey) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithNodeRegistryKey>;
			FFinder Finder(InKey);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}

		FConstGraphClassAccessPtr FConstDocumentAccessPtr::GetRootGraph() const
		{
			return GetMemberAccessPtr<FConstGraphClassAccessPtr>([](const FMetasoundFrontendDocument& InDoc) { return &InDoc.RootGraph; });
		}

		FConstGraphClassAccessPtr FConstDocumentAccessPtr::GetSubgraphWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendDocument, const FMetasoundFrontendGraphClass, FGetDocumentSubgraphs, FIsClassWithID>;
			FFinder Finder{FGetDocumentSubgraphs(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FConstGraphClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FConstDocumentAccessPtr::GetDependencyWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder= TFindInArray<const FMetasoundFrontendDocument, const FMetasoundFrontendClass, FGetDocumentDependencies, FIsClassWithID>;
			FFinder Finder{FGetDocumentDependencies(), FIsClassWithID(InID)};

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}


		FConstClassAccessPtr FConstDocumentAccessPtr::GetClassWithID(const FGuid& InID) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithID>;
			FFinder Finder(InID);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FConstDocumentAccessPtr::GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithMetadata>;
			FFinder Finder(InMetadata);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}

		FConstClassAccessPtr FConstDocumentAccessPtr::GetClassWithInfo(const FNodeClassInfo& InInfo) const
		{
			return { };
		}

		FConstClassAccessPtr FConstDocumentAccessPtr::GetClassWithRegistryKey(const FNodeRegistryKey& InKey) const
		{
			using namespace MetasoundFrontendDocumentAccessPtrPrivate;

			using FFinder = TFindClassInDocument<const FMetasoundFrontendClass, FIsClassWithNodeRegistryKey>;
			FFinder Finder(InKey);

			return GetMemberAccessPtr<FConstClassAccessPtr>(Finder);
		}
	}
}
