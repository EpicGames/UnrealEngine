// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFrontendDocumentBuilder.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "AudioParameter.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "Interfaces/MetasoundFrontendInterfaceBindingRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundAssetBase.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocumentCache.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendNodeUpdateTransform.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundTrace.h"
#include "MetasoundVariableNodes.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendDocumentBuilder)


namespace Metasound::Frontend
{
	namespace DocumentBuilderPrivate
	{
		bool FindInputRegistryClass(FName TypeName, EMetasoundFrontendVertexAccessType AccessType, FMetasoundFrontendClass& OutClass)
		{
			switch (AccessType)
			{
				case EMetasoundFrontendVertexAccessType::Value:
				{
					return IDataTypeRegistry::Get().GetFrontendConstructorInputClass(TypeName, OutClass);
				}

				case EMetasoundFrontendVertexAccessType::Reference:
				{
					return IDataTypeRegistry::Get().GetFrontendInputClass(TypeName, OutClass);
				}

				case EMetasoundFrontendVertexAccessType::Unset:
				default:
				{
					checkNoEntry();
				}
				break;
			}

			return false;
		}

		bool FindOutputRegistryClass(FName TypeName, EMetasoundFrontendVertexAccessType AccessType, FMetasoundFrontendClass& OutClass)
		{
			switch (AccessType)
			{
				case EMetasoundFrontendVertexAccessType::Value:
				{
					return IDataTypeRegistry::Get().GetFrontendConstructorOutputClass(TypeName, OutClass);
				}

				case EMetasoundFrontendVertexAccessType::Reference:
				{
					return IDataTypeRegistry::Get().GetFrontendOutputClass(TypeName, OutClass);
				}

				case EMetasoundFrontendVertexAccessType::Unset:
				default:
				{
					checkNoEntry();
				}
				break;
			}

			return false;
		}

		FName GenerateUniqueName(const TSet<FName>& InExistingNames, const FString& InBaseName)
		{
			int32 PostFixInt = 0;
			FString NewName = InBaseName;

			while (InExistingNames.Contains(*NewName))
			{
				PostFixInt++;
				NewName = FString::Format(TEXT("{0} {1}"), { InBaseName, PostFixInt });
			}

			return FName(*NewName);
		}

		bool NameContainsInterfaceNamespace(FName VertexName, FMetasoundFrontendInterface* OutInterface)
		{
			using namespace Metasound::Frontend;

			FName InterfaceNamespace;
			FName ParamName;
			Audio::FParameterPath::SplitName(VertexName, InterfaceNamespace, ParamName);

			if (!InterfaceNamespace.IsNone())
			{
				FMetasoundFrontendVersion InterfaceVersion;
				if (ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceNamespace, InterfaceVersion))
				{
					if (OutInterface)
					{
						FMetasoundFrontendInterface FoundInterface;
						const bool bInterfaceFound = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, FoundInterface);
						checkf(bInterfaceFound, TEXT("MetaSound Interface found via search engine must exist in registry"))
						*OutInterface = MoveTemp(FoundInterface);
					}
					return true;
				}
			}

			if (OutInterface)
			{
				*OutInterface = { };
			}
			return false;
		}

		bool IsInterfaceInput(FName InputName, FName TypeName, FMetasoundFrontendInterface* OutInterface)
		{
			FMetasoundFrontendInterface Interface;
			if (NameContainsInterfaceNamespace(InputName, &Interface))
			{
				auto IsInput = [&InputName, &TypeName](const FMetasoundFrontendClassInput& InterfaceInput)
				{
					return InputName == InterfaceInput.Name && InterfaceInput.TypeName == TypeName;
				};

				if (Interface.Inputs.ContainsByPredicate(IsInput))
				{
					if (OutInterface)
					{
						*OutInterface = MoveTemp(Interface);
					}
					return true;
				}
			}

			if (OutInterface)
			{
				*OutInterface = { };
			}
			return false;
		}

		bool IsInterfaceOutput(FName OutputName, FName TypeName, FMetasoundFrontendInterface* OutInterface)
		{
			FMetasoundFrontendInterface Interface;
			if (NameContainsInterfaceNamespace(OutputName, &Interface))
			{
				auto IsOutput = [&OutputName, &TypeName](const FMetasoundFrontendClassInput& InterfaceOutput)
				{
					return OutputName == InterfaceOutput.Name && InterfaceOutput.TypeName == TypeName;
				};

				if (Interface.Outputs.ContainsByPredicate(IsOutput))
				{
					if (OutInterface)
					{
						*OutInterface = MoveTemp(Interface);
					}
					return true;
				}
			}

			if (OutInterface)
			{
				*OutInterface = { };
			}
			return false;
		}

		bool TryGetInterfaceBoundEdges(
			const FGuid& InFromNodeID,
			const TSet<FMetasoundFrontendVersion>& InFromNodeInterfaces,
			const FGuid& InToNodeID,
			const TSet<FMetasoundFrontendVersion>& InToNodeInterfaces,
			TSet<FNamedEdge>& OutNamedEdges)
		{
			OutNamedEdges.Reset();
			TSet<FName> InputNames;
			for (const FMetasoundFrontendVersion& InputInterfaceVersion : InToNodeInterfaces)
			{
				TArray<const FInterfaceBindingRegistryEntry*> BindingEntries;
				if (IInterfaceBindingRegistry::Get().FindInterfaceBindingEntries(InputInterfaceVersion, BindingEntries))
				{
					Algo::Sort(BindingEntries, [](const FInterfaceBindingRegistryEntry* A, const FInterfaceBindingRegistryEntry* B)
					{
						check(A);
						check(B);
						return A->GetBindingPriority() < B->GetBindingPriority();
					});

					// Bindings are sorted in registry with earlier entries being higher priority to apply connections,
					// so earlier listed connections are selected over potential collisions with later entries.
					for (const FInterfaceBindingRegistryEntry* BindingEntry : BindingEntries)
					{
						check(BindingEntry);
						if (InFromNodeInterfaces.Contains(BindingEntry->GetOutputInterfaceVersion()))
						{
							for (const FMetasoundFrontendInterfaceVertexBinding& VertexBinding : BindingEntry->GetVertexBindings())
							{
								if (!InputNames.Contains(VertexBinding.InputName))
								{
									InputNames.Add(VertexBinding.InputName);
									OutNamedEdges.Add(FNamedEdge { InFromNodeID, VertexBinding.OutputName, InToNodeID, VertexBinding.InputName });
								}
							}
						}
					}
				}
			};

			return true;
		}

		void SetNodeAndVertexNames(FMetasoundFrontendNode& InOutNode, const FMetasoundFrontendClassVertex& InVertex)
		{
			InOutNode.Name = InVertex.Name;
			// Set name on related vertices of input node
			auto IsVertexWithTypeName = [&InVertex](const FMetasoundFrontendVertex& Vertex) { return Vertex.TypeName == InVertex.TypeName; };
			if (FMetasoundFrontendVertex* InputVertex = InOutNode.Interface.Inputs.FindByPredicate(IsVertexWithTypeName))
			{
				InputVertex->Name = InVertex.Name;
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Node associated with graph vertex of type '%ls' does not contain input vertex of matching type.", *InVertex.TypeName.ToString());
			}

			if (FMetasoundFrontendVertex* OutputVertex = InOutNode.Interface.Outputs.FindByPredicate(IsVertexWithTypeName))
			{
				OutputVertex->Name = InVertex.Name;
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Node associated with graph vertex of type '%ls' does not contain output vertex of matching type.", *InVertex.TypeName.ToString());
			}
		}

		void SetDefaultLiteralOnInputNode(FMetasoundFrontendNode& InOutNode, const FMetasoundFrontendClassInput& InClassInput, const FGuid& PageID)
		{
			// Set the default literal on the nodes inputs so that it gets passed to the instantiated TInputNode on a live
			// auditioned MetaSound
			auto IsVertexWithName = [&Name = InClassInput.Name](const FMetasoundFrontendVertex& InVertex)
			{
				return InVertex.Name == Name;
			};

			if (const FMetasoundFrontendVertex* InputVertex = InOutNode.Interface.Inputs.FindByPredicate(IsVertexWithName))
			{
				auto IsVertexLiteralWithVertexID = [&VertexID = InputVertex->VertexID](const FMetasoundFrontendVertexLiteral& VertexLiteral)
				{
					return VertexLiteral.VertexID == VertexID;
				};
				if (FMetasoundFrontendVertexLiteral* VertexLiteral = InOutNode.InputLiterals.FindByPredicate(IsVertexLiteralWithVertexID))
				{
					// Update existing literal default value with value from class input.
					// It is possible the default has been cooked out for this page when this is called 
					// when rebuilding a preset root graph so ignore it in that case
					const FMetasoundFrontendLiteral* DefaultLiteral = InClassInput.FindConstDefault(PageID);
					if (DefaultLiteral)
					{
						VertexLiteral->Value = *DefaultLiteral;
					}
				}
				else
				{
					// Add literal default value with value from class input.
					// It is possible the default has been cooked out for this page when this is called 
					// when rebuilding a preset root graph so ignore it in that case
					const FMetasoundFrontendLiteral* DefaultLiteral = InClassInput.FindConstDefault(PageID);
					if (DefaultLiteral)
					{
						InOutNode.InputLiterals.Add(FMetasoundFrontendVertexLiteral{ InputVertex->VertexID, *DefaultLiteral });
					}

				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Input node associated with graph input vertex of name '%ls' does not contain input vertex with matching name.", *InClassInput.Name.ToString());
			}
		}

		class FModifyInterfacesImpl
		{
		public:
			FModifyInterfacesImpl(FMetasoundFrontendDocument& InDocument, FModifyInterfaceOptions&& InOptions)
				: Options(MoveTemp(InOptions))
				, Document(InDocument)
			{
				for (const FMetasoundFrontendInterface& FromInterface : Options.InterfacesToRemove)
				{
					if (Document.Interfaces.Contains(FromInterface.Metadata.Version))
					{
						InputsToRemove.Append(FromInterface.Inputs);
						OutputsToRemove.Append(FromInterface.Outputs);
					}
				}

				auto SetNodeID = [this](FMetasoundFrontendClassVertex& InVertex, bool bIsInput)
				{
					FGuid VertexID = FDocumentIDGenerator::Get().CreateVertexID(Document);
					FGuid NodeID = FDocumentIDGenerator::Get().CreateNodeID(Document);

					if (Options.ReferencedBuilder)
					{
						if (bIsInput)
						{
							const FMetasoundFrontendClassInput* GraphInput = Options.ReferencedBuilder->FindGraphInput(InVertex.Name);
							if (GraphInput && GraphInput->TypeName == InVertex.TypeName)
							{
								NodeID = GraphInput->NodeID;
								VertexID = GraphInput->VertexID;
							}
						}
						else // Output
						{
							const FMetasoundFrontendClassOutput* GraphOutput = Options.ReferencedBuilder->FindGraphOutput(InVertex.Name);
							if (GraphOutput && GraphOutput->TypeName == InVertex.TypeName)
							{
								NodeID = GraphOutput->NodeID;
								VertexID = GraphOutput->VertexID;
							}
						}
					}
					InVertex.NodeID = NodeID;
					InVertex.VertexID = VertexID;
				};

				for (const FMetasoundFrontendInterface& ToInterface : Options.InterfacesToAdd)
				{
					Algo::Transform(ToInterface.Inputs, InputsToAdd, [this, &ToInterface, &SetNodeID](const FMetasoundFrontendClassInput& Input)
					{
						FMetasoundFrontendClassInput NewInput = Input;
						SetNodeID(NewInput, true);
						return FInputInterfacePair { MoveTemp(NewInput), &ToInterface };
					});

					Algo::Transform(ToInterface.Outputs, OutputsToAdd, [this, &ToInterface, &SetNodeID](const FMetasoundFrontendClassOutput& Output)
					{
						FMetasoundFrontendClassOutput NewOutput = Output;
						SetNodeID(NewOutput, false);
						return FOutputInterfacePair { MoveTemp(NewOutput), &ToInterface };
					});
				}

				// Iterate in reverse to allow removal from `InputsToAdd`
				for (int32 AddIndex = InputsToAdd.Num() - 1; AddIndex >= 0; AddIndex--)
				{
					const FMetasoundFrontendClassVertex& VertexToAdd = InputsToAdd[AddIndex].Key;

					const int32 RemoveIndex = InputsToRemove.IndexOfByPredicate([&](const FMetasoundFrontendClassVertex& VertexToRemove)
						{
							if (VertexToAdd.TypeName != VertexToRemove.TypeName)
							{
								return false;
							}

							if (Options.NamePairingFunction)
							{
								return Options.NamePairingFunction(VertexToAdd.Name, VertexToRemove.Name);
							}

							FName ParamA;
							FName ParamB;
							FName Namespace;
							VertexToAdd.SplitName(Namespace, ParamA);
							VertexToRemove.SplitName(Namespace, ParamB);

							return ParamA == ParamB;
						});

					if (INDEX_NONE != RemoveIndex)
					{
						PairedInputs.Add(FVertexPair { InputsToRemove[RemoveIndex], InputsToAdd[AddIndex].Key });
						InputsToRemove.RemoveAtSwap(RemoveIndex, EAllowShrinking::No);
						InputsToAdd.RemoveAtSwap(AddIndex, EAllowShrinking::No);
					}
				}

				// Iterate in reverse to allow removal from `OutputsToAdd`
				for (int32 AddIndex = OutputsToAdd.Num() - 1; AddIndex >= 0; AddIndex--)
				{
					const FMetasoundFrontendClassVertex& VertexToAdd = OutputsToAdd[AddIndex].Key;

					const int32 RemoveIndex = OutputsToRemove.IndexOfByPredicate([&](const FMetasoundFrontendClassVertex& VertexToRemove)
						{
							if (VertexToAdd.TypeName != VertexToRemove.TypeName)
							{
								return false;
							}

							if (Options.NamePairingFunction)
							{
								return Options.NamePairingFunction(VertexToAdd.Name, VertexToRemove.Name);
							}

							FName ParamA;
							FName ParamB;
							FName Namespace;
							VertexToAdd.SplitName(Namespace, ParamA);
							VertexToRemove.SplitName(Namespace, ParamB);

							return ParamA == ParamB;
						});

					if (INDEX_NONE != RemoveIndex)
					{
						PairedOutputs.Add(FVertexPair{ OutputsToRemove[RemoveIndex], OutputsToAdd[AddIndex].Key });
						OutputsToRemove.RemoveAtSwap(RemoveIndex);
						OutputsToAdd.RemoveAtSwap(AddIndex);
					}
				}
			}

		private:
			bool AddMissingVertices(FMetaSoundFrontendDocumentBuilder& OutBuilder) const
			{
				if (!InputsToAdd.IsEmpty() || !OutputsToAdd.IsEmpty())
				{
					for (const FInputInterfacePair& Pair: InputsToAdd)
					{
						OutBuilder.AddGraphInput(Pair.Key);
					}

					for (const FOutputInterfacePair& Pair : OutputsToAdd)
					{
						OutBuilder.AddGraphOutput(Pair.Key);
					}

					return true;
				}

				return false;
			}

			bool RemoveUnsupportedVertices(FMetaSoundFrontendDocumentBuilder& OutBuilder) const
			{
				bool bDidEdit = false;

				for (const TPair<FMetasoundFrontendClassInput, const FMetasoundFrontendInterface*>& Pair : InputsToAdd)
				{
					if (OutBuilder.RemoveGraphInput(Pair.Key.Name))
					{
						UE_LOGF(LogMetaSound, Warning, "Removed existing targeted input '%ls' to avoid name collision/member data descrepancies while modifying interface(s). Desired edges may have been removed as a result.", *Pair.Key.Name.ToString());
						bDidEdit = true;
					}
				}

				for (const TPair<FMetasoundFrontendClassOutput, const FMetasoundFrontendInterface*>& Pair : OutputsToAdd)
				{
					if (OutBuilder.RemoveGraphOutput(Pair.Key.Name))
					{
						UE_LOGF(LogMetaSound, Warning, "Removed existing targeted output '%ls' to avoid name collision/member data descrepancies while modifying interface(s). Desired edges may have been removed as a result.", *Pair.Key.Name.ToString());
						bDidEdit = true;
					}
				}

				if (!InputsToRemove.IsEmpty() || !OutputsToRemove.IsEmpty())
				{
					// Remove unsupported inputs
					for (const FMetasoundFrontendClassVertex& InputToRemove : InputsToRemove)
					{
						if (OutBuilder.RemoveGraphInput(InputToRemove.Name))
						{
							bDidEdit = true;
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Failed to remove existing input '%ls', which was an expected member of a removed interface.", *InputToRemove.Name.ToString());
						}
					}

					// Remove unsupported outputs
					for (const FMetasoundFrontendClassVertex& OutputToRemove : OutputsToRemove)
					{
						if (OutBuilder.RemoveGraphOutput(OutputToRemove.Name))
						{
							bDidEdit = true;
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Failed to remove existing output '%ls', which was an expected member of a removed interface.", *OutputToRemove.Name.ToString());
						}
					}

					return true;
				}

				return false;
			}

			bool SwapPairedVertices(FMetaSoundFrontendDocumentBuilder& OutBuilder) const
			{
				bool bDidEdit = false;
				for (const FVertexPair& PairedInput : PairedInputs)
				{
					const bool bSwapped = OutBuilder.SwapGraphInput(PairedInput.Get<0>(), PairedInput.Get<1>());
					bDidEdit |= bSwapped;
				}

				for (const FVertexPair& PairedOutput : PairedOutputs)
				{
					const bool bSwapped = OutBuilder.SwapGraphOutput(PairedOutput.Get<0>(), PairedOutput.Get<1>());
					bDidEdit |= bSwapped;
				}

				return bDidEdit;
			}

#if WITH_EDITORONLY_DATA
			void UpdateAddedVertexNodePositions(
				EMetasoundFrontendClassType ClassType,
				const FMetaSoundFrontendDocumentBuilder& InBuilder,
				TSet<FName>& AddedNames,
				TFunctionRef<int32(const FVertexName&)> InGetSortOrder,
				const FVector2D& InitOffset,
				TArrayView<FMetasoundFrontendNode> OutNodes)
			{
				// Add graph member nodes by sort order
				TSortedMap<int32, FMetasoundFrontendNode*> SortOrderToNode;
				for (FMetasoundFrontendNode& Node : OutNodes)
				{
					if (const FMetasoundFrontendClass* Class = InBuilder.FindDependency(Node.ClassID))
					{
						if (Class->Metadata.GetType() == ClassType)
						{
							const int32 Index = InGetSortOrder(Node.Name);
							SortOrderToNode.Add(Index, &Node);
						}
					}
				}

				// Prime the first location as an offset prior to an existing location (as provided by a swapped member)
				//  to avoid placing away from user's active area if possible.
				FVector2D NextLocation = InitOffset;
				{
					int32 NumBeforeDefined = 1;
					for (const TPair<int32, FMetasoundFrontendNode*>& Pair : SortOrderToNode)
					{
						const FMetasoundFrontendNode* Node = Pair.Value;
						const FName NodeName = Node->Name;
						if (AddedNames.Contains(NodeName))
						{
							NumBeforeDefined++;
						}
						else
						{
							const TMap<FGuid, FVector2D>& Locations = Node->Style.Display.Locations;
							if (!Locations.IsEmpty())
							{
								auto It = Locations.CreateConstIterator();
								const TPair<FGuid, FVector2D>& Location = *It;
								NextLocation = Location.Value - (NumBeforeDefined * DisplayStyle::NodeLayout::DefaultOffsetY);
								break;
							}
						}
					}
				}

				// Iterate through sorted map in sequence, slotting in new locations after
				// existing swapped nodes with predefined locations relative to one another.
				for (TPair<int32, FMetasoundFrontendNode*>& Pair : SortOrderToNode)
				{
					FMetasoundFrontendNode* Node = Pair.Value;
					const FName NodeName = Node->Name;
					if (AddedNames.Contains(NodeName))
					{
						bool bAddedLocation = false;
						for (TPair<FGuid, FVector2D>& LocationPair : Node->Style.Display.Locations)
						{
							bAddedLocation = true;
							LocationPair.Value = NextLocation;
						}
						if (!bAddedLocation)
						{
							Node->Style.Display.Locations.Add(FGuid::NewGuid(), NextLocation);
						}
						NextLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
					}
					else
					{
						for (const TPair<FGuid, FVector2D>& Location : Node->Style.Display.Locations)
						{
							NextLocation = Location.Value + DisplayStyle::NodeLayout::DefaultOffsetY;
						}
					}
				}
			}
#endif // WITH_EDITORONLY_DATA

		public:
			bool Execute(FMetaSoundFrontendDocumentBuilder& OutBuilder, FDocumentModifyDelegates& OutDelegates)
			{
				bool bDidEdit = false;

				for (const FMetasoundFrontendInterface& Interface : Options.InterfacesToRemove)
				{
					if (Document.Interfaces.Contains(Interface.Metadata.Version))
					{
						OutDelegates.InterfaceDelegates.OnRemovingInterface.Broadcast(Interface);
						bDidEdit = true;
#if WITH_EDITORONLY_DATA
						Document.Metadata.ModifyContext.AddInterfaceModified(Interface.Metadata.Version.Name);
#endif // WITH_EDITORONLY_DATA
						Document.Interfaces.Remove(Interface.Metadata.Version);
					}
				}

				for (const FMetasoundFrontendInterface& Interface : Options.InterfacesToAdd)
				{
					bool bAlreadyInSet = false;
					Document.Interfaces.Add(Interface.Metadata.Version, &bAlreadyInSet);
					if (!bAlreadyInSet)
					{
						OutDelegates.InterfaceDelegates.OnInterfaceAdded.Broadcast(Interface);
						bDidEdit = true;
#if WITH_EDITORONLY_DATA
						Document.Metadata.ModifyContext.AddInterfaceModified(Interface.Metadata.Version.Name);
#endif // WITH_EDITORONLY_DATA
					}
				}

				bDidEdit |= RemoveUnsupportedVertices(OutBuilder);
				bDidEdit |= SwapPairedVertices(OutBuilder);
				const bool bAddedVertices = AddMissingVertices(OutBuilder);
				bDidEdit |= bAddedVertices;

				if (bDidEdit)
				{
					OutBuilder.RemoveUnusedDependencies();
				}

#if WITH_EDITORONLY_DATA
				if (bAddedVertices && Options.bSetDefaultNodeLocations && !IsRunningCookCommandlet())
				{
					Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
					{
						TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
						// Sort/Place Inputs
						{
							TSet<FName> NamesToSort;
							Algo::Transform(InputsToAdd, NamesToSort, [](const FInputInterfacePair& Pair) { return Pair.Key.Name; });
							auto GetInputSortOrder = [&OutBuilder](const FVertexName& InVertexName)
							{
								const FMetasoundFrontendClassInput* Input = OutBuilder.FindGraphInput(InVertexName);
								checkf(Input, TEXT("Input must exist by this point of modifying the document's interfaces and respective members"));
								return Input->Metadata.SortOrderIndex;
							};
							UpdateAddedVertexNodePositions(EMetasoundFrontendClassType::Input, OutBuilder, NamesToSort, GetInputSortOrder, FVector2D::Zero(), Nodes);
						}

						// Sort/Place Outputs
						{
							TSet<FName> NamesToSort;
							Algo::Transform(OutputsToAdd, NamesToSort, [](const FOutputInterfacePair& OutputInterfacePair) { return OutputInterfacePair.Key.Name; });
							auto GetOutputSortOrder = [&OutBuilder](const FVertexName& InVertexName)
							{
								const FMetasoundFrontendClassOutput* Output = OutBuilder.FindGraphOutput(InVertexName);
								checkf(Output, TEXT("Output must exist by this point of modifying the document's interfaces and respective members"));
								return Output->Metadata.SortOrderIndex;
							};
							UpdateAddedVertexNodePositions(EMetasoundFrontendClassType::Output, OutBuilder, NamesToSort, GetOutputSortOrder, 3 * DisplayStyle::NodeLayout::DefaultOffsetX, Nodes);
						}
					});
				}
#endif // WITH_EDITORONLY_DATA

				return bDidEdit;
			}

			const FModifyInterfaceOptions Options;

		private:
			FMetasoundFrontendDocument& Document;

			using FVertexPair = TTuple<FMetasoundFrontendClassVertex, FMetasoundFrontendClassVertex>;
			TArray<FVertexPair> PairedInputs;
			TArray<FVertexPair> PairedOutputs;

			using FInputInterfacePair = TPair<FMetasoundFrontendClassInput, const FMetasoundFrontendInterface*>;
			using FOutputInterfacePair = TPair<FMetasoundFrontendClassOutput, const FMetasoundFrontendInterface*>;
			TArray<FInputInterfacePair> InputsToAdd;
			TArray<FOutputInterfacePair> OutputsToAdd;

			TArray<FMetasoundFrontendClassInput> InputsToRemove;
			TArray<FMetasoundFrontendClassOutput> OutputsToRemove;
		};
	} // namespace DocumentBuilderPrivate

	FString LexToString(const EInvalidEdgeReason& InReason)
	{
		switch (InReason)
		{
			case EInvalidEdgeReason::None:
				return TEXT("No reason");

			case EInvalidEdgeReason::MismatchedAccessType:
				return TEXT("Mismatched Access Type");

			case EInvalidEdgeReason::MismatchedDataType:
				return TEXT("Mismatched DataType");

			case EInvalidEdgeReason::MissingInput:
				return TEXT("Missing Input");

			case EInvalidEdgeReason::MissingOutput:
				return TEXT("Missing Output");

			default:
				return TEXT("COUNT");
		}

		static_assert(static_cast<uint32>(EInvalidEdgeReason::COUNT) == 5, "Potential missing case coverage for EInvalidEdgeReason");
	}

	FModifyInterfaceOptions::FModifyInterfaceOptions(const TArray<FMetasoundFrontendInterface>& InInterfacesToRemove, const TArray<FMetasoundFrontendInterface>& InInterfacesToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder)
		: InterfacesToRemove(InInterfacesToRemove)
		, InterfacesToAdd(InInterfacesToAdd)
		, ReferencedBuilder(InReferencedBuilder)
	{
	}

	FModifyInterfaceOptions::FModifyInterfaceOptions(TArray<FMetasoundFrontendInterface>&& InInterfacesToRemove, TArray<FMetasoundFrontendInterface>&& InInterfacesToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder)
		: InterfacesToRemove(MoveTemp(InInterfacesToRemove))
		, InterfacesToAdd(MoveTemp(InInterfacesToAdd))
		, ReferencedBuilder(InReferencedBuilder)
	{
	}

	FModifyInterfaceOptions::FModifyInterfaceOptions(const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToRemove, const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder)
	{
		Algo::Transform(InInterfaceVersionsToRemove, InterfacesToRemove, [](const FMetasoundFrontendVersion& Version)
		{
			FMetasoundFrontendInterface Interface;
			const bool bFromInterfaceFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
			if (!ensureAlways(bFromInterfaceFound))
			{
				UE_LOGF(LogMetaSound, Error, "Failed to find interface '%ls' to remove", *Version.ToString());
			}
			return Interface;
		});

		Algo::Transform(InInterfaceVersionsToAdd, InterfacesToAdd, [](const FMetasoundFrontendVersion& Version)
		{
			FMetasoundFrontendInterface Interface;
			const bool bToInterfaceFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
			if (!ensureAlways(bToInterfaceFound))
			{
				UE_LOGF(LogMetaSound, Error, "Failed to find interface '%ls' to add", *Version.ToString());
			}
			return Interface;
		});
		ReferencedBuilder = InReferencedBuilder;
	}
} // namespace Metasound::Frontend

UMetaSoundBuilderDocument::UMetaSoundBuilderDocument(const FObjectInitializer& ObjectInitializer)
{
	Document.RootGraph.ID = FGuid::NewGuid();
}

UMetaSoundBuilderDocument& UMetaSoundBuilderDocument::Create(const IMetaSoundDocumentInterface& InDocToCopy)
{
	UMetaSoundBuilderDocument* DocObject = NewObject<UMetaSoundBuilderDocument>();
	check(DocObject);
	DocObject->Document = InDocToCopy.GetConstDocument();
	DocObject->MetaSoundUClass = &InDocToCopy.GetBaseMetaSoundUClass();
	DocObject->BuilderUClass = &InDocToCopy.GetBuilderUClass();
	return *DocObject;
}

bool UMetaSoundBuilderDocument::ConformObjectToDocument()
{
	return false;
}

FTopLevelAssetPath UMetaSoundBuilderDocument::GetAssetPathChecked() const
{
	FTopLevelAssetPath Path;
	ensureAlwaysMsgf(Path.TrySetPath(this), TEXT("Failed to set TopLevelAssetPath from transient MetaSound '%s'. MetaSound must be highest level object in package."), *GetPathName());
	ensureAlwaysMsgf(Path.IsValid(), TEXT("Failed to set TopLevelAssetPath from MetaSound '%s'. This may be caused by calling this function when the asset is being destroyed."), *GetPathName());
	return Path;
}

const FMetasoundFrontendDocument& UMetaSoundBuilderDocument::GetConstDocument() const
{
	return Document;
}

#if WITH_EDITOR
TStructView<FMetaSoundFrontendDocumentTemplate> FMetaSoundFrontendDocumentBuilder::GetDocumentTemplate() const
{
	TInstancedStruct<FMetaSoundFrontendDocumentTemplate>& Config = GetDocumentChecked().Template;
	return Config;
}

TStructView<FMetaSoundFrontendDocumentTemplate> FMetaSoundFrontendDocumentBuilder::GetDocumentConfiguration() const
{
	return GetDocumentTemplate();
}
#endif // WITH_EDITOR

TConstStructView<FMetaSoundFrontendDocumentTemplate> FMetaSoundFrontendDocumentBuilder::GetConstDocumentTemplate() const
{
	const TInstancedStruct<FMetaSoundFrontendDocumentTemplate>& Template = GetConstDocumentChecked().Template;
	return Template;
}

EMetasoundFrontendClassAccessFlags UMetaSoundBuilderDocument::GetDefaultAccessFlags() const
{
	return EMetasoundFrontendClassAccessFlags::None;
}

const UClass& UMetaSoundBuilderDocument::GetBaseMetaSoundUClass() const
{
	checkf(MetaSoundUClass, TEXT("BaseMetaSoundUClass must be set upon creation of UMetaSoundBuilderDocument instance"));
	return *MetaSoundUClass;
}

const UClass& UMetaSoundBuilderDocument::GetBuilderUClass() const
{
	checkf(BuilderUClass, TEXT("BuilderUClass must be set upon creation of UMetaSoundBuilderDocument instance"));
	return *BuilderUClass;
}

bool UMetaSoundBuilderDocument::IsActivelyBuilding() const
{
	return true;
}

FMetasoundFrontendDocument& UMetaSoundBuilderDocument::GetDocument()
{
	return Document;
}

void UMetaSoundBuilderDocument::OnBeginActiveBuilder()
{
	// Nothing to do here. UMetaSoundBuilderDocuments are always being used by builders
}

void UMetaSoundBuilderDocument::OnFinishActiveBuilder()
{
	// Nothing to do here. UMetaSoundBuilderDocuments are always being used by builders
}

FMetaSoundFrontendDocumentBuilder::FMetaSoundFrontendDocumentBuilder(TScriptInterface<IMetaSoundDocumentInterface> InDocumentInterface, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> InDocumentDelegates, bool bPrimeCache)
	: DocumentInterface(InDocumentInterface)
	, BuildPageID(Metasound::Frontend::DefaultPageID)
{
	BeginBuilding(InDocumentDelegates, bPrimeCache);
}

FMetaSoundFrontendDocumentBuilder::~FMetaSoundFrontendDocumentBuilder()
{
	FinishBuilding();
}

void FMetaSoundFrontendDocumentBuilder::AddAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags)
{
	GetDocumentChecked().RootGraph.Metadata.AddAccessFlags(AccessFlags);
}

void FMetaSoundFrontendDocumentBuilder::ClearAccessFlags()
{
	GetDocumentChecked().RootGraph.Metadata.ClearAccessFlags();
}

EMetasoundFrontendClassAccessFlags FMetaSoundFrontendDocumentBuilder::GetAccessFlags() const
{
	return GetDocumentChecked().RootGraph.Metadata.GetAccessFlags();
}

void FMetaSoundFrontendDocumentBuilder::RemoveAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags)
{
	GetDocumentChecked().RootGraph.Metadata.RemoveAccessFlags(AccessFlags);
}

void FMetaSoundFrontendDocumentBuilder::SetAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags)
{
	GetDocumentChecked().RootGraph.Metadata.SetAccessFlags(AccessFlags);
}

const FMetasoundFrontendClass* FMetaSoundFrontendDocumentBuilder::ReplaceDependencyInternal(const FMetasoundFrontendClass& OldDependency, FMetasoundFrontendClass NewDependency)
{

	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const FMetasoundFrontendClass* Dependency = nullptr;

	// All 'Graph' dependencies are listed as 'External' from the perspective of the owning document.
	// This makes them implementation agnostic to accommodate nativization of assets.
	if (NewDependency.Metadata.GetType() == EMetasoundFrontendClassType::Graph)
	{
		NewDependency.Metadata.SetType(EMetasoundFrontendClassType::External);
	}
	
	NewDependency.ID = OldDependency.ID;
	const int32* IdxPtr = DocumentCache->FindDependencyIndex(OldDependency.ID);
	checkf(IdxPtr, TEXT("Dependency '%s' to replace not found."), *OldDependency.Metadata.GetClassName().ToString());
	const int32 Idx = *IdxPtr;
	Document.Dependencies.RemoveAt(Idx);
	Dependency = &Document.Dependencies.EmplaceAt_GetRef(Idx, MoveTemp(NewDependency));

	return Dependency;
}

const FMetasoundFrontendClass* FMetaSoundFrontendDocumentBuilder::AddDependency(FMetasoundFrontendClass NewDependency)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const FMetasoundFrontendClass* Dependency = nullptr;

	// All 'Graph' dependencies are listed as 'External' from the perspective of the owning document.
	// This makes them implementation agnostic to accommodate nativization of assets.
	if (NewDependency.Metadata.GetType() == EMetasoundFrontendClassType::Graph)
	{
		NewDependency.Metadata.SetType(EMetasoundFrontendClassType::External);
	}

	NewDependency.ID = FDocumentIDGenerator::Get().CreateClassID(Document);
	Dependency = &Document.Dependencies.Emplace_GetRef(MoveTemp(NewDependency));

	const int32 NewIndex = Document.Dependencies.Num() - 1;
	DocumentDelegates->OnDependencyAdded.Broadcast(NewIndex);

	return Dependency;
}

void FMetaSoundFrontendDocumentBuilder::AddEdge(FMetasoundFrontendEdge&& InNewEdge, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

#if DO_CHECK
	{
		const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
		checkf(!EdgeCache.IsNodeInputConnected(InNewEdge.ToNodeID, InNewEdge.ToVertexID), TEXT("Failed to add edge in MetaSound Builder: Destination input already connected"));

		const EInvalidEdgeReason Reason = IsValidEdge(InNewEdge, &PageID);
		checkf(Reason == Metasound::Frontend::EInvalidEdgeReason::None, TEXT("Attempted call to AddEdge in MetaSound Builder where edge is invalid: %s."), *LexToString(Reason));
	}
#endif // DO_CHECK

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	Graph.Edges.Add(MoveTemp(InNewEdge));
	const int32 NewIndex = Graph.Edges.Num() - 1;
	DocumentDelegates->FindGraphDelegatesChecked(PageID).EdgeDelegates.OnEdgeAdded.Broadcast(NewIndex);
}

bool FMetaSoundFrontendDocumentBuilder::AddNamedEdges(const TSet<Metasound::Frontend::FNamedEdge>& EdgesToMake, TArray<const FMetasoundFrontendEdge*>* OutNewEdges, bool bReplaceExistingConnections, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);

	if (OutNewEdges)
	{
		OutNewEdges->Reset();
	}

	bool bSuccess = true;

	struct FNewEdgeData
	{
		FMetasoundFrontendEdge NewEdge;
		const FMetasoundFrontendVertex* OutputVertex = nullptr;
		const FMetasoundFrontendVertex* InputVertex = nullptr;
	};

	TArray<FNewEdgeData> EdgesToAdd;
	for (const FNamedEdge& Edge : EdgesToMake)
	{
		const FMetasoundFrontendVertex* OutputVertex = NodeCache.FindOutputVertex(Edge.OutputNodeID, Edge.OutputName);
		const FMetasoundFrontendVertex* InputVertex = NodeCache.FindInputVertex(Edge.InputNodeID, Edge.InputName);

		if (OutputVertex && InputVertex)
		{
			FMetasoundFrontendEdge NewEdge = { Edge.OutputNodeID, OutputVertex->VertexID, Edge.InputNodeID, InputVertex->VertexID };
			const EInvalidEdgeReason InvalidEdgeReason = IsValidEdge(NewEdge);
			if (InvalidEdgeReason == EInvalidEdgeReason::None)
			{
				EdgesToAdd.Add(FNewEdgeData { MoveTemp(NewEdge), OutputVertex, InputVertex });
			}
			else
			{
				bSuccess = false;
				UE_LOGF(LogMetaSound, Error, "Failed to add connections between MetaSound output '%ls' and input '%ls': '%ls'.", *Edge.OutputName.ToString(), *Edge.InputName.ToString(), *LexToString(InvalidEdgeReason));
			}
		}
	}

	const TArray<FMetasoundFrontendEdge>& Edges = Graph.Edges;
	const int32 LastIndex = Edges.Num() - 1;
	for (FNewEdgeData& EdgeToAdd : EdgesToAdd)
	{
		if (bReplaceExistingConnections)
		{
#if !NO_LOGGING
			const FMetasoundFrontendNode* OldOutputNode = nullptr;
			const FMetasoundFrontendVertex* OldOutputVertex = FindNodeOutputConnectedToNodeInput(EdgeToAdd.NewEdge.ToNodeID, EdgeToAdd.NewEdge.ToVertexID, &OldOutputNode, &PageID);
#endif // !NO_LOGGING

			const bool bRemovedEdge = RemoveEdgeToNodeInput(EdgeToAdd.NewEdge.ToNodeID, EdgeToAdd.NewEdge.ToVertexID, &PageID);

#if !NO_LOGGING
			if (bRemovedEdge)
			{
				checkf(OldOutputNode, TEXT("MetaSound edge was removed from output but output node not found."));
				checkf(OldOutputVertex, TEXT("MetaSound edge was removed from output but output vertex not found."));

				const FMetasoundFrontendNode* InputNode = FindNode(EdgeToAdd.NewEdge.ToNodeID);
				checkf(InputNode, TEXT("Edge was deemed valid but input parent node is missing"));

				const FMetasoundFrontendNode* OutputNode = FindNode(EdgeToAdd.NewEdge.FromNodeID);
				checkf(OutputNode, TEXT("Edge was deemed valid but output parent node is missing"));

				UE_LOGF(LogMetaSound, Verbose, "Removed connection from node output '%ls:%ls' to node '%ls:%ls' in order to connect to node output '%ls:%ls'",
					*OldOutputNode->Name.ToString(),
					*OldOutputVertex->Name.ToString(),
					*InputNode->Name.ToString(),
					*EdgeToAdd.InputVertex->Name.ToString(),
					*OutputNode->Name.ToString(),
					*EdgeToAdd.OutputVertex->Name.ToString());
			}
#endif // !NO_LOGGING

			AddEdge(MoveTemp(EdgeToAdd.NewEdge), &PageID);
		}
		else if (!IsNodeInputConnected(EdgeToAdd.NewEdge.ToNodeID, EdgeToAdd.NewEdge.ToVertexID, &PageID))
		{
			AddEdge(MoveTemp(EdgeToAdd.NewEdge), &PageID);
		}
		else
		{
			bSuccess = false;

#if !NO_LOGGING
			FMetasoundFrontendEdge EdgeToRemove;
			if (const int32* EdgeIndex = DocumentCache->GetEdgeCache(PageID).FindEdgeIndexToNodeInput(EdgeToAdd.NewEdge.ToNodeID, EdgeToAdd.NewEdge.ToVertexID))
			{
				EdgeToRemove = Graph.Edges[*EdgeIndex];
			}

			const FMetasoundFrontendVertex* Input = FindNodeInput(EdgeToAdd.NewEdge.ToNodeID, EdgeToAdd.NewEdge.ToVertexID, &PageID);
			checkf(Input, TEXT("Prior loop to check edge validity should protect against missing input vertex"));

			const FMetasoundFrontendVertex* Output = FindNodeOutput(EdgeToAdd.NewEdge.FromNodeID, EdgeToAdd.NewEdge.FromVertexID, &PageID);
			checkf(Output, TEXT("Prior loop to check edge validity should protect against missing output vertex"));

			UE_LOGF(LogMetaSound, Warning, "Connection between MetaSound output '%ls' and input '%ls' not added: Input already connected to '%ls'.", *Output->Name.ToString(), *Input->Name.ToString(), *Output->Name.ToString());
#endif // !NO_LOGGING
		}
	}

	if (OutNewEdges)
	{
		for (int32 Index = LastIndex + 1; Index < Edges.Num(); ++Index)
		{
			OutNewEdges->Add(&Edges[Index]);
		}
	}

	return bSuccess;
}

bool FMetaSoundFrontendDocumentBuilder::AddEdgesByNodeClassInterfaceBindings(const FGuid& InFromNodeID, const FGuid& InToNodeID, bool bReplaceExistingConnections, const FGuid* InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	TSet<FMetasoundFrontendVersion> FromInterfaceVersions;
	TSet<FMetasoundFrontendVersion> ToInterfaceVersions;
	if (FindNodeClassInterfaces(InFromNodeID, FromInterfaceVersions, PageID) && FindNodeClassInterfaces(InToNodeID, ToInterfaceVersions, PageID))
	{
		TSet<FNamedEdge> NamedEdges;
		if (DocumentBuilderPrivate::TryGetInterfaceBoundEdges(InFromNodeID, FromInterfaceVersions, InToNodeID, ToInterfaceVersions, NamedEdges))
		{
			return AddNamedEdges(NamedEdges, nullptr, bReplaceExistingConnections, &PageID);
		}
	}

	return false;

}

bool FMetaSoundFrontendDocumentBuilder::AddEdgesFromMatchingInterfaceNodeOutputsToGraphOutputs(const FGuid& InNodeID, TArray<const FMetasoundFrontendEdge*>& OutEdgesCreated, bool bReplaceExistingConnections, const FGuid* InPageID)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::AddEdgesFromMatchingInterfaceNodeOutputsToGraphOutputs);

	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	OutEdgesCreated.Reset();

	TSet<FMetasoundFrontendVersion> NodeInterfaces;
	if (!FindNodeClassInterfaces(InNodeID, NodeInterfaces, PageID))
	{
		// Did not find any node interfaces
		return false;
	}

	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();
	const TSet<FMetasoundFrontendVersion> CommonInterfaces = NodeInterfaces.Intersect(GetDocumentChecked().Interfaces);

	TSet<FNamedEdge> EdgesToMake;
	for (const FMetasoundFrontendVersion& Version : CommonInterfaces)
	{
		FMetasoundFrontendInterface Interface;
		const bool bFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
		if (bFound)
		{
			Algo::Transform(Interface.Outputs, EdgesToMake, [this, &NodeCache, &InterfaceCache, &PageID, InNodeID](const FMetasoundFrontendClassOutput& Output)
			{
				const FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindConstGraphChecked(PageID);
				const FMetasoundFrontendVertex* NodeVertex = NodeCache.FindOutputVertex(InNodeID, Output.Name);
				check(NodeVertex);
				const FMetasoundFrontendClassOutput* OutputClass = InterfaceCache.FindOutput(Output.Name);
				check(OutputClass);
				const FMetasoundFrontendNode* OutputNode = NodeCache.FindNode(OutputClass->NodeID);
				check(OutputNode);
				const TArray<FMetasoundFrontendVertex>& Inputs = OutputNode->Interface.Inputs;
				check(!Inputs.IsEmpty());
				return FNamedEdge { InNodeID, NodeVertex->Name, OutputNode->GetID(), Inputs.Last().Name };
			});
		}
	}

	return AddNamedEdges(EdgesToMake, &OutEdgesCreated, bReplaceExistingConnections, &PageID);
}

bool FMetaSoundFrontendDocumentBuilder::AddEdgesFromMatchingInterfaceNodeInputsToGraphInputs(const FGuid& InNodeID, TArray<const FMetasoundFrontendEdge*>& OutEdgesCreated, bool bReplaceExistingConnections, const FGuid* InPageID)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::AddEdgesFromMatchingInterfaceNodeInputsToGraphInputs);

	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	OutEdgesCreated.Reset();

	TSet<FMetasoundFrontendVersion> NodeInterfaces;
	if (!FindNodeClassInterfaces(InNodeID, NodeInterfaces, PageID))
	{
		// Did not find any node interfaces
		return false;
	}

	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();
	const TSet<FMetasoundFrontendVersion> CommonInterfaces = NodeInterfaces.Intersect(GetDocumentChecked().Interfaces);

	TSet<FNamedEdge> EdgesToMake;
	const FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindConstGraphChecked(PageID);
	for (const FMetasoundFrontendVersion& Version : CommonInterfaces)
	{
		FMetasoundFrontendInterface Interface;
		const bool bFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
		if (bFound)
		{
			Algo::Transform(Interface.Inputs, EdgesToMake, [this, &Graph, &NodeCache, &InterfaceCache, InNodeID](const FMetasoundFrontendClassInput& Input)
			{
				const FMetasoundFrontendVertex* NodeVertex = NodeCache.FindInputVertex(InNodeID, Input.Name);
				check(NodeVertex);
				const FMetasoundFrontendClassInput* InputClass = InterfaceCache.FindInput(Input.Name);
				check(InputClass);
				const FMetasoundFrontendNode* InputNode = NodeCache.FindNode(InputClass->NodeID);
				check(InputNode);
				const TArray<FMetasoundFrontendVertex>& Outputs = InputNode->Interface.Outputs;
				check(!Outputs.IsEmpty());
				return FNamedEdge { InputNode->GetID(), Outputs.Last().Name, InNodeID, NodeVertex->Name };
			});
		}
	}

	return AddNamedEdges(EdgesToMake, &OutEdgesCreated, bReplaceExistingConnections, &PageID);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphInput(FMetasoundFrontendClassInput ClassInput, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	checkf(ClassInput.NodeID.IsValid(), TEXT("Unassigned NodeID when adding graph input"));
	checkf(ClassInput.VertexID.IsValid(), TEXT("Unassigned VertexID when adding graph input"));

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (ClassInput.TypeName.IsNone())
	{
		UE_LOGF(LogMetaSound, Error, "TypeName unset when attempting to add class input '%ls'", *ClassInput.Name.ToString());
		return nullptr;
	}
	else if (const FMetasoundFrontendClassInput* Input = DocumentCache->GetInterfaceCache().FindInput(ClassInput.Name))
	{
		UE_LOGF(LogMetaSound, Error, "Attempting to add MetaSound graph input '%ls' when input with name already exists", *ClassInput.Name.ToString());
		const FMetasoundFrontendNode* OutputNode = DocumentCache->GetNodeCache(PageID).FindNode(Input->NodeID);
		check(OutputNode);
		return OutputNode;
	}
	else if (!IDataTypeRegistry::Get().IsRegistered(ClassInput.TypeName))
	{
		UE_LOGF(LogMetaSound, Error, "Cannot add MetaSound graph input '%ls' with unregistered TypeName '%ls'", *ClassInput.Name.ToString(), *ClassInput.TypeName.ToString());
		return nullptr;
	}

	FNodeRegistryKey ClassKey;
	{
		FMetasoundFrontendClass Class;
		if (!DocumentBuilderPrivate::FindInputRegistryClass(ClassInput.TypeName, ClassInput.AccessType, Class))
		{
			return nullptr;
		}

		ClassKey = FNodeRegistryKey(Class.Metadata);
		if (!FindDependency(Class.Metadata))
		{
			AddDependency(MoveTemp(Class));
		}
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;

	const int32 NewIndex = RootGraph.GetDefaultInterface().Inputs.Num();
	FMetasoundFrontendClassInput& NewInput = RootGraph.GetDefaultInterface().Inputs.Add_GetRef(MoveTemp(ClassInput));

	auto FinalizeNode = [this, &NewInput, &PageID](FMetasoundFrontendNode& InOutNode, const Metasound::Frontend::FNodeRegistryKey&)
	{
		// Sets the name of the node an vertices on the node to match the class vertex name
		DocumentBuilderPrivate::SetNodeAndVertexNames(InOutNode, NewInput);

		// Set the default literal on the nodes inputs so that it gets passed to the instantiated TInputNode on a live
		// auditioned MetaSound.
		DocumentBuilderPrivate::SetDefaultLiteralOnInputNode(InOutNode, NewInput, PageID);
	};

#if WITH_EDITORONLY_DATA
	bool bIsRequired = false;
	FMetasoundFrontendInterface Interface;
	if (DocumentBuilderPrivate::IsInterfaceInput(NewInput.Name, NewInput.TypeName, &Interface))
	{
		if (Document.Interfaces.Contains(Interface.Metadata.Version))
		{
			FText RequiredText;
			bIsRequired = Interface.IsMemberInputRequired(NewInput.Name, RequiredText);
		}
	}
#endif // WITH_EDITORONLY_DATA


	// Must add input node to all paged graphs to maintain API parity for all page implementations
	FMetasoundFrontendNode* NewNode = nullptr;
	RootGraph.IterateGraphPages([&](const FMetasoundFrontendGraph& Graph)
	{
		constexpr int32* NewNodeIndex = nullptr;
		FMetasoundFrontendNode* NewPageNode = AddNodeInternal(ClassKey, FinalizeNode, Graph.PageID, ClassInput.NodeID, NewNodeIndex);
		if (Graph.PageID == PageID)
		{
			NewNode = NewPageNode;
		}

#if WITH_EDITORONLY_DATA
		if (bIsRequired)
		{
			// LocationGuid corresponds with the assigned editor graph node guid when dynamically created.
			// This is added if this is an interface member that is required to force page to create visual
			// representation that can inform the user of its required state.
			FGuid LocationGuid = FDocumentIDGenerator::Get().CreateVertexID(Document);
			SetNodeLocation(NewInput.NodeID, FVector2D::ZeroVector, &LocationGuid, &Graph.PageID);
		}
#endif // WITH_EDITORONLY_DATA

		// Remove the default literal on the node added during the "FinalizeNode" call. This matches how 
		// nodes are serialized in editor. The default literals are only stored on the FMetasoundFrontendClassInputs.
		NewPageNode->InputLiterals.Reset();
	});

	if (NewNode)
	{
		if (!NewInput.VertexID.IsValid())
		{
			NewInput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(Document);
		}

		DocumentDelegates->InterfaceDelegates.OnInputAdded.Broadcast(NewIndex);
#if WITH_EDITORONLY_DATA
		if (FDocumentVertexTemplate* DocTemplate = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
		{
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			DocTemplate->AddVertexMetadata(*this, NewInput);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}

		Document.Metadata.ModifyContext.AddMemberIDModified(NewInput.NodeID);
#endif // WITH_EDITORONLY_DATA

		return NewNode;
	}
	else
	{
		// Undo addition of graph input on failure.
		RootGraph.GetDefaultInterface().Inputs.RemoveAt(NewIndex);
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphOutput(FMetasoundFrontendClassOutput ClassOutput, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	checkf(ClassOutput.NodeID.IsValid(), TEXT("Unassigned NodeID when adding graph output"));
	checkf(ClassOutput.VertexID.IsValid(), TEXT("Unassigned VertexID when adding graph output"));

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (ClassOutput.TypeName.IsNone())
	{
		UE_LOGF(LogMetaSound, Error, "TypeName unset when attempting to add class output '%ls'", *ClassOutput.Name.ToString());
		return nullptr;
	}
	else if (const FMetasoundFrontendClassOutput* Output = DocumentCache->GetInterfaceCache().FindOutput(ClassOutput.Name))
	{
		UE_LOGF(LogMetaSound, Error, "Attempting to add MetaSound graph output '%ls' when output with name already exists", *ClassOutput.Name.ToString());
		return DocumentCache->GetNodeCache(PageID).FindNode(Output->NodeID);
	}
	else if (!IDataTypeRegistry::Get().IsRegistered(ClassOutput.TypeName))
	{
		UE_LOGF(LogMetaSound, Error, "Cannot add MetaSound graph output '%ls' with unregistered TypeName '%ls'", *ClassOutput.Name.ToString(), *ClassOutput.TypeName.ToString());
		return nullptr;
	}

	FNodeRegistryKey ClassKey;
	{
		FMetasoundFrontendClass Class;
		if (!DocumentBuilderPrivate::FindOutputRegistryClass(ClassOutput.TypeName, ClassOutput.AccessType, Class))
		{
			return nullptr;
		}

		ClassKey = FNodeRegistryKey(Class.Metadata);
		if (!FindDependency(Class.Metadata))
		{
			AddDependency(MoveTemp(Class));
		}
	}

	// Add graph output
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	const int32 NewIndex = RootGraph.GetDefaultInterface().Outputs.Num();
	FMetasoundFrontendClassOutput& NewOutput = RootGraph.GetDefaultInterface().Outputs.Add_GetRef(MoveTemp(ClassOutput));

	auto FinalizeNode = [&NewOutput](FMetasoundFrontendNode& InOutNode, const Metasound::Frontend::FNodeRegistryKey&)
	{
		DocumentBuilderPrivate::SetNodeAndVertexNames(InOutNode, NewOutput);
	};

#if WITH_EDITORONLY_DATA
	bool bIsRequired = false;
	FMetasoundFrontendInterface Interface;
	if (DocumentBuilderPrivate::IsInterfaceOutput(NewOutput.Name, NewOutput.TypeName, &Interface))
	{
		FText RequiredText;
		bIsRequired = Interface.IsMemberOutputRequired(NewOutput.Name, RequiredText);
	}
#endif // WITH_EDITORONLY_DATA

	// Add output nodes
	bool bAddedNodes = true;
	FMetasoundFrontendNode* NewNodeToReturn = nullptr;
	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
		FMetasoundFrontendNode* NewNode = AddNodeInternal(ClassKey, FinalizeNode, Graph.PageID, NewOutput.NodeID);
		if (Graph.PageID == PageID)
		{
			NewNodeToReturn = NewNode;
		}

#if WITH_EDITORONLY_DATA
		if (bIsRequired)
		{
			// LocationGuid corresponds with the assigned editor graph node guid when dynamically created.
			// This is added if this is an interface member that is required to force page to create visual
			// representation that can inform the user of its required state.
			FGuid LocationGuid = FDocumentIDGenerator::Get().CreateVertexID(Document);
			SetNodeLocation(NewOutput.NodeID, FVector2D::ZeroVector, &LocationGuid, &Graph.PageID);
		}
#endif // WITH_EDITORONLY_DATA

		if (!NewNode)
		{
			UE_LOGF(LogMetaSound, Error, "Failed to add MetaSound graph output '%ls' node to page ID '%ls'", *ClassOutput.Name.ToString(), *PageID.ToString());
			bAddedNodes = false;
		}
	});

	if (bAddedNodes)
	{
		if (!NewOutput.VertexID.IsValid())
		{
			NewOutput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(Document);
		}

		DocumentDelegates->InterfaceDelegates.OnOutputAdded.Broadcast(NewIndex);

#if WITH_EDITORONLY_DATA
		if (FDocumentVertexTemplate* DocTemplate = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
		{
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			DocTemplate->AddVertexMetadata(*this, NewOutput);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}

		Document.Metadata.ModifyContext.AddMemberIDModified(NewOutput.NodeID);
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		// Remove added output
		RootGraph.GetDefaultInterface().Outputs.RemoveAt(NewIndex);
	}

	return NewNodeToReturn;
}

const FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::AddGraphVariable(FName VariableName, FName DataType, const FMetasoundFrontendLiteral* Literal, const FText* DisplayName, const FText* Description, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (const FMetasoundFrontendVariable* ExistingVariable = FindGraphVariable(VariableName, &PageID))
	{
		UE_LOGF(LogMetaSound, Warning, "AddGraphVariable Failed: Variable already exists with name '%ls' (existing DataType '%ls', requested DataType '%ls')",
			*VariableName.ToString(), *ExistingVariable->TypeName.ToString(), *DataType.ToString());
		return nullptr;
	}

	const IDataTypeRegistry& Registry = IDataTypeRegistry::Get();
	FDataTypeRegistryInfo Info;
	if (!Registry.GetDataTypeInfo(DataType, Info))
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphVariable Failed: Attempted creation of variable '%ls' with unregistered DataType '%ls'", *VariableName.ToString(), *DataType.ToString());
		return nullptr;
	}

	FMetasoundFrontendVariable Variable
	{
		.Name = VariableName,
		.ID = FGuid::NewGuid()
	};

	Variable.TypeName = Info.DataTypeName;
	if (Literal)
	{
		Variable.Literal = *Literal;
	}
	else
	{
		Variable.Literal.SetFromLiteral(Registry.CreateDefaultLiteral(DataType));
	}

#if WITH_EDITORONLY_DATA
	if (DisplayName)
	{
		Variable.DisplayName = *DisplayName;
	}

	if (Description)
	{
		Variable.Description = *Description;
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	GetDocumentChecked().Metadata.ModifyContext.AddMemberIDModified(Variable.ID);
#endif // WITH_EDITOR

	FNodeRegistryKey VariableNodeClassKey;
	{
		FMetasoundFrontendClass VariableNodeClass;
		if (!IDataTypeRegistry::Get().GetFrontendVariableClass(Variable.TypeName, VariableNodeClass))
		{
			return nullptr;
		}

		VariableNodeClassKey = FNodeRegistryKey(VariableNodeClass.Metadata);
		const FMetasoundFrontendClass* Dependency = FindDependency(VariableNodeClass.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(VariableNodeClass));
		}
		check(Dependency);
	}

	auto FinalizeNode = [](FMetasoundFrontendNode& InOutNode, const Metasound::Frontend::FNodeRegistryKey& ClassKey)
	{
#if WITH_EDITOR
		using namespace Metasound::Frontend;

		// Cache the asset name on the node if it node is reference to asset-defined graph.
		const FTopLevelAssetPath Path = IMetaSoundAssetManager::GetChecked().FindAssetPath(FMetaSoundAssetKey(ClassKey.ClassName, ClassKey.Version));
		if (Path.IsValid())
		{
			InOutNode.Name = Path.GetAssetName();
			return;
		}

		InOutNode.Name = ClassKey.ClassName.GetFullName();
#endif // WITH_EDITOR
	};

	if (FMetasoundFrontendNode* VariableNode = AddNodeInternal(VariableNodeClassKey, FinalizeNode, PageID))
	{
		Variable.VariableNodeID = VariableNode->GetID();
		FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
		return &Graph.Variables.Add_GetRef(MoveTemp(Variable));
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphVariableNode(FName VariableName, EMetasoundFrontendClassType ClassType, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	switch (ClassType)
	{
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
			return AddGraphVariableDeferredAccessorNode(VariableName, InNodeID, InPageID);

		case EMetasoundFrontendClassType::VariableAccessor:
			return AddGraphVariableAccessorNode(VariableName, InNodeID, InPageID);

		case EMetasoundFrontendClassType::VariableMutator:
			return AddGraphVariableMutatorNode(VariableName, InNodeID, InPageID);

		default:
		{
			checkNoEntry();
		}
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphVariableAccessorNode(FName VariableName, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID);
	if (!Variable)
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphVariableAccessorNode Failed: Variable does not exists with name '%ls'", *VariableName.ToString());
		return nullptr;
	} 
	
	FNodeRegistryKey VariableNodeClassKey;
	{
		FMetasoundFrontendClass NodeClass;
		if (!IDataTypeRegistry::Get().GetFrontendVariableAccessorClass(Variable->TypeName, NodeClass))
		{
			UE_LOGF(LogMetaSound, Error, "Could not find registered \"get variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
			return nullptr;
		}

		VariableNodeClassKey = FNodeRegistryKey(NodeClass.Metadata);
		const FMetasoundFrontendClass* Dependency = FindDependency(NodeClass.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(NodeClass));
		}
		check(Dependency);
	}

	auto FinalizeNodeFunction = [](const FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&) { };
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (const FMetasoundFrontendNode* NewNode = AddNodeInternal(VariableNodeClassKey, FinalizeNodeFunction, PageID, InNodeID))
	{
		// Connect new node.
		const FMetasoundFrontendVertex* NewInput = FindNodeInput(NewNode->GetID(), METASOUND_GET_PARAM_NAME(InputVariable), InPageID);
		check(NewInput);

		const FMetasoundFrontendNode* TailNode = FindTailNodeInVariableStack(VariableName, InPageID);
		if (!TailNode)
		{
			// variable stack is empty. Connect to init variable node.
			TailNode = FindNode(Variable->VariableNodeID, InPageID);
		}

		if (ensure(TailNode))
		{
			// connect new node to the last "get" node.
			const FMetasoundFrontendVertex* TailNodeOutput = FindNodeOutput(TailNode->GetID(), METASOUND_GET_PARAM_NAME(OutputVariable), InPageID);
			check(TailNodeOutput);

			FMetasoundFrontendEdge NewEdge;
			NewEdge.FromNodeID = TailNode->GetID();
			NewEdge.FromVertexID = TailNodeOutput->VertexID;
			NewEdge.ToNodeID = NewNode->GetID();
			NewEdge.ToVertexID = NewInput->VertexID;
			AddEdge(MoveTemp(NewEdge), InPageID);
		}

		// Add node ID to variable after connecting since the array
		// order of node ids is used to determine whether a node 
		// is the tail node.
		Variable->AccessorNodeIDs.Add(NewNode->GetID());
		return NewNode;
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphVariableDeferredAccessorNode(FName VariableName, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID);
	if (!Variable)
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphVariableGetDelayedNode Failed: Variable does not exists with name '%ls'", *VariableName.ToString());
		return nullptr;
	}

	FNodeRegistryKey ClassKey;
	{
		FMetasoundFrontendClass NodeClass;
		if (!IDataTypeRegistry::Get().GetFrontendVariableDeferredAccessorClass(Variable->TypeName, NodeClass))
		{
			UE_LOGF(LogMetaSound, Error, "AddGraphVariableGetDelayedNode Failed: Could not find registered \"get variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
			return nullptr;
		}

		ClassKey = FNodeRegistryKey(NodeClass.Metadata);
		const FMetasoundFrontendClass* Dependency = FindDependency(NodeClass.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(NodeClass));
		}
		check(Dependency);
	}

	auto FinalizeNodeFunction = [](const FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&) { };
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (const FMetasoundFrontendNode* NewNode = AddNodeInternal(ClassKey, FinalizeNodeFunction, PageID, InNodeID))
	{
		// Connect new node.
		const FMetasoundFrontendVertex* NewNodeOutput = FindNodeOutput(NewNode->GetID(), METASOUND_GET_PARAM_NAME(OutputVariable), InPageID);
		const FMetasoundFrontendNode* HeadNode = FindHeadNodeInVariableStack(VariableName, InPageID);
		if (HeadNode)
		{
			const FMetasoundFrontendVertex* HeadNodeInput = FindNodeInput(HeadNode->GetID(), METASOUND_GET_PARAM_NAME(InputVariable), InPageID);
			check(HeadNodeInput);

			RemoveEdgeToNodeInput(HeadNode->GetID(), HeadNodeInput->VertexID, InPageID);

			FMetasoundFrontendEdge NewEdge;
			NewEdge.FromNodeID = NewNode->GetID();
			NewEdge.FromVertexID = NewNodeOutput->VertexID;
			NewEdge.ToNodeID = HeadNode->GetID();
			NewEdge.ToVertexID = HeadNodeInput->VertexID;
			AddEdge(MoveTemp(NewEdge), InPageID);
		}

		const FMetasoundFrontendVertex* NewNodeInput = FindNodeInput(NewNode->GetID(), METASOUND_GET_PARAM_NAME(InputVariable), InPageID);
		check(NewNodeInput);

		const FMetasoundFrontendNode* VariableNode = FindNode(Variable->VariableNodeID, InPageID);
		check(VariableNode);

		const FMetasoundFrontendVertex* VariableNodeOutput = FindNodeOutput(VariableNode->GetID(), METASOUND_GET_PARAM_NAME(OutputVariable), InPageID);
		check(VariableNodeOutput);

		FMetasoundFrontendEdge NewEdge;
		NewEdge.FromNodeID = VariableNode->GetID();
		NewEdge.FromVertexID = VariableNodeOutput->VertexID;
		NewEdge.ToNodeID = NewNode->GetID();
		NewEdge.ToVertexID = NewNodeInput->VertexID;
		AddEdge(MoveTemp(NewEdge), InPageID);

		// Add node ID to variable after connecting since the array
		// order of node ids is used to determine whether a node 
		// is the tail node.
		Variable->DeferredAccessorNodeIDs.Add(NewNode->GetID());
		return NewNode;
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphVariableMutatorNode(FName VariableName, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID);
	if (!Variable)
	{
		UE_LOGF(LogMetaSound, Error, "AddGraphVariableMutatorNode Failed: Variable does not exists with name '%ls'", *VariableName.ToString());
		return nullptr;
	}

	if (const FMetasoundFrontendNode* ExistingMutatorNode = FindNode(Variable->MutatorNodeID, InPageID))
	{
		UE_LOGF(LogMetaSound, Error, "Cannot add mutator node as one already exists for variable '%ls'.", *VariableName.ToString());
		return nullptr;
	}

	FNodeRegistryKey ClassKey;
	{
		FMetasoundFrontendClass MutatorNodeClass;
		if (!IDataTypeRegistry::Get().GetFrontendVariableMutatorClass(Variable->TypeName, MutatorNodeClass))
		{
			UE_LOGF(LogMetaSound, Error, "Could not find registered \"set variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
			return nullptr;
		}

		ClassKey = FNodeRegistryKey(MutatorNodeClass.Metadata);
		const FMetasoundFrontendClass* Dependency = FindDependency(MutatorNodeClass.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(MutatorNodeClass));
		}
		check(Dependency);
	}

	auto FinalizeNodeFunction = [](const FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&) { };
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendNode* MutatorNode = AddNodeInternal(ClassKey, FinalizeNodeFunction, PageID, InNodeID);
	if (MutatorNode)
	{
		// Initialize mutator default literal value to that of the variable
		const FMetasoundFrontendVertex* MutatorDataInput = FindNodeInput(MutatorNode->GetID(), METASOUND_GET_PARAM_NAME(InputData), InPageID);
		check(MutatorDataInput);
		SetNodeInputDefault(MutatorNode->GetID(), MutatorDataInput->VertexID, Variable->Literal, InPageID);

		Variable->MutatorNodeID = MutatorNode->GetID();
		FGuid SourceVariableNodeID = Variable->VariableNodeID;

		// Connect last delayed getter in variable stack.
		if (!Variable->DeferredAccessorNodeIDs.IsEmpty())
		{
			SourceVariableNodeID = Variable->DeferredAccessorNodeIDs.Last();
		}

		const FMetasoundFrontendNode* SourceVariableNode = FindNode(SourceVariableNodeID, InPageID);
		if (ensure(SourceVariableNode))
		{
			const FMetasoundFrontendVertex* MutatorNodeInput = FindNodeInput(MutatorNode->GetID(), METASOUND_GET_PARAM_NAME(InputVariable), InPageID);
			check(MutatorNodeInput);

			const FMetasoundFrontendNode* VariableSourceNode = FindNode(SourceVariableNodeID, InPageID);
			check(VariableSourceNode);

			const FMetasoundFrontendVertex* SourceVariableNodeOutput = FindNodeOutput(VariableSourceNode->GetID(), METASOUND_GET_PARAM_NAME(OutputVariable), InPageID);
			check(SourceVariableNodeOutput);

			FMetasoundFrontendEdge NewEdge;
			NewEdge.FromNodeID = SourceVariableNodeID;
			NewEdge.FromVertexID = SourceVariableNodeOutput->VertexID;
			NewEdge.ToNodeID = MutatorNode->GetID();
			NewEdge.ToVertexID = MutatorNodeInput->VertexID;
			AddEdge(MoveTemp(NewEdge), InPageID);
		}

		// Connect to first inline getter in variable stack
		if (!Variable->AccessorNodeIDs.IsEmpty())
		{
			const FGuid& HeadAccessorNodeID = Variable->AccessorNodeIDs[0];
			const FMetasoundFrontendVertex* MutatorNodeOutput = FindNodeOutput(MutatorNode->GetID(), METASOUND_GET_PARAM_NAME(OutputVariable), InPageID);
			check(MutatorNodeOutput);

			const FMetasoundFrontendVertex* AccessorNodeInput = FindNodeInput(HeadAccessorNodeID, METASOUND_GET_PARAM_NAME(InputVariable), InPageID);
			check(AccessorNodeInput);

			RemoveEdgeToNodeInput(HeadAccessorNodeID, AccessorNodeInput->VertexID, InPageID);

			FMetasoundFrontendEdge NewEdge;
			NewEdge.FromNodeID = MutatorNode->GetID();
			NewEdge.FromVertexID = MutatorNodeOutput->VertexID;
			NewEdge.ToNodeID = HeadAccessorNodeID;
			NewEdge.ToVertexID = AccessorNodeInput->VertexID;
			AddEdge(MoveTemp(NewEdge), InPageID);
		}

		return MutatorNode;
	}

	return nullptr;
}

bool FMetaSoundFrontendDocumentBuilder::AddInterfaceInternal(FName InterfaceName, bool bAddUserModifiableInterfaceOnly, const FMetaSoundFrontendDocumentBuilder* ReferenceBuilder)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendVersion InterfaceVersion;
	if (ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceName, InterfaceVersion))
	{
		if (GetDocumentChecked().Interfaces.Contains(InterfaceVersion))
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "MetaSound interface '%ls' already found on document. MetaSoundBuilder skipping add request.", *InterfaceName.ToString());
			return true;
		}

		FMetasoundFrontendInterface Interface;
		const bool bFoundInterface = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, Interface);
		check(bFoundInterface);

		// Get all versions of the interface, excluding the latest
		TArray<FMetasoundFrontendVersion> PreviousVersions = ISearchEngine::Get().FindAllRegisteredInterfacesWithName(Interface.Metadata.Version.Name);
		PreviousVersions.Remove(Interface.Metadata.Version);

		// Collect the metasound frontend interface definitions registered to the previous versions
		TArray<FMetasoundFrontendInterface> PreviousVersionInterfaces;

		for (const FMetasoundFrontendVersion& PreviousVersion : PreviousVersions)
		{
			FMetasoundFrontendInterface PreviousVersionInterface;
			if (IInterfaceRegistry::Get().FindInterface(PreviousVersion, PreviousVersionInterface))
			{
				PreviousVersionInterfaces.Add(MoveTemp(PreviousVersionInterface));
			}
		}

		const FTopLevelAssetPath BuilderClassPath = GetBuilderClassPath();
		auto FindClassOptionsPredicate = [&BuilderClassPath](const FMetasoundFrontendInterfaceUClassOptions& Options) { return Options.ClassPath == BuilderClassPath; };
		const FMetasoundFrontendInterfaceUClassOptions* ClassOptions = Interface.Metadata.UClassOptions.FindByPredicate(FindClassOptionsPredicate);
		if (ClassOptions && bAddUserModifiableInterfaceOnly && !ClassOptions->bIsModifiable)
		{
			UE_LOGF(LogMetaSound, Error, "DocumentBuilder failed to add MetaSound Interface '%ls' to document: is not set to be user modifiable for given UClass '%ls'", *InterfaceName.ToString(), *BuilderClassPath.ToString());
			return false;
		}

		// Remove old versions and add new interfaces simultaneously to support node reconnection
		FModifyInterfaceOptions Options(MoveTemp(PreviousVersionInterfaces), { MoveTemp(Interface) }, ReferenceBuilder);
		return ModifyInterfaces(MoveTemp(Options));
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::AddInterface(FName InterfaceName)
{
	return AddInterfaceInternal(InterfaceName);
}

bool FMetaSoundFrontendDocumentBuilder::AddInterface(FName InterfaceName, bool bAddUserModifiableInterfaceOnly, const FMetaSoundFrontendDocumentBuilder* ReferenceBuilder)
{
	return AddInterfaceInternal(InterfaceName, bAddUserModifiableInterfaceOnly, ReferenceBuilder);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddGraphNode(const FMetasoundFrontendGraphClass& InGraphClass, FGuid InNodeID, const FGuid* InPageID)
{
	using FNodeRegistryKey = Metasound::Frontend::FNodeRegistryKey;

	auto FinalizeNode = [](FMetasoundFrontendNode& InOutNode, const Metasound::Frontend::FNodeRegistryKey& ClassKey)
	{
#if WITH_EDITOR
		using namespace Metasound::Frontend;

		// Cache the asset name on the node if it node is reference to asset-defined graph.
		const FTopLevelAssetPath Path = IMetaSoundAssetManager::GetChecked().FindAssetPath(FMetaSoundAssetKey(ClassKey.ClassName, ClassKey.Version));
		if (Path.IsValid())
		{
			InOutNode.Name = Path.GetAssetName();
			return;
		}

		InOutNode.Name = ClassKey.ClassName.GetFullName();
#endif // WITH_EDITOR
	};

	FNodeRegistryKey ClassKey;
	{
		// Dependency is considered "External" when looked up or added
		// on another graph Cast strips GraphClass-specific data as well
		FMetasoundFrontendClass NewClass = InGraphClass;
		NewClass.Metadata.SetType(EMetasoundFrontendClassType::External);

		ClassKey = FNodeRegistryKey(NewClass.Metadata);
		if (!FindDependency(NewClass.Metadata))
		{
			AddDependency(MoveTemp(NewClass));
		}
	}

	constexpr int32* NewNodeIndex = nullptr;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return AddNodeInternal(ClassKey, FinalizeNode, PageID, InNodeID, NewNodeIndex);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeByClassInternal(FMetasoundFrontendClass&& InClass, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClass* Dependency = nullptr;
	const EMetasoundFrontendClassType ClassType = InClass.Metadata.GetType();
	if (ClassType != EMetasoundFrontendClassType::External && ClassType != EMetasoundFrontendClassType::Graph)
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to add new node by class name '%ls': Class is restricted type '%ls' that cannot be added via this function.",
			*InClass.Metadata.GetClassName().ToString(),
			LexToString(ClassType));
		return nullptr;
	}
	
	// Dependency is considered "External" when looked up or added as a dependency to a graph
	InClass.Metadata.SetType(EMetasoundFrontendClassType::External);
	Dependency = FindDependency(InClass.Metadata);
	if (!Dependency)
	{
		Dependency = AddDependency(MoveTemp(InClass));
	}
	else
	{
		Dependency = ReplaceDependencyInternal(*Dependency, MoveTemp(InClass));
	}

	if (Dependency)
	{
		auto FinalizeNode = [](const FMetasoundFrontendNode& Node, const Metasound::Frontend::FNodeRegistryKey& ClassKey) { return Node.Name; };
		constexpr int32* NewNodeIndex = nullptr;
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		return AddNodeInternal(Dependency->Metadata, FinalizeNode, PageID, InNodeID, NewNodeIndex);
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;
	FMetaSoundClassInfo RegistryClassInfo;
	if (!ISearchEngine::Get().FindRegisteredClass(InClassName, InMajorVersion, RegistryClassInfo))
	{
		UE_LOGF(LogMetaSound, Error, "Failed to add new node by class name '%ls' and major version '%d': Class info not registered", *InClassName.ToString(), InMajorVersion);
		return nullptr;
	}

	FMetasoundFrontendClass RegisteredClass;
	if (!INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(RegistryClassInfo.ToRegistryKey(), RegisteredClass))
	{
		UE_LOGF(LogMetaSound, Error, "Failed to add new node by class name '%ls' and major version '%d': Class not found", *InClassName.ToString(), InMajorVersion);
		return nullptr;
	}
	
	return AddNodeByClassInternal(MoveTemp(RegisteredClass), InNodeID, InPageID);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, int32 InMinorVersion, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	FNodeRegistryKey LookupKey(EMetasoundFrontendClassType::External, InClassName, InMajorVersion, InMinorVersion);
	FMetasoundFrontendClass RegisteredClass;
	if (!INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(LookupKey, RegisteredClass))
	{
		UE_LOGF(LogMetaSound, Error, "Failed to add new node by class name '%ls' and major version '%d' and minor version '%d': Class not found", *InClassName.ToString(), InMajorVersion, InMinorVersion);
		return nullptr;
	}
	return AddNodeByClassInternal(MoveTemp(RegisteredClass), InNodeID, InPageID);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeByTemplate(const Metasound::Frontend::INodeTemplate& InTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeID, const FGuid* InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClass& TemplateClass = InTemplate.GetFrontendClass();
	checkf(TemplateClass.Metadata.GetType() == EMetasoundFrontendClassType::Template, TEXT("INodeTemplate ClassType must always be 'Template'"));

	const FMetasoundFrontendClass* Dependency = FindDependency(TemplateClass.Metadata);
	if (!Dependency)
	{
		Dependency = AddDependency(TemplateClass);
	}
	check(Dependency);

	auto FinalizeNodeFunction = [](const FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&) { };
	constexpr int32* NewNodeIndex = nullptr;
	const FGuid & PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendNode* NewNode = AddNodeInternal(Dependency->Metadata, FinalizeNodeFunction, PageID, InNodeID, NewNodeIndex);
	check(NewNode);
	NewNode->Interface = InTemplate.GenerateNodeInterface(MoveTemp(Params));

	return NewNode;
}

TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FMetaSoundFrontendDocumentBuilder::CreateFrontendNodeConfiguration(const FMetasoundFrontendClassMetadata& InClassMetadata) const
{
	return CreateFrontendNodeConfiguration(InClassMetadata.GetType(), Metasound::Frontend::FNodeRegistryKey(InClassMetadata));
}

TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FMetaSoundFrontendDocumentBuilder::CreateFrontendNodeConfiguration(EMetasoundFrontendClassType InClassType, const Metasound::Frontend::FNodeRegistryKey& InClassKey) const
{
	using namespace Metasound::Frontend;

	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NodeConfiguration;

	if (InClassType == EMetasoundFrontendClassType::Template)
	{
		if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(InClassKey))
		{
			NodeConfiguration = Template->CreateFrontendTemplateNodeConfiguration();
		}
	}
	else
	{
		NodeConfiguration = INodeClassRegistry::GetChecked().CreateFrontendNodeConfiguration(InClassKey);
	}

	return NodeConfiguration;
}

FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeInternal(const FMetasoundFrontendClassMetadata& InClassMetadata, FFinalizeNodeFunctionRef FinalizeNode, const FGuid& InPageID, FGuid InNodeID, int32* NewNodeIndex)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::AddNodeInternal);

	using namespace Metasound::Frontend;

	const FNodeRegistryKey ClassKey = FNodeRegistryKey(InClassMetadata);
	return AddNodeInternal(ClassKey, FinalizeNode, InPageID, InNodeID, NewNodeIndex);
}

FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::AddNodeInternal(const Metasound::Frontend::FNodeRegistryKey& InClassKey, FFinalizeNodeFunctionRef FinalizeNode, const FGuid& InPageID, FGuid InNodeID, int32* NewNodeIndex)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::AddNodeInternal);

	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendClass* Dependency = DocumentCache->FindDependency(InClassKey))
	{
		// Get the array which holds the graph's nodes
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(InPageID);
		TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;

		// Create a node configuration for the new node
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NodeConfiguration = CreateFrontendNodeConfiguration(Dependency->Metadata.GetType(), InClassKey);

		// Add the new node into the graphs array
		FMetasoundFrontendNode& Node = Nodes.Emplace_GetRef(*Dependency, MoveTemp(NodeConfiguration));

		// Make sure the node utilizes the correct and unique NodeID
		Node.UpdateID(InNodeID);

		FinalizeNode(Node, InClassKey);

		// Update document builder cache
		const int32 NewIndex = Nodes.Num() - 1;
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(InPageID);
		DocumentDelegates->FindGraphDelegatesChecked(InPageID).NodeDelegates.OnNodeAdded.Broadcast(NewIndex);

		if (NewNodeIndex)
		{
			*NewNodeIndex = NewIndex;
		}

#if WITH_EDITORONLY_DATA
		Document.Metadata.ModifyContext.AddNodeIDModified(InNodeID);
#endif // WITH_EDITORONLY_DATA

		return &Node;
	}

	return nullptr;
}

#if WITH_EDITORONLY_DATA
const FMetasoundFrontendGraph& FMetaSoundFrontendDocumentBuilder::AddGraphPage(const FGuid& InPageID, bool bDuplicateLastGraph, bool bSetAsBuildGraph)
{
	using namespace Metasound::Frontend;

	const FMetasoundFrontendGraph& ToReturn = GetDocumentChecked().RootGraph.AddGraphPage(InPageID, bDuplicateLastGraph);
	DocumentDelegates->AddPageDelegates(InPageID);
	if (bSetAsBuildGraph)
	{
		SetBuildPageID(InPageID);
	}
	return ToReturn;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void FMetaSoundFrontendDocumentBuilder::CacheRegistryMetadata() const
{
	using namespace Metasound::Frontend;
	using FNameDataTypePair = TPair<FName, FName>;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendClassInterface& RootGraphClassInterface = Document.RootGraph.GetDefaultInterface();

	// 1. Gather inputs/outputs managed by interfaces
	TMap<FNameDataTypePair, FMetasoundFrontendClassInput*> Inputs;
	for (FMetasoundFrontendClassInput& Input : RootGraphClassInterface.Inputs)
	{
		Inputs.Add(FNameDataTypePair(Input.Name, Input.TypeName), &Input);
	}

	TMap<FNameDataTypePair, FMetasoundFrontendClassOutput*> Outputs;
	for (FMetasoundFrontendClassOutput& Output : RootGraphClassInterface.Outputs)
	{
		Outputs.Add(FNameDataTypePair(Output.Name, Output.TypeName), &Output);
	}

	// 2. Copy metadata for inputs/outputs managed by interfaces, removing them from maps generated
	auto CacheInterfaceMetadata = [](const FMetasoundFrontendVertexMetadata& InRegistryMetadata, FMetasoundFrontendVertexMetadata& OutMetadata)
	{
		const int32 CachedSortOrderIndex = OutMetadata.SortOrderIndex;
		OutMetadata = InRegistryMetadata;
		OutMetadata.SortOrderIndex = CachedSortOrderIndex;
	};

	const TSet<FMetasoundFrontendVersion>& InterfaceVersions = Document.Interfaces;
	for (const FMetasoundFrontendVersion& Version : InterfaceVersions)
	{
		FMetasoundFrontendInterface Interface;
		const bool bFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);

		UE_CLOGF(!bFound, LogMetaSound, Error,
			"Failed to find interface (%ls) when caching registry data for %ls. "
			"MetaSound inputs and outputs for asset may not function correctly.",
			*Version.ToString(), *GetDebugName());

		if (bFound)
		{
			for (const FMetasoundFrontendClassInput& InterfaceInput : Interface.Inputs)
			{
				const FNameDataTypePair NameDataTypePair = FNameDataTypePair(InterfaceInput.Name, InterfaceInput.TypeName);
				if (FMetasoundFrontendClassInput* Input = Inputs.FindRef(NameDataTypePair))
				{
					CacheInterfaceMetadata(InterfaceInput.Metadata, Input->Metadata);
					Inputs.Remove(NameDataTypePair);
				}
			}

			for (const FMetasoundFrontendClassOutput& InterfaceOutput : Interface.Outputs)
			{
				const FNameDataTypePair NameDataTypePair = FNameDataTypePair(InterfaceOutput.Name, InterfaceOutput.TypeName);
				if (FMetasoundFrontendClassOutput* Output = Outputs.FindRef(NameDataTypePair))
				{
					CacheInterfaceMetadata(InterfaceOutput.Metadata, Output->Metadata);
					Outputs.Remove(NameDataTypePair);
				}
			}
		}
	}

	// 3. Iterate remaining inputs/outputs not managed by interfaces and set to serialize text
	// (in case they were orphaned by an interface no longer being implemented).
	for (const TPair<FNameDataTypePair, FMetasoundFrontendClassInput*>& Pair : Inputs)
	{
		Pair.Value->Metadata.SetSerializeText(true);
	}

	for (const TPair<FNameDataTypePair, FMetasoundFrontendClassOutput*>& Pair : Outputs)
	{
		Pair.Value->Metadata.SetSerializeText(true);
	}

	// 4. Refresh style as order of members could've changed
	{
		FMetasoundFrontendInterfaceStyle InputStyle;
		Algo::ForEach(RootGraphClassInterface.Inputs, [&InputStyle](const FMetasoundFrontendClassInput& Input)
		{
			InputStyle.DefaultSortOrder.Add(Input.Metadata.SortOrderIndex);
		});
		RootGraphClassInterface.SetInputStyle(MoveTemp(InputStyle));
	}

	{
		FMetasoundFrontendInterfaceStyle OutputStyle;
		Algo::ForEach(RootGraphClassInterface.Outputs, [&OutputStyle](const FMetasoundFrontendClassOutput& Output)
		{
			OutputStyle.DefaultSortOrder.Add(Output.Metadata.SortOrderIndex);
		});
		RootGraphClassInterface.SetOutputStyle(MoveTemp(OutputStyle));
	}

	// 5. Cache registry data on document dependencies
	for (FMetasoundFrontendClass& Dependency : Document.Dependencies)
	{
		if (!FMetasoundFrontendClass::CacheGraphDependencyMetadataFromRegistry(Dependency))
		{
			UE_LOGF(LogMetaSound, Warning,
				"'%ls' failed to cache dependency registry data: Registry missing class with key '%ls'",
				*GetDebugName(),
				*Dependency.Metadata.GetClassName().ToString());
			UE_LOGF(LogMetaSound, Warning,
				"Asset '%ls' may fail to build runtime graph unless re-registered after dependency with given key is loaded.",
				*GetDebugName());
		}
	}
}
#endif // WITH_EDITOR

bool FMetaSoundFrontendDocumentBuilder::CanAddEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);

	if (!EdgeCache.IsNodeInputConnected(InEdge.ToNodeID, InEdge.ToVertexID))
	{
		return IsValidEdge(InEdge, InPageID) == EInvalidEdgeReason::None;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::CanAutoUpdate(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return FAutoUpdateRootGraph::CanAutoUpdate(*this, InNodeID, PageID);
}
#endif // WITH_EDITORONLY_DATA

void FMetaSoundFrontendDocumentBuilder::ClearDocument(TSharedRef<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates)
{
	FMetasoundFrontendDocument& Doc = GetDocumentChecked();
	FMetasoundFrontendGraphClass& GraphClass = Doc.RootGraph;

	GraphClass.GetDefaultInterface().Inputs.Empty();
	GraphClass.GetDefaultInterface().Outputs.Empty();
	GraphClass.GetDefaultInterface().Environment.Empty();

#if WITH_EDITOR
	GraphClass.GetDefaultInterface().SetInputStyle({ });
	GraphClass.GetDefaultInterface().SetOutputStyle({ });
#endif // WITH_EDITOR

	// Removing graph pages is not necessary when editor only data is not available as graph mutation
	// is only supported in builds with editor data loaded. Otherwise, anything calling ClearDocument
	// should only be a transient, non serialized asset graph which does not support page mutation.
#if WITH_EDITORONLY_DATA
	constexpr bool bClearDefaultGraph = true;
	ResetGraphPages(bClearDefaultGraph);
#else // !WITH_EDITORONLY_DATA
	UObject& DocObject = CastDocumentObjectChecked<UObject>();
	checkf(!DocObject.IsAsset(), TEXT("Cannot call clear document on asset '%s': builder API does not support document mutation on serialized objects without editor data loaded"), *GetDebugName());

	GraphClass.IterateGraphPages([] (FMetasoundFrontendGraph& Graph)
	{
		Graph.Nodes.Empty();
		Graph.Edges.Empty();
		Graph.Variables.Empty();
	});
#endif // !WITH_EDITORONLY_DATA

	Doc.Interfaces.Empty();
	Doc.Dependencies.Empty();

#if WITH_EDITORONLY_DATA
	Doc.Metadata.MemberMetadata.Empty();
#endif // WITH_EDITORONLY_DATA

	Reload(ModifyDelegates);
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::ClearMemberMetadata(const FGuid& InMemberID)
{
	DocumentDelegates->InterfaceDelegates.OnRemovingMemberMetadata.Broadcast(InMemberID);
	return GetDocumentChecked().Metadata.MemberMetadata.Remove(InMemberID) > 0;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::ConformGraphInputNodeToClass(const FMetasoundFrontendClassInput& GraphInput)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendClass Class;
	const bool bClassFound = DocumentBuilderPrivate::FindInputRegistryClass(GraphInput.TypeName, GraphInput.AccessType, Class);
	if (ensureAlways(bClassFound))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		const FMetasoundFrontendClass* Dependency = FindDependency(Class.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(Class));
		}

		if (ensureAlways(Dependency))
		{
			Document.RootGraph.IterateGraphPages([this, &Document, &Dependency, &GraphInput](FMetasoundFrontendGraph& Graph)
			{
				const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
				if (const int32* NodeIndexPtr = NodeCache.FindNodeIndex(GraphInput.NodeID))
				{
					TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
					FMetasoundFrontendNode& Node = Nodes[*NodeIndexPtr];
					FNodeModifyDelegates& NodeDelegates = DocumentDelegates->FindGraphDelegatesChecked(Graph.PageID).NodeDelegates;
					const int32 RemovalIndex = *NodeIndexPtr; // Have to cache as next delegate broadcast invalidates index pointer
					NodeDelegates.OnRemoveSwappingNode.Broadcast(RemovalIndex, Nodes.Num() - 1);
					FMetasoundFrontendNode NewNode = MoveTemp(Node);
					Nodes.RemoveAtSwap(RemovalIndex, EAllowShrinking::No);
					NewNode.ClassID = Dependency->ID;
					NewNode.Interface.Inputs.Last().TypeName = GraphInput.TypeName;
					NewNode.Interface.Outputs.Last().TypeName = GraphInput.TypeName;

#if WITH_EDITORONLY_DATA
					Document.Metadata.ModifyContext.AddNodeIDModified(NewNode.GetID());
#endif // WITH_EDITORONLY_DATA

					// Set the default literal on the nodes inputs so that it gets passed to the instantiated TInputNode on a live
					// auditioned MetaSound.
					DocumentBuilderPrivate::SetDefaultLiteralOnInputNode(NewNode, GraphInput, Graph.PageID);

					FMetasoundFrontendNode& NewNodeRef = Nodes.Add_GetRef(MoveTemp(NewNode));
					NodeDelegates.OnNodeAdded.Broadcast(Nodes.Num() - 1);

					// Remove the default literal on the node added during the "FinalizeNode" call. This matches how 
					// nodes are serialized in editor. The default literals are only stored on the FMetasoundFrontendClassInputs.
					NewNodeRef.InputLiterals.Reset();
				}
			});

			RemoveUnusedDependencies();
			return true;
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::ConformGraphOutputNodeToClass(const FMetasoundFrontendClassOutput& GraphOutput)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendClass Class;
	const bool bClassFound = DocumentBuilderPrivate::FindOutputRegistryClass(GraphOutput.TypeName, GraphOutput.AccessType, Class);
	if (ensureAlways(bClassFound))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		const FMetasoundFrontendClass* Dependency = FindDependency(Class.Metadata);
		if (!Dependency)
		{
			Dependency = AddDependency(MoveTemp(Class));
		}

		if (ensureAlways(Dependency))
		{
			Document.RootGraph.IterateGraphPages([this, &Document, &Dependency, &GraphOutput](FMetasoundFrontendGraph& Graph)
			{
				const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
				if (const int32* NodeIndexPtr = NodeCache.FindNodeIndex(GraphOutput.NodeID))
				{
					TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
					FMetasoundFrontendNode& Node = Nodes[*NodeIndexPtr];
					FNodeModifyDelegates& NodeDelegates = DocumentDelegates->FindGraphDelegatesChecked(Graph.PageID).NodeDelegates;
					const int32 RemovalIndex = *NodeIndexPtr; // Have to cache as next delegate broadcast invalidates index pointer
					NodeDelegates.OnRemoveSwappingNode.Broadcast(RemovalIndex, Nodes.Num() - 1);
					FMetasoundFrontendNode NewNode = MoveTemp(Node);
					Nodes.RemoveAtSwap(RemovalIndex, EAllowShrinking::No);
					NewNode.ClassID = Dependency->ID;
					NewNode.Interface.Inputs.Last().TypeName = GraphOutput.TypeName;
					NewNode.Interface.Outputs.Last().TypeName = GraphOutput.TypeName;

#if WITH_EDITORONLY_DATA
					Document.Metadata.ModifyContext.AddNodeIDModified(NewNode.GetID());
#endif // WITH_EDITORONLY_DATA
					Nodes.Add(MoveTemp(NewNode));
					NodeDelegates.OnNodeAdded.Broadcast(Nodes.Num() - 1);
				}	
			});	

			RemoveUnusedDependencies();
			return true;
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::ConformObjectToDocument()
{
	return GetDocumentInterfaceChecked().ConformObjectToDocument();
}

bool FMetaSoundFrontendDocumentBuilder::ConfigureDocument()
{
	using namespace Metasound::Frontend;

	if (bInTemplateScope)
	{
		UE_LOGF(LogMetaSound, Error, "Cannot call Configure Document: Configure already called higher in stack");
		return false;
	}

	bInTemplateScope = true;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	if (FMetaSoundFrontendDocumentTemplate* DocTemplate = Document.Template.GetMutablePtr())
	{
		if (DocTemplate->ConfigureDocument(*this))
		{
			GetDocumentInterfaceChecked().ConformObjectToDocument();

#if WITH_EDITORONLY_DATA
			FInputNodeTemplate::GetChecked().Inject(*this);
#endif // WITH_EDITORONLY_DATA

			bInTemplateScope = false;
			return true;
		}

		TSharedRef<FDocumentModifyDelegates> ModifyDelegates = MakeShared<FDocumentModifyDelegates>(*DocumentDelegates);
		ClearDocument(ModifyDelegates);

#if WITH_EDITORONLY_DATA
		SetIsGraphEditable(false);
#endif // WITH_EDITORONLY_DATA

		// Add required interfaces, but disable editing to ensure document maintains valid state
		// (ex. MetaSound sources need  the source interface to properly initialize a null-rendering
		// MetaSound generator.)
		TArray<FMetasoundFrontendVersion> InitVersions = ISearchEngine::Get().FindUClassDefaultInterfaceVersions(GetBuilderClassPath());
		FModifyInterfaceOptions Options({ }, MoveTemp(InitVersions));
		ModifyInterfaces(MoveTemp(Options));
	}
#if WITH_EDITORONLY_DATA
	else
	{
		SetIsGraphEditable(true);
	}
#endif // WITH_EDITORONLY_DATA

	bInTemplateScope = false;
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::ContainsDependencyOfType(EMetasoundFrontendClassType ClassType) const
{
	return DocumentCache->ContainsDependencyOfType(ClassType);
}

bool FMetaSoundFrontendDocumentBuilder::ContainsEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(InPageID ? *InPageID : BuildPageID);
	return EdgeCache.ContainsEdge(InEdge);
}

bool FMetaSoundFrontendDocumentBuilder::ContainsNode(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(InPageID ? * InPageID : BuildPageID);
	return NodeCache.ContainsNode(InNodeID);
}

bool FMetaSoundFrontendDocumentBuilder::ConvertFromPreset()
{
	using namespace Metasound::Frontend;

	if (IsPreset())
	{
		const bool bConfigChanged = GetDocumentChecked().Template.IsValid();

		// Don't configure after nulling the template in order to "freeze" the current document/MetaSound UObject state (i.e. not clear it).
		GetDocumentChecked().Template.Reset();

#if WITH_EDITORONLY_DATA
		SetIsGraphEditable(true);
#endif // WITH_EDITORONLY_DATA?

		if (bConfigChanged)
		{
			DocumentDelegates->OnDocumentTemplateChanged.Broadcast(FDocumentTemplateChangedArgs { .Type = EDocumentTemplateChangeType::Struct });
		}

		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::ConvertToPreset(const FMetasoundFrontendDocument& InReferencedDocument, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates)
{
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::ConvertToPreset(const FMetaSoundFrontendDocumentBuilder& InReferencedDocumentBuilder, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	using FPresetConfigStruct = TInstancedStruct<FMetaSoundFrontendPresetTemplate>;
	FPresetConfigStruct NewConfig = FPresetConfigStruct::Make<FMetaSoundFrontendPresetTemplate>();
	NewConfig.GetMutable().Parent = &InReferencedDocumentBuilder.CastDocumentObjectChecked<UObject>();
	SetTemplate(MoveTemp(NewConfig));
	ConfigureDocument();

	return true;
}

TOptional<Metasound::FAnyDataReference> FMetaSoundFrontendDocumentBuilder::CreateDataReference(
	const Metasound::FOperatorSettings& InOperatorSettings,
	FName DataType,
	const Metasound::FLiteral& InLiteral,
	Metasound::EDataReferenceAccessType AccessType)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	return IDataTypeRegistry::Get().CreateDataReference(DataType, AccessType, InLiteral, InOperatorSettings);
};

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::NodeInterfaceMatchesClassInterface(const FGuid& InNodeID, const FGuid& InPageID) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendNode* Node = FindNode(InNodeID, &InPageID);
	if (!Node)
	{
		return false;
	}

	const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID);
	if (!Class)
	{
		return false;
	}

	const FMetasoundFrontendClassInterface& NodeClassInterface = Class->GetInterfaceForNode(*Node);
	const FMetasoundFrontendNodeInterface& NodeInterface = Node->Interface;

	if ((NodeClassInterface.Inputs.Num() != NodeInterface.Inputs.Num()) ||
		(NodeClassInterface.Outputs.Num() != NodeInterface.Outputs.Num()))
	{
		return false;
	}
	
	for (const FMetasoundFrontendClassInput& Input : NodeClassInterface.Inputs)
	{
		auto IsEquivalent = [&Input](const FMetasoundFrontendVertex& RegistryVertex)
		{
			return FMetasoundFrontendVertex::IsFunctionalEquivalent(Input, RegistryVertex);
		};

		const int32 Index = NodeInterface.Inputs.FindLastByPredicate(IsEquivalent);
		if (Index == INDEX_NONE)
		{
			return false;
		}
	}
	
	for (const FMetasoundFrontendClassOutput& Output : NodeClassInterface.Outputs)
	{
		auto IsEquivalent = [&Output](const FMetasoundFrontendVertex& RegistryVertex)
		{
			return FMetasoundFrontendVertex::IsFunctionalEquivalent(Output, RegistryVertex);
		};

		const int32 Index = NodeInterface.Outputs.FindLastByPredicate(IsEquivalent);
		if (Index == INDEX_NONE)
		{
			return false;
		}
	}
	return true;
}

bool FMetaSoundFrontendDocumentBuilder::DiffAgainstRegistryInterface(const FGuid& InNodeID, const FGuid& InPageID, bool bInUseHighestMinorVersion, Metasound::Frontend::FClassInterfaceUpdates& OutInterfaceUpdates, bool bInForceRegenerateClassInterfaceOverride) const
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::DiffAgainstRegistryInterface);

	using namespace Metasound;
	using namespace Metasound::Frontend;

	OutInterfaceUpdates = FClassInterfaceUpdates();

	const FMetasoundFrontendNode* Node = FindNode(InNodeID, &InPageID);
	if (!Node)
	{
		return false;
	}

	const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID);
	if (!Class)
	{
		return false;
	}

	const FMetasoundFrontendClassMetadata& NodeClassMetadata = Class->Metadata;
	const FMetasoundFrontendClassInterface& NodeClassInterface = Class->GetInterfaceForNode(*Node);

	FNodeClassName NodeClassName = NodeClassMetadata.GetClassName().ToNodeClassName();

	auto ClearInterface = [&]()
	{
		Algo::Transform(NodeClassInterface.Inputs, OutInterfaceUpdates.RemovedInputs, [&](const FMetasoundFrontendClassInput& Input) { return &Input; });
		Algo::Transform(NodeClassInterface.Outputs, OutInterfaceUpdates.RemovedOutputs, [&](const FMetasoundFrontendClassOutput& Output) { return &Output; });
	};

	// Get class 
	if (bInUseHighestMinorVersion)
	{
		FMetaSoundClassInfo ClassInfo;
		if (!ISearchEngine::Get().FindRegisteredClass(NodeClassName, NodeClassMetadata.GetVersion().Major, ClassInfo))
		{
			ClearInterface();
			return false;
		}

		bool bFoundRegisteredClass = INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(ClassInfo.ToRegistryKey(), OutInterfaceUpdates.RegistryClass);
		if (!ensureMsgf(bFoundRegisteredClass, TEXT("SearchEngine returned registered class info for class that is not registered")))
		{
			ClearInterface();
			return false;
		}
	}
	else
	{
		// Find class with same metadata in the node registry.
		bool bFoundRegisteredClass = INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(FNodeRegistryKey(NodeClassMetadata), OutInterfaceUpdates.RegistryClass);
		if (!bFoundRegisteredClass)
		{
			// If the class was not found, mark all inputs and outputs as removed.
			UE_LOGF(LogMetaSound, Warning, "Could not find registered version of interface. %ls %ls is not registered.", *NodeClassMetadata.GetClassName().ToString(), *NodeClassMetadata.GetVersion().ToString());
			ClearInterface();
			return false;
		}
	}

	// Get updates of the node's configuration
	FNodeConfigurationUpdateData NodeConfigUpdates;
	FindNodeConfigurationUpdates(InNodeID, InPageID, OutInterfaceUpdates.RegistryClass, NodeConfigUpdates, bInForceRegenerateClassInterfaceOverride);

	// Based upon all the node configuration updates, determine which class interface is the correct up-to-date interface
	const FMetasoundFrontendClassInterface& ApplicableRegistryInterface = GetApplicableRegistryInterface(OutInterfaceUpdates.RegistryClass, NodeConfigUpdates);
	
	// Diff interface 
	Algo::Transform(ApplicableRegistryInterface.Inputs, OutInterfaceUpdates.AddedInputs, [](const FMetasoundFrontendClassInput& Input) { return &Input; });
	for (const FMetasoundFrontendClassInput& Input : NodeClassInterface.Inputs)
	{
		auto IsEquivalent = [&Input](const FMetasoundFrontendClassInput* RegistryInput)
		{
			return FMetasoundFrontendClassInput::IsFunctionalEquivalent(Input, *RegistryInput);
		};

		const int32 Index = OutInterfaceUpdates.AddedInputs.FindLastByPredicate(IsEquivalent);
		if (Index == INDEX_NONE)
		{
			OutInterfaceUpdates.RemovedInputs.Add(&Input);
		}
		else
		{
			OutInterfaceUpdates.AddedInputs.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}


	Algo::Transform(ApplicableRegistryInterface.Outputs, OutInterfaceUpdates.AddedOutputs, [](const FMetasoundFrontendClassOutput& Output) { return &Output; });
	for (const FMetasoundFrontendClassOutput& Output : NodeClassInterface.Outputs)
	{
		auto IsFunctionalEquivalent = [&Output](const FMetasoundFrontendClassOutput* Iter)
		{
			return FMetasoundFrontendClassVertex::IsFunctionalEquivalent(Output, *Iter);
		};

		const int32 Index = OutInterfaceUpdates.AddedOutputs.FindLastByPredicate(IsFunctionalEquivalent);
		if (Index == INDEX_NONE)
		{
			OutInterfaceUpdates.RemovedOutputs.Add(&Output);
		}
		else
		{
			OutInterfaceUpdates.AddedOutputs.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}

	const bool bFoundDifferenceInInterface = OutInterfaceUpdates.ContainsRemovedMembers() || OutInterfaceUpdates.ContainsAddedMembers();

	// Add node config and class interface override info if necessary. 
	if (ShouldReplaceExistingNodeConfig(NodeConfigUpdates.RegisteredConfig, NodeConfigUpdates.ExistingConfig))
	{
		OutInterfaceUpdates.AddedConfiguration = MoveTemp(NodeConfigUpdates.RegisteredConfig);
		OutInterfaceUpdates.RemovedConfiguration = NodeConfigUpdates.ExistingConfig;
	}

	if (NodeConfigUpdates.bDidUpdateClassInterfaceOverride)
	{
		// The class interface override is updated if there was any difference found
		// in the interface, or the override was added/removed. 
		const bool bReplaceClassInterfaceOverride = bFoundDifferenceInInterface || (NodeConfigUpdates.ExistingClassInterfaceOverride.IsValid() != NodeConfigUpdates.RegeneratedClassInterfaceOverride.IsValid());
		if (bReplaceClassInterfaceOverride)
		{
			OutInterfaceUpdates.AddedClassInterfaceOverride = MoveTemp(NodeConfigUpdates.RegeneratedClassInterfaceOverride);
			OutInterfaceUpdates.RemovedClassInterfaceOverride = NodeConfigUpdates.ExistingClassInterfaceOverride;
		}
	}

	return true;
}
#endif // WITH_EDITORONLY_DATA

const FMetasoundFrontendClassInput* FMetaSoundFrontendDocumentBuilder::DuplicateGraphInput(FName ExistingName, FName NewName)
{
	using namespace Metasound;

	const FMetasoundFrontendClassInput* ExistingInput = FindGraphInput(ExistingName);
	if (!ExistingInput)
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph input '%ls': input does not exist", *ExistingName.ToString());
		return nullptr;
	}

	if (FindGraphInput(NewName))
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph input '%ls': input with name '%ls' already exists", *ExistingName.ToString(), *NewName.ToString());
		return nullptr;
	}

	Frontend::FDocumentIDGenerator& IDGenerator = Frontend::FDocumentIDGenerator::Get();
	const FMetasoundFrontendDocument& Doc = GetConstDocumentChecked();

	FMetasoundFrontendClassInput ClassInput = *ExistingInput;
	ClassInput.NodeID = IDGenerator.CreateNodeID(Doc);
	ClassInput.VertexID = IDGenerator.CreateVertexID(Doc);
#if WITH_EDITORONLY_DATA
	ClassInput.Metadata.SetDisplayName(FText::GetEmpty());
#endif // WITH_EDITORONLY_DATA
	ClassInput.Name = NewName;

	AddGraphInput(MoveTemp(ClassInput));
	return FindGraphInput(NewName);
}

const FMetasoundFrontendClassOutput* FMetaSoundFrontendDocumentBuilder::DuplicateGraphOutput(FName ExistingName, FName NewName)
{
	using namespace Metasound::Frontend;

	const FMetasoundFrontendClassOutput* ExistingOutput = FindGraphOutput(ExistingName);
	if (!ExistingOutput)
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph output '%ls', output does not exist", *ExistingName.ToString());
		return nullptr;

	}

	if (FindGraphOutput(NewName))
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph output '%ls', output with name '%ls' already exists", *ExistingName.ToString(), *NewName.ToString());
		return nullptr;
	}

	FDocumentIDGenerator& IDGenerator = FDocumentIDGenerator::Get();
	const FMetasoundFrontendDocument& Doc = GetConstDocumentChecked();

	FMetasoundFrontendClassOutput ClassOutput = *ExistingOutput;
	ClassOutput.NodeID = IDGenerator.CreateNodeID(Doc);
	ClassOutput.VertexID = IDGenerator.CreateVertexID(Doc);
#if WITH_EDITORONLY_DATA
	ClassOutput.Metadata.SetDisplayName(FText::GetEmpty());
#endif // WITH_EDITORONLY_DATA
	ClassOutput.Name = NewName;

	AddGraphOutput(MoveTemp(ClassOutput));
	return FindGraphOutput(NewName);
}

const FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::DuplicateGraphVariable(FName ExistingName, FName NewName, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	if (FindGraphVariable(NewName, InPageID))
	{
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph variable '%ls': variable with name '%ls' already exists", *ExistingName.ToString(), *NewName.ToString());
		return nullptr;
	}

	if (const FMetasoundFrontendVariable* ExistingVariable = FindGraphVariable(ExistingName, InPageID))
	{
#if WITH_EDITORONLY_DATA
		const FText* Description = &ExistingVariable->Description;
#else
		const FText* Description = nullptr;
#endif // WITH_EDITORONLY_DATA

		const FMetasoundFrontendVariable* NewVariable = AddGraphVariable(
			NewName,
			ExistingVariable->TypeName,
			&ExistingVariable->Literal,
			&FText::GetEmpty(), // Don't copy to ensure no confusion over identical display names
			Description,
			InPageID
		);
		return NewVariable;
	}
	else
	{
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		UE_LOGF(LogMetaSound, Warning, "Failed to duplicate graph variable '%ls' on page '%ls': variable does not exist", *ExistingName.ToString(), *PageID.ToString());
	}

	return nullptr;
}

FMetasoundFrontendGraph& FMetaSoundFrontendDocumentBuilder::FindBuildGraphChecked() const
{
	return GetDocumentChecked().RootGraph.FindGraphChecked(BuildPageID);
}

const FMetasoundFrontendGraph& FMetaSoundFrontendDocumentBuilder::FindConstBuildGraphChecked() const
{
	return GetConstDocumentChecked().RootGraph.FindConstGraphChecked(BuildPageID);
}

TConstStructView<FMetasoundFrontendClassInterface> FMetaSoundFrontendDocumentBuilder::FindClassInterfaceOverride(const FGuid& InNodeID, const FGuid* InPageID) const
{
	TConstStructView<FMetasoundFrontendClassInterface> ClassInterfaceOverride;
	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		ClassInterfaceOverride = Node->ClassInterfaceOverride;
	}
	return ClassInterfaceOverride;
}

bool FMetaSoundFrontendDocumentBuilder::FindDeclaredInterfaces(TArray<const Metasound::Frontend::IInterfaceRegistryEntry*>& OutInterfaces) const
{
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::FindDeclaredInterfaces(const FMetasoundFrontendDocument& InDocument, TArray<const Metasound::Frontend::IInterfaceRegistryEntry*>& OutInterfaces)
{
	return false;
}

const FMetasoundFrontendClass* FMetaSoundFrontendDocumentBuilder::FindDependency(const FGuid& InClassID) const
{
	return DocumentCache->FindDependency(InClassID);
}

const FMetasoundFrontendClass* FMetaSoundFrontendDocumentBuilder::FindDependency(const FMetasoundFrontendClassMetadata& InMetadata) const
{
	using namespace Metasound::Frontend;

	checkf(InMetadata.GetType() != EMetasoundFrontendClassType::Graph,
		TEXT("Dependencies are never listed as 'Graph' types. Graphs are considered 'External' from the perspective of the parent document to allow for nativization."));
	const FNodeRegistryKey RegistryKey = FNodeRegistryKey(InMetadata);
	return DocumentCache->FindDependency(RegistryKey);
}

const FMetasoundFrontendClass* FMetaSoundFrontendDocumentBuilder::FindDependency(const Metasound::Frontend::FNodeRegistryKey& InRegistryKey) const
{
	return DocumentCache->FindDependency(InRegistryKey);
}

TArray<const FMetasoundFrontendEdge*> FMetaSoundFrontendDocumentBuilder::FindEdges(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(InPageID ? *InPageID : BuildPageID);
	return EdgeCache.FindEdges(InNodeID, InVertexID);
}

#if WITH_EDITORONLY_DATA
const FMetasoundFrontendEdgeStyle* FMetaSoundFrontendDocumentBuilder::FindConstEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID) const
{
	auto IsEdgeStyle = [&InNodeID, &OutputName](const FMetasoundFrontendEdgeStyle& EdgeStyle)
	{
		return EdgeStyle.NodeID == InNodeID && EdgeStyle.OutputName == OutputName;
	};

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
	return Graph.Style.EdgeStyles.FindByPredicate(IsEdgeStyle);
}

FMetasoundFrontendEdgeStyle* FMetaSoundFrontendDocumentBuilder::FindEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID)
{
	auto IsEdgeStyle = [&InNodeID, &OutputName](const FMetasoundFrontendEdgeStyle& EdgeStyle)
	{
		return EdgeStyle.NodeID == InNodeID && EdgeStyle.OutputName == OutputName;
	};

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	return Graph.Style.EdgeStyles.FindByPredicate(IsEdgeStyle);
}

FMetasoundFrontendEdgeStyle& FMetaSoundFrontendDocumentBuilder::FindOrAddEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID)
{
	if (FMetasoundFrontendEdgeStyle* Style = FindEdgeStyle(InNodeID, OutputName, InPageID))
	{
		return *Style;
	}

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	FMetasoundFrontendEdgeStyle& EdgeStyle = Graph.Style.EdgeStyles.AddDefaulted_GetRef();

	checkf(ContainsNode(InNodeID), TEXT("Cannot add edge style for node that does not exist"));
	EdgeStyle.NodeID = InNodeID;
	EdgeStyle.OutputName = OutputName;
	return EdgeStyle;
}

const FMetaSoundFrontendGraphComment* FMetaSoundFrontendDocumentBuilder::FindGraphComment(const FGuid& InCommentID, const FGuid* InPageID) const
{
	check(InCommentID.IsValid());
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const TMap<FGuid, FMetaSoundFrontendGraphComment>& Comments = Document.RootGraph.FindConstGraphChecked(PageID).Style.Comments;
	return Comments.Find(InCommentID);
}

FMetaSoundFrontendGraphComment* FMetaSoundFrontendDocumentBuilder::FindGraphComment(const FGuid& InCommentID, const FGuid* InPageID)
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TMap<FGuid, FMetaSoundFrontendGraphComment>& Comments = Document.RootGraph.FindGraphChecked(PageID).Style.Comments;
	return Comments.Find(InCommentID);
}
#endif // WITH_EDITORONLY_DATA


const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindHeadNodeInVariableStack(FName VariableName, const FGuid* InPageID) const
{
	// The variable "stack" is [GetDelayedNodes, SetNode, GetNodes].
	if (const FMetasoundFrontendVariable* Variable = FindGraphVariable(VariableName, InPageID))
	{
		if (!Variable->DeferredAccessorNodeIDs.IsEmpty())
		{
			return FindNode(Variable->DeferredAccessorNodeIDs[0], InPageID);
		}

		if (Variable->MutatorNodeID.IsValid())
		{
			return FindNode(Variable->MutatorNodeID, InPageID);
		}

		if (!Variable->AccessorNodeIDs.IsEmpty())
		{
			return FindNode(Variable->AccessorNodeIDs[0], InPageID);
		}
	}

	return nullptr;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindTailNodeInVariableStack(FName VariableName, const FGuid* InPageID) const
{
	// The variable "stack" is [GetDelayedNodes, SetNode, GetNodes].
	if (const FMetasoundFrontendVariable* Variable = FindGraphVariable(VariableName, InPageID))
	{
		if (!Variable->AccessorNodeIDs.IsEmpty())
		{
			return FindNode(Variable->AccessorNodeIDs.Last(), InPageID);
		}

		if (Variable->MutatorNodeID.IsValid())
		{
			return FindNode(Variable->MutatorNodeID, InPageID);
		}

		if (!Variable->DeferredAccessorNodeIDs.IsEmpty())
		{
			return FindNode(Variable->DeferredAccessorNodeIDs.Last(), InPageID);
		}
	}

	return nullptr;
}

bool FMetaSoundFrontendDocumentBuilder::FindInterfaceInputNodes(FName InterfaceName, TArray<const FMetasoundFrontendNode*>& OutInputs, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	OutInputs.Reset();

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendVersion InterfaceVersion;
	const TSet<FMetasoundFrontendVersion>& Interfaces = GetConstDocumentChecked().Interfaces;
	if (ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceName, InterfaceVersion))
	{
		if (Interfaces.Contains(InterfaceVersion))
		{
			FMetasoundFrontendInterface Interface;
			const bool bFoundInterface = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, Interface);
			check(bFoundInterface);

			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
			const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();

			TArray<const FMetasoundFrontendNode*> InterfaceInputs;
			for (const FMetasoundFrontendClassInput& Input : Interface.Inputs)
			{
				const FMetasoundFrontendClassInput* ClassInput = InterfaceCache.FindInput(Input.Name);
				if (!ClassInput)
				{
					return false;
				}

				if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(ClassInput->NodeID))
				{
					InterfaceInputs.Add(Node);
				}
				else
				{
					return false;
				}
			}

			OutInputs = MoveTemp(InterfaceInputs);
			return true;
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::FindInterfaceOutputNodes(FName InterfaceName, TArray<const FMetasoundFrontendNode*>& OutOutputs, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	OutOutputs.Reset();

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	const TSet<FMetasoundFrontendVersion>& Interfaces = GetConstDocumentChecked().Interfaces;
	FMetasoundFrontendVersion InterfaceVersion;
	if (ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceName, InterfaceVersion))
	{
		if (Interfaces.Contains(InterfaceVersion))
		{
			FMetasoundFrontendInterface Interface;
			if (IInterfaceRegistry::Get().FindInterface(InterfaceVersion, Interface))
			{
				const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
				const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();

				TArray<const FMetasoundFrontendNode*> InterfaceOutputs;
				for (const FMetasoundFrontendClassOutput& Output : Interface.Outputs)
				{
					const FMetasoundFrontendClassOutput* ClassOutput = InterfaceCache.FindOutput(Output.Name);
					if (!ClassOutput)
					{
						return false;
					}

					if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(ClassOutput->NodeID))
					{
						InterfaceOutputs.Add(Node);
					}
					else
					{
						return false;
					}
				}

				OutOutputs = MoveTemp(InterfaceOutputs);
				return true;
			}
		}
	}

	return false;
}

const FMetasoundFrontendClassInput* FMetaSoundFrontendDocumentBuilder::FindGraphInput(FName InputName) const
{
	return DocumentCache->GetInterfaceCache().FindInput(InputName);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindGraphInputNode(FName InputName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendClassInput* InputClass = FindGraphInput(InputName))
	{
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
		return NodeCache.FindNode(InputClass->NodeID);
	}

	return nullptr;
}

const FMetasoundFrontendClassOutput* FMetaSoundFrontendDocumentBuilder::FindGraphOutput(FName OutputName) const
{
	return DocumentCache->GetInterfaceCache().FindOutput(OutputName);
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindGraphOutputNode(FName OutputName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendClassOutput* OutputClass = FindGraphOutput(OutputName))
	{
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
		return NodeCache.FindNode(OutputClass->NodeID);
	}

	return nullptr;
}

const FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::FindGraphVariable(const FGuid& InVariableID, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
	auto MatchesID = [&InVariableID](const FMetasoundFrontendVariable& Variable) { return Variable.ID == InVariableID; };
	return Graph.Variables.FindByPredicate(MatchesID);
}

const FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::FindGraphVariable(FName VariableName, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
	auto MatchesName = [&VariableName](const FMetasoundFrontendVariable& Variable) { return Variable.Name == VariableName; };
	return Graph.Variables.FindByPredicate(MatchesName);
}

FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::FindGraphVariableInternal(FName VariableName, const FGuid* InPageID)
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	auto MatchesName = [&VariableName](const FMetasoundFrontendVariable& Variable) { return Variable.Name == VariableName; };
	return Graph.Variables.FindByPredicate(MatchesName);
}

const FMetasoundFrontendVariable* FMetaSoundFrontendDocumentBuilder::FindGraphVariableByNodeID(const FGuid& InNodeID, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
	auto ContainsNodeWithID = [&InNodeID](const FMetasoundFrontendVariable& Variable)
	{
		return Variable.VariableNodeID == InNodeID
		|| Variable.MutatorNodeID == InNodeID
		|| Variable.DeferredAccessorNodeIDs.Contains(InNodeID)
		|| Variable.AccessorNodeIDs.Contains(InNodeID);
	};
	return Graph.Variables.FindByPredicate(ContainsNodeWithID);
}

#if WITH_EDITOR
UMetaSoundFrontendMemberMetadata* FMetaSoundFrontendDocumentBuilder::FindMemberMetadata(const FGuid& InMemberID) const
{
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const TMap<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>& LiteralMetadata = Document.Metadata.MemberMetadata;
	TObjectPtr<UMetaSoundFrontendMemberMetadata> ToReturn = LiteralMetadata.FindRef(InMemberID);
	return ToReturn;
}
#endif // WITH_EDITOR

TArray<const FMetasoundFrontendVertex*> FMetaSoundFrontendDocumentBuilder::FindUserModifiableNodeInputs(const FGuid& InNodeID, const FGuid* InPageID) const
{
	TArray<const FMetasoundFrontendVertex*> Vertices;
	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		Algo::TransformIf(Node->Interface.Inputs, Vertices,
			[this, &InNodeID](const FMetasoundFrontendVertex& Vertex)
			{
				return IsNodeInputConnectionUserModifiable(InNodeID, Vertex.VertexID);
			},
			[](const FMetasoundFrontendVertex& Vertex)
			{
				return &Vertex;
			}
		);
	}

	return Vertices;
}

TArray<const FMetasoundFrontendVertex*> FMetaSoundFrontendDocumentBuilder::FindUserModifiableNodeOutputs(const FGuid& InNodeID, const FGuid* InPageID) const
{
	TArray<const FMetasoundFrontendVertex*> Vertices;
	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		Algo::TransformIf(Node->Interface.Outputs, Vertices,
			[this, &InNodeID](const FMetasoundFrontendVertex& Vertex)
			{
				return IsNodeOutputConnectionUserModifiable(InNodeID, Vertex.VertexID);
			},
			[](const FMetasoundFrontendVertex& Vertex)
			{
				return &Vertex;
			}
		);
	}

	return Vertices;
}

const FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindNode(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindNode(InNodeID);
}

const TConstStructView<FMetaSoundFrontendNodeConfiguration> FMetaSoundFrontendDocumentBuilder::FindNodeConfiguration(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		return Node->Configuration;
	}
	return TConstStructView<FMetaSoundFrontendNodeConfiguration>();
}

TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FMetaSoundFrontendDocumentBuilder::FindNodeConfiguration(const FGuid& InNodeID, const FGuid* InPageID) 
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		return Node->Configuration;
	}
	return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
}

bool FMetaSoundFrontendDocumentBuilder::ShouldReplaceExistingNodeConfig(TConstStructView<FMetaSoundFrontendNodeConfiguration> InRegisteredNodeConfig, TConstStructView<FMetaSoundFrontendNodeConfiguration> InExistingConfig) const
{
	// The existing node configuration is not always replaced. Replacing
	// the node configuration struct loses any modified state on the existing
	// node config. In the scenario where the node configurations point to
	// the same derived class type, we do not replace them. 
	return InRegisteredNodeConfig.GetScriptStruct() != InExistingConfig.GetScriptStruct();
}

const FMetasoundFrontendClassInterface& FMetaSoundFrontendDocumentBuilder::GetApplicableRegistryInterface(const FMetasoundFrontendClass& InRegisteredClass, const FNodeConfigurationUpdateData& InNodeConfigurationUpdates) const
{
	// Check for the latest class interface overrides
	if (InNodeConfigurationUpdates.bDidUpdateClassInterfaceOverride)
	{
		if (const FMetasoundFrontendClassInterface* Interface = InNodeConfigurationUpdates.RegeneratedClassInterfaceOverride.GetPtr())
		{
			return *Interface;
		}
	}
	else
	{
		// The class interface override wasn't updated. Use the original if it exists. 
		if (const FMetasoundFrontendClassInterface* Interface = InNodeConfigurationUpdates.ExistingClassInterfaceOverride.GetPtr())
		{
			return *Interface;
		}
	}

	// If the override is invalid, use the interface on the class.
	return InRegisteredClass.GetDefaultInterface();
}

void FMetaSoundFrontendDocumentBuilder::FindNodeConfigurationUpdates(const FGuid& InNodeID, const FGuid& InPageID, const FMetasoundFrontendClass& InRegisteredClass, FNodeConfigurationUpdateData& OutNodeConfigurationUpdates, bool bInForceRegenerateClassInterfaceOverride) const
{
	using namespace Metasound::Frontend;

	OutNodeConfigurationUpdates.ExistingConfig = FindNodeConfiguration(InNodeID, &InPageID);
	OutNodeConfigurationUpdates.ExistingClassInterfaceOverride = FindClassInterfaceOverride(InNodeID, &InPageID);

	OutNodeConfigurationUpdates.RegisteredConfig = CreateFrontendNodeConfiguration(InRegisteredClass.Metadata);

	// Determine which node config will be used to create the class interface override;
	const bool bWillReplaceNodeConfig = ShouldReplaceExistingNodeConfig(OutNodeConfigurationUpdates.RegisteredConfig, OutNodeConfigurationUpdates.ExistingConfig);
	TConstStructView<FMetaSoundFrontendNodeConfiguration> EffectiveNodeConfig = bWillReplaceNodeConfig ? OutNodeConfigurationUpdates.RegisteredConfig : OutNodeConfigurationUpdates.ExistingConfig;

	// Update the node config
	if (bWillReplaceNodeConfig || bInForceRegenerateClassInterfaceOverride)
	{
		// Set this flag to true even if the effective node configuration pointer 
		// is invalid. If the effective node configuration is null, this captures
		// the fact that a null node configuration should produce an empty 
		// ClassInterfaceOverride. 
		OutNodeConfigurationUpdates.bDidUpdateClassInterfaceOverride = true;
		if (const FMetaSoundFrontendNodeConfiguration* ConfigPtr = EffectiveNodeConfig.GetPtr())
		{
			OutNodeConfigurationUpdates.RegeneratedClassInterfaceOverride = ConfigPtr->OverrideDefaultInterface(InRegisteredClass);
		}
	}
}

const int32* FMetaSoundFrontendDocumentBuilder::FindNodeIndex(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindNodeIndex(InNodeID);
}

bool FMetaSoundFrontendDocumentBuilder::FindNodeClassInterfaces(const FGuid& InNodeID, TSet<FMetasoundFrontendVersion>& OutInterfaces, const FGuid& InPageID) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(InPageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		if (const FMetasoundFrontendClass* NodeClass = DocumentCache->FindDependency(Node->ClassID))
		{
			const FNodeRegistryKey NodeClassRegistryKey = FNodeRegistryKey(NodeClass->Metadata);
			return INodeClassRegistry::Get()->FindImplementedInterfacesFromRegistered(NodeClassRegistryKey, OutInterfaces);
		}
	}

	return false;
}

const FMetasoundFrontendVertex* FMetaSoundFrontendDocumentBuilder::FindNodeInput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindInputVertex(InNodeID, InVertexID);
}

const FMetasoundFrontendVertex* FMetaSoundFrontendDocumentBuilder::FindNodeInput(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindInputVertex(InNodeID, InVertexName);
}

const TArray<FMetasoundFrontendClassInputDefault>* FMetaSoundFrontendDocumentBuilder::FindNodeClassInputDefaults(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID))
		{
			const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
			switch (ClassType)
			{
				case EMetasoundFrontendClassType::External:
				{
					auto MatchesName = [&InVertexName](const FMetasoundFrontendClassInput& Input) { return Input.Name == InVertexName; };
					const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(*Node);
					if (const FMetasoundFrontendClassInput* Input = ClassInterface.Inputs.FindByPredicate(MatchesName))
					{
						return &Input->GetDefaults();
					}
				}
				break;

				case EMetasoundFrontendClassType::Input:
				case EMetasoundFrontendClassType::Output:
				case EMetasoundFrontendClassType::Literal:
				{
					return &Class->GetInterfaceForNode(*Node).Inputs.Last().GetDefaults();
				}

				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					using namespace VariableNames;
					auto IsDataInput = [](const FMetasoundFrontendClassInput& Input) { return Input.Name == METASOUND_GET_PARAM_NAME(InputData); };

					const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(*Node);
					if (const FMetasoundFrontendClassInput* Input = ClassInterface.Inputs.FindByPredicate(IsDataInput))
					{
						return &Input->GetDefaults();
					}
				}
				break;

				case EMetasoundFrontendClassType::Template:
				{
					const Frontend::FNodeRegistryKey Key = Frontend::FNodeRegistryKey(Class->Metadata);
					const Frontend::INodeTemplate* Template = Frontend::INodeTemplateRegistry::Get().FindTemplate(Key);
					check(Template);
					const FGuid PageID = InPageID ? *InPageID : BuildPageID;
					return Template->FindNodeClassInputDefaults(*this, PageID, InNodeID, InVertexName);
				}

				case EMetasoundFrontendClassType::Graph:
				case EMetasoundFrontendClassType::Invalid:
				default:
				{
					checkNoEntry();
				}
				break;
			}
		}
	}

	return nullptr;
}

const FMetasoundFrontendVertexLiteral* FMetaSoundFrontendDocumentBuilder::FindNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		auto VertexLiteralMatchesID = [&InVertexID](const FMetasoundFrontendVertexLiteral& VertexLiteral)
		{
			return VertexLiteral.VertexID == InVertexID;
		};
		return Node->InputLiterals.FindByPredicate(VertexLiteralMatchesID);
	}

	return nullptr;
}

const FMetasoundFrontendVertexLiteral* FMetaSoundFrontendDocumentBuilder::FindNodeInputDefault(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	if (const FMetasoundFrontendVertex* Vertex = FindNodeInput(InNodeID, InVertexName, InPageID))
	{
		return FindNodeInputDefault(InNodeID, Vertex->VertexID, InPageID);
	}

	return nullptr;
}

TArray<const FMetasoundFrontendVertex*> FMetaSoundFrontendDocumentBuilder::FindNodeInputs(const FGuid& InNodeID, FName TypeName, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return DocumentCache->GetNodeCache(PageID).FindNodeInputs(InNodeID, TypeName);
}

TArray<const FMetasoundFrontendVertex*> FMetaSoundFrontendDocumentBuilder::FindNodeInputsConnectedToNodeOutput(const FGuid& InOutputNodeID, const FGuid& InOutputVertexID, TArray<const FMetasoundFrontendNode*>* ConnectedInputNodes, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);

	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();

	if (ConnectedInputNodes)
	{
		ConnectedInputNodes->Reset();
	}

	TArray<const FMetasoundFrontendVertex*> Inputs;
	const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
	const TArrayView<const int32> Indices = EdgeCache.FindEdgeIndicesFromNodeOutput(InOutputNodeID, InOutputVertexID);
	Algo::Transform(Indices, Inputs, [&Graph, &Document, &NodeCache, &ConnectedInputNodes](const int32& Index)
	{
		const FMetasoundFrontendEdge& Edge = Graph.Edges[Index];
		if (ConnectedInputNodes)
		{
			ConnectedInputNodes->Add(NodeCache.FindNode(Edge.ToNodeID));
		}
		return NodeCache.FindInputVertex(Edge.ToNodeID, Edge.ToVertexID);
	});
	return Inputs;
}

FMetasoundFrontendNode* FMetaSoundFrontendDocumentBuilder::FindNodeInternal(const FGuid& InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
		return &Graph.Nodes[*NodeIndex];
	}

	return nullptr;
}

const FMetasoundFrontendVertex* FMetaSoundFrontendDocumentBuilder::FindNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindOutputVertex(InNodeID, InVertexID);
}

const FMetasoundFrontendVertex* FMetaSoundFrontendDocumentBuilder::FindNodeOutput(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	return NodeCache.FindOutputVertex(InNodeID, InVertexName);
}

TArray<const FMetasoundFrontendVertex*> FMetaSoundFrontendDocumentBuilder::FindNodeOutputs(const FGuid& InNodeID, FName TypeName, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return DocumentCache->GetNodeCache(PageID).FindNodeOutputs(InNodeID, TypeName);
}

const FMetasoundFrontendVertex* FMetaSoundFrontendDocumentBuilder::FindNodeOutputConnectedToNodeInput(const FGuid& InInputNodeID, const FGuid& InInputVertexID, const FMetasoundFrontendNode** ConnectedOutputNode, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
	if (const int32* Index = EdgeCache.FindEdgeIndexToNodeInput(InInputNodeID, InInputVertexID))
	{
		const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
		const FMetasoundFrontendEdge& Edge = Document.RootGraph.FindConstGraphChecked(PageID).Edges[*Index];
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
		if (ConnectedOutputNode)
		{
			(*ConnectedOutputNode) = NodeCache.FindNode(Edge.FromNodeID);
		}
		return NodeCache.FindOutputVertex(Edge.FromNodeID, Edge.FromVertexID);
	}

	if (ConnectedOutputNode)
	{
		*ConnectedOutputNode = nullptr;
	}
	return nullptr;
}

int32 FMetaSoundFrontendDocumentBuilder::FindGraphPageIndex(const FGuid& InPageID) const
{
	const FMetasoundFrontendDocument& Document = GetDocumentChecked();
	const TArray<FMetasoundFrontendGraph>& GraphPages = Document.RootGraph.GetConstGraphPages();
	auto MatchesPageID = [this, &InPageID](const FMetasoundFrontendGraph& Iter) { return Iter.PageID == InPageID; };
	return GraphPages.IndexOfByPredicate(MatchesPageID);
}

#if WITH_EDITORONLY_DATA
FMetaSoundFrontendGraphComment& FMetaSoundFrontendDocumentBuilder::FindOrAddGraphComment(const FGuid& InCommentID, const FGuid* InPageID)
{
	check(InCommentID.IsValid());
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TMap<FGuid, FMetaSoundFrontendGraphComment>& Comments = Document.RootGraph.FindGraphChecked(PageID).Style.Comments;
	return Comments.FindOrAdd(InCommentID);
}
#endif // WITH_EDITORONLY_DATA

FMetasoundFrontendClassName FMetaSoundFrontendDocumentBuilder::GenerateNewClassName()
{
	using namespace Metasound::Frontend;
	FMetasoundFrontendClassMetadata& Metadata = GetDocumentChecked().RootGraph.Metadata;
	const FMetasoundFrontendClassName NewClassName(FName(), FName(*FGuid::NewGuid().ToString()), FName());
	Metadata.SetClassName(NewClassName);
	return NewClassName;
}

FName FMetaSoundFrontendDocumentBuilder::GenerateUniqueInputName(const FString& InBaseName) const
{
	using namespace Metasound::Frontend;

	TSet<FName> ExistingNames;
	auto GetName = [&](const FMetasoundFrontendClass&, const FMetasoundFrontendNode& Node) { ExistingNames.Add(Node.Name); };
	IterateNodesByClassType(GetName, EMetasoundFrontendClassType::Input);
	return DocumentBuilderPrivate::GenerateUniqueName(ExistingNames, InBaseName);
}

FName FMetaSoundFrontendDocumentBuilder::GenerateUniqueOutputName(const FString& InBaseName) const
{
	using namespace Metasound::Frontend;

	TSet<FName> ExistingNames;
	auto GetName = [&](const FMetasoundFrontendClass&, const FMetasoundFrontendNode& Node) { ExistingNames.Add(Node.Name); };
	IterateNodesByClassType(GetName, EMetasoundFrontendClassType::Output);
	return DocumentBuilderPrivate::GenerateUniqueName(ExistingNames, InBaseName);
}

FName FMetaSoundFrontendDocumentBuilder::GenerateUniqueVariableName(const FString& InBaseName) const
{
	using namespace Metasound::Frontend;

	TSet<FName> ExistingVariableNames;
	Algo::Transform(FindConstBuildGraphChecked().Variables, ExistingVariableNames, [](const FMetasoundFrontendVariable& Var) { return Var.Name; });
	return DocumentBuilderPrivate::GenerateUniqueName(ExistingVariableNames, InBaseName);
}

const FTopLevelAssetPath FMetaSoundFrontendDocumentBuilder::GetBuilderClassPath() const
{
	IMetaSoundDocumentInterface* Interface = DocumentInterface.GetInterface();
	checkf(Interface, TEXT("Failed to return class path; interface must always be valid while builder is operating on MetaSound UObject!"));
	return Interface->GetBaseMetaSoundUClass().GetClassPathName();
}

const FMetasoundFrontendDocument& FMetaSoundFrontendDocumentBuilder::GetConstDocumentChecked() const
{
	return GetConstDocumentInterfaceChecked().GetConstDocument();
}

const IMetaSoundDocumentInterface& FMetaSoundFrontendDocumentBuilder::GetConstDocumentInterfaceChecked() const
{
	const IMetaSoundDocumentInterface* Interface = DocumentInterface.GetInterface();
	checkf(Interface, TEXT("Failed to return document; interface must always be valid while builder is operating on MetaSound UObject! Builder constructed with asset at %s"), *HintPath.ToString());
	return *Interface;
}

const FString FMetaSoundFrontendDocumentBuilder::GetDebugName() const
{
	using namespace Metasound::Frontend;

	UObject& MetaSoundObject = CastDocumentObjectChecked<UObject>();
	return MetaSoundObject.GetPathName();
}

FMetasoundFrontendDocument& FMetaSoundFrontendDocumentBuilder::GetDocumentChecked() const
{
	return GetDocumentInterfaceChecked().GetDocument();
}

Metasound::Frontend::FDocumentModifyDelegates& FMetaSoundFrontendDocumentBuilder::GetDocumentDelegates()
{
	return *DocumentDelegates;
}

IMetaSoundDocumentInterface& FMetaSoundFrontendDocumentBuilder::GetDocumentInterfaceChecked() const
{
	IMetaSoundDocumentInterface* Interface = DocumentInterface.GetInterface();
	checkf(Interface, TEXT("Failed to return document; interface must always be valid while builder is operating on MetaSound UObject! Builder constructed with asset at %s"), *HintPath.ToString());
	return *Interface;
}

TArray<const FMetasoundFrontendNode*> FMetaSoundFrontendDocumentBuilder::GetGraphInputTemplateNodes(FName InInputName, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	TArray<const FMetasoundFrontendNode*> TemplateNodes;

	const FGuid PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendGraphClass& RootGraph = GetDocumentChecked().RootGraph;
	if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InInputName))
	{
		const FMetasoundFrontendClassInput& InputClass = RootGraph.GetDefaultInterface().Inputs[*Index];
		const FMetasoundFrontendGraph& Graph = RootGraph.FindConstGraphChecked(PageID);
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
		const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);

		if (const FMetasoundFrontendNode* InputNode = NodeCache.FindNode(InputClass.NodeID))
		{
			const FGuid OutputVertexID = InputNode->Interface.Outputs.Last().VertexID;
			const TArray<const FMetasoundFrontendEdge*> ConnectedEdges = EdgeCache.FindEdges(InputClass.NodeID, OutputVertexID);
			for (const FMetasoundFrontendEdge* Edge : ConnectedEdges)
			{
				check(Edge);
				if (const int32* ConnectedNodeIndex = NodeCache.FindNodeIndex(Edge->ToNodeID))
				{
					const FMetasoundFrontendNode& ConnectedNode = Graph.Nodes[*ConnectedNodeIndex];
					if (const FMetasoundFrontendClass* ConnectedNodeClass = FindDependency(ConnectedNode.ClassID))
					{
						if (ConnectedNodeClass->Metadata.GetClassName() == FInputNodeTemplate::ClassName)
						{
							TemplateNodes.Add(&ConnectedNode);
						}
					}
				}
			}
		}
	}

	return TemplateNodes;
}

const TSet<FName>* FMetaSoundFrontendDocumentBuilder::GetGraphInputsInheritingDefault() const
{
	return nullptr;
}

const FTopLevelAssetPath& FMetaSoundFrontendDocumentBuilder::GetHintPath() const
{
	return HintPath;
}

FMetasoundAssetBase& FMetaSoundFrontendDocumentBuilder::GetMetasoundAsset() const
{
	using namespace Metasound::Frontend;

	UObject* Object = DocumentInterface.GetObject();
	check(Object);
	FMetasoundAssetBase* Asset = IMetaSoundAssetManager::GetChecked().GetAsAsset(*Object);
	check(Asset);
	return *Asset;
}

FMetasoundAssetBase* FMetaSoundFrontendDocumentBuilder::GetReferencedPresetAsset() const
{
	using namespace Metasound::Frontend;

	if (const FMetaSoundFrontendPresetTemplate* PresetTemplate = GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>())
	{
		if (UObject* Parent = PresetTemplate->Parent.GetObject())
		{
			return IMetaSoundAssetManager::GetChecked().GetAsAsset(*Parent);
		}
	}

	return nullptr;
}

const FGuid& FMetaSoundFrontendDocumentBuilder::GetBuildPageID() const
{
	return BuildPageID;
}

const FMetasoundFrontendLiteral* FMetaSoundFrontendDocumentBuilder::GetGraphInputDefault(FName InputName, const FGuid* InPageID) const
{
	if (const FMetasoundFrontendClassInput* GraphInput = FindGraphInput(InputName))
	{
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		return GraphInput->FindConstDefault(PageID);
	}
	return nullptr;
}

const FMetasoundFrontendLiteral* FMetaSoundFrontendDocumentBuilder::GetGraphVariableDefault(FName VariableName, const FGuid* InPageID) const
{
	if (const FMetasoundFrontendVariable* Variable = FindGraphVariable(VariableName, InPageID))
	{
		return &Variable->Literal;
	}
	return nullptr;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::GetIsGraphEditable(const FGuid* PageID) const
{
	bool bIsEditable = true;
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	Document.RootGraph.IterateGraphPages([&bIsEditable, &PageID](const FMetasoundFrontendGraph& PageGraph)
	{
		if (!PageID || PageGraph.PageID == *PageID)
		{
			bIsEditable &= PageGraph.Style.bIsGraphEditable;
		}
	});
	return bIsEditable;
}
#endif // WITH_EDITORONLY_DATA

EMetasoundFrontendVertexAccessType FMetaSoundFrontendDocumentBuilder::GetNodeInputAccessType(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		const FMetasoundFrontendGraph& Graph = GetConstDocumentChecked().RootGraph.FindConstGraphChecked(PageID);
		const FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
		auto IsVertexID = [&InVertexID](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == InVertexID; };
		if (const FMetasoundFrontendClass* Class = DocumentCache->FindDependency(Node.ClassID))
		{
			const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
			switch (ClassType)
			{
				case EMetasoundFrontendClassType::Template:
				{
					const FNodeRegistryKey Key = FNodeRegistryKey(Class->Metadata);
					const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
					if (ensureMsgf(Template, TEXT("Failed to find MetaSound node template registered with key '%s'"), *Key.ToString()))
					{
						if (Template->IsInputAccessTypeDynamic())
						{
							return Template->GetNodeInputAccessType(*this, PageID, InNodeID, InVertexID);
						}
					}
				}
				break;

				case EMetasoundFrontendClassType::Output:
				{
					const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(Node);
					const FMetasoundFrontendClassInput& ClassInput = ClassInterface.Inputs.Last();
					return ClassInput.AccessType;
				}

				default:
				break;
			}
			static_assert(static_cast<uint32>(EMetasoundFrontendClassType::Invalid) == 10, "Potential missing case coverage for EMetasoundFrontendClassType");

			if (const FMetasoundFrontendVertex* Vertex = Node.Interface.Inputs.FindByPredicate(IsVertexID))
			{
				auto IsClassInput = [VertexName = Vertex->Name](const FMetasoundFrontendClassInput& Input) { return Input.Name == VertexName; };
				const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(Node);
				if (const FMetasoundFrontendClassInput* ClassInput = ClassInterface.Inputs.FindByPredicate(IsClassInput))
				{
					return ClassInput->AccessType;
				}
			}
		}
	}

	return EMetasoundFrontendVertexAccessType::Unset;
}

#if WITH_EDITORONLY_DATA
FText FMetaSoundFrontendDocumentBuilder::GetNodeInputDisplayName(const FGuid& InNodeID, const FName VertexName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID);
	if (!Node)
	{
		return { };
	}

	const FMetasoundFrontendClass* Class = DocumentCache->FindDependency(Node->ClassID);
	if (!Class)
	{
		return { };
	}

	const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
	switch (ClassType)
	{
		case EMetasoundFrontendClassType::Input:
		case EMetasoundFrontendClassType::Output:
		{
			return FText::FromName(Node->Name);
		}

		case EMetasoundFrontendClassType::Variable:
		case EMetasoundFrontendClassType::VariableAccessor:
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
		case EMetasoundFrontendClassType::VariableMutator:
		{
			return FText::FromName(METASOUND_GET_PARAM_NAME(InputData));
		}

		case EMetasoundFrontendClassType::Literal:
		case EMetasoundFrontendClassType::External:
		{
			auto MatchesName = [&VertexName](const FMetasoundFrontendClassVertex& OtherVertex) { return OtherVertex.Name == VertexName; };
			const FMetasoundFrontendClassVertex* ClassVertex = Class->GetInterfaceForNode(*Node).Inputs.FindByPredicate(MatchesName);
			if (ClassVertex)
			{
				FName Namespace, ParamName;
				ClassVertex->SplitName(Namespace, ParamName);
				const FText DisplayName = ClassVertex->Metadata.GetDisplayName();
				if (DisplayName.IsEmptyOrWhitespace())
				{
					if (Namespace.IsNone())
					{
						return FText::FromName(ParamName);
					}
					else
					{
						return FText::Format(NSLOCTEXT("MetaSoundFrontend", "ClassMetadataDisplayNameWithNamespaceFormat", "{0} ({1})"), FText::FromName(ParamName), FText::FromName(Namespace));
					}
				}

				return DisplayName;
			}
		}
		break;

		case EMetasoundFrontendClassType::Template:
		{
			const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Class->Metadata.GetClassName());
			if (ensure(Template))
			{
				return Template->GetInputVertexDisplayName(*this, PageID, InNodeID, VertexName);
			}
		}
		break;

		case EMetasoundFrontendClassType::Graph:
		case EMetasoundFrontendClassType::Invalid:
		default:
		{
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing EMetasoundFrontendClassType case coverage");
		}
		break;
	}

	return FText::FromName(VertexName);
}

FText FMetaSoundFrontendDocumentBuilder::GetNodeOutputDisplayName(const FGuid& InNodeID, const FName VertexName, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID);
	if (!Node)
	{
		return { };
	}

	const FMetasoundFrontendClass* Class = DocumentCache->FindDependency(Node->ClassID);
	if (!Class)
	{
		return { };
	}

	const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
	switch (ClassType)
	{
		case EMetasoundFrontendClassType::Input:
		case EMetasoundFrontendClassType::Output:
		{
			return FText::FromName(Node->Name);
		}

		case EMetasoundFrontendClassType::Variable:
		case EMetasoundFrontendClassType::VariableAccessor:
		case EMetasoundFrontendClassType::VariableDeferredAccessor:
		case EMetasoundFrontendClassType::VariableMutator:
		{
			return FText::FromName(METASOUND_GET_PARAM_NAME(OutputData));
		}

		case EMetasoundFrontendClassType::Literal:
		case EMetasoundFrontendClassType::External:
		{
			auto PinMatchesClassVertex = [&VertexName](const FMetasoundFrontendClassVertex& OtherVertex) { return OtherVertex.Name == VertexName; };
			const FMetasoundFrontendClassVertex* ClassVertex = Class->GetInterfaceForNode(*Node).Outputs.FindByPredicate(PinMatchesClassVertex);
			if (ClassVertex)
			{
				FName Namespace, ParamName;
				ClassVertex->SplitName(Namespace, ParamName);
				const FText DisplayName = ClassVertex->Metadata.GetDisplayName();
				if (DisplayName.IsEmptyOrWhitespace())
				{
					if (Namespace.IsNone())
					{
						return FText::FromName(ParamName);
					}
					else
					{
						return FText::Format(NSLOCTEXT("MetaSoundFrontend", "ClassMetadataDisplayNameWithNamespaceFormat", "{0} ({1})"), FText::FromName(ParamName), FText::FromName(Namespace));
					}
				}

				return DisplayName;
			}
		}
		break;

		case EMetasoundFrontendClassType::Template:
		{
			const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Class->Metadata.GetClassName());
			if (ensure(Template))
			{
				return Template->GetOutputVertexDisplayName(*this, PageID, InNodeID, VertexName);
			}
		}
		break;

		case EMetasoundFrontendClassType::Graph:
		case EMetasoundFrontendClassType::Invalid:
		default:
		{
			static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing EMetasoundFrontendClassType case coverage");
		}
		break;
	}

	return FText::FromName(VertexName);
}
#endif // WITH_EDITORONLY_DATA

EMetasoundFrontendVertexAccessType FMetaSoundFrontendDocumentBuilder::GetNodeOutputAccessType(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		const FMetasoundFrontendGraph& Graph = GetConstDocumentChecked().RootGraph.FindConstGraphChecked(PageID);
		const FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
		if (const FMetasoundFrontendClass* Class = DocumentCache->FindDependency(Node.ClassID))
		{
			const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
			switch (ClassType)
			{
				case EMetasoundFrontendClassType::Template:
				{
					const FNodeRegistryKey Key = FNodeRegistryKey(Class->Metadata);
					const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
					if (ensureMsgf(Template, TEXT("Failed to find MetaSound node template registered with key '%s'"), *Key.ToString()))
					{
						if (Template->IsOutputAccessTypeDynamic())
						{
							return Template->GetNodeOutputAccessType(*this, PageID, InNodeID, InVertexID);
						}
					}
				}
				break;

				case EMetasoundFrontendClassType::Input:
				{
					const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(Node);
					const FMetasoundFrontendClassOutput& ClassOutput = ClassInterface.Outputs.Last();
					return ClassOutput.AccessType;
				}

				default:
				break;
			}
			static_assert(static_cast<uint32>(EMetasoundFrontendClassType::Invalid) == 10, "Potential missing case coverage for EMetasoundFrontendClassType");

			auto IsVertexID = [&InVertexID](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == InVertexID; };
			if (const FMetasoundFrontendVertex* Vertex = Node.Interface.Outputs.FindByPredicate(IsVertexID))
			{
				auto IsClassInput = [VertexName = Vertex->Name](const FMetasoundFrontendClassInput& Output) { return Output.Name == VertexName; };
				const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(Node);
				if (const FMetasoundFrontendClassOutput* ClassOutput = ClassInterface.Outputs.FindByPredicate(IsClassInput))
				{
					return ClassOutput->AccessType;
				}
			}
		}
	}

	return EMetasoundFrontendVertexAccessType::Unset;
}

#if WITH_EDITORONLY_DATA
const bool FMetaSoundFrontendDocumentBuilder::GetIsAdvancedDisplay(const FName MemberName, const EMetasoundFrontendClassType Type) const
{
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();

	//Input
	if (Type == EMetasoundFrontendClassType::Input)
	{
		if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(MemberName))
		{
			const FMetasoundFrontendClassInput& GraphInput = Document.RootGraph.GetDefaultInterface().Inputs[*Index];
			return GraphInput.Metadata.bIsAdvancedDisplay;
		}
	}
	//Output
	else if (Type == EMetasoundFrontendClassType::Output)
	{
		if (const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(MemberName))
		{
			const FMetasoundFrontendClassOutput& GraphOutput = Document.RootGraph.GetDefaultInterface().Outputs[*Index];
			return GraphOutput.Metadata.bIsAdvancedDisplay;
		}
	}
	return false;
}
#endif // WITH_EDITORONLY_DATA

void FMetaSoundFrontendDocumentBuilder::InitDocument(const FMetasoundFrontendDocument* InDocumentTemplate, const FMetasoundFrontendClassName* InNewClassName, bool bResetVersion)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::InitDocument);

	FMetasoundFrontendDocument& Document = GetDocumentChecked();

	// If template provided, copy & initialize from that.
	if (InDocumentTemplate)
	{
#if WITH_EDITORONLY_DATA
		// Ensure member metadata is parented to new document owner
		UObject* Parent = DocumentInterface.GetObject();
		check(Parent);
		TMap<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>> NewMetadata;
		for (const TPair<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>& Pair : InDocumentTemplate->Metadata.MemberMetadata)
		{
			NewMetadata.Add(Pair.Key, NewObject<UMetaSoundFrontendMemberMetadata>(Parent, Pair.Value->GetClass(), { }, { }, Pair.Value.Get()));
		}
#endif // WITH_EDITORONLY_DATA

		Document = *InDocumentTemplate;

#if WITH_EDITORONLY_DATA
		Document.Metadata.MemberMetadata = MoveTemp(NewMetadata);
#endif // WITH_EDITORONLY_DATA

		if (Document.RootGraph.GetConstGraphPages().IsEmpty())
		{
			Document.RootGraph.InitDefaultGraphPage();
			DocumentDelegates->AddPageDelegates(Frontend::DefaultPageID);
		}

		InitGraphClassMetadata(bResetVersion, InNewClassName);
	}
	// Initialize class using default data
	else
	{
		// Initialize class using default data
		if (Document.RootGraph.GetConstGraphPages().IsEmpty())
		{
			Document.RootGraph.InitDefaultGraphPage();
			DocumentDelegates->AddPageDelegates(Frontend::DefaultPageID);
		}

		InitGraphClassMetadata(bResetVersion, InNewClassName);

#if WITH_EDITORONLY_DATA
		// Set default doc version Metadata
		{
			FMetasoundFrontendDocumentMetadata& DocMetadata = Document.Metadata;
			DocMetadata.Version.Number = FVersioningManager::GetMaxDocumentVersion();
		}
#endif // WITH_EDITORONLY_DATA

		// Add default interfaces for given UClass
		{
			TArray<FMetasoundFrontendVersion> InitVersions = ISearchEngine::Get().FindUClassDefaultInterfaceVersions(GetBuilderClassPath());
			FModifyInterfaceOptions Options({ }, MoveTemp(InitVersions));
			ModifyInterfaces(MoveTemp(Options));
		}

		// Set default access flags as provided by document object class
		SetAccessFlags(DocumentInterface->GetDefaultAccessFlags());
	}
}

bool FMetaSoundFrontendDocumentBuilder::IsValid() const
{
	return DocumentInterface.GetObject() != nullptr;
}

int32 FMetaSoundFrontendDocumentBuilder::GetTransactionCount() const
{
	using namespace Metasound::Frontend;

	if (DocumentCache.IsValid())
	{
		return StaticCastSharedPtr<FDocumentCache>(DocumentCache)->GetTransactionCount();
	}

	return 0;
}

void FMetaSoundFrontendDocumentBuilder::InitGraphClassMetadata(FMetasoundFrontendClassMetadata& InOutMetadata, bool bResetVersion, const FMetasoundFrontendClassName* NewClassName)
{
	if (NewClassName)
	{
		InOutMetadata.SetClassName(*NewClassName);
	}
	else
	{
		InOutMetadata.SetClassName(FMetasoundFrontendClassName(FName(), *FGuid::NewGuid().ToString(), FName()));
	}

	if (bResetVersion)
	{
		InOutMetadata.SetVersion({ 1, 0 });
	}

	InOutMetadata.SetType(EMetasoundFrontendClassType::Graph);
}

void FMetaSoundFrontendDocumentBuilder::InitGraphClassMetadata(bool bResetVersion, const FMetasoundFrontendClassName* NewClassName)
{
	FMetasoundFrontendClassMetadata& ThisMetadata = GetDocumentChecked().RootGraph.Metadata;

#if DO_CHECK
	using namespace Metasound::Frontend;
	if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
	{
		TArray<FMetaSoundFrontendDocumentBuilder*> RegisteredBuilders = BuilderRegistry->FindBuilders(ThisMetadata.GetClassName());
		auto IsThis = [this](const FMetaSoundFrontendDocumentBuilder* Builder)
		{
			checkf(Builder, TEXT("FindBuilders should always return valid pointers to builders"));
			return this == Builder;
		};
		checkf(!Algo::AnyOf(RegisteredBuilders, IsThis),
			TEXT("Renaming graph class will result in duplicate builder entries in the builder registry. "
				"Builder(s) '%s' must be unregistered prior to updating class name"),
			*FString::JoinBy(RegisteredBuilders, TEXT(", "), [](const FMetaSoundFrontendDocumentBuilder* RegisteredBuilder)
			{
				return RegisteredBuilder->GetDebugName();
			}));
	}
#endif // DO_CHECK

	InitGraphClassMetadata(ThisMetadata, bResetVersion, NewClassName);
}

void FMetaSoundFrontendDocumentBuilder::InitNodeLocations()
{
#if WITH_EDITORONLY_DATA
	using namespace Metasound;
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
		FVector2D InputNodeLocation = FVector2D::ZeroVector;
		FVector2D ExternalNodeLocation = InputNodeLocation + DisplayStyle::NodeLayout::DefaultOffsetX;
		FVector2D OutputNodeLocation = ExternalNodeLocation + DisplayStyle::NodeLayout::DefaultOffsetX;

		TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
		for (FMetasoundFrontendNode& Node : Nodes)
		{
			if (const int32* ClassIndex = DocumentCache->FindDependencyIndex(Node.ClassID))
			{
				FMetasoundFrontendClass& Class = Document.Dependencies[*ClassIndex];

				const EMetasoundFrontendClassType NodeType = Class.Metadata.GetType();
				FVector2D NewLocation;
				if (NodeType == EMetasoundFrontendClassType::Input)
				{
					NewLocation = InputNodeLocation;
					InputNodeLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
				}
				else if (NodeType == EMetasoundFrontendClassType::Output)
				{
					NewLocation = OutputNodeLocation;
					OutputNodeLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
				}
				else
				{
					NewLocation = ExternalNodeLocation;
					ExternalNodeLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
				}

				// TODO: Find consistent location for controlling node locations.
				// Currently it is split between MetasoundEditor and MetasoundFrontend modules.
				FMetasoundFrontendNodeStyle& Style = Node.Style;
				if (Style.Display.Locations.IsEmpty())
				{
					Style.Display.Locations = { { FGuid::NewGuid(), NewLocation } };
				}
				// Initialize the position if the location hasn't been assigned yet.  This can happen
				// if default interfaces were assigned to the given MetaSound but not placed with respect
				// to one another.  In this case, node location initialization takes "priority" to avoid
				// visual overlap.
				else if (Style.Display.Locations.Num() == 1 && Style.Display.Locations.Contains(FGuid()))
				{
					Style.Display.Locations = { { FGuid::NewGuid(), NewLocation } };
				}
			}
		}
	});
#endif // WITH_EDITORONLY_DATA
}

bool FMetaSoundFrontendDocumentBuilder::IsConfiguringDocument() const
{
	return bInTemplateScope;
}

bool FMetaSoundFrontendDocumentBuilder::IsDependencyReferenced(const FGuid& InClassID) const
{
	bool bIsReferenced = false;
	GetConstDocumentChecked().RootGraph.IterateGraphPages([this, &InClassID, &bIsReferenced](const FMetasoundFrontendGraph& Graph)
	{
		using namespace Metasound::Frontend;
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
		bIsReferenced |= NodeCache.ContainsNodesOfClassID(InClassID);
	});
	return bIsReferenced;
}

bool FMetaSoundFrontendDocumentBuilder::IsNodeInputConnected(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return DocumentCache->GetEdgeCache(PageID).IsNodeInputConnected(InNodeID, InVertexID);
}

bool FMetaSoundFrontendDocumentBuilder::IsNodeInputConnectionUserModifiable(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID))
		{
			switch (Class->Metadata.GetType())
			{
				case EMetasoundFrontendClassType::Template:
				{
					const FNodeClassRegistryKey Key(Class->Metadata);
					const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
					if (ensure(Template))
					{
						return Template->IsInputConnectionUserModifiable();
					}
				}
				break;

				case EMetasoundFrontendClassType::Input:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					if (const FMetasoundFrontendVertex* Vertex = NodeCache.FindInputVertex(InNodeID, InVertexID))
					{
						return Vertex->Name == METASOUND_GET_PARAM_NAME(InputData);
					}
					return false;
				}

				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				{
					return false;
				}

				default:
				break;
			}
		}
	}

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::IsNodeOutputConnected(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	return DocumentCache->GetEdgeCache(PageID).IsNodeOutputConnected(InNodeID, InVertexID);
}

bool FMetaSoundFrontendDocumentBuilder::IsNodeOutputConnectionUserModifiable(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID))
		{
			switch (Class->Metadata.GetType())
			{
				case EMetasoundFrontendClassType::Template:
				{
					const FNodeClassRegistryKey Key(Class->Metadata);
					const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
					if (ensure(Template))
					{
						return Template->IsOutputConnectionUserModifiable();
					}
				}
				break;

				case EMetasoundFrontendClassType::Output:
				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					return false;
				}

				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				{
					if (const FMetasoundFrontendVertex* Vertex = NodeCache.FindOutputVertex(InNodeID, InVertexID))
					{
						return Vertex->Name == METASOUND_GET_PARAM_NAME(OutputData);
					}
					return false;
				}

				default:
					break;
			}
		}
	}

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::IsInterfaceDeclared(FName InInterfaceName) const
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendVersion InterfaceVersion;
	if (ISearchEngine::Get().FindHighestInterfaceVersion(InInterfaceName, InterfaceVersion))
	{
		return IsInterfaceDeclared(InterfaceVersion);
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::IsInterfaceDeclared(const FMetasoundFrontendVersion& InInterfaceVersion) const
{
	return GetConstDocumentChecked().Interfaces.Contains(InInterfaceVersion);
}

bool FMetaSoundFrontendDocumentBuilder::IsPreset() const
{
	using namespace Metasound::Frontend;

	return IsDocumentConfiguredBy<FPresetTemplate>();
}

Metasound::Frontend::EInvalidEdgeReason FMetaSoundFrontendDocumentBuilder::IsValidEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);

	const FMetasoundFrontendVertex* OutputVertex = NodeCache.FindOutputVertex(InEdge.FromNodeID, InEdge.FromVertexID);
	if (!OutputVertex)
	{
		return EInvalidEdgeReason::MissingOutput;
	}

	const FMetasoundFrontendVertex* InputVertex = NodeCache.FindInputVertex(InEdge.ToNodeID, InEdge.ToVertexID);
	if (!InputVertex)
	{
		return EInvalidEdgeReason::MissingInput;
	}

	if (!Metasound::IsCastable(OutputVertex->TypeName, InputVertex->TypeName))
	{
		return EInvalidEdgeReason::MismatchedDataType;
	}

	// TODO: Add cycle detection here

	const EMetasoundFrontendVertexAccessType OutputAccessType = GetNodeOutputAccessType(InEdge.FromNodeID, InEdge.FromVertexID, InPageID);
	const EMetasoundFrontendVertexAccessType InputAccessType = GetNodeInputAccessType(InEdge.ToNodeID, InEdge.ToVertexID, InPageID);
	if (!FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(OutputAccessType, InputAccessType))
	{
		return EInvalidEdgeReason::MismatchedAccessType;
	}

	return EInvalidEdgeReason::None;
}

void FMetaSoundFrontendDocumentBuilder::IterateNodesConnectedWithVertex(const FMetasoundFrontendVertexHandle& Vertex, TFunctionRef<void(const FMetasoundFrontendEdge&, FMetasoundFrontendNode&)> NodeIndexIterFunc, const FGuid& InPageID)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(InPageID);
	TArray<FMetasoundFrontendEdge> EdgesToConnectedNodes; // Have to cache to avoid pointers becoming garbage in subsequent removal loop
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(InPageID);
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(InPageID);
	const TArray<const FMetasoundFrontendEdge*> Edges = EdgeCache.FindEdges(Vertex.NodeID, Vertex.VertexID);
	Algo::Transform(Edges, EdgesToConnectedNodes, [](const FMetasoundFrontendEdge* Edge) { check(Edge); return *Edge; });
	for (const FMetasoundFrontendEdge& Edge : EdgesToConnectedNodes)
	{
		const FGuid& ConnectedNodeID = Edge.ToNodeID == Vertex.NodeID ? Edge.FromNodeID : Edge.ToNodeID;
		if (const int32* ConnectedNodeIndex = NodeCache.FindNodeIndex(ConnectedNodeID))
		{
			FMetasoundFrontendNode& Node = Graph.Nodes[*ConnectedNodeIndex];
			NodeIndexIterFunc(Edge, Node);
		}
	}
}

void FMetaSoundFrontendDocumentBuilder::IterateNodes(Metasound::Frontend::FConstClassAndNodeFunctionRef Func, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Doc = GetConstDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Doc.RootGraph.FindConstGraphChecked(PageID);
	for (const FMetasoundFrontendNode& Node : Graph.Nodes)
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID))
		{
			Func(*Class, Node);
		}
	}
}

void FMetaSoundFrontendDocumentBuilder::IterateNodesByPredicate(Metasound::Frontend::FConstClassAndNodeAndPageIDFunctionRef Func, Metasound::Frontend::FNodePredicateFunctionRef PredicateFunc, const FGuid* InPageID, bool bIterateAllPages) const
{
	using namespace Metasound::Frontend;
	FMetasoundFrontendDocument& Doc = GetDocumentChecked();
	if (bIterateAllPages)
	{
		Doc.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
		{
			for (const FMetasoundFrontendNode& Node : Graph.Nodes)
			{
				if (PredicateFunc(Node))
				{
					if (const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID))
					{
						Func(*Class, Node, Graph.PageID);
					}
				}
			}
		});
	}
	else
	{
		const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
		const FMetasoundFrontendGraph& Graph = Doc.RootGraph.FindGraphChecked(PageID);

		for (const FMetasoundFrontendNode& Node : Graph.Nodes)
		{
			if (PredicateFunc(Node))
			{
				if (const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID))
				{
					Func(*Class, Node, Graph.PageID);
				}
			}
		}
	}
}

void FMetaSoundFrontendDocumentBuilder::IterateNodesByClassType(Metasound::Frontend::FConstClassAndNodeFunctionRef Func, EMetasoundFrontendClassType ClassType, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	check(ClassType != EMetasoundFrontendClassType::Invalid);

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const FMetasoundFrontendDocument& Doc = GetConstDocumentChecked();
	const FMetasoundFrontendGraph& Graph = Doc.RootGraph.FindConstGraphChecked(PageID);
	for (const FMetasoundFrontendNode& Node : Graph.Nodes)
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID))
		{
			if (Class->Metadata.GetType() == ClassType)
			{
				Func(*Class, Node);
			}
		}
	}
}

bool FMetaSoundFrontendDocumentBuilder::ModifyInterfaces(Metasound::Frontend::FModifyInterfaceOptions&& InOptions)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Doc = GetDocumentChecked();
	DocumentBuilderPrivate::FModifyInterfacesImpl Context(Doc, MoveTemp(InOptions));
	return Context.Execute(*this, *DocumentDelegates);
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::TransformTemplateNodes()
{
	using namespace Metasound::Frontend;

	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::TransformTemplateNodes);

	struct FTemplateTransformParams
	{
		const Metasound::Frontend::INodeTemplate* Template = nullptr;
		TArray<FGuid> NodeIDs;
	};
	using FTemplateTransformParamsMap = TSortedMap<FGuid, FTemplateTransformParams>;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClass>& Dependencies = Document.Dependencies;

	FTemplateTransformParamsMap TemplateParams;
	for (const FMetasoundFrontendClass& Dependency : Dependencies)
	{
		if (Dependency.Metadata.GetType() == EMetasoundFrontendClassType::Template)
		{
			const FNodeRegistryKey Key = FNodeRegistryKey(Dependency.Metadata);
			const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
			ensureMsgf(Template, TEXT("Template not found for template class reference '%s'"), *Dependency.Metadata.GetClassName().ToString());
			TemplateParams.Add(Dependency.ID, FTemplateTransformParams { Template });
		}
	}

	if (TemplateParams.IsEmpty())
	{
		return false;
	}

	// 1. Execute generated template node transform on copy of node array,
	// which allows for addition/removal of nodes to/from original array container
	// without template transform having to worry about mutation while iterating
	TArray<FGuid> TemplateNodeIDs;
	bool bModified = false;
	Document.RootGraph.IterateGraphPages([this, &Dependencies, &TemplateParams, &bModified](FMetasoundFrontendGraph& Graph)
	{
		for (const FMetasoundFrontendNode& Node : Graph.Nodes)
		{
			if (FTemplateTransformParams* Params = TemplateParams.Find(Node.ClassID))
			{
				Params->NodeIDs.Add(Node.GetID());
			}
		}

		for (TPair<FGuid, FTemplateTransformParams>& Pair : TemplateParams)
		{
			FTemplateTransformParams& Params = Pair.Value;
			if (Params.Template)
			{
				TUniquePtr<INodeTemplateTransform> NodeTransform = Params.Template->GenerateNodeTransform();
				check(NodeTransform.IsValid());

				for (const FGuid& NodeID : Params.NodeIDs)
				{
					bModified = true;
					NodeTransform->Transform(Graph.PageID, NodeID, *this);
				}
			}
			Params.NodeIDs.Reset();
		}
	});

	// 2. Remove template classes from dependency list
	for (int32 i = Dependencies.Num() - 1; i >= 0; --i)
	{
		const FMetasoundFrontendClass& Class = Dependencies[i];
		if (TemplateParams.Contains(Class.ID))
		{
			DocumentDelegates->OnRemoveSwappingDependency.Broadcast(i, Dependencies.Num() - 1);
			Dependencies.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}
	Dependencies.Shrink();

	return bModified;
}
#endif // WITH_EDITORONLY_DATA

void FMetaSoundFrontendDocumentBuilder::BeginBuilding(TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> Delegates, bool bPrimeCache)
{
	using namespace Metasound::Frontend;

	HintPath = { };
	if (DocumentInterface)
	{
		HintPath = DocumentInterface->GetAssetPathChecked();

		// Potentially at cook and runtime, the default graph may have been cooked away,
		// so initialize build page to a valid page ID if possible. On initial construction,
		// it may be possible the default graph has yet to be initialized, so don't error if
		// default page graph has yet to be created (BuildPageID is then left as ctor Default
		// PageID).
		const FMetasoundFrontendDocument& Document = DocumentInterface->GetConstDocument();
		bool bPageIDSet = false;
		Document.RootGraph.IterateGraphPages([&bPageIDSet, this](const FMetasoundFrontendGraph& Graph)
		{
			if (Graph.PageID == Metasound::Frontend::DefaultPageID)
			{
				bPageIDSet = true;
				BuildPageID = Graph.PageID;
			}
			else if (!bPageIDSet)
			{
				BuildPageID = Graph.PageID;
			}
		});

		if (UE_LOG_ACTIVE(LogMetaSound, VeryVerbose))
		{
			FTopLevelAssetPath DebugPath;
			if (DebugPath.TrySetPath(DocumentInterface.GetObject()))
			{
				UE_LOGF(LogMetaSound, VeryVerbose, "MetaSoundFrontendDocumentBuilder::BeginBuilding for asset '%ls': BuildPageID initialized to '%ls'", *DebugPath.ToString(), *BuildPageID.ToString());
			}
		}
	}

	if (Delegates.IsValid())
	{
		DocumentDelegates = Delegates;
	}
	else
	{
		if (DocumentInterface)
		{
			const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
			DocumentDelegates = MakeShared<FDocumentModifyDelegates>(Document);
		}
		else
		{
			DocumentDelegates = MakeShared<FDocumentModifyDelegates>();
		}
	}

	if (DocumentInterface)
	{
		DocumentInterface->OnBeginActiveBuilder();

		const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
		DocumentCache = FDocumentCache::Create(Document, DocumentDelegates.ToSharedRef(), BuildPageID, bPrimeCache);
	}
}

void FMetaSoundFrontendDocumentBuilder::FinishBuilding()
{
	using namespace Metasound::Frontend;

	if (DocumentInterface)
	{
		DocumentInterface->OnFinishActiveBuilder();
		DocumentInterface = { };
	}

	DocumentDelegates.Reset();
	DocumentCache.Reset();
}

bool FMetaSoundFrontendDocumentBuilder::RemoveDependency(const FGuid& InClassID)
{
	using namespace Metasound::Frontend;

	bool bSuccess = false;
	if (const int32* IndexPtr = DocumentCache->FindDependencyIndex(InClassID))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		TArray<FMetasoundFrontendClass>& Dependencies = Document.Dependencies;
		const int32 Index = *IndexPtr;

		bSuccess = true;
		Document.RootGraph.IterateGraphPages([this, &bSuccess, &InClassID](const FMetasoundFrontendGraph& Graph)
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			TArray<const FMetasoundFrontendNode*> Nodes = NodeCache.FindNodesOfClassID(InClassID);
			for (const FMetasoundFrontendNode* Node : Nodes)
			{
				bSuccess &= RemoveNode(Node->GetID(), &Graph.PageID);
			}
		});

		RemoveSwapDependencyInternal(Index);
	}

	return bSuccess;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveDependency(EMetasoundFrontendClassType ClassType, const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InClassVersionNumber)
{
	using namespace Metasound::Frontend;

	bool bSuccess = false;
	const FNodeRegistryKey ClassKey(ClassType, InClassName, InClassVersionNumber);
	if (const int32* IndexPtr = DocumentCache->FindDependencyIndex(ClassKey))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		TArray<FMetasoundFrontendClass>& Dependencies = Document.Dependencies;
		const int32 Index = *IndexPtr;

		bSuccess = true;
		Document.RootGraph.IterateGraphPages([this, &bSuccess, &Dependencies, &Index](const FMetasoundFrontendGraph& Graph)
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			TArray<const FMetasoundFrontendNode*> Nodes = NodeCache.FindNodesOfClassID(Dependencies[Index].ID);
			for (const FMetasoundFrontendNode* Node : Nodes)
			{
				bSuccess &= RemoveNode(Node->GetID(), &Graph.PageID);
			}
		});

		RemoveSwapDependencyInternal(Index);
	}

	return bSuccess;
}

void FMetaSoundFrontendDocumentBuilder::RemoveSwapDependencyInternal(int32 Index)
{
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClass>& Dependencies = Document.Dependencies;
	const int32 LastIndex = Dependencies.Num() - 1;
	DocumentDelegates->OnRemoveSwappingDependency.Broadcast(Index, LastIndex);
	Dependencies.RemoveAtSwap(Index, EAllowShrinking::No);
}

bool FMetaSoundFrontendDocumentBuilder::RemoveEdge(const FMetasoundFrontendEdge& EdgeToRemove, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
	TArray<FMetasoundFrontendEdge>& Edges = Graph.Edges;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
	if (const int32* IndexPtr = EdgeCache.FindEdgeIndexToNodeInput(EdgeToRemove.ToNodeID, EdgeToRemove.ToVertexID))
	{
		const int32 Index = *IndexPtr;
		FMetasoundFrontendEdge& FoundEdge = Edges[Index];
		if (EdgeToRemove.FromNodeID == FoundEdge.FromNodeID && EdgeToRemove.FromVertexID == FoundEdge.FromVertexID)
		{
			const int32 LastIndex = Edges.Num() - 1;
			DocumentDelegates->FindGraphDelegatesChecked(PageID).EdgeDelegates.OnRemoveSwappingEdge.Broadcast(Index, LastIndex);
			Edges.RemoveAtSwap(Index, EAllowShrinking::No);
			return true;
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::RemoveEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID)
{
	auto IsEdgeStyle = [&InNodeID, &OutputName](const FMetasoundFrontendEdgeStyle& EdgeStyle)
	{
		return EdgeStyle.NodeID == InNodeID && EdgeStyle.OutputName == OutputName;
	};

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	return Graph.Style.EdgeStyles.RemoveAllSwap(IsEdgeStyle) > 0;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::RemoveNamedEdges(const TSet<Metasound::Frontend::FNamedEdge>& InNamedEdgesToRemove, TArray<FMetasoundFrontendEdge>* OutRemovedEdges, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);

	if (OutRemovedEdges)
	{
		OutRemovedEdges->Reset();
	}

	bool bSuccess = true;

	TArray<FMetasoundFrontendEdge> EdgesToRemove;
	for (const FNamedEdge& NamedEdge : InNamedEdgesToRemove)
	{
		const FMetasoundFrontendVertex* OutputVertex = NodeCache.FindOutputVertex(NamedEdge.OutputNodeID, NamedEdge.OutputName);
		const FMetasoundFrontendVertex* InputVertex = NodeCache.FindInputVertex(NamedEdge.InputNodeID, NamedEdge.InputName);

		if (OutputVertex && InputVertex)
		{
			FMetasoundFrontendEdge NewEdge = { NamedEdge.OutputNodeID, OutputVertex->VertexID, NamedEdge.InputNodeID, InputVertex->VertexID };
			if (EdgeCache.ContainsEdge(NewEdge))
			{
				EdgesToRemove.Add(MoveTemp(NewEdge));
			}
			else
			{
				bSuccess = false;
				UE_LOGF(LogMetaSound, Warning, "Failed to remove connection between MetaSound node output '%ls' and input '%ls': No connection found.", *NamedEdge.OutputName.ToString(), *NamedEdge.InputName.ToString());
			}
		}
	}

	for (const FMetasoundFrontendEdge& EdgeToRemove : EdgesToRemove)
	{
		const bool bRemovedEdge = RemoveEdgeToNodeInput(EdgeToRemove.ToNodeID, EdgeToRemove.ToVertexID, InPageID);
		if (ensureAlwaysMsgf(bRemovedEdge, TEXT("Failed to remove MetaSound graph edge via DocumentBuilder when prior step validated edge remove was valid")))
		{
			if (OutRemovedEdges)
			{
				OutRemovedEdges->Add(EdgeToRemove);
			}
		}
		else
		{
			bSuccess = false;
		}
	}

	return bSuccess;
}

void FMetaSoundFrontendDocumentBuilder::Reload(TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> Delegates, bool bPrimeCache)
{
	using namespace Metasound::Frontend;

	if (DocumentInterface)
	{
		DocumentInterface->OnFinishActiveBuilder();
	}

	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	DocumentDelegates = Delegates.IsValid() ? Delegates : MakeShared<FDocumentModifyDelegates>(Document);

	if (DocumentInterface)
	{
		DocumentCache = FDocumentCache::Create(Document, DocumentDelegates.ToSharedRef(), BuildPageID, bPrimeCache);
		DocumentInterface->OnBeginActiveBuilder();
	}
}

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphInputDefault(FName InputName, const FGuid& InPageID, bool bClearInheritsDefault)
{
	return RemoveGraphInputDefault(InputName, InPageID);
}

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphInputDefault(FName InputName, const FGuid& InPageID)
{
	using namespace Metasound::Frontend;

	// Default page input default should not be removed at editor/runtime but can be cooked out depending on platform/cook settings. Use StripGraphInputDefault for that case instead.
	if (InPageID == DefaultPageID)
	{
		UE_LOGF(LogMetaSound, Error, "Document Builder failed to remove graph input default '%ls': Cannot remove default page input default except in cook. If this happens during cook, use StripGraphInputDefault instead.", *InputName.ToString());
		return false;
	}
	
	return RemoveGraphInputDefaultInternal(InputName, InPageID);
}

bool FMetaSoundFrontendDocumentBuilder::StripGraphInputDefault(FName InputName, const FGuid& InPageID)
{
	return RemoveGraphInputDefaultInternal(InputName, InPageID);	
}

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphInputDefaultInternal(FName InputName, const FGuid& InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	auto NameMatchesInput = [&InputName](const FMetasoundFrontendClassInput& Input) { return Input.Name == InputName; };
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClassInput>& Inputs = Document.RootGraph.GetDefaultInterface().Inputs;

	const int32 Index = Inputs.IndexOfByPredicate(NameMatchesInput);
	if (Index != INDEX_NONE)
	{
		FMetasoundFrontendClassInput& Input = Inputs[Index];
		const bool bRemovedDefault = Input.RemoveDefault(InPageID);
		if (bRemovedDefault)
		{
			DocumentDelegates->InterfaceDelegates.OnInputDefaultChanged.Broadcast(
				FDocumentArrayPagedInputArgs
				{
					.Index = Index,
					.PageID = InPageID,
					.ChangeType = EMetaSoundLiteralChangeType::Remove
				});
			return true;
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
		FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];

		auto IsVertex = [&InVertexID](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == InVertexID; };
		const int32 VertexIndex = Node.Interface.Inputs.IndexOfByPredicate(IsVertex);
		if (VertexIndex != INDEX_NONE)
		{
			auto IsLiteral = [&InVertexID](const FMetasoundFrontendVertexLiteral& Literal) { return Literal.VertexID == InVertexID; };
			const int32 LiteralIndex = Node.InputLiterals.IndexOfByPredicate(IsLiteral);
			if (LiteralIndex != INDEX_NONE)
			{
				FNodeModifyDelegates& NodeDelegates = DocumentDelegates->FindGraphDelegatesChecked(PageID).NodeDelegates;
				const FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray& OnRemovingNodeInputLiteral = NodeDelegates.OnRemovingNodeInputLiteral;
				const int32 LastIndex = Node.InputLiterals.Num() - 1;
				OnRemovingNodeInputLiteral.Broadcast(*NodeIndex, VertexIndex, LastIndex);
				if (LiteralIndex != LastIndex)
				{
					OnRemovingNodeInputLiteral.Broadcast(*NodeIndex, VertexIndex, LiteralIndex);
				}

				Node.InputLiterals.RemoveAtSwap(LiteralIndex, EAllowShrinking::No);

#if WITH_EDITORONLY_DATA
				Document.Metadata.ModifyContext.AddNodeIDModified(Node.GetID());
#endif // WITH_EDITORONLY_DATA

				if (LiteralIndex != LastIndex)
				{
					const FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray& OnNodeInputLiteralSet = NodeDelegates.OnNodeInputLiteralSet;
					OnNodeInputLiteralSet.Broadcast(*NodeIndex, VertexIndex, LiteralIndex);
				}
				return true;
			}
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveEdges(const FGuid& InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::VariableNames;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const FMetasoundFrontendNode* Node = NodeCache.FindNode(InNodeID))
	{
		const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);

		for (const FMetasoundFrontendVertex& Vertex : Node->Interface.Inputs)
		{
			RemoveEdgeToNodeInput(InNodeID, Vertex.VertexID, InPageID);
		}

		TArray<FMetasoundFrontendVertexHandle> ToVertexHandles;
		for (const FMetasoundFrontendVertex& Vertex : Node->Interface.Outputs)
		{
			RemoveEdgesFromNodeOutput(InNodeID, Vertex.VertexID, InPageID);
		}

		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveEdgesByNodeClassInterfaceBindings(const FGuid& InFromNodeID, const FGuid& InToNodeID, const FGuid* InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	TSet<FMetasoundFrontendVersion> FromInterfaceVersions;
	TSet<FMetasoundFrontendVersion> ToInterfaceVersions;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	if (FindNodeClassInterfaces(InFromNodeID, FromInterfaceVersions, PageID) && FindNodeClassInterfaces(InToNodeID, ToInterfaceVersions, PageID))
	{
		TSet<FNamedEdge> NamedEdges;
		if (DocumentBuilderPrivate::TryGetInterfaceBoundEdges(InFromNodeID, FromInterfaceVersions, InToNodeID, ToInterfaceVersions, NamedEdges))
		{
			constexpr TArray<FMetasoundFrontendEdge>* RemovedEdges = nullptr;
			return RemoveNamedEdges(NamedEdges, RemovedEdges, InPageID);
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveEdgesFromNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
	const TArrayView<const int32> Indices = EdgeCache.FindEdgeIndicesFromNodeOutput(InNodeID, InVertexID);
	if (!Indices.IsEmpty())
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);

		// Copy off indices and sort descending as the edge array will be modified when notifying the cache in the loop below
		TArray<int32> IndicesCopy(Indices.GetData(), Indices.Num());
		Algo::Sort(IndicesCopy, [](const int32& L, const int32& R) { return L > R; });
		FEdgeModifyDelegates& EdgeDelegates = DocumentDelegates->FindGraphDelegatesChecked(PageID).EdgeDelegates;
		for (int32 Index : IndicesCopy)
		{
#if WITH_EDITORONLY_DATA
			if (const FMetasoundFrontendVertex* Vertex = FindNodeOutput(InNodeID, InVertexID))
			{
				auto IsEdgeStyle = [&InNodeID, OutputName = Vertex->Name](const FMetasoundFrontendEdgeStyle& EdgeStyle)
				{
					return EdgeStyle.NodeID == InNodeID && EdgeStyle.OutputName == OutputName;
				};
				Graph.Style.EdgeStyles.RemoveAllSwap(IsEdgeStyle);
			}
#endif // WITH_EDITORONLY_DATA

			const int32 LastIndex = Graph.Edges.Num() - 1;
			EdgeDelegates.OnRemoveSwappingEdge.Broadcast(Index, LastIndex);
			Graph.Edges.RemoveAtSwap(Index, EAllowShrinking::No);
		}

#if WITH_EDITORONLY_DATA
		Document.Metadata.ModifyContext.AddNodeIDModified(InNodeID);
#endif // WITH_EDITORONLY_DATA

		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveEdgeToNodeInput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);
	if (const int32* IndexPtr = EdgeCache.FindEdgeIndexToNodeInput(InNodeID, InVertexID))
	{
		FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
		const int32 Index = *IndexPtr; // Copy off indices as the pointer may be modified when notifying the cache below

#if WITH_EDITORONLY_DATA
		if (const FMetasoundFrontendVertex* Vertex = FindNodeOutput(Graph.Edges[Index].FromNodeID, Graph.Edges[Index].FromVertexID))
		{
			auto IsEdgeStyle = [&InNodeID, OutputName = Vertex->Name](const FMetasoundFrontendEdgeStyle& EdgeStyle)
			{
				return EdgeStyle.NodeID == InNodeID && EdgeStyle.OutputName == OutputName;
			};
			Graph.Style.EdgeStyles.RemoveAllSwap(IsEdgeStyle);
		}
#endif // WITH_EDITORONLY_DATA

		const FEdgeModifyDelegates& EdgeDelegates = DocumentDelegates->FindGraphDelegatesChecked(PageID).EdgeDelegates;
		const int32 LastIndex = Graph.Edges.Num() - 1;
		EdgeDelegates.OnRemoveSwappingEdge.Broadcast(Index, LastIndex);
		Graph.Edges.RemoveAtSwap(Index, EAllowShrinking::No);

#if WITH_EDITORONLY_DATA
		GetDocumentChecked().Metadata.ModifyContext.AddNodeIDModified(InNodeID);
#endif // WITH_EDITORONLY_DATA

		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::RemoveGraphComment(const FGuid& InCommentID, const FGuid* InPageID)
{
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(InPageID ? *InPageID : BuildPageID);
	if (Graph.Style.Comments.Remove(InCommentID) > 0)
	{
		Document.Metadata.ModifyContext.SetDocumentModified();

		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphInput(FName InputName, bool bRemoveTemplateInputNodes)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	if (const int32* IndexPtr = DocumentCache->GetInterfaceCache().FindInputIndex(InputName))
	{
		TArray<FMetasoundFrontendClassInput>& Inputs = Document.RootGraph.GetDefaultInterface().Inputs;
		const FGuid NodeID = Inputs[*IndexPtr].NodeID;
		FGuid ClassID;
		bool bNodesRemoved = true;
		Document.RootGraph.IterateGraphPages([&](const FMetasoundFrontendGraph& Graph)
		{
			TArray<FGuid> NodeIDsToRemove { NodeID };

			if (const FMetasoundFrontendNode* Node = FindNode(NodeID, &Graph.PageID))
			{
				ClassID = Node->ClassID;
			}
			else
			{
				bNodesRemoved = false;
				return;
			}

			if (bRemoveTemplateInputNodes)
			{
				const TArray<const FMetasoundFrontendNode*> TemplateNodes = GetGraphInputTemplateNodes(InputName, &Graph.PageID);
				Algo::Transform(TemplateNodes, NodeIDsToRemove, [](const FMetasoundFrontendNode* Node) { return Node->GetID(); });
			}

			for (const FGuid& ToRemove : NodeIDsToRemove)
			{
				if (RemoveNode(ToRemove, &Graph.PageID))
				{
#if WITH_EDITORONLY_DATA
					Document.Metadata.ModifyContext.AddNodeIDModified(ToRemove);
#endif // WITH_EDITORONLY_DATA
				}
				else
				{
					bNodesRemoved = false;
				}
			}
		});

		if (bNodesRemoved)
		{
#if WITH_EDITORONLY_DATA
			if (FDocumentVertexTemplate* DocTemplate = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
			{
				DocTemplate->RemoveVertexMetadata(NodeID);
			}
#endif // WITH_EDITORONLY_DATA

			const int32 Index = *IndexPtr;
			DocumentDelegates->InterfaceDelegates.OnRemovingInput.Broadcast(Index);

			const int32 LastIndex = Inputs.Num() - 1;
			if (Index != LastIndex)
			{
				DocumentDelegates->InterfaceDelegates.OnRemovingInput.Broadcast(LastIndex);
			}
			Inputs.RemoveAtSwap(Index, EAllowShrinking::No);
			if (Index != LastIndex)
			{
				DocumentDelegates->InterfaceDelegates.OnInputAdded.Broadcast(Index);
			}

#if WITH_EDITORONLY_DATA
			ClearMemberMetadata(NodeID);
			Document.Metadata.ModifyContext.AddMemberIDModified(NodeID);
#endif // WITH_EDITORONLY_DATA

			const bool bDependencyReferenced = IsDependencyReferenced(ClassID);
			if (bDependencyReferenced || RemoveDependency(ClassID))
			{
				return true;
			}
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphOutput(FName OutputName)
{
	using namespace Metasound::Frontend;

	bool bNodesRemoved = true;
	FGuid ClassID;
	FGuid NodeID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	Document.RootGraph.IterateGraphPages([this, &OutputName, &Document, &ClassID, &NodeID, &bNodesRemoved](const FMetasoundFrontendGraph& Graph)
	{
		if (const FMetasoundFrontendNode* Node = FindGraphOutputNode(OutputName, &Graph.PageID))
		{
			ClassID = Node->ClassID;
			NodeID = Node->GetID();
			if (!RemoveNode(NodeID, &Graph.PageID))
			{
				bNodesRemoved = false;
				return;
			}

#if WITH_EDITORONLY_DATA
			Document.Metadata.ModifyContext.AddNodeIDModified(NodeID);
#endif // WITH_EDITORONLY_DATA
		}
	});

	if (bNodesRemoved)
	{
#if WITH_EDITORONLY_DATA
		if (FDocumentVertexTemplate* DocTemplate = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
		{
			DocTemplate->RemoveVertexMetadata(NodeID);
		}
#endif // WITH_EDITORONLY_DATA

		TArray<FMetasoundFrontendClassOutput>& Outputs = Document.RootGraph.GetDefaultInterface().Outputs;
		auto OutputNameMatches = [OutputName](const FMetasoundFrontendClassOutput& Output) { return Output.Name == OutputName; };
		const int32 Index = Outputs.IndexOfByPredicate(OutputNameMatches);
		if (Index != INDEX_NONE)
		{
			DocumentDelegates->InterfaceDelegates.OnRemovingOutput.Broadcast(Index);

			const int32 LastIndex = Outputs.Num() - 1;
			if (Index != LastIndex)
			{
				DocumentDelegates->InterfaceDelegates.OnRemovingOutput.Broadcast(LastIndex);
			}
			Outputs.RemoveAtSwap(Index, EAllowShrinking::No);
			if (Index != LastIndex)
			{
				DocumentDelegates->InterfaceDelegates.OnOutputAdded.Broadcast(Index);
			}

#if WITH_EDITORONLY_DATA
			ClearMemberMetadata(NodeID);
			Document.Metadata.ModifyContext.AddMemberIDModified(NodeID);
#endif // WITH_EDITORONLY_DATA

			const bool bDependencyReferenced = IsDependencyReferenced(ClassID);
			if (bDependencyReferenced || RemoveDependency(ClassID))
			{
				return true;
			}
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::RemoveGraphPage(const FGuid& InPageID)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FGuid AdjacentPageID;

	if (Document.RootGraph.ContainsGraphPage(InPageID))
	{
		DocumentDelegates->RemovePageDelegates(InPageID);
	}

	const bool bPageRemoved = Document.RootGraph.RemoveGraphPage(InPageID, &AdjacentPageID);
	if (bPageRemoved)
	{
		if (InPageID == BuildPageID)
		{
			ensureAlwaysMsgf(SetBuildPageID(AdjacentPageID), TEXT("AdjacentPageID returned is always expected to be valid"));
		}
		RemoveUnusedDependencies();
	}

	return bPageRemoved;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::RemoveGraphVariable(FName VariableName, const FGuid* InPageID)
{
	if (const FMetasoundFrontendVariable* Variable = FindGraphVariable(VariableName, InPageID))
	{
		RemoveNode(Variable->VariableNodeID);
		RemoveNode(Variable->MutatorNodeID);

		// Copy ids as node removal will update AccessorNodeIDs array on FrontendVariable internally to RemoveNode call
		TArray<FGuid> AccessorNodeIDs = Variable->AccessorNodeIDs;
		for (const FGuid& NodeID : AccessorNodeIDs)
		{
			RemoveNode(NodeID, InPageID);
		}

		// Copy ids as node removal will update DeferredAccessorNodeIDs array on FrontendVariable internally to RemoveNode call
		TArray<FGuid> DeferredAccessorNodeIDs = Variable->DeferredAccessorNodeIDs;
		for (const FGuid& NodeID : DeferredAccessorNodeIDs)
		{
			RemoveNode(NodeID, InPageID);
		}

		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(InPageID ? *InPageID : BuildPageID);

		// VariableID must be cached to avoid being mutated when variable
		// is ultimately removed in the RemoveAllSwap below.
		const FGuid VariableID = Variable->ID;
		auto IsVariableWithID = [VariableID](const FMetasoundFrontendVariable& InVariable)
		{
			return InVariable.ID == VariableID;
		};
		Graph.Variables.RemoveAllSwap(IsVariableWithID);

		// Clean-up/remove variable dependencies that may or may no longer be referenced.
		RemoveUnusedDependencies();

#if WITH_EDITOR
		Document.Metadata.ModifyContext.AddMemberIDModified(VariableID);
#endif // WITH_EDITOR
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveInterface(FName InterfaceName)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendVersion InterfaceVersion;
	if (ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceName, InterfaceVersion))
	{
		if (!GetDocumentChecked().Interfaces.Contains(InterfaceVersion))
		{
			UE_LOGF(LogMetaSound, VeryVerbose, "MetaSound interface '%ls' not found on document. MetaSoundBuilder skipping remove request.", *InterfaceName.ToString());
			return true;
		}

		FMetasoundFrontendInterface Interface;
		const bool bFoundInterface = IInterfaceRegistry::Get().FindInterface(InterfaceVersion, Interface);
		if (bFoundInterface)
		{
			const FTopLevelAssetPath BuilderClassPath = GetBuilderClassPath();
			auto FindClassOptionsPredicate = [&BuilderClassPath](const FMetasoundFrontendInterfaceUClassOptions& Options) { return Options.ClassPath == BuilderClassPath; };
			const FMetasoundFrontendInterfaceUClassOptions* ClassOptions = Interface.Metadata.UClassOptions.FindByPredicate(FindClassOptionsPredicate);
			if (ClassOptions && !ClassOptions->bIsModifiable)
			{
				UE_LOGF(LogMetaSound, Error, "DocumentBuilder failed to remove MetaSound Interface '%ls' to document: is not set to be modifiable for given UClass '%ls'", *InterfaceName.ToString(), *BuilderClassPath.ToString());
				return false;
			}

			FModifyInterfaceOptions Options({ MoveTemp(Interface) }, { });
			return ModifyInterfaces(MoveTemp(Options));
		}
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::RemoveNode(const FGuid& InNodeID, const FGuid* InPageID)
{
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetaSoundFrontendDocumentBuilder::RemoveNode);

	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(PageID);

	if (const int32* IndexPtr = NodeCache.FindNodeIndex(InNodeID))
	{
		const int32 Index = *IndexPtr; // Copy off indices as the pointer may be modified when notifying the cache below

		FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
		TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
		const FMetasoundFrontendNode& Node = Nodes[Index];
		const FGuid& NodeID = Node.GetID();

		const FMetasoundFrontendClass* NodeClass = DocumentCache->FindDependency(Node.ClassID);
		check(NodeClass);
		const EMetasoundFrontendClassType ClassType = NodeClass->Metadata.GetType();
		switch (ClassType)
		{
			case EMetasoundFrontendClassType::Variable:
			case EMetasoundFrontendClassType::VariableDeferredAccessor:
			case EMetasoundFrontendClassType::VariableAccessor:
			case EMetasoundFrontendClassType::VariableMutator:
			{
				const bool bVariableNodeUnlinked = UnlinkVariableNode(NodeID, PageID);
				ensureAlwaysMsgf(bVariableNodeUnlinked, TEXT("Failed to unlink %s node with ID '%s"), LexToString(ClassType), *InNodeID.ToString());
			}
			break;
		}

		RemoveEdges(NodeID, InPageID);
		const int32 LastIndex = Nodes.Num() - 1;
		FNodeModifyDelegates& NodeDelegates = DocumentDelegates->FindGraphDelegatesChecked(PageID).NodeDelegates;
		NodeDelegates.OnRemoveSwappingNode.Broadcast(Index, LastIndex);
		Nodes.RemoveAtSwap(Index, EAllowShrinking::No);

#if WITH_EDITORONLY_DATA
		GetDocumentChecked().Metadata.ModifyContext.AddNodeIDModified(InNodeID);
#endif // WITH_EDITORONLY_DATA

		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
int32 FMetaSoundFrontendDocumentBuilder::RemoveNodeLocation(const FGuid& InNodeID, const FGuid* InLocationGuid, const FGuid* InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
		FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
		FMetasoundFrontendNodeStyle& Style = Node.Style;
		if (InLocationGuid)
		{
			return Style.Display.Locations.Remove(*InLocationGuid);
		}
		else
		{
			const int32 NumLocationsRemoved = Style.Display.Locations.Num();
			Style.Display.Locations.Reset();
			return NumLocationsRemoved;
		}
	}

	return 0;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::RemoveUnusedDependencies()
{
	const FMetasoundFrontendDocument& Document = GetConstDocumentChecked();
	const FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	const TArray<FMetasoundFrontendClass>& Dependencies = Document.Dependencies;

	bool bDidEdit = false;

	// Remove unused dependencies
	for (int32 Index = Dependencies.Num() - 1; Index >= 0; --Index)
	{
		const FGuid& ClassID = Dependencies[Index].ID;
		const bool bIsReferenced = IsDependencyReferenced(ClassID);
		if (!bIsReferenced)
		{
			RemoveSwapDependencyInternal(Index);
			bDidEdit = true;
		}
	}

	return bDidEdit;
}

bool FMetaSoundFrontendDocumentBuilder::ReplaceDependency(const Metasound::Frontend::FNodeClassRegistryKey& OldNodeClassKey, const Metasound::Frontend::FNodeClassRegistryKey& NewNodeClassKey,TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedInputs, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedOutputs)
{
	using namespace Metasound::Frontend;
	using namespace Metasound::Frontend::NodeReplacement;

	// Early out if there is no dependency to replace
	const FMetasoundFrontendClass* OldDependency = FindDependency(OldNodeClassKey);
	if (!OldDependency)
	{
		return false;
	}
	
	// For all matching nodes, cache off connections and other data 
	struct FNodeReplacementData
	{
		const FGuid NodeID;
		const FGuid PageID;
#if WITH_EDITORONLY_DATA
		FNodeInstanceReplacementData NodeInstanceReplacementData;
#endif // WITH_EDITORONLY_DATA
	};
	TArray<FNodeReplacementData> NodeDataToReplace;
	auto CaptureNodeData = [&](const FMetasoundFrontendClass&, const FMetasoundFrontendNode& Node, const FGuid& PageID)
	{
#if WITH_EDITORONLY_DATA
		FNodeInstanceReplacementData NodeInstanceData = CaptureNodeInstanceReplacementData(Node.GetID(), &PageID);
		NodeDataToReplace.Emplace(FNodeReplacementData { Node.GetID(), PageID, MoveTemp(NodeInstanceData) });
#else
		NodeDataToReplace.Emplace(FNodeReplacementData { Node.GetID(), PageID });
#endif // WITH_EDITORONLY_DATA
	};

	auto GetNodesOfOldClass = [&](const FMetasoundFrontendNode& Node)
	{
		const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID);
		if (Class)
		{
			const FNodeClassRegistryKey KeyToCompare(Class->Metadata);
			return KeyToCompare == OldNodeClassKey;
		}
		return false;
	};

	IterateNodesByPredicate(CaptureNodeData, GetNodesOfOldClass, /*InPageID =*/nullptr, /*bIterateAllPages=*/true);
	
	// Remove dependency and associated nodes
	bool bSuccess = RemoveDependency(OldNodeClassKey.Type, OldNodeClassKey.ClassName, OldNodeClassKey.Version);

	// Clean up dependency
	RemoveUnusedDependencies();

	// Readd nodes with new class
	for (const FNodeReplacementData& NodeReplacementData : NodeDataToReplace)
	{
		const FMetasoundFrontendNode* NewNode = AddNodeByClassName(NewNodeClassKey.ClassName, NewNodeClassKey.Version.Major, NewNodeClassKey.Version.Minor, NodeReplacementData.NodeID, &NodeReplacementData.PageID);
		bSuccess |= (NewNode != nullptr);
	}
	
#if WITH_EDITORONLY_DATA
	// Reapply connections and node data 
	const bool bNodeVersionUpdated = NewNodeClassKey.Version > OldNodeClassKey.Version;
	for (FNodeReplacementData& NodeReplacementData : NodeDataToReplace)
	{
		const FGuid& NodeID = NodeReplacementData.NodeID;
		const FGuid& PageID = NodeReplacementData.PageID;
		
		FMetasoundFrontendNode* NewNode = FindNodeInternal(NodeID, &PageID);
		if (NewNode)
		{
			const FMetasoundFrontendClass* Class = FindDependency(NewNode->ClassID);
			if (Class)
			{
				NodeReplacementData.NodeInstanceReplacementData.Style.bMessageNodeUpdated = bNodeVersionUpdated;
				ApplyNodeInstanceReplacementData(MoveTemp(NodeReplacementData.NodeInstanceReplacementData), NodeID, &PageID, OutDisconnectedInputs, OutDisconnectedOutputs);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
	return bSuccess;
}

#if WITH_EDITORONLY_DATA
void FMetaSoundFrontendDocumentBuilder::ApplyNodeInstanceReplacementData(Metasound::Frontend::NodeReplacement::FNodeInstanceReplacementData&& InInstanceData, const FGuid& InNodeID, const FGuid* InPageID, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedInputs, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedOutputs)
{
	using namespace Metasound::Frontend;
	using namespace NodeReplacement;
	
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	
	FMetasoundFrontendNode* ReplacementNode = FindNodeInternal(InNodeID, InPageID);
	check(ReplacementNode);
	ReplacementNode->Style = MoveTemp(InInstanceData.Style);

	// The logic may appear a little backwards here because adding the node already instantiated the replacement
	// node configuration instances. The purpose is to revert the replacement if it should not have happened in 
	// the first place.
	if (!ShouldReplaceExistingNodeConfig(ReplacementNode->Configuration, InInstanceData.Configuration))
	{
		// We should have kept the original node configuration. Set back to the original
		SetNodeConfiguration(InNodeID, MoveTemp(InInstanceData.Configuration), InPageID);
	}
	
	// Set literal and reconnect input connections if possible
	for (FMetasoundFrontendVertex& InputVertex : ReplacementNode->Interface.Inputs)
	{
		const FVertexNameAndType ConnectionKey(InputVertex.Name, InputVertex.TypeName);
		if (FInputConnectionInfo* ConnectionInfo = InInstanceData.InputConnections.Find(ConnectionKey))
		{
			if (ConnectionInfo->bLiteralSet)
			{
				// Only apply literal if needs to be overwritten (an overwritten value exists, or it is not equal to class input default)
				if (const FMetasoundFrontendVertexLiteral* CurrentLiteral = FindNodeInputDefault(InNodeID, InputVertex.VertexID, InPageID))
				{
					SetNodeInputDefault(InNodeID, InputVertex.VertexID, ConnectionInfo->DefaultValue, InPageID);
				}
				else if (const TArray<FMetasoundFrontendClassInputDefault>* NodeClassDefaults = FindNodeClassInputDefaults(InNodeID, InputVertex.Name, InPageID))
				{
					// Paged defaults not currently supported on external node inputs, so just use the default page one 
					const FMetasoundFrontendClassInputDefault& NodeClassDefault = NodeClassDefaults->Last();
					if (!NodeClassDefault.Literal.IsEqual(ConnectionInfo->DefaultValue))
					{
						SetNodeInputDefault(InNodeID, InputVertex.VertexID, ConnectionInfo->DefaultValue, InPageID);
					}
				}
			}

			if (ConnectionInfo->ConnectedOutput.IsSet())
			{
				FMetasoundFrontendEdge NewEdge
				{ 
					ConnectionInfo->ConnectedOutput.NodeID,
					ConnectionInfo->ConnectedOutput.VertexID,
					InNodeID, 
					InputVertex.VertexID, 
				};

				const bool bIsValidEdge = IsValidEdge(NewEdge, InPageID) == Metasound::Frontend::EInvalidEdgeReason::None;
				// It is possible the edge already exists if the edge is between 
				// this and another node of the same dependency that just added it when it was replaced 
				if (bIsValidEdge && !ContainsEdge(NewEdge, InPageID))
				{
					AddEdge(MoveTemp(NewEdge), InPageID);
				}

				// Remove connection to track missing connections between 
				// node versions.
				InInstanceData.InputConnections.Remove(ConnectionKey);
			}
		}
	}

	// Track missing input connections
	if (nullptr != OutDisconnectedInputs)
	{
		for (const auto& ConnectionInfoKV : InInstanceData.InputConnections)
		{
			const FInputConnectionInfo& ConnectionInfo = ConnectionInfoKV.Value;
			if (ConnectionInfo.ConnectedOutput.IsSet())
			{
				OutDisconnectedInputs->Add(FVertexNameAndType{ ConnectionInfo.Name, ConnectionInfo.DataType });
			}
		}
	}

	// Reconnect output connections if possible
	for (FMetasoundFrontendVertex& OutputVertex : ReplacementNode->Interface.Outputs)
	{
		const FVertexNameAndType ConnectionKey(OutputVertex.Name, OutputVertex.TypeName);
		if (FOutputConnectionInfo* ConnectionInfo = InInstanceData.OutputConnections.Find(ConnectionKey))
		{
			bool bConnectionSuccess = false;
			for (const FMetasoundFrontendVertexHandle& ConnectedInputHandle : ConnectionInfo->ConnectedInputs)
			{
				if (ConnectedInputHandle.IsSet())
				{
					FMetasoundFrontendEdge NewEdge
					{ 
						InNodeID,
						OutputVertex.VertexID,
						ConnectedInputHandle.NodeID,
						ConnectedInputHandle.VertexID
					};

					const bool bIsValidEdge = IsValidEdge(NewEdge, InPageID) == Metasound::Frontend::EInvalidEdgeReason::None;
					// It is possible the edge already exists if the edge is between 
					// this and another node of the same dependency that just added it when it was replaced 
					if (bIsValidEdge && !ContainsEdge(NewEdge, InPageID))
					{
						AddEdge(MoveTemp(NewEdge), InPageID);
					}
					bConnectionSuccess = true;
				}
			}

			// Remove connection to track missing connections between 
			// node versions.
			if (bConnectionSuccess)
			{
				InInstanceData.OutputConnections.Remove(ConnectionKey);
			}
		}
	}

	// Track missing output connections
	if (nullptr != OutDisconnectedOutputs)
	{
		for (const auto& ConnectionInfoKV : InInstanceData.OutputConnections)
		{
			const FOutputConnectionInfo& ConnectionInfo = ConnectionInfoKV.Value;
			const bool bAnyConnectedInputs = Algo::AnyOf(ConnectionInfo.ConnectedInputs, [](const FMetasoundFrontendVertexHandle& Input) { return Input.IsSet(); });
			if (bAnyConnectedInputs)
			{
				OutDisconnectedOutputs->Add(FVertexNameAndType{ ConnectionInfo.Name, ConnectionInfo.DataType });
			}
		}
	}
}

Metasound::Frontend::NodeReplacement::FNodeInstanceReplacementData FMetaSoundFrontendDocumentBuilder::CaptureNodeInstanceReplacementData(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;
	using namespace NodeReplacement;
	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;

	FNodeInstanceReplacementData ReplacementData;

	const FMetasoundFrontendNode* ReplacementNode = FindNode(InNodeID, InPageID);
	check(ReplacementNode);
	ReplacementData.Style = ReplacementNode->Style;

	// Move any configuration and override data living on the node.
	ReplacementData.Configuration = ReplacementNode->Configuration;
	ReplacementData.ClassInterfaceOverride = ReplacementNode->ClassInterfaceOverride;

	// Cache input/output connections by name to try so they can be
	// hooked back up after swapping to the new class version.
	for (const FMetasoundFrontendVertex& InputVertex : ReplacementNode->Interface.Inputs)
	{
		bool bLiteralSet = false;
		FMetasoundFrontendLiteral DefaultLiteral;

		auto VertexLiteralMatchesID = [&](const FMetasoundFrontendVertexLiteral& VertexLiteral)
		{
			return VertexLiteral.VertexID == InputVertex.VertexID;
		};

		if (const FMetasoundFrontendVertexLiteral* VertexLiteral = ReplacementNode->InputLiterals.FindByPredicate(VertexLiteralMatchesID))
		{
			DefaultLiteral = VertexLiteral->Value;
			bLiteralSet = true;
		}

		const FVertexNameAndType ConnectionKey(InputVertex.Name, InputVertex.TypeName);
		const FMetasoundFrontendNode* ConnectedOutputNode = nullptr;
		const FMetasoundFrontendVertex* ConnectedOutputVertex = FindNodeOutputConnectedToNodeInput(ReplacementNode->GetID(), InputVertex.VertexID, &ConnectedOutputNode, &PageID);
		FMetasoundFrontendVertexHandle ConnectedOutputHandle = (ConnectedOutputNode && ConnectedOutputVertex) ?
			FMetasoundFrontendVertexHandle{ ConnectedOutputNode->GetID(), ConnectedOutputVertex->VertexID } : FMetasoundFrontendVertexHandle();
		ReplacementData.InputConnections.Add(ConnectionKey, FInputConnectionInfo
		{
			MoveTemp(ConnectedOutputHandle),
			InputVertex.Name, 
			InputVertex.TypeName,
			MoveTemp(DefaultLiteral),
			bLiteralSet
		});
	}

	for (const FMetasoundFrontendVertex& OutputVertex : ReplacementNode->Interface.Outputs)
	{
		const FVertexNameAndType ConnectionKey(OutputVertex.Name, OutputVertex.TypeName);
		TArray<const FMetasoundFrontendNode*> ConnectedInputNodes;
		TArray<const FMetasoundFrontendVertex*> ConnectedInputVertices = FindNodeInputsConnectedToNodeOutput(ReplacementNode->GetID(), OutputVertex.VertexID, &ConnectedInputNodes, &PageID);
		TArray<FMetasoundFrontendVertexHandle> ConnectedInputInfo;
		check(ConnectedInputNodes.Num() == ConnectedInputVertices.Num());
		for (int i = 0; i < ConnectedInputVertices.Num(); ++i)
		{
			check(ConnectedInputNodes[i]);
			check(ConnectedInputVertices[i]);
			ConnectedInputInfo.Add(FMetasoundFrontendVertexHandle{ ConnectedInputNodes[i]->GetID(), ConnectedInputVertices[i]->VertexID });
		}
		ReplacementData.OutputConnections.Add(ConnectionKey, FOutputConnectionInfo
		{
			ConnectedInputInfo,
			OutputVertex.Name, 
			OutputVertex.TypeName
		});
	}
	
	return ReplacementData;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::ResetGraphInputDefault(FName InputName)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	auto NameMatchesInput = [&InputName](const FMetasoundFrontendClassInput& Input) { return Input.Name == InputName; };
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClassInput>& Inputs = Document.RootGraph.GetDefaultInterface().Inputs;

	const int32 Index = Inputs.IndexOfByPredicate(NameMatchesInput);
	if (Index != INDEX_NONE)
	{
		FMetasoundFrontendClassInput& Input = Inputs[Index];
		TArray<const FGuid> PageIDs;
		Input.IterateDefaults([&](const FGuid& PageID, FMetasoundFrontendLiteral&)
		{
			PageIDs.Add(PageID);
		});
		Input.ResetDefaults();

		for (const FGuid& PageID : PageIDs)
		{
			DocumentDelegates->InterfaceDelegates.OnInputDefaultChanged.Broadcast(
			FDocumentArrayPagedInputArgs
			{
				.Index = Index,
				.PageID = PageID,
				.ChangeType = EMetaSoundLiteralChangeType::Reset
			});
		}
		
		if (FDocumentVertexTemplate* Config = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
		{
			if (FMetaSoundFrontendDocumentTemplateVertexMetadata* VertexMetadata = Config->FindVertexMetadata(Input.NodeID))
			{
				VertexMetadata->OnDefaultReset(*this, Input);
			}
		}

#if WITH_EDITORONLY_DATA
		Document.Metadata.ModifyContext.AddMemberIDModified(Input.NodeID);
#endif // WITH_EDITORONLY_DATA
		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void FMetaSoundFrontendDocumentBuilder::ResetGraphPages(bool bClearDefaultGraph)
{
	using namespace Metasound;

	FMetasoundFrontendGraphClass& RootGraph = GetDocumentChecked().RootGraph;
	TArray<FGuid> PageDelegatesToRemove;
	RootGraph.IterateGraphPages([this, &PageDelegatesToRemove](FMetasoundFrontendGraph& Graph)
	{
		if (Graph.PageID != Frontend::DefaultPageID)
		{
			DocumentDelegates->PageDelegates.OnRemovingPage.Broadcast(Frontend::FDocumentMutatePageArgs{ Graph.PageID });
			PageDelegatesToRemove.Add(Graph.PageID);
		}
	});

	RootGraph.ResetGraphPages(bClearDefaultGraph);

	// Must be called after reset to avoid re-initializing delegates
	// prematurely, which is handled by delegate responding to prior
	// OnRemovingPage broadcast.
	constexpr bool bBroadcastNotify = false;
	for (const FGuid& PageID : PageDelegatesToRemove)
	{
		DocumentDelegates->RemovePageDelegates(PageID, bBroadcastNotify);
	}

	Reload(DocumentDelegates);
	SetBuildPageID(Frontend::DefaultPageID);
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void FMetaSoundFrontendDocumentBuilder::SetAuthor(const FString& InAuthor)
{
	FMetasoundFrontendClassMetadata& ClassMetadata = GetDocumentChecked().RootGraph.Metadata;
	ClassMetadata.SetAuthor(InAuthor);
	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetBuildPageID(const FGuid& InBuildPageID, bool bBroadcastDelegate)
{
	using namespace Metasound::Frontend;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	if (const FMetasoundFrontendGraph* BuildGraph = Document.RootGraph.FindConstGraph(InBuildPageID))
	{
		if (BuildPageID != BuildGraph->PageID)
		{
			BuildPageID = BuildGraph->PageID;

			constexpr bool bPrimeCache = false;
			DocumentCache->SetBuildPageID(BuildPageID);
			if (bBroadcastDelegate)
			{
				DocumentDelegates->PageDelegates.OnPageSet.Broadcast({ BuildPageID });
			}
		}
		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

void FMetaSoundFrontendDocumentBuilder::SetTemplate(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> InConfig)
{
	using namespace Metasound::Frontend;

	GetDocumentChecked().Template = MoveTemp(InConfig);
	DocumentDelegates->OnDocumentTemplateChanged.Broadcast({ EDocumentTemplateChangeType::Struct });
}

void FMetaSoundFrontendDocumentBuilder::SetTemplateProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertyFunction)
{
	using namespace Metasound::Frontend;
	SetPropertyFunction(GetDocumentChecked().Template);
	DocumentDelegates->OnDocumentTemplateChanged.Broadcast({ EDocumentTemplateChangeType::Property });
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetGraphInputAdvancedDisplay(const FName InputName, const bool InAdvancedDisplay)
{
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName))
	{
		FMetasoundFrontendClassInput& GraphInput = RootGraph.GetDefaultInterface().Inputs[*Index];
		if (GraphInput.Metadata.bIsAdvancedDisplay != InAdvancedDisplay)
		{
			GraphInput.Metadata.SetIsAdvancedDisplay(InAdvancedDisplay);
			Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.VertexID);
			DocumentDelegates->InterfaceDelegates.OnInputIsAdvancedDisplayChanged.Broadcast(*Index);
			return true;
		}
	}

	return false;
}

void FMetaSoundFrontendDocumentBuilder::SetIsGraphEditable(bool bIsEditable, const FGuid* PageID) const
{
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	Document.RootGraph.IterateGraphPages([bIsEditable, &PageID](FMetasoundFrontendGraph& PageGraph)
	{
		if (!PageID || PageGraph.PageID == *PageID)
		{
			PageGraph.Style.bIsGraphEditable = bIsEditable;
		}
	});
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputAccessType(FName InputName, EMetasoundFrontendVertexAccessType AccessType)
{
	using namespace Metasound::Frontend;

	if (!ensureMsgf(AccessType != EMetasoundFrontendVertexAccessType::Unset, TEXT("Cannot set graph input access type to '%s'"), LexToString(AccessType)))
	{
		return false;
	}

	const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName);
	if (!Index)
	{
		return false;
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	FMetasoundFrontendClassInput& GraphInput = RootGraph.GetDefaultInterface().Inputs[*Index];

	if (GraphInput.AccessType != AccessType)
	{
		GraphInput.AccessType = AccessType;

		RootGraph.IterateGraphPages([this, &Document, &GraphInput, &AccessType](FMetasoundFrontendGraph& Graph)
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphInput.NodeID))
			{
				FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
				const FMetasoundFrontendVertex& NodeOutput = Node.Interface.Outputs.Last();
				IterateNodesConnectedWithVertex({ GraphInput.NodeID, NodeOutput.VertexID }, [this, &Document, &Graph, &AccessType](const FMetasoundFrontendEdge& Edge, FMetasoundFrontendNode& ConnectedNode)
				{
					if (const FMetasoundFrontendClass* ConnectedNodeClass = FindDependency(ConnectedNode.ClassID))
					{
						// If connected to an input template node, disconnect the template node from other nodes as the data type is
						// about to be mismatched.  Otherwise, direct connection to other nodes (i.e. at runtime when template
						// nodes aren't injected) forcefully remove to avoid data type mismatch.
						if (ConnectedNodeClass->Metadata.GetClassName() == FInputNodeTemplate::ClassName)
						{
#if WITH_EDITORONLY_DATA
							// Even if not disconnecting, need to bump the template node so that listeners
							// such as the editor know to re-evaluate internal node access type resolution
							// (in the case of the base MetaSound editor, this in turn redraws the pins
							// accordingly as it caches the AccessType for better performance)
							if (Graph.PageID == GetBuildPageID())
							{
								Document.Metadata.ModifyContext.AddNodeIDModified(ConnectedNode.GetID());
							}
#endif // WITH_EDITORONLY_DATA

							if (AccessType == EMetasoundFrontendVertexAccessType::Reference)
							{
								const FMetasoundFrontendVertex& ConnectedNodeOutput = ConnectedNode.Interface.Outputs.Last();
								IterateNodesConnectedWithVertex({ Edge.ToNodeID, ConnectedNodeOutput.VertexID }, [this, &Graph, &AccessType](const FMetasoundFrontendEdge& TempEdge, FMetasoundFrontendNode&)
								{
									const EMetasoundFrontendVertexAccessType ConnectedAccessType = GetNodeInputAccessType(TempEdge.ToNodeID, TempEdge.ToVertexID, &Graph.PageID);
									if (!FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(AccessType, ConnectedAccessType))
									{
										RemoveEdgeToNodeInput(TempEdge.ToNodeID, TempEdge.ToVertexID, &Graph.PageID);
									}
								}, Graph.PageID);
							}
						}
						else if (AccessType == EMetasoundFrontendVertexAccessType::Reference)
						{
							const EMetasoundFrontendVertexAccessType ConnectedAccessType = GetNodeInputAccessType(Edge.ToNodeID, Edge.ToVertexID, &Graph.PageID);
							if (!FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(AccessType, ConnectedAccessType))
							{
								RemoveEdgeToNodeInput(Edge.ToNodeID, Edge.ToVertexID, &Graph.PageID);
							}
						}
					}
				}, Graph.PageID);
			}
		});

		const bool bNodeConformed = ConformGraphInputNodeToClass(GraphInput);
		if (!bNodeConformed)
		{
			return false;
		}

		DocumentDelegates->InterfaceDelegates.OnInputIsConstructorPinChanged.Broadcast(*Index);

#if WITH_EDITORONLY_DATA
		RootGraph.GetDefaultInterface().UpdateChangeID();
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
#endif // WITH_EDITORONLY_DATA
	}

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputDataType(FName InputName, FName DataType)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	if (IDataTypeRegistry::Get().IsRegistered(DataType))
	{
		const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName);
		if (!Index)
		{
			return false;
		}

		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
		FMetasoundFrontendClassInput& GraphInput = RootGraph.GetDefaultInterface().Inputs[*Index];
		if (GraphInput.TypeName != DataType)
		{
			// If input is currently a constructor (value) type, but is being set to a data type 
			// that does not support constructor types, set it back to reference
			if (GraphInput.AccessType == EMetasoundFrontendVertexAccessType::Value)
			{
				FDataTypeRegistryInfo Info;
				if (IDataTypeRegistry::Get().GetDataTypeInfo(DataType, Info))
				{
					if (!Info.bIsConstructorType)
					{
						GraphInput.AccessType = EMetasoundFrontendVertexAccessType::Reference;
					}
				}
			}
			GraphInput.TypeName = DataType;
			TArray<const FGuid> PageIDsModified;
			GraphInput.IterateDefaults([&](const FGuid& PageID, FMetasoundFrontendLiteral&)
			{
				PageIDsModified.Add(PageID);
			});
			GraphInput.ResetDefaults();

			RootGraph.IterateGraphPages([this, &DataType, &GraphInput](FMetasoundFrontendGraph& Graph)
			{
				const Frontend::IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
				if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphInput.NodeID))
				{
					FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
					FMetasoundFrontendVertex& NodeOutput = Node.Interface.Outputs.Last();
					IterateNodesConnectedWithVertex({ GraphInput.NodeID, NodeOutput.VertexID }, [this, &Graph, &DataType](const FMetasoundFrontendEdge& Edge, FMetasoundFrontendNode& ConnectedNode)
					{
						const FMetasoundFrontendClass* ConnectedNodeClass = FindDependency(ConnectedNode.ClassID);
						if (ensure(ConnectedNodeClass))
						{
							// If connected to an input template node, disconnect the template node from other nodes as the data type is
							// about to be mismatched.  Otherwise, direct connection to other nodes (i.e. at runtime when template
							// nodes aren't injected) forcefully remove to avoid data type mismatch.
							if (ConnectedNodeClass->Metadata.GetClassName() == Frontend::FInputNodeTemplate::ClassName)
							{
								RemoveEdgesFromNodeOutput(Edge.ToNodeID, ConnectedNode.Interface.Outputs.Last().VertexID, &Graph.PageID);
								ConnectedNode.Interface.Inputs.Last().TypeName = DataType;
								ConnectedNode.Interface.Outputs.Last().TypeName = DataType;
							}
							else
							{
								RemoveEdgeToNodeInput(Edge.ToNodeID, Edge.ToVertexID, &Graph.PageID);
							}
						}
					}, Graph.PageID);
				}
			});

			const bool bNodeConformed = ConformGraphInputNodeToClass(GraphInput);
			if (!bNodeConformed)
			{
				return false;
			}

			for (const FGuid& PageID : PageIDsModified)
			{
				DocumentDelegates->InterfaceDelegates.OnInputDefaultChanged.Broadcast(
				FDocumentArrayPagedInputArgs
				{
					.Index = *Index,
					.PageID = PageID,
					.ChangeType = EMetaSoundLiteralChangeType::Reset
				});
			}
			DocumentDelegates->InterfaceDelegates.OnInputDataTypeChanged.Broadcast(*Index);

			RemoveUnusedDependencies();

#if WITH_EDITORONLY_DATA
			ClearMemberMetadata(GraphInput.NodeID);
			RootGraph.GetDefaultInterface().UpdateChangeID();
			Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
			Document.Metadata.ModifyContext.AddNodeIDModified(GraphInput.NodeID);
#endif // WITH_EDITORONLY_DATA
		}
	}

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputDefault(FName InputName, FMetasoundFrontendLiteral InDefaultLiteral, const FGuid* InPageID, EMetaSoundLiteralChangeType InChangeType)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	auto NameMatchesInput = [&InputName](const FMetasoundFrontendClassInput& Input) { return Input.Name == InputName; };
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClassInput>& Inputs = Document.RootGraph.GetDefaultInterface().Inputs;

	const int32 Index = Inputs.IndexOfByPredicate(NameMatchesInput);
	if (Index != INDEX_NONE)
	{
		FMetasoundFrontendClassInput& Input = Inputs[Index];
		if (Frontend::IDataTypeRegistry::Get().IsLiteralTypeSupported(Input.TypeName, InDefaultLiteral.GetType()))
		{
			const FGuid PageID = InPageID ? *InPageID : BuildPageID;
			bool bFound = false;
			Input.IterateDefaults([&bFound, &PageID, &InDefaultLiteral](const FGuid& InputPageID, FMetasoundFrontendLiteral& InputLiteral)
			{
				if (!bFound && InputPageID == PageID)
				{
					bFound = true;
					InputLiteral = MoveTemp(InDefaultLiteral);
				}
			});
			if (!bFound)
			{
				Input.AddDefault(PageID) = MoveTemp(InDefaultLiteral);
			}
			DocumentDelegates->InterfaceDelegates.OnInputDefaultChanged.Broadcast(
				FDocumentArrayPagedInputArgs
				{
					.Index = Index,
					.PageID = PageID,
					.ChangeType = InChangeType
				});

			if (const FTemplateVertexMetadata* ConfigVertexMetadata = FindConstVertexMetadata<FTemplateVertexMetadata>(Input.NodeID))
			{
				const_cast<FMetaSoundFrontendDocumentTemplateVertexMetadata*>(ConfigVertexMetadata)->OnDefaultUpdated(*this, Input);
			}

			return true;
		}
		UE_LOGF(LogMetaSound, Error, "Attempting to set graph input of type '%ls' with unsupported literal type", *Input.TypeName.ToString());
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputDefaults(FName InputName, TArray<FMetasoundFrontendClassInputDefault> Defaults, EMetaSoundLiteralChangeType InChangeType)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	auto NameMatchesInput = [&InputName](const FMetasoundFrontendClassInput& Input) { return Input.Name == InputName; };
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	TArray<FMetasoundFrontendClassInput>& Inputs = Document.RootGraph.GetDefaultInterface().Inputs;

	const int32 Index = Inputs.IndexOfByPredicate(NameMatchesInput);
	if (Index != INDEX_NONE)
	{
		FMetasoundFrontendClassInput& Input = Inputs[Index];
		TArray<FGuid> PageIDsModified;

		// If defaults are being removed or reset, compare with existing ones to report the correct page id being modified
		if (InChangeType == EMetaSoundLiteralChangeType::Remove)
		{
			Input.IterateDefaults([&](const FGuid& PageID, const FMetasoundFrontendLiteral& Literal)
			{
				PageIDsModified.Emplace(PageID);
			});
			for (const FMetasoundFrontendClassInputDefault& Default : Defaults)
			{
				PageIDsModified.Remove(Default.PageID);
			}
		}
		else if (InChangeType == EMetaSoundLiteralChangeType::Set || InChangeType == EMetaSoundLiteralChangeType::Reset)
		{
			Input.IterateDefaults([&](const FGuid& PageID, const FMetasoundFrontendLiteral& Literal)
			{
				PageIDsModified.Emplace(PageID);
			});
			for (const FMetasoundFrontendClassInputDefault& Default : Defaults)
			{
				if (const FMetasoundFrontendLiteral* ExistingLiteral = Input.FindConstDefault(Default.PageID))
				{
					if (Default.Literal.IsEqual(*ExistingLiteral))
					{
						PageIDsModified.Remove(Default.PageID);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Possible unhandled EMetaSoundLiteralChangeType in SetGraphInputDefaults");
		}
		
		bool bAllSupported = Algo::AllOf(Defaults, [&Input](const FMetasoundFrontendClassInputDefault& Default)
		{
			return Frontend::IDataTypeRegistry::Get().IsLiteralTypeSupported(Input.TypeName, Default.Literal.GetType());
		});
		if (bAllSupported)
		{
			Input.SetDefaults(MoveTemp(Defaults));
			for (const FGuid& PageID : PageIDsModified)
			{
				DocumentDelegates->InterfaceDelegates.OnInputDefaultChanged.Broadcast(
				FDocumentArrayPagedInputArgs
				{
					.Index = Index,
					.PageID = PageID,
					.ChangeType = InChangeType
				});
			}

			if (FDocumentVertexTemplate* DocTemplate = Document.Template.GetMutablePtr<FDocumentVertexTemplate>())
			{
				if (FTemplateVertexMetadata* VertexMetadata = DocTemplate->FindVertexMetadata(Input.NodeID))
				{
					VertexMetadata->OnDefaultUpdated(*this, Input);
				}
			}

			return true;
		}
		UE_LOGF(LogMetaSound, Error, "Attempting to set graph input of type '%ls' with unsupported literal type(s)", *Input.TypeName.ToString());
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetGraphInputDescription(FName InputName, FText Description)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassInput& GraphInput = Document.RootGraph.GetDefaultInterface().Inputs[*Index];
		GraphInput.Metadata.SetDescription(Description);
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnInputDescriptionChanged.Broadcast(*Index);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputDisplayName(FName InputName, FText DisplayName)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassInput& GraphInput = Document.RootGraph.GetDefaultInterface().Inputs[*Index];
		GraphInput.Metadata.SetDisplayName(DisplayName);
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnInputDisplayNameChanged.Broadcast(*Index);
		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputInheritsDefault(FName InName, bool bInputInheritsDefault, bool bForceUpdate)
{
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputsInheritingDefault(TSet<FName>&& InNames)
{
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphInputName(FName InputName, FName NewName)
{
	using namespace Metasound::Frontend;

	if (InputName == NewName)
	{
		return true;
	}

	const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName);
	if (!Index)
	{
		return false;
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;

	FMetasoundFrontendClassInput& GraphInput = RootGraph.GetDefaultInterface().Inputs[*Index];
	GraphInput.Name = NewName;

	RootGraph.IterateGraphPages([this, &GraphInput, &NewName](FMetasoundFrontendGraph& Graph)
	{
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
		if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphInput.NodeID))
		{
			FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
			Node.Name = NewName;
			for (FMetasoundFrontendVertex& Vertex : Node.Interface.Inputs)
			{
				Vertex.Name = NewName;
			}
			for (FMetasoundFrontendVertex& Vertex : Node.Interface.Outputs)
			{
				Vertex.Name = NewName;
			}
		}
	});

	DocumentDelegates->InterfaceDelegates.OnInputNameChanged.Broadcast(InputName, NewName);

#if WITH_EDITORONLY_DATA
	RootGraph.GetDefaultInterface().UpdateChangeID();
	Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
#endif // WITH_EDITORONLY_DATA

	return true;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetGraphInputSortOrderIndex(const FName InputName, const int32 InSortOrderIndex)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindInputIndex(InputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassInput& GraphInput = Document.RootGraph.GetDefaultInterface().Inputs[*Index];
		GraphInput.Metadata.SortOrderIndex = InSortOrderIndex;
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphInput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnInputSortOrderIndexChanged.Broadcast(*Index);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputSortOrderIndex(const FName OutputName, const int32 InSortOrderIndex)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassOutput& GraphOutput = Document.RootGraph.GetDefaultInterface().Outputs[*Index];
		GraphOutput.Metadata.SortOrderIndex = InSortOrderIndex;
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnOutputSortOrderIndexChanged.Broadcast(*Index);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputAdvancedDisplay(const FName OutputName, const bool InAdvancedDisplay)
{
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName))
	{
		FMetasoundFrontendClassOutput& GraphOutput = Document.RootGraph.GetDefaultInterface().Outputs[*Index];
		if (GraphOutput.Metadata.bIsAdvancedDisplay != InAdvancedDisplay)
		{
			GraphOutput.Metadata.SetIsAdvancedDisplay(InAdvancedDisplay);
			Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.VertexID);
			DocumentDelegates->InterfaceDelegates.OnOutputIsAdvancedDisplayChanged.Broadcast(*Index);
			return true;
		}
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputAccessType(FName OutputName, EMetasoundFrontendVertexAccessType AccessType)
{
	using namespace Metasound::Frontend;

	if (!ensureMsgf(AccessType != EMetasoundFrontendVertexAccessType::Unset, TEXT("Cannot set graph output access type to '%s'"), LexToString(AccessType)))
	{
		return false;
	}

	const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName);
	if (!Index)
	{
		return false;
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	FMetasoundFrontendClassOutput& GraphOutput = RootGraph.GetDefaultInterface().Outputs[*Index];
	if (GraphOutput.AccessType != AccessType)
	{
		GraphOutput.AccessType = AccessType;
		if (AccessType == EMetasoundFrontendVertexAccessType::Value)
		{
			RootGraph.IterateGraphPages([this, &GraphOutput, &AccessType](FMetasoundFrontendGraph& Graph)
			{
				const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
				if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphOutput.NodeID))
				{
					FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
					const FMetasoundFrontendVertex& NodeInput = Node.Interface.Inputs.Last();
					IterateNodesConnectedWithVertex({ GraphOutput.NodeID, NodeInput.VertexID }, [this, &Graph, &AccessType](const FMetasoundFrontendEdge& Edge, FMetasoundFrontendNode& ConnectedNode)
					{
						if (const FMetasoundFrontendClass* ConnectedNodeClass = FindDependency(ConnectedNode.ClassID))
						{
							const FMetasoundFrontendVertex& ConnectedNodeOutput = ConnectedNode.Interface.Outputs.Last();
							const EMetasoundFrontendVertexAccessType ConnectedAccessType = GetNodeOutputAccessType(ConnectedNode.GetID(), ConnectedNodeOutput.VertexID, &Graph.PageID);
							if (!FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(ConnectedAccessType, AccessType))
							{
								RemoveEdgeToNodeInput(Edge.ToNodeID, Edge.ToVertexID, &Graph.PageID);
							}
						}
					}, Graph.PageID);
				}
			});
		}

		const bool bNodeConformed = ConformGraphOutputNodeToClass(GraphOutput);
		if (!bNodeConformed)
		{
			return false;
		}

		DocumentDelegates->InterfaceDelegates.OnOutputIsConstructorPinChanged.Broadcast(*Index);

#if WITH_EDITORONLY_DATA
		RootGraph.GetDefaultInterface().UpdateChangeID();
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
#endif // WITH_EDITORONLY_DATA
	}

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputDataType(FName OutputName, FName DataType)
{
	using namespace Metasound::Frontend;

	if (!IDataTypeRegistry::Get().IsRegistered(DataType))
	{
		return false;
	}

	const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName);
	if (!Index)
	{
		return false;
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	FMetasoundFrontendClassOutput& GraphOutput = RootGraph.GetDefaultInterface().Outputs[*Index];
	if (GraphOutput.TypeName != DataType)
	{
		// If output is currently a constructor (value) type, but is being set to a data type 
		// that does not support constructor types, set it back to reference
		if (GraphOutput.AccessType == EMetasoundFrontendVertexAccessType::Value)
		{
			FDataTypeRegistryInfo Info;
			if (IDataTypeRegistry::Get().GetDataTypeInfo(DataType, Info))
			{
				if (!Info.bIsConstructorType)
				{
					GraphOutput.AccessType = EMetasoundFrontendVertexAccessType::Reference;
				}
			}
		}
		GraphOutput.TypeName = DataType;
		RootGraph.IterateGraphPages([this, &GraphOutput, &DataType](FMetasoundFrontendGraph& Graph)
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphOutput.NodeID))
			{
				FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];

				FMetasoundFrontendLiteral DefaultLiteral;
				DefaultLiteral.SetFromLiteral(IDataTypeRegistry::Get().CreateDefaultLiteral(DataType));
				FMetasoundFrontendVertex& NodeInput = Node.Interface.Inputs.Last();
				Node.InputLiterals = { FMetasoundFrontendVertexLiteral { NodeInput.VertexID, MoveTemp(DefaultLiteral) } };

				RemoveEdgeToNodeInput(GraphOutput.NodeID, NodeInput.VertexID, &Graph.PageID);
			}
		});

		const bool bNodeConformed = ConformGraphOutputNodeToClass(GraphOutput);
		if (!bNodeConformed)
		{
			return false;
		}

		DocumentDelegates->InterfaceDelegates.OnOutputDataTypeChanged.Broadcast(*Index);

#if WITH_EDITORONLY_DATA
		RootGraph.GetDefaultInterface().UpdateChangeID();
		ClearMemberMetadata(GraphOutput.NodeID);
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
#endif // WITH_EDITORONLY_DATA
	}

	return true;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputDescription(FName OutputName, FText Description)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassOutput& GraphOutput = Document.RootGraph.GetDefaultInterface().Outputs[*Index];
		GraphOutput.Metadata.SetDescription(Description);
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnOutputDescriptionChanged.Broadcast(*Index);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputDisplayName(FName OutputName, FText DisplayName)
{
	using namespace Metasound;

	if (const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName))
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		FMetasoundFrontendClassOutput& GraphOutput = Document.RootGraph.GetDefaultInterface().Outputs[*Index];
		GraphOutput.Metadata.SetDisplayName(DisplayName);
		Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
		DocumentDelegates->InterfaceDelegates.OnOutputDisplayNameChanged.Broadcast(*Index);
		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::SetGraphOutputName(FName OutputName, FName NewName)
{
	using namespace Metasound::Frontend;

	if (OutputName == NewName)
	{
		return true;
	}

	const int32* Index = DocumentCache->GetInterfaceCache().FindOutputIndex(OutputName);
	if (!Index)
	{
		return false;
	}

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
	FMetasoundFrontendClassInterface& Interface = GraphClass.GetDefaultInterface();
	Interface.UpdateChangeID();

	FMetasoundFrontendClassOutput& GraphOutput = Interface.Outputs[*Index];
	GraphOutput.Name = NewName;
	
	GraphClass.IterateGraphPages([this, &GraphOutput, &NewName](FMetasoundFrontendGraph& Graph)
	{
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
		if (const int32* NodeIndex = NodeCache.FindNodeIndex(GraphOutput.NodeID))
		{
			FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
			Node.Name = NewName;
			for (FMetasoundFrontendVertex& Vertex : Node.Interface.Inputs)
			{
				Vertex.Name = NewName;
			}
			for (FMetasoundFrontendVertex& Vertex : Node.Interface.Outputs)
			{
				Vertex.Name = NewName;
			}
		}
	});
	DocumentDelegates->InterfaceDelegates.OnOutputNameChanged.Broadcast(OutputName, NewName);
	
#if WITH_EDITORONLY_DATA
	GraphClass.GetDefaultInterface().UpdateChangeID();
	Document.Metadata.ModifyContext.AddMemberIDModified(GraphOutput.NodeID);
#endif // WITH_EDITORONLY_DATA

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphVariableDefault(FName VariableName, FMetasoundFrontendLiteral InDefaultLiteral, const FGuid* InPageID)
{
	using namespace Metasound;

	if (FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID))
	{
		if (Frontend::IDataTypeRegistry::Get().IsLiteralTypeSupported(Variable->TypeName, InDefaultLiteral.GetType()))
		{
			Variable->Literal = MoveTemp(InDefaultLiteral);
			return true;
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool FMetaSoundFrontendDocumentBuilder::SetGraphVariableDescription(FName VariableName, FText Description, const FGuid* InPageID)
{
	using namespace Metasound;

	if (FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID))
	{
		Variable->Description = Description;
		GetDocumentChecked().Metadata.ModifyContext.AddMemberIDModified(Variable->ID);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetGraphVariableDisplayName(FName VariableName, FText DisplayName, const FGuid* InPageID)
{
	using namespace Metasound;

	if (FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID))
	{
		Variable->DisplayName = DisplayName;
		GetDocumentChecked().Metadata.ModifyContext.AddMemberIDModified(Variable->ID);
		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

bool FMetaSoundFrontendDocumentBuilder::SetGraphVariableName(FName VariableName, FName NewName, const FGuid* InPageID)
{
	using namespace Metasound;

	if (FMetasoundFrontendVariable* Variable = FindGraphVariableInternal(VariableName, InPageID))
	{
		Variable->Name = NewName;
#if WITH_EDITORONLY_DATA
		GetDocumentChecked().Metadata.ModifyContext.AddMemberIDModified(Variable->ID);
#endif // WITH_EDITORONLY_DATA
		return true;
	}

	return false;
}

#if WITH_EDITOR
void FMetaSoundFrontendDocumentBuilder::SetDisplayName(const FText& InDisplayName)
{
	DocumentInterface->GetDocument().RootGraph.Metadata.SetDisplayName(InDisplayName);
	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}

void FMetaSoundFrontendDocumentBuilder::SetDescription(const FText& InDescription)
{
	DocumentInterface->GetDocument().RootGraph.Metadata.SetDescription(InDescription);
	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}

void FMetaSoundFrontendDocumentBuilder::SetKeywords(const TArray<FText>& InKeywords)
{
	DocumentInterface->GetDocument().RootGraph.Metadata.SetKeywords(InKeywords);
	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}

void FMetaSoundFrontendDocumentBuilder::SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy)
{
	DocumentInterface->GetDocument().RootGraph.Metadata.SetCategoryHierarchy(InCategoryHierarchy);
	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}

void FMetaSoundFrontendDocumentBuilder::SetIsDeprecated(const bool bInIsDeprecated)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DocumentInterface->GetDocument().RootGraph.Metadata.SetIsDeprecated(bInIsDeprecated);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DocumentDelegates->OnDocumentMetadataChanged.Broadcast();
}

void FMetaSoundFrontendDocumentBuilder::SetInputStyle(FMetasoundFrontendInterfaceStyle&& Style)
{
	FMetasoundFrontendClassInterface& RootGraphClassInterface = GetDocumentChecked().RootGraph.GetDefaultInterface();
	RootGraphClassInterface.SetInputStyle(MoveTemp(Style));
}

void FMetaSoundFrontendDocumentBuilder::SetOutputStyle(FMetasoundFrontendInterfaceStyle&& Style)
{
	FMetasoundFrontendClassInterface& RootGraphClassInterface = GetDocumentChecked().RootGraph.GetDefaultInterface();
	RootGraphClassInterface.SetOutputStyle(MoveTemp(Style));
}

void FMetaSoundFrontendDocumentBuilder::SetMemberMetadata(UMetaSoundFrontendMemberMetadata& NewMetadata)
{
	check(NewMetadata.MemberID.IsValid());

	TMap<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>& LiteralMetadata = GetDocumentChecked().Metadata.MemberMetadata;
	LiteralMetadata.Remove(NewMetadata.MemberID);
	LiteralMetadata.Add(NewMetadata.MemberID, &NewMetadata);
	DocumentDelegates->InterfaceDelegates.OnMemberMetadataSet.Broadcast(NewMetadata.MemberID);
}

bool FMetaSoundFrontendDocumentBuilder::SetNodeComment(const FGuid& InNodeID, FString&& InNewComment, const FGuid* InPageID)
{
	if (FMetasoundFrontendNode* Node = FindNodeInternal(InNodeID, InPageID))
	{
		Node->Style.Display.Comment = MoveTemp(InNewComment);
		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetNodeCommentVisible(const FGuid& InNodeID, bool bIsVisible, const FGuid* InPageID)
{
	if (FMetasoundFrontendNode* Node = FindNodeInternal(InNodeID, InPageID))
	{
		Node->Style.Display.bCommentVisible = bIsVisible;
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

bool FMetaSoundFrontendDocumentBuilder::SetNodeConfiguration(const FGuid& InNodeID, TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];
		// Node config derived type must match
		if (const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID))
		{
			const FNodeClassRegistryKey NodeClassKey(Class->Metadata);
			if (!INodeClassRegistry::Get()->IsCompatibleNodeConfiguration(NodeClassKey, InNodeConfiguration))
			{
#if !NO_LOGGING
				TInstancedStruct<FMetaSoundFrontendNodeConfiguration> RegisteredConfiguration = CreateFrontendNodeConfiguration(Class->Metadata);
				UE_LOGF(LogMetaSound, Warning, "Cannot set node configuration to type %ls, expected registered type %ls", *GetNameSafe(InNodeConfiguration.GetScriptStruct()), *GetNameSafe(RegisteredConfiguration.GetScriptStruct()));
#endif // !NO_LOGGING
				return false;
			}

			Node.Configuration = MoveTemp(InNodeConfiguration);
			return UpdateNodeInterfaceFromConfiguration(InNodeID, InPageID);
		}
	}
	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(PageID);
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	if (const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID))
	{
		FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];

		auto IsVertex = [&InVertexID](const FMetasoundFrontendVertex& Vertex) { return Vertex.VertexID == InVertexID; };
		int32 VertexIndex = Node.Interface.Inputs.IndexOfByPredicate(IsVertex);
		if (VertexIndex != INDEX_NONE)
		{
			FMetasoundFrontendVertexLiteral NewVertexLiteral;
			NewVertexLiteral.VertexID = InVertexID;
			NewVertexLiteral.Value = InLiteral;

			auto IsLiteral = [&InVertexID](const FMetasoundFrontendVertexLiteral& Literal) { return Literal.VertexID == InVertexID; };
			int32 LiteralIndex = Node.InputLiterals.IndexOfByPredicate(IsLiteral);
			if (LiteralIndex == INDEX_NONE)
			{
				LiteralIndex = Node.InputLiterals.Num();
				Node.InputLiterals.Add(MoveTemp(NewVertexLiteral));
			}
			else
			{
				Node.InputLiterals[LiteralIndex] = MoveTemp(NewVertexLiteral);
			}

			FNodeModifyDelegates& NodeDelegates = DocumentDelegates->FindGraphDelegatesChecked(PageID).NodeDelegates;
			const FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray& OnNodeInputLiteralSet = NodeDelegates.OnNodeInputLiteralSet;

#if WITH_EDITORONLY_DATA
			Document.Metadata.ModifyContext.AddNodeIDModified(Node.GetID());
#endif // WITH_EDITORONLY_DATA

			OnNodeInputLiteralSet.Broadcast(*NodeIndex, VertexIndex, LiteralIndex);
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
bool FMetaSoundFrontendDocumentBuilder::SetNodeLocation(const FGuid& InNodeID, const FVector2D& InLocation, const FGuid* InLocationGuid, const FGuid* InPageID)
{
	if (FMetasoundFrontendNode* Node = FindNodeInternal(InNodeID, InPageID))
	{
		FMetasoundFrontendNodeStyle& Style = Node->Style;
		if (InLocationGuid)
		{
			if (InLocationGuid->IsValid())
			{
				auto ExistingLocationWithOtherId = [&](const TPair<FGuid, FVector2D> Location) {
					return Location.Key != *InLocationGuid;
				};
				// If there are multiple locations or a single location with a nonmatching location guid (which would result in multiple locations after this location is added)
				if (Style.Display.Locations.Num() > 1 || Algo::AnyOf(Style.Display.Locations, ExistingLocationWithOtherId))
				{
					UE_LOGF(LogMetaSound, Warning, "Multiple locations per node no longer supported, removing other display locations for node ID '%ls'", *InNodeID.ToString());
					Style.Display.Locations.Empty();
				}

				Style.Display.Locations.FindOrAdd(*InLocationGuid) = InLocation;
				return true;
			}

			UE_LOGF(LogMetaSound, Display, "Invalid Location Guid no longer supported, resetting display location for node with ID '%ls'", *InNodeID.ToString());
		}

		if (Style.Display.Locations.IsEmpty())
		{
			Style.Display.Locations = { { FGuid::NewGuid(), InLocation } };
		}
		else
		{
			Algo::ForEach(Style.Display.Locations, [InLocation](TPair<FGuid, FVector2D>& Pair)
			{
				Pair.Value = InLocation;
			});
		}

		ensureMsgf(Style.Display.Locations.Num() <= 1, TEXT("Nodes should not have more than 1 location post ed graph migration (doc version 1.12)."));

		return true;
	}

	return false;
}

bool FMetaSoundFrontendDocumentBuilder::SetNodeUnconnectedPinsHidden(const FGuid& InNodeID, const bool bUnconnectedPinsHidden, const FGuid* InPageID)
{
	if (FMetasoundFrontendNode* Node = FindNodeInternal(InNodeID, InPageID))
	{
		Node->Style.bUnconnectedPinsHidden = bUnconnectedPinsHidden;
		return true;
	}

	return false;
}

const FMetasoundFrontendNodeStyle* FMetaSoundFrontendDocumentBuilder::GetNodeStyle(const FGuid& InNodeID, const FGuid* InPageID) const
{
	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		return &Node->Style;
	}

	return nullptr;
}

FText FMetaSoundFrontendDocumentBuilder::GetNodeTitle(const FGuid& InNodeID, const FGuid* InPageID) const
{
	using namespace Metasound::Frontend;

	if (const FMetasoundFrontendNode* Node = FindNode(InNodeID, InPageID))
	{
		if (const FMetasoundFrontendClass* Class = FindDependency(Node->ClassID))
		{
			const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
			switch (ClassType)
			{
				case EMetasoundFrontendClassType::External:
				case EMetasoundFrontendClassType::Template:
				{
					const FText DisplayName = Class->Metadata.GetDisplayName();
					if (!DisplayName.IsEmptyOrWhitespace())
					{
						return DisplayName;
					}

					const FTopLevelAssetPath Path = IMetaSoundAssetManager::GetChecked().FindAssetPath(FMetaSoundAssetKey(Class->Metadata));
					if (Path.IsValid())
					{
						return FText::FromName(Path.GetAssetName());
					}

					return FText::FromName(Class->Metadata.GetClassName().Name);
				}

				case EMetasoundFrontendClassType::Input:
				{
					return NSLOCTEXT("MetasoundFrontend", "InputNode_Title", "Input");
				}

				case EMetasoundFrontendClassType::Output:
				{
					return NSLOCTEXT("MetasoundFrontend", "OutputNode_Title", "Output");
				}

				case EMetasoundFrontendClassType::Variable:
				case EMetasoundFrontendClassType::VariableDeferredAccessor:
				case EMetasoundFrontendClassType::VariableAccessor:
				case EMetasoundFrontendClassType::VariableMutator:
				{
					return NSLOCTEXT("MetasoundFrontend", "OutputNode_Variable", "Variable");
				}

				case EMetasoundFrontendClassType::Graph:
				case EMetasoundFrontendClassType::Invalid:
				case EMetasoundFrontendClassType::Literal:
				default:
				{
					checkNoEntry();
				}
				break;
			}
		}
	}

	return { };
}
#endif // WITH_EDITOR

void FMetaSoundFrontendDocumentBuilder::SetVersionNumber(const FMetasoundFrontendVersionNumber& InDocumentVersionNumber)
{
	GetDocumentChecked().Metadata.Version.Number = InDocumentVersionNumber;
}

bool FMetaSoundFrontendDocumentBuilder::SpliceVariableNodeFromStack(const FGuid& InNodeID, const FGuid& InPageID)
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	bool bSpliced = false;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraph& Graph = Document.RootGraph.FindGraphChecked(InPageID);
	const IDocumentGraphEdgeCache& EdgeCache = DocumentCache->GetEdgeCache(InPageID);
	FMetasoundFrontendVertexHandle FromVariableVertexHandle;
	{
		// InputVertex may be null if provided ID corresponds to the base variable node (which is always at head of stack and has no inputs)
		if (const FMetasoundFrontendVertex* InputVertex = FindNodeInput(InNodeID, VariableNames::InputVariableName, &InPageID))
		{
			if (const int32* InputEdgeIndex = EdgeCache.FindEdgeIndexToNodeInput(InNodeID, InputVertex->VertexID))
			{
				FromVariableVertexHandle = Graph.Edges[*InputEdgeIndex].GetFromVertexHandle();
				bSpliced = RemoveEdgeToNodeInput(InNodeID, InputVertex->VertexID, &InPageID);
			}
		}
	}

	if (const FMetasoundFrontendVertex* OutputVertex = FindNodeOutput(InNodeID, VariableNames::OutputVariableName, &InPageID))
	{
		TArray<FMetasoundFrontendVertexHandle> ToVertexHandles;
		const TArrayView<const int32> OutputEdgeIndices = EdgeCache.FindEdgeIndicesFromNodeOutput(InNodeID, OutputVertex->VertexID);
		Algo::Transform(OutputEdgeIndices, ToVertexHandles, [&Graph](const int32& VertIndex) { return Graph.Edges[VertIndex].GetToVertexHandle(); });

		bSpliced |= RemoveEdgesFromNodeOutput(InNodeID, OutputVertex->VertexID, &InPageID);

		if (FromVariableVertexHandle.IsSet())
		{
			for (const FMetasoundFrontendVertexHandle& ToHandle : ToVertexHandles)
			{
				AddEdge(FMetasoundFrontendEdge
				{
					FromVariableVertexHandle.NodeID,
					FromVariableVertexHandle.VertexID,
					ToHandle.NodeID,
					ToHandle.VertexID
				}, &InPageID);
			}
		}
	}

	return bSpliced;
}

bool FMetaSoundFrontendDocumentBuilder::SwapGraphInput(const FMetasoundFrontendClassVertex& InExistingInputVertex, const FMetasoundFrontendClassVertex& InNewInputVertex)
{
	using namespace Metasound::Frontend;

	// 1. Check if equivalent and early out if functionally do not match
	{
		const FMetasoundFrontendClassInput* ClassInput = FindGraphInput(InExistingInputVertex.Name);
		if (!ClassInput || !FMetasoundFrontendVertex::IsFunctionalEquivalent(*ClassInput, InExistingInputVertex))
		{
			return false;
		}
	}

	const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();

#if WITH_EDITORONLY_DATA
	using FPageNodeLocations = TMap<FGuid, FVector2D>;
	TMap<FGuid, FPageNodeLocations> PageNodeLocations;
#endif // WITH_EDITORONLY_DATA

	// 2. Gather data from existing member/node needed to swap
	TMultiMap<FGuid, FMetasoundFrontendEdge> RemovedEdgesPerPage;

	const FMetasoundFrontendClassInput* ExistingInputClass = InterfaceCache.FindInput(InExistingInputVertex.Name);
	checkf(ExistingInputClass, TEXT("'SwapGraphInput' failed to find original graph input"));
	const FGuid NodeID = ExistingInputClass->NodeID;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
		const FMetasoundFrontendNode* ExistingInputNode = NodeCache.FindNode(NodeID);
		check(ExistingInputNode);

#if WITH_EDITORONLY_DATA
		PageNodeLocations.Add(Graph.PageID, ExistingInputNode->Style.Display.Locations);
#endif // WITH_EDITORONLY_DATA

		const FGuid VertexID = ExistingInputNode->Interface.Outputs.Last().VertexID;
		TArray<const FMetasoundFrontendEdge*> Edges = DocumentCache->GetEdgeCache(Graph.PageID).FindEdges(NodeID, VertexID);
		Algo::Transform(Edges, RemovedEdgesPerPage, [PageID = Graph.PageID](const FMetasoundFrontendEdge* Edge)
		{
			return TPair<FGuid, FMetasoundFrontendEdge>(PageID, *Edge);
		});
	});

	// 3. Remove existing graph vertex
	{
		// Access & Data Types will be preserved, so no reason to remove template nodes.
		// (Removal can additionally cause associated edges to be removed rendering cached
		// RemovedEdgesPerPage above to be stale, so leaving template input nodes in place
		// preserves that data's validity.
		constexpr bool bRemoveTemplateInputNodes = false;
		const bool bRemovedVertex = RemoveGraphInput(InExistingInputVertex.Name, bRemoveTemplateInputNodes);
		checkf(bRemovedVertex, TEXT("Failed to swap MetaSound input expected to exist"));
	}

	// 4. Add new graph vertex
	FGuid VertexID;
	{
		FMetasoundFrontendClassInput NewInput = InNewInputVertex;
		NewInput.NodeID = NodeID;
#if WITH_EDITORONLY_DATA
		NewInput.Metadata.SetSerializeText(InExistingInputVertex.Metadata.GetSerializeText());
#endif // WITH_EDITORONLY_DATA

		const FMetasoundFrontendNode* NewInputNode = AddGraphInput(MoveTemp(NewInput));
		checkf(NewInputNode, TEXT("Failed to add new Input node when swapping graph inputs"));
		checkf(NewInputNode->GetID() == NodeID, TEXT("Expected new node added to build graph to have same ID as provided input"));
		VertexID = NewInputNode->Interface.Outputs.Last().VertexID;
	}

	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
#if WITH_EDITORONLY_DATA
		// 5a. Add to new copy existing node locations
		if (const FPageNodeLocations* Locations = PageNodeLocations.Find(Graph.PageID))
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			const int32* NodeIndex = NodeCache.FindNodeIndex(NodeID);
			checkf(NodeIndex, TEXT("Cache was not updated to reflect newly added input node"));
			FMetasoundFrontendNode& NewNode = Graph.Nodes[*NodeIndex];
			NewNode.Style.Display.Locations = *Locations;
		}
#endif // WITH_EDITORONLY_DATA

		// 5b. Add to new copy existing node edges
		TArray<FMetasoundFrontendEdge> RemovedEdges;
		RemovedEdgesPerPage.MultiFind(Graph.PageID, RemovedEdges);
		for (const FMetasoundFrontendEdge& RemovedEdge : RemovedEdges)
		{
			FMetasoundFrontendEdge NewEdge = RemovedEdge;
			NewEdge.FromNodeID = NodeID;
			NewEdge.FromVertexID = VertexID;
			AddEdge(MoveTemp(NewEdge), &Graph.PageID);
		}
	});

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::SwapGraphOutput(const FMetasoundFrontendClassVertex& InExistingOutputVertex, const FMetasoundFrontendClassVertex& InNewOutputVertex)
{
	using namespace Metasound::Frontend;

	// 1. Check if equivalent and early out if functionally do not match
	{
		const FMetasoundFrontendClassOutput* ClassOutput = FindGraphOutput(InExistingOutputVertex.Name);
		if (!ClassOutput || !FMetasoundFrontendVertex::IsFunctionalEquivalent(*ClassOutput, InExistingOutputVertex))
		{
			return false;
		}
	}

	const IDocumentGraphInterfaceCache& InterfaceCache = DocumentCache->GetInterfaceCache();

#if WITH_EDITORONLY_DATA
	using FPageNodeLocations = TMap<FGuid, FVector2D>;
	TMap<FGuid, FPageNodeLocations> PageNodeLocations;
#endif // WITH_EDITORONLY_DATA

	// 2. Gather data from existing page member/node needed to swap
	TMultiMap<FGuid, FMetasoundFrontendEdge> RemovedEdgesPerPage;

	const FMetasoundFrontendClassOutput* ExistingOutputClass = InterfaceCache.FindOutput(InExistingOutputVertex.Name);
	checkf(ExistingOutputClass, TEXT("'SwapGraphOutput' failed to find original graph output"));
	const FGuid NodeID = ExistingOutputClass->NodeID;

	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	FMetasoundFrontendGraphClass& RootGraph = Document.RootGraph;
	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
		const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
		const FMetasoundFrontendNode* ExistingOutputNode = NodeCache.FindNode(NodeID);
		check(ExistingOutputNode);

#if WITH_EDITORONLY_DATA
		PageNodeLocations.Add(Graph.PageID, ExistingOutputNode->Style.Display.Locations);
#endif // WITH_EDITORONLY_DATA

		const FGuid VertexID = ExistingOutputNode->Interface.Inputs.Last().VertexID;
		TArray<const FMetasoundFrontendEdge*> Edges = DocumentCache->GetEdgeCache(Graph.PageID).FindEdges(NodeID, VertexID);
		Algo::Transform(Edges, RemovedEdgesPerPage, [PageID = Graph.PageID](const FMetasoundFrontendEdge* Edge)
		{
			return TPair<FGuid, FMetasoundFrontendEdge>(PageID, *Edge);
		});
	});

	// 3. Remove existing graph vertex
	{
		const bool bRemovedVertex = RemoveGraphOutput(InExistingOutputVertex.Name);
		checkf(bRemovedVertex, TEXT("Failed to swap output expected to exist while swapping MetaSound outputs"));
	}
	
	// 4. Add new graph vertex
	FGuid VertexID;
	{
		FMetasoundFrontendClassOutput NewOutput = InNewOutputVertex;
		NewOutput.NodeID = NodeID;
#if WITH_EDITORONLY_DATA
		NewOutput.Metadata.SetSerializeText(InExistingOutputVertex.Metadata.GetSerializeText());
#endif // WITH_EDITORONLY_DATA

		const FMetasoundFrontendNode* NewOutputNode = AddGraphOutput(MoveTemp(NewOutput));
		checkf(NewOutputNode, TEXT("Failed to add new output node when swapping graph outputs"));
		checkf(NewOutputNode->GetID() == NodeID, TEXT("Expected new node added to build graph to have same ID as provided output"));
		VertexID = NewOutputNode->Interface.Inputs.Last().VertexID;
	}

	Document.RootGraph.IterateGraphPages([&](FMetasoundFrontendGraph& Graph)
	{
#if WITH_EDITORONLY_DATA
		// 5a. Add to new copy existing node locations
		if (const FPageNodeLocations* Locations = PageNodeLocations.Find(Graph.PageID))
		{
			const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(Graph.PageID);
			const int32* NodeIndex = NodeCache.FindNodeIndex(NodeID);
			checkf(NodeIndex, TEXT("Cache was not updated to reflect newly added output node"));
			FMetasoundFrontendNode& NewNode = Graph.Nodes[*NodeIndex];
			NewNode.Style.Display.Locations = *Locations;
		}
#endif // WITH_EDITORONLY_DATA

		// 5b. Add to new copy existing node edges
		TArray<FMetasoundFrontendEdge> RemovedEdges;
		RemovedEdgesPerPage.MultiFind(Graph.PageID, RemovedEdges);
		for (const FMetasoundFrontendEdge& RemovedEdge : RemovedEdges)
		{
			FMetasoundFrontendEdge NewEdge = RemovedEdge;
			NewEdge.ToNodeID = NodeID;
			NewEdge.ToVertexID = VertexID;
			AddEdge(MoveTemp(NewEdge), &Graph.PageID);
		}
	});

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::UpdateNodeInterfaceFromConfiguration(const FGuid& InNodeID, const FGuid* InPageID)
{
	using namespace Metasound::Frontend;

	const FGuid& PageID = InPageID ? *InPageID : BuildPageID;
	const IDocumentGraphNodeCache& NodeCache = DocumentCache->GetNodeCache(PageID);
	const int32* NodeIndex = NodeCache.FindNodeIndex(InNodeID);
	if (!NodeIndex)
	{
		return false;
	}

	FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(PageID);
	FMetasoundFrontendNode& Node = Graph.Nodes[*NodeIndex];

	const FMetasoundFrontendClass* Class = FindDependency(Node.ClassID);
	check(Class);

	// Update class interface override
	if (const FMetaSoundFrontendNodeConfiguration* ConfigurationPtr = Node.Configuration.GetPtr())
	{
		Node.ClassInterfaceOverride = ConfigurationPtr->OverrideDefaultInterface(*Class);
	}
	else
	{
		// Set class interface override back to default if no node configuration
		Node.ClassInterfaceOverride = TInstancedStruct<FMetasoundFrontendClassInterface>();
	}

	// Update node interface
	const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(Node);

	auto DisconnectInput = [this, &InNodeID, &InPageID](const FMetasoundFrontendVertex& InNodeInput)
	{
		RemoveEdgeToNodeInput(InNodeID, InNodeInput.VertexID, InPageID);
	};

	auto DisconnectOutput = [this, &InNodeID, &InPageID](const FMetasoundFrontendVertex& InNodeOutput)
	{
		RemoveEdgesFromNodeOutput(InNodeID, InNodeOutput.VertexID, InPageID);
	};

	const bool bInterfaceUpdated = Node.Interface.Update(ClassInterface, DisconnectInput, DisconnectOutput);
	DocumentDelegates->FindGraphDelegatesChecked(PageID).NodeDelegates.OnNodeConfigurationUpdated.Broadcast(*NodeIndex);

#if WITH_EDITORONLY_DATA
	if (bInterfaceUpdated)
	{
		GetDocumentChecked().Metadata.ModifyContext.AddNodeIDModified(InNodeID);
	}
#endif // WITH_EDITORONLY_DATA

	return true;
}

bool FMetaSoundFrontendDocumentBuilder::UnlinkVariableNode(const FGuid& InNodeID, const FGuid& InPageID)
{
	auto IsNodeID = [&InNodeID](const FGuid& TestID) { return TestID == InNodeID; };

	FMetasoundFrontendGraph& Graph = GetDocumentChecked().RootGraph.FindGraphChecked(InPageID);
	for (FMetasoundFrontendVariable& Variable : Graph.Variables)
	{
		if (Variable.MutatorNodeID == InNodeID)
		{
			Variable.MutatorNodeID = FGuid();
			SpliceVariableNodeFromStack(InNodeID, InPageID);
			return true;
		}

		if (Variable.VariableNodeID == InNodeID)
		{
			Variable.VariableNodeID = FGuid();
			SpliceVariableNodeFromStack(InNodeID, InPageID);
			return true;
		}

		// Removal must maintain array order to preserve head/tail positions in stack
		const bool bRemovedDeferredNode = Variable.DeferredAccessorNodeIDs.RemoveAll(IsNodeID) > 0;
		if (bRemovedDeferredNode)
		{
			SpliceVariableNodeFromStack(InNodeID, InPageID);
			return true;
		}
		
		// Removal must maintain array order to preserve head/tail positions in stack
		const bool bRemovedAccessorNode = Variable.AccessorNodeIDs.RemoveAll(IsNodeID) > 0;
		if (bRemovedAccessorNode)
		{
			SpliceVariableNodeFromStack(InNodeID, InPageID);
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
bool FMetaSoundFrontendDocumentBuilder::SynchronizeDependencyMetadata(TArray<const FMetasoundFrontendClass*>* InOutModifiedClasses)
{
	using namespace Metasound::Frontend;

	if (InOutModifiedClasses)
	{
		InOutModifiedClasses->Reset();
	}

	bool bSuccess = true;
	FMetasoundFrontendDocument& Document = GetDocumentChecked();
	for (FMetasoundFrontendClass& Dependency : Document.Dependencies)
	{
		FMetasoundFrontendClass RegistryVersion;
		const FNodeRegistryKey RegistryKey(Dependency.Metadata);
		bool bFoundRegisteredClass = INodeClassRegistry::Get()->FindFrontendClassFromRegistered(RegistryKey, RegistryVersion);
		if (bFoundRegisteredClass)
		{
			if (Dependency.Metadata.GetChangeID() != RegistryVersion.Metadata.GetChangeID())
			{
				Dependency.Metadata = MoveTemp(RegistryVersion.Metadata);
				if (InOutModifiedClasses)
				{
					InOutModifiedClasses->Add(&Dependency);
				}
			}
		}
		else
		{
			bSuccess = false;
		}
	}
	return bSuccess;
}

bool FMetaSoundFrontendDocumentBuilder::UpdateDependencyRegistryData(const TMap<Metasound::Frontend::FNodeRegistryKey, Metasound::Frontend::FNodeRegistryKey>& OldToNewClassKeys)
{
	using namespace Metasound::Frontend;

	bool bUpdated = false;
	if (DocumentDelegates.IsValid())
	{
		FMetasoundFrontendDocument& Document = GetDocumentChecked();
		for (FMetasoundFrontendClass& Dependency : Document.Dependencies)
		{
			const FNodeRegistryKey OldKey(Dependency.Metadata);
			if (const FNodeRegistryKey* NewKey = OldToNewClassKeys.Find(OldKey))
			{
				if (Dependency.Metadata.GetType() == EMetasoundFrontendClassType::External)
				{
					bUpdated = true;
					const int32* DependencyIndex = DocumentCache->FindDependencyIndex(Dependency.ID);
					check(DependencyIndex);
					DocumentDelegates->OnRenamingDependencyClass.Broadcast(*DependencyIndex, NewKey->ClassName);
					Dependency.Metadata.SetType(NewKey->Type);
					Dependency.Metadata.SetClassName(NewKey->ClassName);
					Dependency.Metadata.SetVersion(NewKey->Version);
				}
			}
		}

#if WITH_EDITORONLY_DATA
		if (bUpdated)
		{
			Document.Metadata.ModifyContext.SetDocumentModified();
		}
#endif // WITH_EDITORONLY_DATA
	}

	return bUpdated;
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
FMetasoundFrontendDocument& FMetaSoundFrontendDocumentBuilder::IPropertyVersionTransform::GetDocumentUnsafe(const FMetaSoundFrontendDocumentBuilder& Builder)
{
	return Builder.GetDocumentChecked();
}
#endif // WITH_EDITORONLY_DATA
