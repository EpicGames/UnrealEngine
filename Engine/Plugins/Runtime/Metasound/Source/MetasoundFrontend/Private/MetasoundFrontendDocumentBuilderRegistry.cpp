// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocumentBuilderRegistry.h"

#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorBuilderSettings.h"


namespace Metasound::Frontend
{
	namespace DocumentInterfacePrivate
	{
		TUniquePtr<IDocumentBuilderRegistry> Instance;
	}

	IDocumentBuilderRegistry* IDocumentBuilderRegistry::Get()
	{
		return DocumentInterfacePrivate::Instance.Get();
	}

	IDocumentBuilderRegistry& IDocumentBuilderRegistry::GetChecked()
	{
		check(DocumentInterfacePrivate::Instance.IsValid());
		return *DocumentInterfacePrivate::Instance.Get();
	}

	void IDocumentBuilderRegistry::Deinitialize()
	{
		check(DocumentInterfacePrivate::Instance.IsValid());
		DocumentInterfacePrivate::Instance.Reset();
	}

	void IDocumentBuilderRegistry::Initialize(TUniquePtr<IDocumentBuilderRegistry>&& InInstance)
	{
		check(InInstance.IsValid());
		check(!DocumentInterfacePrivate::Instance.IsValid());
		DocumentInterfacePrivate::Instance = MoveTemp(InInstance);
	}
} // namespace Metasound::Frontend
