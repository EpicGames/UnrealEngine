// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocumentCache.h"

#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundTrace.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"


namespace Metasound::Frontend
{
	namespace DocumentCachePrivate
	{
		// Modifies cache map of an attempted array index swap removal, where the
		// given cache map is a map of keys to indices.
		template <typename TMapType, typename TKeyType>
		void RemoveSwapMapIndexChecked(const TKeyType& InSwapKey, const TKeyType& InLastKey, TMapType& OutMap)
		{
			// Remove swap item
			int32 SwapIndex = INDEX_NONE;
			const bool bRemoved = OutMap.RemoveAndCopyValue(InSwapKey, SwapIndex);
			checkf(bRemoved, TEXT("Failed to remove/swap expected MetaSound Document Cache key/value pair"));

			// Swap last item index if not the same entry
			if (InLastKey != InSwapKey)
			{
				OutMap.FindChecked(InLastKey) = SwapIndex;
			}
		}

		// Modifies cache map of an attempted array index swap removal, where the
		// given cache map is a map of keys to arrays of indices.
		template <typename TMapArrayType, typename TKeyType>
		void RemoveSwapMapArrayIndexChecked(
			const TKeyType& InSwapKey,
			int32 SwapIndex,
			const TKeyType& InLastKey,
			int32 LastIndex,
			TMapArrayType& OutMap
		)
		{
			auto IndexIsLast = [&LastIndex](const int32& Index) { return Index == LastIndex; };

			if (SwapIndex == LastIndex)
			{
				check(InSwapKey == InLastKey);
				TArray<int32>& Indices = OutMap.FindChecked(InLastKey);
				Indices.RemoveAllSwap(IndexIsLast, EAllowShrinking::No);
				if (Indices.IsEmpty())
				{
					OutMap.Remove(InLastKey);
				}
			}
			else
			{
				if (InSwapKey == InLastKey)
				{
					TArray<int32>& Indices = OutMap.FindChecked(InLastKey);
					Indices.RemoveAllSwap(IndexIsLast, EAllowShrinking::No);
					check(!Indices.IsEmpty()); // has to still contain the key that now points to *just* the swap index
				}
				else
				{
					{
						auto IndexIsSwap = [&SwapIndex](const int32& Index) { return Index == SwapIndex; };
						TArray<int32>& Indices = OutMap.FindChecked(InSwapKey);
						Indices.RemoveAllSwap(IndexIsSwap, EAllowShrinking::No);
						if (Indices.IsEmpty())
						{
							OutMap.Remove(InSwapKey);
						}
					}

					{
						TArray<int32>& Indices = OutMap.FindChecked(InLastKey);
						Algo::ForEachIf(Indices,
							IndexIsLast,
							[&SwapIndex](int32& Index) { Index = SwapIndex; }
						);
					}
				}
			}
		}
	}

	FDocumentCache::FDocumentCache()
	{
	}

	FDocumentCache::FDocumentCache(const FMetasoundFrontendDocument& InDocument, TSharedRef<FDocumentModifyDelegates> Delegates)
		: Document(&InDocument)
		, ModifyDelegates(Delegates)
	{
	}

	void FDocumentCache::Init(const FGuid& InBuildPageID, bool bPrimeCache)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentCache::Init);
		check(Document);

		BuildPageID = InBuildPageID;
		if (bPrimeCache)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentCache::Init_Prime);

			TSharedRef<const FDocumentCache> ThisShared = StaticCastSharedRef<const FDocumentCache>(AsShared());
			DependencyCache = MakeShared<FDocumentDependencyCache>(GetDocument());

			TSharedRef<FDocumentGraphEdgeCache> EdgeCache = FDocumentGraphEdgeCache::Create(ThisShared, InBuildPageID, ModifyDelegates.Get());
			EdgeCacheMap.Add(InBuildPageID, EdgeCache);
			TSharedRef<FDocumentGraphNodeCache> NodeCache = FDocumentGraphNodeCache::Create(ThisShared, InBuildPageID, ModifyDelegates.Get());
			NodeCacheMap.Add(InBuildPageID, NodeCache);

			InterfaceCache = FDocumentGraphInterfaceCache::Create(ThisShared, ModifyDelegates->InterfaceDelegates);
		}

		ModifyDelegates->OnDependencyAdded.AddSP(this, &FDocumentCache::OnDependencyAdded);
		ModifyDelegates->OnRemoveSwappingDependency.AddSP(this, &FDocumentCache::OnRemoveSwappingDependency);
		ModifyDelegates->OnRenamingDependencyClass.AddSP(this, &FDocumentCache::OnRenamingDependencyClass);

		ModifyDelegates->PageDelegates.OnRemovingPage.AddSPLambda(this, [this](const FDocumentMutatePageArgs& Args)
		{
			EdgeCacheMap.Remove(Args.PageID);
			NodeCacheMap.Remove(Args.PageID);
		});
	}

	const FMetasoundFrontendDocument& FDocumentCache::GetDocument() const
	{
		check(Document);
		return *Document;
	}

	TSharedRef<FDocumentCache> FDocumentCache::Create(const FMetasoundFrontendDocument& InDocument, TSharedRef<FDocumentModifyDelegates> Delegates, const FGuid& InBuildPageID, bool bPrimeCache)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentCache::Create);

		// Factory style constructor as restrictions on construction of shared pointers disallows passing of this document cache's pointer to sub-caches
		TSharedRef<FDocumentCache> Cache = MakeShared<FDocumentCache>(InDocument, Delegates);

		Cache->Init(InBuildPageID, bPrimeCache);
		return Cache;
	}

	bool FDocumentCache::ContainsDependency(const FNodeRegistryKey& InClassKey) const
	{
		const FDocumentDependencyCache& Cache = GetDependencyCache();
		return Cache.KeyToIndex.Contains(InClassKey);
	}

	bool FDocumentCache::ContainsDependencyOfType(EMetasoundFrontendClassType ClassType) const
	{
		const TArray<FMetasoundFrontendClass>& Dependencies = GetDocument().Dependencies;
		auto IsTemplateDependency = [&ClassType](const FMetasoundFrontendClass& Class) { return Class.Metadata.GetType() == ClassType; };
		return Algo::AnyOf(Dependencies, IsTemplateDependency);
	}

	const FMetasoundFrontendClass* FDocumentCache::FindDependency(const FNodeRegistryKey& InClassKey) const
	{
		const FDocumentDependencyCache& Cache = GetDependencyCache();
		if (const int32* Index = Cache.KeyToIndex.Find(InClassKey))
		{
			return &GetDocument().Dependencies[*Index];
		}

		return nullptr;
	}

	const FMetasoundFrontendClass* FDocumentCache::FindDependency(const FGuid& InClassID) const
	{
		const FDocumentDependencyCache& Cache = GetDependencyCache();
		if (const int32* Index = Cache.IDToIndex.Find(InClassID))
		{
			return &GetDocument().Dependencies[*Index];
		}

		return nullptr;
	}

	const int32* FDocumentCache::FindDependencyIndex(const Metasound::Frontend::FNodeRegistryKey& InClassKey) const
	{
		const FDocumentDependencyCache& Cache = GetDependencyCache();
		return Cache.KeyToIndex.Find(InClassKey);
	}

	const int32* FDocumentCache::FindDependencyIndex(const FGuid& InClassID) const
	{
		const FDocumentDependencyCache& Cache = GetDependencyCache();
		return Cache.IDToIndex.Find(InClassID);
	}

	FDocumentCache::FDocumentDependencyCache& FDocumentCache::GetDependencyCache()
	{
		if (!DependencyCache.IsValid())
		{
			DependencyCache = MakeShared<FDocumentDependencyCache>(GetDocument());
		}
		return *DependencyCache;
	}

	const FDocumentCache::FDocumentDependencyCache& FDocumentCache::GetDependencyCache() const
	{
		if (!DependencyCache.IsValid())
		{
			DependencyCache = MakeShared<FDocumentDependencyCache>(GetDocument());
		}
		return *DependencyCache;
	}

	const IDocumentGraphEdgeCache& FDocumentCache::GetEdgeCache(const FGuid& InPageID) const
	{
		TSharedPtr<FDocumentGraphEdgeCache> EdgeCache = EdgeCacheMap.FindRef(InPageID);
		if (!EdgeCache.IsValid())
		{
			TSharedRef<const FDocumentCache> ThisShared = StaticCastSharedRef<const FDocumentCache>(AsShared());
			EdgeCache = FDocumentGraphEdgeCache::Create(ThisShared, InPageID, ModifyDelegates.Get());
			EdgeCacheMap.Add(InPageID, EdgeCache);
		}

		return *EdgeCache.Get();
	}

	const IDocumentGraphNodeCache& FDocumentCache::GetNodeCache(const FGuid& InPageID) const
	{
		TSharedPtr<FDocumentGraphNodeCache> NodeCache = NodeCacheMap.FindRef(InPageID);
		if (!NodeCache.IsValid())
		{
			TSharedRef<const FDocumentCache> ThisShared = StaticCastSharedRef<const FDocumentCache>(AsShared());
			NodeCache = FDocumentGraphNodeCache::Create(ThisShared, InPageID, ModifyDelegates.Get());
			NodeCacheMap.Add(InPageID, NodeCache);
		}

		return *NodeCache.Get();
	}

	const IDocumentGraphInterfaceCache& FDocumentCache::GetInterfaceCache() const
	{
		if (!InterfaceCache.IsValid())
		{
			TSharedRef<const FDocumentCache> ThisShared = StaticCastSharedRef<const FDocumentCache>(AsShared());
			InterfaceCache = FDocumentGraphInterfaceCache::Create(ThisShared, ModifyDelegates->InterfaceDelegates);
		}
		return *InterfaceCache;
	}

	int32 FDocumentCache::GetTransactionCount() const
	{
		return GetTransactionCount(BuildPageID);
	}

	int32 FDocumentCache::GetTransactionCount(const FGuid& InPageID) const
	{
		int32 TotalCount = TransactionCount;
		TSharedPtr<FDocumentGraphNodeCache> NodeCache = NodeCacheMap.FindRef(InPageID);
		if (NodeCache.IsValid())
		{
			TotalCount += NodeCache->GetTransactionCount();
		}

		TSharedPtr<FDocumentGraphEdgeCache> EdgeCache = EdgeCacheMap.FindRef(InPageID);
		if (EdgeCache.IsValid())
		{
			TotalCount += EdgeCache->GetTransactionCount();
		}

		if (InterfaceCache.IsValid())
		{
			TotalCount += InterfaceCache->GetTransactionCount();
		}

		return TotalCount;
	}

	void FDocumentCache::OnDependencyAdded(int32 NewIndex)
	{
		using namespace Metasound::Frontend;

		const FMetasoundFrontendClass& Dependency = GetDocument().Dependencies[NewIndex];
		const FNodeRegistryKey Key = FNodeRegistryKey(Dependency.Metadata);

		FDocumentDependencyCache& Cache = GetDependencyCache();
		Cache.KeyToIndex.Add(Key, NewIndex);
		Cache.IDToIndex.Add(Dependency.ID, NewIndex);

		++TransactionCount;
	}

	void FDocumentCache::OnRemoveSwappingDependency(int32 SwapIndex, int32 LastIndex)
	{
		using namespace Metasound::Frontend;

		const TArray<FMetasoundFrontendClass>& Dependencies = GetDocument().Dependencies;
		const FMetasoundFrontendClass& SwapDependency = Dependencies[SwapIndex];
		const FMetasoundFrontendClass& LastDependency = Dependencies[LastIndex];

		const FNodeRegistryKey SwapKey = FNodeRegistryKey(SwapDependency.Metadata);
		const FNodeRegistryKey LastKey = FNodeRegistryKey(LastDependency.Metadata);

		FDocumentDependencyCache& Cache = GetDependencyCache();

		DocumentCachePrivate::RemoveSwapMapIndexChecked(SwapKey, LastKey, Cache.KeyToIndex);
		DocumentCachePrivate::RemoveSwapMapIndexChecked(SwapDependency.ID, LastDependency.ID, Cache.IDToIndex);

		++TransactionCount;
	}

	void FDocumentCache::OnRenamingDependencyClass(const int32 IndexBeingRenamed, const FMetasoundFrontendClassName& NewName)
	{
		using namespace Metasound::Frontend;

		const FMetasoundFrontendClass& DependencyBeingRemoved = GetDocument().Dependencies[IndexBeingRenamed];
		const FNodeRegistryKey OldKey = FNodeRegistryKey(DependencyBeingRemoved.Metadata);
		FMetasoundFrontendClassMetadata NewMetadata = DependencyBeingRemoved.Metadata;
		NewMetadata.SetClassName(NewName);
		const FNodeRegistryKey NewKey = FNodeRegistryKey(NewMetadata);

		FDocumentDependencyCache& Cache = GetDependencyCache();
		Cache.KeyToIndex.Add(NewKey, IndexBeingRenamed);
		Cache.KeyToIndex.Remove(OldKey);

		++TransactionCount;
	}

#if WITH_EDITORONLY_DATA
	void FDocumentCache::SetBuildPageID(const FGuid& InPageID)
	{
		if (InPageID != BuildPageID)
		{
			check(Document->RootGraph.ContainsGraphPage(InPageID));
			BuildPageID = InPageID;
		}
	}
#endif // WITH_EDITORONLY_DATA

	FDocumentGraphInterfaceCache::FDocumentGraphInterfaceCache(TSharedRef<const FDocumentCache> ParentCache)
		: Parent(ParentCache)
	{
		const FMetasoundFrontendGraphClass& GraphClass = ParentCache->GetDocument().RootGraph;
		const TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass.GetDefaultInterface().Inputs;
		for (int32 Index = 0; Index < Inputs.Num(); ++Index)
		{
			InputNameToIndex.Add(Inputs[Index].Name, Index);
		};

		const TArray<FMetasoundFrontendClassOutput>& Outputs = GraphClass.GetDefaultInterface().Outputs;
		for (int32 Index = 0; Index < Outputs.Num(); ++Index)
		{
			OutputNameToIndex.Add(Outputs[Index].Name, Index);
		};
	}

	TSharedRef<FDocumentGraphInterfaceCache> FDocumentGraphInterfaceCache::Create(TSharedRef<const FDocumentCache> ParentCache, FInterfaceModifyDelegates& OutDelegates)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentGraphInterfaceCache::Create);

		// Factory style constructor as restrictions on construction of shared pointers disallows passing of this document cache's pointer to sub-caches
		TSharedRef<FDocumentGraphInterfaceCache> Cache = MakeShared<FDocumentGraphInterfaceCache>(ParentCache);
		Cache->Init(OutDelegates);
		return Cache;
	}

	const FMetasoundFrontendClassInput* FDocumentGraphInterfaceCache::FindInput(FName InputName) const
	{
		if (const int32* Index = InputNameToIndex.Find(InputName))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
				return &Doc.RootGraph.GetDefaultInterface().Inputs[*Index];
			}
		}

		return nullptr;
	}

	const int32* FDocumentGraphInterfaceCache::FindInputIndex(FName InputName) const
	{
		return InputNameToIndex.Find(InputName);
	}

	const FMetasoundFrontendClassOutput* FDocumentGraphInterfaceCache::FindOutput(FName OutputName) const
	{
		if (const int32* Index = OutputNameToIndex.Find(OutputName))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
				return &Doc.RootGraph.GetDefaultInterface().Outputs[*Index];
			}
		}

		return nullptr;
	}

	const int32* FDocumentGraphInterfaceCache::FindOutputIndex(FName OutputName) const
	{
		return OutputNameToIndex.Find(OutputName);
	}

	int32 FDocumentGraphInterfaceCache::GetTransactionCount() const
	{
		return TransactionCount;
	}

	void FDocumentGraphInterfaceCache::Init(FInterfaceModifyDelegates& OutDelegates)
	{
		OutDelegates.OnInputAdded.AddSP(this, &FDocumentGraphInterfaceCache::OnInputAdded);
		OutDelegates.OnRemovingInput.AddSP(this, &FDocumentGraphInterfaceCache::OnRemovingInput);
		OutDelegates.OnInputNameChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputNameChanged);
		OutDelegates.OnInputDataTypeChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputDataTypeChanged);
		OutDelegates.OnInputDefaultChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputDefaultChanged);
		OutDelegates.OnInputInheritsDefaultChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputInheritsDefaultChanged);

#if WITH_EDITOR
		OutDelegates.OnInputDisplayNameChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputDisplayNameChanged);
		OutDelegates.OnInputDescriptionChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputDescriptionChanged);
		OutDelegates.OnInputSortOrderIndexChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputSortOrderIndexChanged);
		OutDelegates.OnInputIsAdvancedDisplayChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnInputIsAdvancedDisplayChanged);
#endif // WITH_EDITOR

		OutDelegates.OnOutputAdded.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputAdded);
		OutDelegates.OnRemovingOutput.AddSP(this, &FDocumentGraphInterfaceCache::OnRemovingOutput);
		OutDelegates.OnOutputNameChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputNameChanged);
		OutDelegates.OnOutputDataTypeChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputDataTypeChanged);

#if WITH_EDITOR
		OutDelegates.OnOutputDisplayNameChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputDisplayNameChanged);
		OutDelegates.OnOutputDescriptionChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputDescriptionChanged);
		OutDelegates.OnOutputSortOrderIndexChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputSortOrderIndexChanged);
		OutDelegates.OnOutputIsAdvancedDisplayChanged.AddSP(this, &FDocumentGraphInterfaceCache::OnOutputIsAdvancedDisplayChanged);
#endif // WITH_EDITOR
	}

	void FDocumentGraphInterfaceCache::OnInputAdded(int32 NewIndex)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
			const FMetasoundFrontendClassInput& Input = GraphClass.GetDefaultInterface().Inputs[NewIndex];
			InputNameToIndex.Add(Input.Name, NewIndex);

			++TransactionCount;
		}
	}

	void FDocumentGraphInterfaceCache::OnRemovingInput(int32 IndexBeingRemoved)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
			const FMetasoundFrontendClassInput& Input = GraphClass.GetDefaultInterface().Inputs[IndexBeingRemoved];
			InputNameToIndex.Remove(Input.Name);

			++TransactionCount;
		}
	}

	void FDocumentGraphInterfaceCache::OnInputNameChanged(FName OldName, FName NewName)
	{
		int32 Index = INDEX_NONE;
		const bool bRemoved = InputNameToIndex.RemoveAndCopyValue(OldName, Index);
		check(bRemoved);
		InputNameToIndex.Add(NewName, Index);

		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnInputDataTypeChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnInputDefaultChanged(const FDocumentArrayPagedInputArgs& Args)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnInputInheritsDefaultChanged(int32 NewIndex)
	{
		++TransactionCount;
	}

#if WITH_EDITOR
	void FDocumentGraphInterfaceCache::OnInputDescriptionChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnInputSortOrderIndexChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnInputIsAdvancedDisplayChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnOutputDisplayNameChanged(int32 Index)
	{
		++TransactionCount;
	}
#endif // WITH_EDITOR

	void FDocumentGraphInterfaceCache::OnOutputAdded(int32 NewIndex)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
			const FMetasoundFrontendClassOutput& Output = GraphClass.GetDefaultInterface().Outputs[NewIndex];
			OutputNameToIndex.Add(Output.Name, NewIndex);

			++TransactionCount;
		}
	}

	void FDocumentGraphInterfaceCache::OnRemovingOutput(int32 IndexBeingRemoved)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
			const FMetasoundFrontendClassOutput& Output = GraphClass.GetDefaultInterface().Outputs[IndexBeingRemoved];
			OutputNameToIndex.Remove(Output.Name);

			++TransactionCount;
		}
	}

	void FDocumentGraphInterfaceCache::OnOutputNameChanged(FName OldName, FName NewName)
	{
		int32 Index = INDEX_NONE;
		const bool bRemoved = OutputNameToIndex.RemoveAndCopyValue(OldName, Index);
		check(bRemoved);
		OutputNameToIndex.Add(NewName, Index);

		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnOutputDataTypeChanged(int32 Index)
	{
		++TransactionCount;
	}

#if WITH_EDITOR
	void FDocumentGraphInterfaceCache::OnInputDisplayNameChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnOutputDescriptionChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnOutputSortOrderIndexChanged(int32 Index)
	{
		++TransactionCount;
	}

	void FDocumentGraphInterfaceCache::OnOutputIsAdvancedDisplayChanged(int32 Index)
	{
		++TransactionCount;
	}
#endif // WITH_EDITOR

	FDocumentGraphNodeCache::FDocumentGraphNodeCache()
		: Parent(MakeShared<FDocumentCache>())
	{
	}

	FDocumentGraphNodeCache::FDocumentGraphNodeCache(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID)
		: Parent(ParentCache)
		, PageID(InPageID)
	{
		const FMetasoundFrontendGraph& Graph = ParentCache->GetDocument().RootGraph.FindConstGraphChecked(InPageID);
		const TArray<FMetasoundFrontendNode>& Nodes = Graph.Nodes;
		for (int32 Index = 0; Index < Nodes.Num(); ++Index)
		{
			const FMetasoundFrontendNode& Node = Nodes[Index];
			const FMetasoundFrontendClass* Class = ParentCache->FindDependency(Node.ClassID);
			check(Class);

			const FGuid& NodeID = Node.GetID();
			IDToIndex.Add(NodeID, Index);
			ClassIDToNodeIndices.FindOrAdd(Node.ClassID).Add(Index);
		}
	}

	bool FDocumentGraphNodeCache::ContainsNode(const FGuid& InNodeID) const
	{
		return IDToIndex.Contains(InNodeID);
	}

	TSharedRef<FDocumentGraphNodeCache> FDocumentGraphNodeCache::Create(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID, FDocumentModifyDelegates& OutDelegates)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentGraphNodeCache::Create);

		// Factory style constructor as restrictions on construction of shared pointers disallows passing of this document cache's pointer to sub-caches
		TSharedRef<FDocumentGraphNodeCache> Cache = MakeShared<FDocumentGraphNodeCache>(ParentCache, InPageID);
		Cache->Init(OutDelegates.FindGraphDelegatesChecked(InPageID).NodeDelegates);
		return Cache;
	}

	const int32* FDocumentGraphNodeCache::FindNodeIndex(const FGuid& InNodeID) const
	{
		return IDToIndex.Find(InNodeID);
	}

	TArray<const FMetasoundFrontendNode*> FDocumentGraphNodeCache::FindNodesOfClassID(const FGuid& InClassID) const
	{
		TArray<const FMetasoundFrontendNode*> Nodes;
		if (const TArray<int32>* NodeIndices = ClassIDToNodeIndices.Find(InClassID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
				const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
				Algo::Transform(*NodeIndices, Nodes, [&Graph](const int32& Index) { return &Graph.Nodes[Index]; });
			}
		}
		return Nodes;
	}

	const FMetasoundFrontendNode* FDocumentGraphNodeCache::FindNode(const FGuid& InNodeID) const
	{
		if (const int32* NodeIndex = IDToIndex.Find(InNodeID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
				const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
				return &Graph.Nodes[*NodeIndex];
			}
		}

		return nullptr;
	}

	int32 FDocumentGraphNodeCache::GetTransactionCount() const
	{
		return TransactionCount;
	}

	bool FDocumentGraphNodeCache::ContainsNodesOfClassID(const FGuid& InClassID) const
	{
		if (const TArray<int32>* NodeIndices = ClassIDToNodeIndices.Find(InClassID))
		{
			return !NodeIndices->IsEmpty();
		}

		return false;
	}

	TArray<const FMetasoundFrontendVertex*> FDocumentGraphNodeCache::FindNodeInputs(const FGuid& InNodeID, FName TypeName) const
	{
		TArray<const FMetasoundFrontendVertex*> FoundVertices;
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto IsVertexOfType = [&TypeName](const FMetasoundFrontendVertex& Vertex) { return TypeName.IsNone() || Vertex.TypeName == TypeName; };
			auto GetVertexPtr = [](const FMetasoundFrontendVertex& Vertex) { return &Vertex; };
			Algo::TransformIf(Node->Interface.Inputs, FoundVertices, IsVertexOfType, GetVertexPtr);
		}

		return FoundVertices;
	}

	TArray<const FMetasoundFrontendVertex*> FDocumentGraphNodeCache::FindNodeOutputs(const FGuid& InNodeID, FName TypeName) const
	{
		TArray<const FMetasoundFrontendVertex*> FoundVertices;
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto IsVertexOfType = [&TypeName](const FMetasoundFrontendVertex& Vertex) { return TypeName.IsNone() || Vertex.TypeName == TypeName; };
			auto GetVertexPtr = [](const FMetasoundFrontendVertex& Vertex) { return &Vertex; };
			Algo::TransformIf(Node->Interface.Outputs, FoundVertices, IsVertexOfType, GetVertexPtr);
		}

		return FoundVertices;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindInputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto VertexMatchesPredicate = [&InVertexID](const FMetasoundFrontendVertex& Vertex)
			{
				return Vertex.VertexID == InVertexID;
			};
			return Node->Interface.Inputs.FindByPredicate(VertexMatchesPredicate);
		}

		return nullptr;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindInputVertex(const FGuid& InNodeID, FName InVertexName) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto VertexMatchesPredicate = [&InVertexName](const FMetasoundFrontendVertex& Vertex)
			{
				return Vertex.Name == InVertexName;
			};
			return Node->Interface.Inputs.FindByPredicate(VertexMatchesPredicate);
		}

		return nullptr;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto VertexMatchesPredicate = [&InVertexID](const FMetasoundFrontendVertex& Vertex)
			{
				return Vertex.VertexID == InVertexID;
			};
			return Node->Interface.Outputs.FindByPredicate(VertexMatchesPredicate);
		}

		return nullptr;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindOutputVertex(const FGuid& InNodeID, FName InVertexName) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			auto VertexMatchesPredicate = [&InVertexName](const FMetasoundFrontendVertex& Vertex)
			{
				return Vertex.Name == InVertexName;
			};
			return Node->Interface.Outputs.FindByPredicate(VertexMatchesPredicate);
		}

		return nullptr;
	}

	TArray<const FMetasoundFrontendVertex*> FDocumentGraphNodeCache::FindReroutedInputVertices(const FGuid& InNodeID, const FGuid& InVertexID, TArray<const FMetasoundFrontendNode*>* ConnectedNodes, bool* bOutIsRerouted) const
	{
		TArray<const FMetasoundFrontendVertex*> Vertices;

		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendClass* Class = ParentCache->FindDependency(Node->ClassID);
				check(Class);
				if (Class->Metadata.GetClassName() == FRerouteNodeTemplate::ClassName)
				{
					if (bOutIsRerouted)
					{
						*bOutIsRerouted = true;
					}

					const IDocumentGraphEdgeCache& EdgeCache = ParentCache->GetEdgeCache(PageID);
					const FMetasoundFrontendVertex& RerouteOutput = Node->Interface.Outputs.Last();
					TArrayView<const int32> EdgeIndices = EdgeCache.FindEdgeIndicesFromNodeOutput(InNodeID, RerouteOutput.VertexID);
					const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
					for (int32 Index : EdgeIndices)
					{
						const FMetasoundFrontendEdge& EdgeConnectedToOutput = Doc.RootGraph.FindConstGraphChecked(PageID).Edges[Index];
						TArray<const FMetasoundFrontendVertex*> ReroutedVertices;
						if (ConnectedNodes)
						{
							TArray<const FMetasoundFrontendNode*> ReroutedConnectedNodes;
							ReroutedVertices = FindReroutedInputVertices(EdgeConnectedToOutput.ToNodeID, EdgeConnectedToOutput.ToVertexID, &ReroutedConnectedNodes, bOutIsRerouted);
							ConnectedNodes->Append(ReroutedConnectedNodes);
						}
						else
						{
							ReroutedVertices = FindReroutedInputVertices(EdgeConnectedToOutput.ToNodeID, EdgeConnectedToOutput.ToVertexID, nullptr, bOutIsRerouted);
						}
						Vertices.Append(MoveTemp(ReroutedVertices));
					}
				}
				else
				{
					auto VertexMatchesPredicate = [&InVertexID](const FMetasoundFrontendVertex& Vertex)
						{
							return Vertex.VertexID == InVertexID;
						};
					const FMetasoundFrontendVertex* InputVertex = Node->Interface.Inputs.FindByPredicate(VertexMatchesPredicate);
					check(InputVertex);
					Vertices.Add(InputVertex);
					if (ConnectedNodes)
					{
						ConnectedNodes->Add(Node);
					}
				}
			}
		}

		return Vertices;
	}

	TArray<const FMetasoundFrontendVertex*> FDocumentGraphNodeCache::FindReroutedInputVertices(const FGuid& InNodeID, FName InVertexName, TArray<const FMetasoundFrontendNode*>* ConnectedNodes, bool* bOutIsRerouted) const
	{
		TArray<const FMetasoundFrontendVertex*> Vertices;

		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendClass* Class = ParentCache->FindDependency(Node->ClassID);
				check(Class);
				if (Class->Metadata.GetClassName() == FRerouteNodeTemplate::ClassName)
				{
					if (bOutIsRerouted)
					{
						*bOutIsRerouted = true;
					}

					const IDocumentGraphEdgeCache& EdgeCache = ParentCache->GetEdgeCache(PageID);
					const FMetasoundFrontendVertex& RerouteOutput = Node->Interface.Outputs.Last();
					TArrayView<const int32> EdgeIndices = EdgeCache.FindEdgeIndicesFromNodeOutput(InNodeID, RerouteOutput.VertexID);
					const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
					for (int32 Index : EdgeIndices)
					{
						const FMetasoundFrontendEdge& EdgeConnectedToOutput = Doc.RootGraph.FindConstGraphChecked(PageID).Edges[Index];

						TArray<const FMetasoundFrontendVertex*> ReroutedVertices;
						if (ConnectedNodes)
						{
							TArray<const FMetasoundFrontendNode*> ReroutedConnectedNodes;
							ReroutedVertices = FindReroutedInputVertices(EdgeConnectedToOutput.ToNodeID, EdgeConnectedToOutput.ToVertexID, &ReroutedConnectedNodes, bOutIsRerouted);
							ConnectedNodes->Append(MoveTemp(ReroutedConnectedNodes));
						}
						else
						{
							ReroutedVertices = FindReroutedInputVertices(EdgeConnectedToOutput.ToNodeID, EdgeConnectedToOutput.ToVertexID, nullptr, bOutIsRerouted);
						}
						Vertices.Append(MoveTemp(ReroutedVertices));
					}
				}
				else
				{
					auto VertexMatchesPredicate = [&InVertexName](const FMetasoundFrontendVertex& Vertex)
						{
							return Vertex.Name == InVertexName;
						};
					const FMetasoundFrontendVertex* InputVertex = Node->Interface.Inputs.FindByPredicate(VertexMatchesPredicate);
					check(InputVertex);
					Vertices.Add(InputVertex);
					if (ConnectedNodes)
					{
						ConnectedNodes->Add(Node);
					}
				}
			}
		}

		return Vertices;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindReroutedOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID, const FMetasoundFrontendNode** ConnectedNode, bool* bOutIsRerouted) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendClass* Class = ParentCache->FindDependency(Node->ClassID);
				check(Class);
				if (Class->Metadata.GetClassName() == FRerouteNodeTemplate::ClassName)
				{
					if (bOutIsRerouted)
					{
						*bOutIsRerouted = true;
					}

					const IDocumentGraphEdgeCache& EdgeCache = ParentCache->GetEdgeCache(PageID);
					const FMetasoundFrontendVertex& RerouteInput = Node->Interface.Inputs.Last();
					const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
					if (const int32* ConnectedEdgeIndex = EdgeCache.FindEdgeIndexToNodeInput(InNodeID, RerouteInput.VertexID))
					{
						const FMetasoundFrontendEdge& EdgeConnectedToInput = Doc.RootGraph.FindConstGraphChecked(PageID).Edges[*ConnectedEdgeIndex];
						return FindReroutedOutputVertex(EdgeConnectedToInput.FromNodeID, EdgeConnectedToInput.FromVertexID, ConnectedNode, bOutIsRerouted);
					}

					return nullptr;
				}

				if (ConnectedNode)
				{
					*ConnectedNode = Node;
				}
				auto VertexMatchesPredicate = [&InVertexID](const FMetasoundFrontendVertex& Vertex)
					{
						return Vertex.VertexID == InVertexID;
					};
				return Node->Interface.Outputs.FindByPredicate(VertexMatchesPredicate);
			}
		}

		return nullptr;
	}

	const FMetasoundFrontendVertex* FDocumentGraphNodeCache::FindReroutedOutputVertex(const FGuid& InNodeID, FName InVertexName, const FMetasoundFrontendNode** ConnectedNode,  bool* bOutIsRerouted) const
	{
		if (const FMetasoundFrontendNode* Node = FindNode(InNodeID))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendClass* Class = ParentCache->FindDependency(Node->ClassID);
				check(Class);
				if (Class->Metadata.GetClassName() == FRerouteNodeTemplate::ClassName)
				{
					if (bOutIsRerouted)
					{
						*bOutIsRerouted = true;
					}

					const IDocumentGraphEdgeCache& EdgeCache = ParentCache->GetEdgeCache(PageID);
					const FMetasoundFrontendVertex& RerouteInput = Node->Interface.Inputs.Last();
					const FMetasoundFrontendDocument& Doc = ParentCache->GetDocument();
					if (const int32* ConnectedEdgeIndex = EdgeCache.FindEdgeIndexToNodeInput(InNodeID, RerouteInput.VertexID))
					{
						const FMetasoundFrontendEdge& EdgeConnectedToInput = Doc.RootGraph.FindConstGraphChecked(PageID).Edges[*ConnectedEdgeIndex];
						return FindReroutedOutputVertex(EdgeConnectedToInput.FromNodeID, EdgeConnectedToInput.FromVertexID, ConnectedNode, bOutIsRerouted);
					}

					return nullptr;
				}

				auto VertexMatchesPredicate = [&InVertexName](const FMetasoundFrontendVertex& Vertex)
				{
					return Vertex.Name == InVertexName;
				};
				if (ConnectedNode)
				{
					*ConnectedNode = Node;
				}
				return Node->Interface.Outputs.FindByPredicate(VertexMatchesPredicate);
			}
		}

		return nullptr;
	}

	void FDocumentGraphNodeCache::Init(FNodeModifyDelegates& OutDelegates)
	{
		OutDelegates.OnNodeAdded.AddSP(this, &FDocumentGraphNodeCache::OnNodeAdded);
		OutDelegates.OnNodeInputLiteralSet.AddSP(this, &FDocumentGraphNodeCache::OnNodeInputLiteralSet);
		OutDelegates.OnRemoveSwappingNode.AddSP(this, &FDocumentGraphNodeCache::OnRemoveSwappingNode);
		OutDelegates.OnRemovingNodeInputLiteral.AddSP(this, &FDocumentGraphNodeCache::OnRemovingNodeInputLiteral);
	}

	void FDocumentGraphNodeCache::OnNodeAdded(int32 InNewIndex)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
			const FMetasoundFrontendNode& Node = Graph.Nodes[InNewIndex];
			IDToIndex.Add(Node.GetID(), InNewIndex);
			ClassIDToNodeIndices.FindOrAdd(Node.ClassID).Add(InNewIndex);

			++TransactionCount;
		}
	}

	void FDocumentGraphNodeCache::OnNodeInputLiteralSet(int32 /* NodeIndex */, int32 /* VertexIndex */, int32 /* LiteralIndex */)
	{
		++TransactionCount;
	}

	void FDocumentGraphNodeCache::OnRemoveSwappingNode(int32 SwapIndex, int32 LastIndex)
	{
		using namespace Metasound::Frontend;
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const TArray<FMetasoundFrontendNode>& Nodes = Document.RootGraph.FindConstGraphChecked(PageID).Nodes;
			const FMetasoundFrontendNode& SwapNode = Nodes[SwapIndex];
			const FMetasoundFrontendNode& LastNode = Nodes[LastIndex];

			DocumentCachePrivate::RemoveSwapMapArrayIndexChecked(SwapNode.ClassID, SwapIndex, LastNode.ClassID, LastIndex, ClassIDToNodeIndices);
			DocumentCachePrivate::RemoveSwapMapIndexChecked(SwapNode.GetID(), LastNode.GetID(), IDToIndex);

			++TransactionCount;
		}
	}

	void FDocumentGraphNodeCache::OnRemovingNodeInputLiteral(int32 /* NodeIndex */, int32 /* VertexIndex */, int32 /* LiteralIndex */)
	{
		++TransactionCount;
	}

	FDocumentGraphEdgeCache::FDocumentGraphEdgeCache()
		: Parent(MakeShared<const FDocumentCache>())
	{
	}

	FDocumentGraphEdgeCache::FDocumentGraphEdgeCache(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID)
		: Parent(ParentCache)
		, PageID(InPageID)
	{
		const TArray<FMetasoundFrontendEdge>& Edges = ParentCache->GetDocument().RootGraph.FindConstGraphChecked(InPageID).Edges;
		for (int32 Index = 0; Index < Edges.Num(); ++Index)
		{
			const FMetasoundFrontendEdge& Edge = Edges[Index];
			OutputToEdgeIndices.FindOrAdd(Edge.GetFromVertexHandle()).Add(Index);
			InputToEdgeIndex.Add(Edge.GetToVertexHandle()) = Index;
		}
	}

	bool FDocumentGraphEdgeCache::ContainsEdge(const FMetasoundFrontendEdge& InEdge) const
	{
		const FMetasoundFrontendVertexHandle InputPair = { InEdge.ToNodeID, InEdge.ToVertexID };
		if (const int32* Index = InputToEdgeIndex.Find(InputPair))
		{
			if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
			{
				const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
				const FMetasoundFrontendEdge& Edge = Document.RootGraph.FindConstGraphChecked(PageID).Edges[*Index];
				return Edge.FromNodeID == InEdge.FromNodeID && Edge.FromVertexID == InEdge.FromVertexID;
			}
		}

		return false;
	}

	TSharedRef<FDocumentGraphEdgeCache> FDocumentGraphEdgeCache::Create(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID, FDocumentModifyDelegates& OutDelegates)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentGraphEdgeCache::Create);

		// Factory style constructor as restrictions on construction of shared pointers disallows passing of this document cache's pointer to sub-caches
		TSharedRef<FDocumentGraphEdgeCache> Cache = MakeShared<FDocumentGraphEdgeCache>(ParentCache, InPageID);
		Cache->Init(OutDelegates.FindGraphDelegatesChecked(InPageID).EdgeDelegates);
		return Cache;
	}

	TArray<const FMetasoundFrontendEdge*> FDocumentGraphEdgeCache::FindEdges(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		TArray<const FMetasoundFrontendEdge*> Edges;
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);

			const FMetasoundFrontendVertexHandle Handle{ InNodeID, InVertexID };
			if (const int32* Index = InputToEdgeIndex.Find(Handle))
			{
				Edges.Add(&Graph.Edges[*Index]);
			}

			if (const TArray<int32>* Indices = OutputToEdgeIndices.Find(Handle))
			{
				Algo::Transform(*Indices, Edges, [&Graph](const int32& Index) { return &Graph.Edges[Index]; });
			}
		}

		return Edges;
	}

	const int32* FDocumentGraphEdgeCache::FindEdgeIndexToNodeInput(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		return InputToEdgeIndex.Find({ InNodeID, InVertexID });
	}

	const TArrayView<const int32> FDocumentGraphEdgeCache::FindEdgeIndicesFromNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		if (const TArray<int32>* Array = OutputToEdgeIndices.Find({ InNodeID, InVertexID }))
		{
			return TArrayView<const int32>(*Array);
		}

		return { };
	}

	int32 FDocumentGraphEdgeCache::GetTransactionCount() const
	{
		return TransactionCount;
	}

	void FDocumentGraphEdgeCache::Init(FEdgeModifyDelegates& OutDelegates)
	{
		OutDelegates.OnEdgeAdded.AddSP(this, &FDocumentGraphEdgeCache::OnEdgeAdded);
		OutDelegates.OnRemoveSwappingEdge.AddSP(this, &FDocumentGraphEdgeCache::OnRemoveSwappingEdge);
	}

	bool FDocumentGraphEdgeCache::IsNodeInputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		return InputToEdgeIndex.Contains(FMetasoundFrontendVertexHandle { InNodeID, InVertexID });
	}

	bool FDocumentGraphEdgeCache::IsNodeOutputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		return OutputToEdgeIndices.Contains(FMetasoundFrontendVertexHandle { InNodeID, InVertexID });
	}

	void FDocumentGraphEdgeCache::OnEdgeAdded(int32 InNewIndex)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const FMetasoundFrontendGraph& Graph = Document.RootGraph.FindConstGraphChecked(PageID);
			const FMetasoundFrontendEdge& NewEdge = Graph.Edges[InNewIndex];
			InputToEdgeIndex.Add(NewEdge.GetToVertexHandle(), InNewIndex);
			OutputToEdgeIndices.FindOrAdd(NewEdge.GetFromVertexHandle()).Add(InNewIndex);

			++TransactionCount;
		}
	}

	void FDocumentGraphEdgeCache::OnRemoveSwappingEdge(int32 SwapIndex, int32 LastIndex)
	{
		if (TSharedPtr<const FDocumentCache> ParentCache = Parent.Pin())
		{
			const FMetasoundFrontendDocument& Document = ParentCache->GetDocument();
			const TArray<FMetasoundFrontendEdge>& Edges = Document.RootGraph.FindConstGraphChecked(PageID).Edges;

			const FMetasoundFrontendEdge& SwapEdge = Edges[SwapIndex];
			const FMetasoundFrontendEdge& LastEdge = Edges[LastIndex];
			DocumentCachePrivate::RemoveSwapMapIndexChecked(SwapEdge.GetToVertexHandle(), LastEdge.GetToVertexHandle(), InputToEdgeIndex);
			DocumentCachePrivate::RemoveSwapMapArrayIndexChecked(SwapEdge.GetFromVertexHandle(), SwapIndex, LastEdge.GetFromVertexHandle(), LastIndex, OutputToEdgeIndices);

			++TransactionCount;
		}
	}

	FDocumentCache::FDocumentDependencyCache::FDocumentDependencyCache(const FMetasoundFrontendDocument& InDocument)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FDocumentDependencyCache::FDocumentDependencyCache);

		for (int32 Index = 0; Index < InDocument.Dependencies.Num(); ++Index)
		{
			const FMetasoundFrontendClass& Class = InDocument.Dependencies[Index];
			const FMetasoundFrontendClassMetadata& Metadata = Class.Metadata;
			FNodeRegistryKey Key = FNodeRegistryKey(Metadata);
			KeyToIndex.Add(MoveTemp(Key), Index);
#if !NO_LOGGING
			if (IDToIndex.Contains(Class.ID))
			{
				UE_LOGF(LogMetaSound, Display, "MetaSound document with class name %ls found containing multiple dependencies with the same ID while initializing builder cache. Document may be currently versioning or require versioning past version 1.15", *InDocument.RootGraph.Metadata.GetClassName().ToString());
			}
#endif // !NO_LOGGING
			IDToIndex.Add(Class.ID, Index);
		}
	}
} // namespace Metasound::Frontend
