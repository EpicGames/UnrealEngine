// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentCacheInterface.h"
#include "Containers/SortedMap.h"

// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;


namespace Metasound::Frontend
{
	// Forward Declarations
	class FDocumentCache;

	class FDocumentGraphEdgeCache : public IDocumentGraphEdgeCache
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		// Only exists to make TSharedFromThis happy, but should never be called
		FDocumentGraphEdgeCache();
		FDocumentGraphEdgeCache(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID);

	public:
		static TSharedRef<FDocumentGraphEdgeCache> Create(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID, FDocumentModifyDelegates& OutDelegates);
		virtual ~FDocumentGraphEdgeCache() = default;

		// IDocumentGraphEdgeCache implementation
		virtual bool ContainsEdge(const FMetasoundFrontendEdge& InEdge) const override;

		virtual bool IsNodeInputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const override;
		virtual bool IsNodeOutputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const override;

		virtual TArray<const FMetasoundFrontendEdge*> FindEdges(const FGuid& InNodeID, const FGuid& InVertexID) const override;
		virtual const int32* FindEdgeIndexToNodeInput(const FGuid& InNodeID, const FGuid& InVertexID) const override;
		virtual const TArrayView<const int32> FindEdgeIndicesFromNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID) const override;

		int32 GetTransactionCount() const;

	private:
		void Init(FEdgeModifyDelegates& OutDelegates);
		void OnEdgeAdded(int32 InNewIndex);
		void OnRemoveSwappingEdge(int32 SwapIndex, int32 LastIndex);

		// Cache of outputs NodeId/VertexId pairs to associated edge indices
		TMap<FMetasoundFrontendVertexHandle, TArray<int32>> OutputToEdgeIndices;

		// Cache of Input NodeId/VertexId pairs to associated edge indices
		TMap<FMetasoundFrontendVertexHandle, int32> InputToEdgeIndex;

		TWeakPtr<const FDocumentCache> Parent;
		FGuid PageID;
		int32 TransactionCount = 0;
	};


	class FDocumentGraphNodeCache : public IDocumentGraphNodeCache
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		// Only exists to make TSharedFromThis happy, but should never be called
		FDocumentGraphNodeCache();
		FDocumentGraphNodeCache(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID);

	public:
		static TSharedRef<FDocumentGraphNodeCache> Create(TSharedRef<const FDocumentCache> ParentCache, const FGuid& InPageID, FDocumentModifyDelegates& OutDelegates);
		virtual ~FDocumentGraphNodeCache() = default;

		// IDocumentGraphNodeCache implementation
		virtual bool ContainsNode(const FGuid& InNodeID) const override;
		virtual bool ContainsNodesOfClassID(const FGuid& InClassID) const override;

		virtual const int32* FindNodeIndex(const FGuid& InNodeID) const override;
		virtual TArray<const FMetasoundFrontendNode*> FindNodesOfClassID(const FGuid& InClassID) const override;
		virtual const FMetasoundFrontendNode* FindNode(const FGuid& InNodeID) const override;

		virtual TArray<const FMetasoundFrontendVertex*> FindNodeInputs(const FGuid& InNodeID, FName TypeName = FName()) const override;
		virtual TArray<const FMetasoundFrontendVertex*> FindNodeOutputs(const FGuid& InNodeID, FName TypeName = FName()) const override;

		virtual const FMetasoundFrontendVertex* FindInputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const override;
		virtual const FMetasoundFrontendVertex* FindInputVertex(const FGuid& InNodeID, FName InVertexName) const override;
		virtual const FMetasoundFrontendVertex* FindOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const override;
		virtual const FMetasoundFrontendVertex* FindOutputVertex(const FGuid& InNodeID, FName InVertexName) const override;

		virtual TArray<const FMetasoundFrontendVertex*> FindReroutedInputVertices(const FGuid& InNodeID, const FGuid& InVertexID, TArray<const FMetasoundFrontendNode*>* ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const override;
		virtual TArray<const FMetasoundFrontendVertex*> FindReroutedInputVertices(const FGuid& InNodeID, FName InVertexName, TArray<const FMetasoundFrontendNode*>* ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const override;
		virtual const FMetasoundFrontendVertex* FindReroutedOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID, const FMetasoundFrontendNode** ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const override;
		virtual const FMetasoundFrontendVertex* FindReroutedOutputVertex(const FGuid& InNodeID, FName InVertexName, const FMetasoundFrontendNode** ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const override;

		int32 GetTransactionCount() const;

	private:
		void Init(FNodeModifyDelegates& OutDelegates);
		void OnNodeAdded(int32 NewIndex);
		void OnNodeInputLiteralSet(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex);
		void OnRemoveSwappingNode(int32 IndexBeingRemoved, int32 LastIndex);
		void OnRemovingNodeInputLiteral(int32 NodeIndex, int32 VertexIndex, int32 LiteralIndex);

		// Cache of NodeId to array index of node
		TSortedMap<FGuid, int32> IDToIndex;

		// Cache of ClassID to referencing node indices
		TSortedMap<FGuid, TArray<int32>> ClassIDToNodeIndices;

		TWeakPtr<const FDocumentCache> Parent;
		FGuid PageID;
		int32 TransactionCount = 0;
	};


	class FDocumentGraphInterfaceCache : public IDocumentGraphInterfaceCache
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		FDocumentGraphInterfaceCache(TSharedRef<const FDocumentCache> ParentCache);

	public:
		static TSharedRef<FDocumentGraphInterfaceCache> Create(TSharedRef<const FDocumentCache> ParentCache, FInterfaceModifyDelegates& OutDelegates);
		virtual ~FDocumentGraphInterfaceCache() = default;

		// IDocumentGraphInterfaceCache implementation
		virtual const FMetasoundFrontendClassInput* FindInput(FName InputName) const override;
		virtual const int32* FindInputIndex(FName InputName) const override;
		virtual const FMetasoundFrontendClassOutput* FindOutput(FName OutputName) const override;
		virtual const int32* FindOutputIndex(FName OutputName) const override;

		int32 GetTransactionCount() const;

	private:
		void Init(FInterfaceModifyDelegates& OutDelegates);

		void OnInputAdded(int32 NewIndex);
		void OnRemovingInput(int32 IndexBeingRemoved);
		void OnInputNameChanged(FName OldName, FName NewName);
		void OnInputDataTypeChanged(int32 Index);
		void OnInputDefaultChanged(const FDocumentArrayPagedInputArgs& Args);
		void OnInputInheritsDefaultChanged(int32 NewIndex);
#if WITH_EDITOR
		void OnInputDisplayNameChanged(int32 Index);
		void OnInputDescriptionChanged(int32 Index);
		void OnInputSortOrderIndexChanged(int32 Index);
		void OnInputIsAdvancedDisplayChanged(int32 Index);
#endif // WITH_EDITOR

		void OnOutputAdded(int32 NewIndex);
		void OnRemovingOutput(int32 IndexBeingRemoved);
		void OnOutputNameChanged(FName OldName, FName NewName);
		void OnOutputDataTypeChanged(int32 Index);
#if WITH_EDITOR
		void OnOutputDisplayNameChanged(int32 Index);
		void OnOutputDescriptionChanged(int32 Index);
		void OnOutputSortOrderIndexChanged(int32 Index);
		void OnOutputIsAdvancedDisplayChanged(int32 Index);
#endif // WITH_EDITOR

		// Cache of Input name to array index of input
		TMap<FName, int32> InputNameToIndex;

		// Cache of Output name to array index of output
		TMap<FName, int32> OutputNameToIndex;

		TWeakPtr<const FDocumentCache> Parent;

		int32 TransactionCount = 0;
	};


	class FDocumentCache : public IDocumentCache
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		// Only exists to make TSharedFromThis happy, but should never be called
		FDocumentCache();
		FDocumentCache(const FMetasoundFrontendDocument& InDocument, TSharedRef<FDocumentModifyDelegates> Delegates);

	public:
		static TSharedRef<FDocumentCache> Create(const FMetasoundFrontendDocument& InDocument, TSharedRef<FDocumentModifyDelegates> Delegates, const FGuid& InBuildPageID, bool bPrimeCache);
		virtual ~FDocumentCache() = default;

		virtual bool ContainsDependency(const FNodeRegistryKey& InClassKey) const override;
		virtual bool ContainsDependencyOfType(EMetasoundFrontendClassType ClassType) const override;

		virtual const FMetasoundFrontendClass* FindDependency(const FNodeRegistryKey& InClassKey) const override;
		virtual const FMetasoundFrontendClass* FindDependency(const FGuid& InClassID) const override;
		virtual const int32* FindDependencyIndex(const Metasound::Frontend::FNodeRegistryKey& InClassKey) const override;
		virtual const int32* FindDependencyIndex(const FGuid& InClassID) const override;

		virtual const FMetasoundFrontendDocument& GetDocument() const override;
		virtual const IDocumentGraphEdgeCache& GetEdgeCache(const FGuid& InPageID) const override;
		virtual const IDocumentGraphNodeCache& GetNodeCache(const FGuid& InPageID) const override;
		virtual const IDocumentGraphInterfaceCache& GetInterfaceCache() const override;

#if WITH_EDITORONLY_DATA
		virtual void SetBuildPageID(const FGuid& InPageID) override;
#endif // WITH_EDITORONLY_DATA

		int32 GetTransactionCount() const;
		int32 GetTransactionCount(const FGuid& InPageID) const;

	private:
		void Init(const FGuid& InBuildPageID, bool bPrimeCache);

		void OnDependencyAdded(int32 InNewIndex);
		void OnRemoveSwappingDependency(int32 SwapIndex, int32 LastIndex);
		void OnRenamingDependencyClass(const int32 IndexBeingRenamed, const FMetasoundFrontendClassName& NewName);

		struct FDocumentDependencyCache : public TSharedFromThis<FDocumentDependencyCache>
		{
			FDocumentDependencyCache(const FMetasoundFrontendDocument& InDocument);
			~FDocumentDependencyCache() = default;

			// Cache of dependency (Class) ID to corresponding class dependency index
			TSortedMap<FGuid, int32> IDToIndex;

			// Cache of version data to corresponding class dependency index
			TSortedMap<FNodeRegistryKey, int32> KeyToIndex;
		};

		// Private as cached queried data for fast access at document layer is minimal (i.e. dependencies by class id),
		// so API is just passed through document cache interface. Opaque create getter API kept similar to calls above
		// to separately implemented caches for parity.
		FDocumentDependencyCache& GetDependencyCache();
		const FDocumentDependencyCache& GetDependencyCache() const;

		// Caches are mutable to be able to be loaded on demand if cache does not exist at the time of an initial query
		mutable TSharedPtr<FDocumentDependencyCache> DependencyCache;

		// Sorted map of PageID to EdgeCache
		mutable TSortedMap<FGuid, TSharedPtr<FDocumentGraphEdgeCache>> EdgeCacheMap;

		// Sorted map of PageID to NodeCache
		mutable TSortedMap<FGuid, TSharedPtr<FDocumentGraphNodeCache>> NodeCacheMap;

		mutable TSharedPtr<FDocumentGraphInterfaceCache> InterfaceCache;

		const FMetasoundFrontendDocument* Document = nullptr;
		TSharedRef<FDocumentModifyDelegates> ModifyDelegates;

		// Number of transactions processed since builder was instantiated
		int32 TransactionCount = 0;

		// Currently set build PageID, which is the default PageID for API calls that do not specify a targeted PageID.
		FGuid BuildPageID;
	};
} // namespace Metasound::Frontend
