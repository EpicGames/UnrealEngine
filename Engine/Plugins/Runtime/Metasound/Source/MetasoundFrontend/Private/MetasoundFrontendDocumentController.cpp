// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocumentController.h"

#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "HAL/FileManager.h"
#include "MetasoundFrontendGraphController.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendInvalidController.h"
#include "MetasoundJsonBackend.h"
#include "StructSerializer.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontendDocumentController"

namespace Metasound
{
	namespace Frontend
	{
		//
		// FDocumentController
		//
		FDocumentController::FDocumentController(FDocumentAccessPtr InDocumentPtr)
		:	DocumentPtr(InDocumentPtr)
		{
		}

		bool FDocumentController::IsValid() const
		{
			return (nullptr != DocumentPtr.Get());
		}

		const TArray<FMetasoundFrontendClass>& FDocumentController::GetDependencies() const
		{
			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				return Document->Dependencies;
			}

			return Invalid::GetInvalidClassArray();
		}

		void FDocumentController::IterateDependencies(TFunctionRef<void(FMetasoundFrontendClass&)> InFunction)
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				Algo::ForEach(Document->Dependencies, InFunction);
			}
		}

		void FDocumentController::IterateDependencies(TFunctionRef<void(const FMetasoundFrontendClass&)> InFunction) const
		{
			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				Algo::ForEach(Document->Dependencies, InFunction);
			}
		}

		const TArray<FMetasoundFrontendGraphClass>& FDocumentController::GetSubgraphs() const
		{
			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				return Document->Subgraphs;
			}
			return Invalid::GetInvalidGraphClassArray();
		}

		const FMetasoundFrontendGraphClass& FDocumentController::GetRootGraphClass() const
		{
			if (const FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
				return Doc->RootGraph;
			}
			return Invalid::GetInvalidGraphClass();
		}

		void FDocumentController::SetRootGraphClass(FMetasoundFrontendGraphClass&& InClass)
		{
			if (FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
				Doc->RootGraph = MoveTemp(InClass);
			}
		}

		bool FDocumentController::AddDuplicateSubgraph(const FMetasoundFrontendGraphClass& InGraphToCopy, const FMetasoundFrontendDocument& InOtherDocument)
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				// Direct copy of subgraph
				bool bSuccess = true;
				FMetasoundFrontendGraphClass SubgraphCopy(InGraphToCopy);

				for (FMetasoundFrontendNode& Node : SubgraphCopy.GetDefaultGraph().Nodes)
				{
					const FGuid OriginalClassID = Node.ClassID;

					auto IsClassWithClassID = [&](const FMetasoundFrontendClass& InClass) -> bool
					{
						return InClass.ID == OriginalClassID;
					};

					if (const FMetasoundFrontendClass* OriginalNodeClass = InOtherDocument.Dependencies.FindByPredicate(IsClassWithClassID))
					{
						// Should not be a graph class since it's in the dependencies list
						check(EMetasoundFrontendClassType::Graph != OriginalNodeClass->Metadata.GetType());

						if (const FMetasoundFrontendClass* NodeClass = FindOrAddClass(OriginalNodeClass->Metadata).Get())
						{
							// All this just to update this ID. Maybe having globally 
							// consistent class IDs would help. Or using the classname & version as
							// a class ID. 
							Node.ClassID = NodeClass->ID;
						}
						else
						{
							UE_LOGF(LogMetaSound, Error, "Failed to add subgraph dependency [Class:%ls]", *OriginalNodeClass->Metadata.GetClassName().ToString());
							bSuccess = false;
						}
					}
					else if (const FMetasoundFrontendGraphClass* OriginalNodeGraphClass = InOtherDocument.Subgraphs.FindByPredicate(IsClassWithClassID))
					{
						bSuccess = bSuccess && AddDuplicateSubgraph(*OriginalNodeGraphClass, InOtherDocument);
						if (!bSuccess)
						{
							break;
						}
					}
					else
					{
						bSuccess = false;
						UE_LOGF(LogMetaSound, Error, "Failed to copy subgraph. Subgraph document is missing dependency info for node [Node:%ls, NodeID:%ls]", *Node.Name.ToString(), *Node.GetID().ToString());
					}
				}

				if (bSuccess)
				{
					Document->Subgraphs.Add(SubgraphCopy);
				}

				return bSuccess;
			}

			return false;
		}

		const TSet<FMetasoundFrontendVersion>& FDocumentController::GetInterfaceVersions() const
		{
			if (const FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
				return Doc->Interfaces;
			}

			static const TSet<FMetasoundFrontendVersion> EmptySet;
			return EmptySet;
		}

		void FDocumentController::AddInterfaceVersion(const FMetasoundFrontendVersion& InVersion)
		{
			if (FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
#if WITH_EDITOR
				Doc->Metadata.ModifyContext.AddInterfaceModified(InVersion.Name);
#endif // WITH_EDITOR
				Doc->Interfaces.Add(InVersion);
			}
		}

		void FDocumentController::RemoveInterfaceVersion(const FMetasoundFrontendVersion& InVersion)
		{
			if (FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
#if WITH_EDITOR
				Doc->Metadata.ModifyContext.AddInterfaceModified(InVersion.Name);
#endif // WITH_EDITOR
				Doc->Interfaces.Remove(InVersion);
			}
		}

		void FDocumentController::ClearInterfaceVersions()
		{
			if (FMetasoundFrontendDocument* Doc = DocumentPtr.Get())
			{
				Doc->Interfaces.Reset();
			}
		}

		FGraphHandle FDocumentController::AddDuplicateSubgraph(const IGraphController& InGraph)
		{
			// TODO: class IDs have issues.. 
			// Currently ClassIDs are just used for internal linking. They need to be fixed up
			// here if swapping documents. In the future, ClassIDs should be unique and consistent
			// across documents and platforms.

			FConstDocumentAccess GraphDocumentAccess = GetSharedAccess(*InGraph.GetOwningDocument());
			const FMetasoundFrontendDocument* OtherDocument = GraphDocumentAccess.ConstDocument.Get();
			if (nullptr == OtherDocument)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add subgraph from invalid document");
				return IGraphController::GetInvalidHandle();
			}

			FConstDocumentAccess GraphAccess = GetSharedAccess(InGraph);
			const FMetasoundFrontendGraphClass* OtherGraph = GraphAccess.ConstGraphClass.Get();
			if (nullptr == OtherGraph)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add invalid subgraph to document");
				return IGraphController::GetInvalidHandle();
			}

			if (AddDuplicateSubgraph(*OtherGraph, *OtherDocument))
			{
				if (const FMetasoundFrontendClass* SubgraphClass = FindClass(OtherGraph->Metadata).Get())
				{
					return GetSubgraphWithClassID(SubgraphClass->ID);
				}
			}

			return IGraphController::GetInvalidHandle();
		}
 

		FConstClassAccessPtr FDocumentController::FindDependencyWithID(FGuid InClassID) const 
		{
			return DocumentPtr.GetDependencyWithID(InClassID);
		}

		FConstGraphClassAccessPtr FDocumentController::FindSubgraphWithID(FGuid InClassID) const
		{
			return DocumentPtr.GetSubgraphWithID(InClassID);
		}

		FConstClassAccessPtr FDocumentController::FindClassWithID(FGuid InClassID) const
		{
			FConstClassAccessPtr MetasoundClass = FindDependencyWithID(InClassID);

			if (nullptr == MetasoundClass.Get())
			{
				MetasoundClass = FindSubgraphWithID(InClassID);
			}

			return MetasoundClass;
		}

		void FDocumentController::SetMetadata(const FMetasoundFrontendDocumentMetadata& InMetadata)
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				Document->Metadata = InMetadata;
			}
		}

		const FMetasoundFrontendDocumentMetadata& FDocumentController::GetMetadata() const
		{
			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				return Document->Metadata;
			}

			return Invalid::GetInvalidDocumentMetadata();
		}

		FMetasoundFrontendDocumentMetadata* FDocumentController::GetMetadata()
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				return &Document->Metadata;
			}

			return nullptr;
		}

		FConstClassAccessPtr FDocumentController::FindClass(const FNodeRegistryKey& InKey) const
		{
			return DocumentPtr.GetClassWithRegistryKey(InKey);
		}

		FConstClassAccessPtr FDocumentController::FindOrAddClass(const FNodeRegistryKey& InKey, bool bInRefreshFromRegistry)
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				FClassAccessPtr ClassPtr = DocumentPtr.GetClassWithRegistryKey(InKey);

				auto AddClass = [this, InKey, Document](FMetasoundFrontendClass&& NewClassDescription, const FGuid& NewClassID)
				{
					FConstClassAccessPtr NewClassPtr;

					// Cannot add a subgraph using this method because dependencies
					// of external graph are not added in this method.
					check(EMetasoundFrontendClassType::Graph != NewClassDescription.Metadata.GetType());
					NewClassDescription.ID = NewClassID;

					Document->Dependencies.Add(MoveTemp(NewClassDescription));
					NewClassPtr = FindClass(InKey);

					return NewClassPtr;
				};
				
				if (FMetasoundFrontendClass* MetasoundClass = ClassPtr.Get())
				{
					// External node classes must match version to return shared definition.
					if (MetasoundClass->Metadata.GetType() == EMetasoundFrontendClassType::Template
						|| MetasoundClass->Metadata.GetType() == EMetasoundFrontendClassType::External)
					{
						// TODO: Assuming we want to recheck classes when they add another
						// node, this should be replace with a call to synchronize a 
						// single class.
						FMetasoundFrontendClass NewClass = GenerateClass(InKey);
						if (NewClass.Metadata.GetVersion().Major != MetasoundClass->Metadata.GetVersion().Major)
						{
							const FGuid ClassId = FDocumentIDGenerator::Get().CreateClassID(*Document);
							return AddClass(MoveTemp(NewClass), ClassId);
						}
					}

					if (bInRefreshFromRegistry)
					{
						FGuid ClassID = MetasoundClass->ID;
						*MetasoundClass = GenerateClass(InKey);
						MetasoundClass->ID = ClassID;
						return FindClass(InKey);
					}

					return ClassPtr;
				}

				FMetasoundFrontendClass NewClass = GenerateClass(InKey);
				const FGuid ClassId = FDocumentIDGenerator::Get().CreateClassID(*Document);
				return AddClass(MoveTemp(NewClass), ClassId);
			}

			return FConstClassAccessPtr();
		}

		FConstClassAccessPtr FDocumentController::FindClass(const FMetasoundFrontendClassMetadata& InMetadata) const
		{
			return DocumentPtr.GetClassWithMetadata(InMetadata);
		}

		FConstClassAccessPtr FDocumentController::FindOrAddClass(const FMetasoundFrontendClassMetadata& InMetadata)
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;

			FConstClassAccessPtr ClassPtr = FindClass(InMetadata);
			
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				// External & Template node classes must match major version to return shared definition.
				if (EMetasoundFrontendClassType::External == InMetadata.GetType()
					|| EMetasoundFrontendClassType::Template == InMetadata.GetType())
				{
					if (InMetadata.GetVersion().Major != Class->Metadata.GetVersion().Major)
					{
						// Mismatched major version. Reset class pointer to null.
						ClassPtr = FConstClassAccessPtr();
					}
				}
			}


			const bool bNoMatchingClassFoundInDocument = (nullptr == ClassPtr.Get());
			if (bNoMatchingClassFoundInDocument)
			{
				// If no matching class found, attempt to add a class matching the metadata.
				if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
				{
					switch (InMetadata.GetType())
					{
						case EMetasoundFrontendClassType::External:
						case EMetasoundFrontendClassType::Template:
						case EMetasoundFrontendClassType::Input:
						case EMetasoundFrontendClassType::Output:
						{
							FMetasoundFrontendClass NewClass;
							FNodeRegistryKey Key = FNodeRegistryKey(InMetadata);

							if (FRegistry::GetFrontendClassFromRegistered(Key, NewClass))
							{
								NewClass.ID = FGuid::NewGuid();
								Document->Dependencies.Add(NewClass);
							}
							else
							{
#if WITH_EDITOR
								UE_LOGF(LogMetaSound, Error,
									"Cannot add external dependency. No Metasound class found with matching registry key [Key:%ls, Name:%ls, Version:%ls]. Suggested solution \"%ls\" by %ls.",
									*Key.ToString(),
									*InMetadata.GetClassName().ToString(),
									*InMetadata.GetVersion().ToString(),
									*InMetadata.GetPromptIfMissing().ToString(),
									*InMetadata.GetAuthor());
#else
								UE_LOGF(LogMetaSound, Error,
									"Cannot add external dependency. No Metasound class found with matching registry key [Key:%ls, Name:%ls, Version:%ls].",
									*Key.ToString(),
									*InMetadata.GetClassName().ToString(),
									*InMetadata.GetVersion().ToString());
#endif // !WITH_EDITOR
							}
						}
						break;

						case EMetasoundFrontendClassType::Graph:
						{
							FMetasoundFrontendGraphClass NewClass;
							NewClass.ID = FGuid::NewGuid();
							NewClass.Metadata = InMetadata;
							NewClass.InitDefaultGraphPage();
							Document->Subgraphs.Add(NewClass);
						}
						break;

						default:
						{
							UE_LOGF(LogMetaSound, Error, "Unsupported metasound class type for node: \"%ls\" (%ls).",
								*InMetadata.GetClassName().ToString(),
								*InMetadata.GetVersion().ToString());
							checkNoEntry();
						}
					}

					ClassPtr = FindClass(InMetadata);
				}
			}

			return ClassPtr;
		}

		void FDocumentController::DeduplicateDependencies()
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				auto CompareForUnique = [&](const FMetasoundFrontendClass& InLHS, const FMetasoundFrontendClass& InRHS)
				{
					return InLHS.ID == InRHS.ID;
				};

				// Remove duplicate entries, keeping the highest version dependencies. 
				auto CompareForSort = [&](const FMetasoundFrontendClass& InLHS, const FMetasoundFrontendClass& InRHS)
				{
					// Sort by ID
					if (InLHS.ID < InRHS.ID)
					{
						return true;
					}
					else if (InRHS.ID < InLHS.ID)
					{
						return false;
					}
					// If IDs are equal, sort by version number descending
					else if (InLHS.Metadata.GetVersion() > InRHS.Metadata.GetVersion())
					{
						return true;
					}
					else if (InLHS.Metadata.GetVersion() < InRHS.Metadata.GetVersion())
					{
						return false ;
					}
					else
					{
						// if IDs and version numbers are equal, sort by number of inputs & outputs descending
						const int32 NumLHSVertices = InLHS.GetDefaultInterface().Inputs.Num() + InLHS.GetDefaultInterface().Outputs.Num();
						const int32 NumRHSVertices = InRHS.GetDefaultInterface().Inputs.Num() + InRHS.GetDefaultInterface().Outputs.Num();

						return NumLHSVertices > NumRHSVertices;
					}
				};

				Algo::Sort(Document->Dependencies, CompareForSort);
				Document->Dependencies.SetNum(Algo::Unique(Document->Dependencies, CompareForUnique));
			}
		}
		void FDocumentController::RemoveUnreferencedDependencies()
		{
			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				// Remove duplicate entries, keeping the highest version dependencies. 
				DeduplicateDependencies();

				int32 NumDependenciesRemovedThisItr = 0;

				// Repeatedly remove unreferenced dependencies until there are
				// no unreferenced dependencies left.
				do
				{
					TSet<FGuid> ReferencedDependencyIDs;
					auto AddGraphNodeClassIDsToSet = [&ReferencedDependencyIDs](const FMetasoundFrontendGraphClass& GraphClass)
					{
						GraphClass.IterateGraphPages([&ReferencedDependencyIDs](const FMetasoundFrontendGraph& Graph)
						{
							auto AddNodeClassIDToSet = [&ReferencedDependencyIDs](const FMetasoundFrontendNode& Node)
							{
								ReferencedDependencyIDs.Add(Node.ClassID);
							};
							Algo::ForEach(Graph.Nodes, AddNodeClassIDToSet);
						});
					};

					// Referenced dependencies in RootGraph
					AddGraphNodeClassIDsToSet(Document->RootGraph);

					// Referenced dependencies in Subgraphs
					Algo::ForEach(Document->Subgraphs, AddGraphNodeClassIDsToSet);

					auto IsDependencyUnreferenced = [&ReferencedDependencyIDs](const FMetasoundFrontendClass& ClassDependency)
					{
						return !ReferencedDependencyIDs.Contains(ClassDependency.ID);
					};

					NumDependenciesRemovedThisItr = Document->Dependencies.RemoveAllSwap(IsDependencyUnreferenced);
				} while (NumDependenciesRemovedThisItr > 0);
			}
		}

		TArray<FConstClassAccessPtr> FDocumentController::SynchronizeDependencyMetadata()
		{
			TArray<FConstClassAccessPtr> UpdatedClassPtrs;

			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				for (FMetasoundFrontendClass& Class : Document->Dependencies)
				{
					FMetasoundFrontendClass RegistryVersion;
					FNodeRegistryKey RegistryKey(Class.Metadata);
					if (FMetasoundFrontendRegistryContainer::Get()->FindFrontendClassFromRegistered(RegistryKey, RegistryVersion))
					{
						if (Class.Metadata.GetChangeID() != RegistryVersion.Metadata.GetChangeID())
						{
							Class.Metadata = MoveTemp(RegistryVersion.Metadata);
							UpdatedClassPtrs.Add(FindClass(RegistryKey));
						}
					}
				}
			}

			return UpdatedClassPtrs;
		}

		FGraphHandle FDocumentController::GetRootGraph()
		{
			if (IsValid())
			{
				FGraphClassAccessPtr GraphClass = DocumentPtr.GetRootGraph();
				return FGraphController::CreateGraphHandle(FGraphController::FInitParams{GraphClass, this->AsShared()});
			}

			return IGraphController::GetInvalidHandle();
		}

		FConstGraphHandle FDocumentController::GetRootGraph() const
		{
			if (IsValid())
			{
				FConstGraphClassAccessPtr GraphClass = DocumentPtr.GetRootGraph();
				return FGraphController::CreateConstGraphHandle(FGraphController::FInitParams
					{
						ConstCastAccessPtr<FGraphClassAccessPtr>(GraphClass),
						ConstCastSharedRef<IDocumentController>(this->AsShared())
					});
			}
			return IGraphController::GetInvalidHandle();
		}

		FDocumentAccessPtr FDocumentController::GetDocumentPtr()
		{
			return DocumentPtr;
		}

		const FDocumentAccessPtr FDocumentController::GetDocumentPtr() const
		{
			return DocumentPtr;
		}

		TArray<FGraphHandle> FDocumentController::GetSubgraphHandles() 
		{
			TArray<FGraphHandle> Subgraphs;

			if (FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				for (FMetasoundFrontendGraphClass& GraphClass : Document->Subgraphs)
				{
					Subgraphs.Add(GetSubgraphWithClassID(GraphClass.ID));
				}
			}

			return Subgraphs;
		}

		TArray<FConstGraphHandle> FDocumentController::GetSubgraphHandles() const 
		{
			TArray<FConstGraphHandle> Subgraphs;

			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				for (const FMetasoundFrontendGraphClass& GraphClass : Document->Subgraphs)
				{
					Subgraphs.Add(GetSubgraphWithClassID(GraphClass.ID));
				}
			}

			return Subgraphs;
		}

		FGraphHandle FDocumentController::GetSubgraphWithClassID(FGuid InClassID)
		{
			FGraphClassAccessPtr GraphClassPtr = DocumentPtr.GetSubgraphWithID(InClassID);

			return FGraphController::CreateGraphHandle(FGraphController::FInitParams{GraphClassPtr, this->AsShared()});
		}

		FConstGraphHandle FDocumentController::GetSubgraphWithClassID(FGuid InClassID) const
		{
			FConstGraphClassAccessPtr GraphClassPtr = DocumentPtr.GetSubgraphWithID(InClassID);

			return FGraphController::CreateConstGraphHandle(FGraphController::FInitParams{ConstCastAccessPtr<FGraphClassAccessPtr>(GraphClassPtr), ConstCastSharedRef<IDocumentController>(this->AsShared())});
		}

		bool FDocumentController::ExportToJSONAsset(const FString& InAbsolutePath) const
		{
			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*InAbsolutePath)))
				{
					TJsonStructSerializerBackend<DefaultCharType> Backend(*FileWriter, EStructSerializerBackendFlags::Default);
					FStructSerializer::Serialize<FMetasoundFrontendDocument>(*Document, Backend);
			
					FileWriter->Close();

					return true;
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Failed to export Metasound json asset. Could not write to path \"%ls\".", *InAbsolutePath);
				}
			}

			return false;
		}

		FString FDocumentController::ExportToJSON() const
		{
			FString Output;

			if (const FMetasoundFrontendDocument* Document = DocumentPtr.Get())
			{
				TArray<uint8> WriterBuffer;
				FMemoryWriter MemWriter(WriterBuffer);

				Metasound::TJsonStructSerializerBackend<Metasound::DefaultCharType> Backend(MemWriter, EStructSerializerBackendFlags::Default);
				FStructSerializer::Serialize<FMetasoundFrontendDocument>(*Document, Backend);

				MemWriter.Close();

				// null terminator
				WriterBuffer.AddZeroed(sizeof(ANSICHAR));

				Output.AppendChars(reinterpret_cast<ANSICHAR*>(WriterBuffer.GetData()), WriterBuffer.Num() / sizeof(ANSICHAR));
			}

			return Output;
		}
		
		FDocumentAccess FDocumentController::ShareAccess() 
		{
			FDocumentAccess Access;

			Access.Document = DocumentPtr;
			Access.ConstDocument = DocumentPtr;

			return Access;
		}

		FConstDocumentAccess FDocumentController::ShareAccess() const 
		{
			FConstDocumentAccess Access;

			Access.ConstDocument = DocumentPtr;

			return Access;
		}
	}
}
#undef LOCTEXT_NAMESPACE
