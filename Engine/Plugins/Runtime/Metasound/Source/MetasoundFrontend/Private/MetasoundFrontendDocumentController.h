// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"

namespace Metasound
{
	namespace Frontend
	{
		/** FDocumentController represents an entire Metasound document. */
		class FDocumentController : public IDocumentController 
		{

		public:
			/** Construct a FDocumentController.
			 *
			 * @param InDocument - Document to be manipulated.
			 */
			FDocumentController(FDocumentAccessPtr InDocumentPtr);

			static FDocumentHandle CreateDocumentHandle(FDocumentAccessPtr InDocument, bool bReloadBuilder = true)
			{
				// Unit test builds may not load the builder registry (i.e. via the engine module).  Creating and
				// manipulating documents via controllers/handles must be supported for backward compat in this context,
				// so registry is not required to exist.
				if (bReloadBuilder)
				{
					if (IDocumentBuilderRegistry* Registry = IDocumentBuilderRegistry::Get())
					{
						const FMetasoundFrontendClassName& ClassName = InDocument.Get()->RootGraph.Metadata.GetClassName();
						Registry->ReloadBuilder(ClassName);
					}
				}
				return MakeShared<FDocumentController>(InDocument);
			}

			/** Create a FDocumentController.
			 *
			 * @param InDocument - Document to be manipulated.
			 *
			 * @return A document handle. 
			 */
			static FConstDocumentHandle CreateDocumentHandle(FConstDocumentAccessPtr InDocument)
			{
				return MakeShared<const FDocumentController>(ConstCastAccessPtr<FDocumentAccessPtr>(InDocument));
			}

			virtual ~FDocumentController() = default;

			bool IsValid() const override;

			const TArray<FMetasoundFrontendClass>& GetDependencies() const override;
			void IterateDependencies(TFunctionRef<void(FMetasoundFrontendClass&)> InFunction) override;
			void IterateDependencies(TFunctionRef<void(const FMetasoundFrontendClass&)> InFunction) const override;
			const TArray<FMetasoundFrontendGraphClass>& GetSubgraphs() const override;
			const FMetasoundFrontendGraphClass& GetRootGraphClass() const override;
			void SetRootGraphClass(FMetasoundFrontendGraphClass&& InClass) override;

			FConstClassAccessPtr FindDependencyWithID(FGuid InClassID) const override;
			FConstGraphClassAccessPtr FindSubgraphWithID(FGuid InClassID) const override;
			FConstClassAccessPtr FindClassWithID(FGuid InClassID) const override;

			FConstClassAccessPtr FindClass(const FNodeRegistryKey& InKey) const override;
			FConstClassAccessPtr FindClass(const FMetasoundFrontendClassMetadata& InMetadata) const override;

			FConstClassAccessPtr FindOrAddClass(const FNodeRegistryKey& InKey, bool bInRefreshFromRegistry) override;
			FConstClassAccessPtr FindOrAddClass(const FMetasoundFrontendClassMetadata& InMetadata) override;

			FGraphHandle AddDuplicateSubgraph(const IGraphController& InGraph) override;

			const TSet<FMetasoundFrontendVersion>& GetInterfaceVersions() const override;
			void AddInterfaceVersion(const FMetasoundFrontendVersion& InVersion) override;
			void RemoveInterfaceVersion(const FMetasoundFrontendVersion& InVersion) override;
			void ClearInterfaceVersions() override;

			void SetMetadata(const FMetasoundFrontendDocumentMetadata& InMetadata) override;
			const FMetasoundFrontendDocumentMetadata& GetMetadata() const override;
			FMetasoundFrontendDocumentMetadata* GetMetadata() override;

			void RemoveUnreferencedDependencies() override;
			TArray<FConstClassAccessPtr> SynchronizeDependencyMetadata() override;

			FGraphHandle GetRootGraph() override;
			FConstGraphHandle GetRootGraph() const override;

			FDocumentAccessPtr GetDocumentPtr() override;
			const FDocumentAccessPtr GetDocumentPtr() const override;

			/** Returns an array of all subgraphs for this document. */
			TArray<FGraphHandle> GetSubgraphHandles() override;

			/** Returns an array of all subgraphs for this document. */
			TArray<FConstGraphHandle> GetSubgraphHandles() const override;

			/** Returns a graphs in the document with the given class ID.*/
			FGraphHandle GetSubgraphWithClassID(FGuid InClassID) override;

			/** Returns a graphs in the document with the given class ID.*/
			FConstGraphHandle GetSubgraphWithClassID(FGuid InClassID) const override;

			bool ExportToJSONAsset(const FString& InAbsolutePath) const override;
			FString ExportToJSON() const override;

		protected:
			FDocumentAccess ShareAccess() override;
			FConstDocumentAccess ShareAccess() const override;
		private:

			void DeduplicateDependencies();
			bool AddDuplicateSubgraph(const FMetasoundFrontendGraphClass& InGraphToCopy, const FMetasoundFrontendDocument& InOtherDocument);

			FDocumentAccessPtr DocumentPtr;
		};
	}
}

