// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFrontendDocumentIdGenerator.h"

#include "MetasoundAssetManager.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentVersioning.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace DocumentIDGeneratorPrivate
		{
			std::atomic<uint64> GlobalAtomicMetasoundIdCounter = 1; // First ID will be 1. This is because we will make this into an FGuid which cannot be zero.
		}

		FDocumentIDGenerator::FScopeDeterminism::FScopeDeterminism(bool bInIsDeterministic)
		{
			FDocumentIDGenerator& IDGen = Get();
			bOriginalValue = IDGen.GetDeterminism();
			IDGen.SetDeterminism(bInIsDeterministic);
		}

		bool FDocumentIDGenerator::FScopeDeterminism::GetDeterminism() const
		{
			return Get().GetDeterminism();
		}

		FDocumentIDGenerator::FScopeDeterminism::~FScopeDeterminism()
		{
			Get().SetDeterminism(bOriginalValue);
		}

		void FDocumentIDGenerator::SetDeterminism(bool bInIsDeterministic)
		{
			bIsDeterministic = bInIsDeterministic;
		}

		bool FDocumentIDGenerator::GetDeterminism() const
		{
			return bIsDeterministic;
		}
		
		FDocumentIDGenerator& FDocumentIDGenerator::Get()
		{
			thread_local FDocumentIDGenerator IDGenerator;
			return IDGenerator;
		}

		FGuid FDocumentIDGenerator::CreateNodeID(const FMetasoundFrontendDocument& InDocument) const
		{
			return CreateIDFromDocument(InDocument);
		}

		FGuid FDocumentIDGenerator::CreateVertexID(const FMetasoundFrontendDocument& InDocument) const
		{
			return CreateIDFromDocument(InDocument);
		}

		FGuid FDocumentIDGenerator::CreateClassID(const FMetasoundFrontendDocument& InDocument) const
		{
			return CreateIDFromDocument(InDocument);
		}

		FGuid FDocumentIDGenerator::CreateIDFromDocument(const FMetasoundFrontendDocument& InDocument) const
		{
			if (bIsDeterministic)
			{
				// A bug caused collisions between serialized content and newly generated values.
				// The use of this base guid ensures no such collision generation continues.
				constexpr FGuid BaseGuid(0x8BC4C7C3, 0x591449C4, 0xA35830F8, 0xE7F9052E);

				const uint32 Value = InDocument.GetNextIdCounter();
				const FGuid CounterGuid = FGuid(Value << 6, Value << 4, Value << 2, Value);
				const FGuid UpdatedGuid = FGuid::Combine(CounterGuid, BaseGuid);

				FGuid ClassID;
				const FMetasoundFrontendClassName& ClassName = InDocument.RootGraph.Metadata.GetClassName();
				ensureAlwaysMsgf(IMetaSoundAssetManager::GetChecked().TryGetAssetIDFromClassName(ClassName, ClassID), TEXT("Failed to retrieve AssetID from MetaSoundClassName"));

				return FGuid::Combine(ClassID, UpdatedGuid);
			}
			else
			{
				return FGuid::NewGuid();
			}
		}

		FClassIDGenerator& FClassIDGenerator::Get()
		{
			static FClassIDGenerator IDGenerator;
			return IDGenerator;
		}

		FGuid FClassIDGenerator::CreateInputID(const FMetasoundFrontendClassInput& Input) const
		{
			constexpr FGuid ClassInputNamespaceGuid = FGuid(0x149FEB6E, 0xB9F947A6, 0xAD4FB786, 0x55F6EBE8);
			FString NameToHash = FString::Printf(TEXT("ClassInput.%s.%s.%s"), *Input.Name.ToString(), *Input.TypeName.ToString(), LexToString(Input.AccessType));

			return CreateNamespacedIDFromString(ClassInputNamespaceGuid, NameToHash);
		}

		FGuid FClassIDGenerator::CreateInputID(const Audio::FParameterInterface::FInput& Input) const
		{
			constexpr FGuid ParameterInterfaceInputNamespaceGuid(0xD9E893C0, 0x92B34CB4, 0x83064525, 0xABEACADD);
			FString NameToHash = FString::Printf(TEXT("ParameterInterfaceInput.%s.%s"), *Input.InitValue.ParamName.ToString(), *Input.DataType.ToString());

			return CreateNamespacedIDFromString(ParameterInterfaceInputNamespaceGuid, NameToHash);
		}

		FGuid FClassIDGenerator::CreateOutputID(const FMetasoundFrontendClassOutput& Output) const
		{
			constexpr FGuid ClassOutputNamespaceGuid(0xC7B3ED2C, 0x44074B2A, 0x91447F11, 0x08387EBB);
			FString NameToHash = FString::Printf(TEXT("ClassOutput.%s.%s.%s"), *Output.Name.ToString(), *Output.TypeName.ToString(), LexToString(Output.AccessType));

			return CreateNamespacedIDFromString(ClassOutputNamespaceGuid, NameToHash);
		}

		FGuid FClassIDGenerator::CreateOutputID(const Audio::FParameterInterface::FOutput& Output) const
		{
			constexpr FGuid ParameterInterfaceOutputNamespaceGuid(0x6F41342A, 0x24364462, 0x81A08517, 0x887BB729);
			FString NameToHash = FString::Printf(TEXT("ParameterInterfaceOutput.%s.%s"), *Output.ParamName.ToString(), *Output.DataType.ToString());

			return CreateNamespacedIDFromString(ParameterInterfaceOutputNamespaceGuid, NameToHash);
		}

		FGuid FClassIDGenerator::CreateNamespacedIDFromString(const FGuid& NamespaceGuid, const FString& StringToHash) const
		{
			FSHA1 Hasher;
			Hasher.Update(reinterpret_cast<const uint8*>(&NamespaceGuid), sizeof(FGuid));
			Hasher.UpdateWithString(*StringToHash, StringToHash.Len());
			FSHAHash HashValue = Hasher.Finalize();
			const uint32* HashUint32 = reinterpret_cast<const uint32*>(HashValue.Hash);

			return FGuid(HashUint32[0], HashUint32[1], HashUint32[2], HashUint32[3]);
		}

		FGuid CreateLocallyUniqueId()
		{
			using namespace DocumentIDGeneratorPrivate;
			uint64 NextId = GlobalAtomicMetasoundIdCounter.fetch_add(1, std::memory_order_relaxed);
			return FGuid(0, 0, NextId >> 32, NextId & 0xFFFFFFFF);
		}
	}
}
