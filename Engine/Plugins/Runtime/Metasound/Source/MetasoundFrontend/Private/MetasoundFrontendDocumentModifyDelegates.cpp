// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendDocumentModifyDelegates.h"

#if WITH_EDITORONLY_DATA
#include "MetasoundFrontendDocumentVersioning.h"
#endif // WITH_EDITORONLY_DATA

namespace Metasound::Frontend
{
	FDocumentModifyDelegates::FDocumentModifyDelegates()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: OnPresetStateChanged()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FDocumentModifyDelegates::FDocumentModifyDelegates(const FMetasoundFrontendDocument& Document)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: OnPresetStateChanged()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		Document.RootGraph.IterateGraphPages([this](const FMetasoundFrontendGraph& Graph)
		{
			AddPageDelegates(Graph.PageID);
		});

#if WITH_EDITORONLY_DATA
		// Modify delegates may register when a builder is constructed prior to a document
		// being versioned. In this legacy case, add the default page ID delegates as the
		// default page is the only version that exists on legacy documents.
		if (Document.Metadata.Version.Number < FVersioningManager::GetPageMigrationVersion())
		{
			if (Document.RootGraph.GetConstGraphPages().IsEmpty())
			{
				AddPageDelegates(Frontend::DefaultPageID);
			}
		}
#endif // WITH_EDITORONLY_DATA
	}

	void FDocumentModifyDelegates::AddPageDelegates(const FGuid& InPageID)
	{
		if (!GraphDelegates.Contains(InPageID))
		{
			GraphDelegates.Add(InPageID, FGraphModifyDelegates());
			PageDelegates.OnPageAdded.Broadcast(FDocumentMutatePageArgs{ InPageID });
		}
	}

	FNodeModifyDelegates& FDocumentModifyDelegates::FindNodeDelegatesChecked(const FGuid& InPageID)
	{
		return GraphDelegates.FindChecked(InPageID).NodeDelegates;
	}

	FEdgeModifyDelegates& FDocumentModifyDelegates::FindEdgeDelegatesChecked(const FGuid& InPageID)
	{
		return GraphDelegates.FindChecked(InPageID).EdgeDelegates;
	}

	FGraphModifyDelegates& FDocumentModifyDelegates::FindGraphDelegatesChecked(const FGuid& InPageID)
	{
		return GraphDelegates.FindChecked(InPageID);
	}

	void FDocumentModifyDelegates::IterateGraphEdgeDelegates(TFunctionRef<void(FEdgeModifyDelegates&)> Func)
	{
		IterateGraphDelegates([&](FGraphModifyDelegates& GraphDelegates)
		{
			Func(GraphDelegates.EdgeDelegates);
		});
	}

	void FDocumentModifyDelegates::IterateGraphNodeDelegates(TFunctionRef<void(FNodeModifyDelegates&)> Func)
	{
		IterateGraphDelegates([&](FGraphModifyDelegates& GraphDelegates)
		{
			Func(GraphDelegates.NodeDelegates);
		});
	}

	void FDocumentModifyDelegates::IterateGraphDelegates(TFunctionRef<void(FGraphModifyDelegates&)> Func)
	{
		for (TPair<FGuid, FGraphModifyDelegates>& Pair : GraphDelegates)
		{
			Func(Pair.Value);
		}
	}

	void FDocumentModifyDelegates::RemovePageDelegates(const FGuid& InPageID, bool bBroadcastNotify)
	{
		if (GraphDelegates.Contains(InPageID))
		{
			if (bBroadcastNotify)
			{
				PageDelegates.OnRemovingPage.Broadcast(FDocumentMutatePageArgs{ InPageID });
			}
			GraphDelegates.Remove(InPageID);
		}
	}
} // namespace Metasound::Frontend
