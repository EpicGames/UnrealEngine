// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFrontendDocumentVersioning.h"

#include "Misc/CoreMiscDefines.h"

#if WITH_EDITORONLY_DATA
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "CoreGlobals.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/MetasoundFrontendInterface.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundAccessPtr.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentController.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"
#include "Misc/App.h"
#include "UObject/UObjectHash.h"

namespace Metasound::Frontend
{
	namespace VersioningPrivate
	{
		static int32 MetaSoundAutoUpdateUseAssetChangeIDsCVar = 0;
		static TUniquePtr<FVersioningManager> VersioningManager;

		FAutoConsoleVariableRef CVarMetaSoundAutoUpdateUseAssetChangeIDs(
			TEXT("au.MetaSound.AutoUpdate.UseChangeIDs"),
			MetaSoundAutoUpdateUseAssetChangeIDsCVar,
			TEXT("If true, use soft-deprecated Change ID system to speed up diffing interface or metadata changes during auto-update. If false, ignore change IDs and always do \"deep check\" for changes. \n")
			TEXT("0: Don't use Change Ids (default), !0: Use ChangeIDs"),
			ECVF_Default);

		FMetasoundFrontendVersion GetTargetInterfaceVersion(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendVersion& InterfaceVersion)
		{
			using namespace Metasound::Frontend;

			// Find registered target interface.
			FMetasoundFrontendVersion TargetInterfaceVersion;
			bool bFoundTargetInterface = ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceVersion.Name, TargetInterfaceVersion);
			if (!bFoundTargetInterface)
			{
				METASOUND_VERSIONING_LOG(Warning,
					TEXT("Could not check for interface updates. Target interface is not registered [InterfaceVersion:%s] when attempting to update root graph of asset (%s). "
						"Ensure that the module which registers the interface has been loaded before the asset is loaded."),
					*InterfaceVersion.ToString(),
					*Builder.GetDebugName());
				return { };
			}

			if (TargetInterfaceVersion == InterfaceVersion)
			{
				return { };
			}

			return TargetInterfaceVersion;
		}

		void GetUpdatePathForDocument(const FMetasoundFrontendVersion& InCurrentVersion, const FMetasoundFrontendVersion& InTargetVersion, TArray<TSharedPtr<IBuilderVersionTransform>>& OutUpgradePath)
		{
			OutUpgradePath.Reset();

			if (InCurrentVersion.Name == InTargetVersion.Name)
			{
				// Get all associated registered interfaces
				TArray<FMetasoundFrontendVersion> RegisteredVersions = ISearchEngine::Get().FindAllRegisteredInterfacesWithName(InTargetVersion.Name);

				// Filter registry entries that exist between current version and target version
				auto FilterRegistryEntries = [&InCurrentVersion, &InTargetVersion](const FMetasoundFrontendVersion& InVersion)
				{
					const bool bIsGreaterThanCurrent = InVersion.Number > InCurrentVersion.Number;
					const bool bIsLessThanOrEqualToTarget = InVersion.Number <= InTargetVersion.Number;

					return bIsGreaterThanCurrent && bIsLessThanOrEqualToTarget;
				};
				RegisteredVersions = RegisteredVersions.FilterByPredicate(FilterRegistryEntries);

				// sort registry entries to create an ordered upgrade path.
				RegisteredVersions.Sort();

				// Get registry entries from registry keys.
				auto GetUpdateTransform = [](const FMetasoundFrontendVersion& InVersion)
				{
					return IInterfaceRegistry::Get().FindInterfaceUpdateTransform(InVersion);
				};
				Algo::Transform(RegisteredVersions, OutUpgradePath, GetUpdateTransform);
			}
		}

		bool UpdateDocumentInterface(const TArray<TSharedPtr<IBuilderVersionTransform>>& InUpgradePath, const FMetasoundFrontendVersion& InterfaceVersion, FMetaSoundFrontendDocumentBuilder& Builder)
		{
			const FMetasoundFrontendVersionNumber* LastVersionUpdated = nullptr;
			for (const TSharedPtr<IBuilderVersionTransform>& Transform : InUpgradePath)
			{
				if (ensureAlwaysMsgf(Transform.IsValid(), TEXT("Transform should always be provided, even if referencing the default transform, by this point in updating document interfaces")))
				{
					if (Transform->Transform(Builder))
					{
						LastVersionUpdated = &Transform->GetVersion().Number;
					}
				}
			}

			if (LastVersionUpdated)
			{
				const FMetasoundFrontendClassMetadata& Metadata = Builder.GetMetasoundAsset().GetDocumentChecked().RootGraph.Metadata;
#if WITH_EDITOR
				const FString AssetName = *Metadata.GetDisplayName().ToString();
#else
				const FString AssetName = *Metadata.GetClassName().ToString();
#endif // !WITH_EDITOR
				METASOUND_VERSIONING_LOG(Display, TEXT("Asset '%s' interface '%s' updated: '%s' --> '%s'"),
					*AssetName,
					*InterfaceVersion.Name.ToString(),
					*InterfaceVersion.Number.ToString(),
					*LastVersionUpdated->ToString());
				return true;
			}

			return false;
		}

		// Checks if version is up-to-date. If so, returns true. If false, updates the interfaces within the given asset's document to the most recent version.
		bool TryUpdateInterfaceFromVersion(const FMetasoundFrontendVersion& Version, FMetaSoundFrontendDocumentBuilder& Builder)
		{
			using namespace Metasound::Frontend;

			const FMetasoundFrontendVersion TargetInterfaceVersion = GetTargetInterfaceVersion(Builder, Version);
			if (TargetInterfaceVersion.IsValid())
			{
				TArray<TSharedPtr<IBuilderVersionTransform>> UpgradePath;
				GetUpdatePathForDocument(Version, TargetInterfaceVersion, UpgradePath);
				const bool bUpdated = UpdateDocumentInterface(UpgradePath, Version, Builder);
				ensureMsgf(bUpdated, TEXT("Target interface '%s' was out-of-date but interface failed to be updated"), *TargetInterfaceVersion.ToString());
				return bUpdated;
			}

			return false;
		}

		class FMigratePagePropertiesTransform : public FMetaSoundFrontendDocumentBuilder::IPropertyVersionTransform
		{
		public:
			virtual ~FMigratePagePropertiesTransform() = default;

			bool Transform(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				using namespace Metasound;

				bool bUpdated = false;
				auto MigrateInterfaceInputDefaults = [&](FMetasoundFrontendClassInterface& OutInterface)
				{
					for (FMetasoundFrontendClassInput& Input : OutInterface.Inputs)
					{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						if (Input.DefaultLiteral.IsValid())
						{
							// There were cases where the default literal may have
							// been set to 'None' incorrectly on struct initialization
							// instead of invalid after versioning.  In this case, don't
							// port the input default as a paged value if valid data
							// already exists.
							if (Input.GetDefaults().IsEmpty())
							{
								Input.AddDefault(Frontend::DefaultPageID) = MoveTemp(Input.DefaultLiteral);
							}

							// Always set default literal to invalid due to a minor versioning issue
							// where the input default used to initialize its type set to 'None' incorrectly.
							Input.DefaultLiteral = FMetasoundFrontendLiteral::GetInvalid();
							bUpdated = true;
						}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				};

				FMetasoundFrontendDocument& Document = GetDocumentUnsafe(OutBuilder);
				FMetasoundFrontendGraphClass& GraphClass = Document.RootGraph;
				MigrateInterfaceInputDefaults(GraphClass.GetDefaultInterface());
				for (FMetasoundFrontendClass& Dependency : Document.Dependencies)
				{
					MigrateInterfaceInputDefaults(Dependency.GetDefaultInterface());
				}

				if (bUpdated)
				{
					METASOUND_VERSIONING_LOG(Display, TEXT("Migrated Class Interface paged input defaults for MetaSound '%s'"), *OutBuilder.GetDebugName());
				}

				struct FMigratePageGraphs : public FMetasoundFrontendGraphClass::IPropertyVersionTransform
				{
					const UObject& PageGraphMS;
				public:
					FMigratePageGraphs(const UObject& InMetaSound)
						: PageGraphMS(InMetaSound)
					{
					}

					virtual ~FMigratePageGraphs() = default;

					virtual bool Transform(FMetasoundFrontendGraphClass& OutClass) const override
					{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						TArray<FMetasoundFrontendGraph>& Pages = GetPagesUnsafe(OutClass);
						if (Pages.IsEmpty())
						{
							Pages.Add(MoveTemp(OutClass.Graph));
							OutClass.Graph = { };
							METASOUND_VERSIONING_LOG(Display, TEXT("Migrated Class Interface paged graph for MetaSound '%s'"), *PageGraphMS.GetPathName());
							return true;
						}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

						return false;
					}
				};

				bUpdated |= FMigratePageGraphs(OutBuilder.CastDocumentObjectChecked<UObject>()).Transform(GraphClass);

				struct FMigrateNodeIds : public FMetasoundFrontendGraphClass::IPropertyVersionTransform
				{
					const UObject& NodeIDMS;
					const FMetasoundFrontendDocument& NodeIDDoc;

				public:
					FMigrateNodeIds(const UObject& InMetaSound, const FMetasoundFrontendDocument& InDocument)
						: NodeIDMS(InMetaSound)
						, NodeIDDoc(InDocument)
					{
					}

					virtual ~FMigrateNodeIds() = default;

					virtual bool Transform(FMetasoundFrontendGraphClass& OutClass) const override
					{
						TArray<FMetasoundFrontendGraph>& Pages = GetPagesUnsafe(OutClass);

						bool bNodeIDModified = false;

						for (FMetasoundFrontendGraph& Graph : Pages)
						{
							FGuid NewNodeID;
							for (FMetasoundFrontendNode& Node : Graph.Nodes)
							{
								if (!Node.GetID().IsValid())
								{
									checkf(!NewNodeID.IsValid(), TEXT("More than one node found with zero guid in %s: page %s.  Cannot recover from document corrupted with nodes of the same invalid id"),
										*NodeIDMS.GetPathName(),
										*Graph.PageID.ToString());
									NewNodeID = FDocumentIDGenerator::Get().CreateNodeID(NodeIDDoc);
									Node.UpdateID(NewNodeID);
									METASOUND_VERSIONING_LOG(Display, TEXT("Found node (in '%s': page %s) with zero ID. Updating to '%s'"),
										*NodeIDMS.GetPathName(),
										*Graph.PageID.ToString(),
										*NewNodeID.ToString())

									bNodeIDModified = true;
									for (FMetasoundFrontendEdge& Edge : Graph.Edges)
									{
										if (!Edge.ToNodeID.IsValid())
										{
											Edge.ToNodeID = NewNodeID;
										}

										if (!Edge.FromNodeID.IsValid())
										{
											Edge.FromNodeID = NewNodeID;
										}
									}
								}
							}
						}

						return bNodeIDModified;
					}
				};
				bUpdated |= FMigrateNodeIds(OutBuilder.CastDocumentObjectChecked<UObject>(), Document).Transform(GraphClass);

				return bUpdated;
			}
		};

		class FMigratePropertiesTransform : public FMetaSoundFrontendDocumentBuilder::IPropertyVersionTransform
		{
		public:
			virtual ~FMigratePropertiesTransform() = default;

			bool Transform(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				const bool bMigratedProperties = GetDocumentUnsafe(OutBuilder).MigrateProperties();

				// Page Graph migration must be completed for graph accessor back
				// compat prior to all controller versioning, so just do it here.
				const bool bMigratedPages = FMigratePagePropertiesTransform().Transform(OutBuilder);

				return bMigratedProperties || bMigratedPages;
			}
		};

		class FVersionDocumentTransform
		{
			public:
				virtual ~FVersionDocumentTransform() = default;

			protected:
				virtual FMetasoundFrontendVersionNumber GetTargetVersion() const = 0;

				virtual void TransformInternal(FDocumentHandle) const
				{
					checkNoEntry();
				}

				virtual void TransformInternal(FMetasoundFrontendDocument& OutDocument) const
				{
					FDocumentAccessPtr DocAccessPtr = MakeAccessPtr<FDocumentAccessPtr>(OutDocument.AccessPoint, OutDocument);
					return TransformInternal(FDocumentController::CreateDocumentHandle(DocAccessPtr));
				}

				virtual void TransformInternal(FMetaSoundFrontendDocumentBuilder&) const
				{
				}

			public:
				bool Transform(FDocumentHandle InDocument) const
				{
					if (FMetasoundFrontendDocumentMetadata* Metadata = InDocument->GetMetadata())
					{
						const FMetasoundFrontendVersionNumber TargetVersion = GetTargetVersion();
						if (Metadata->Version.Number < TargetVersion)
						{
							TransformInternal(InDocument);
							Metadata->Version.Number = TargetVersion;
							return true;
						}
					}

					return false;
				}

				virtual bool Transform(FMetaSoundFrontendDocumentBuilder& OutDocumentBuilder) const 
				{
					const FMetasoundFrontendDocumentMetadata& Metadata = OutDocumentBuilder.GetConstDocumentChecked().Metadata;

					const FMetasoundFrontendVersionNumber TargetVersion = GetTargetVersion();
					if (Metadata.Version.Number < TargetVersion)
					{
						TransformInternal(OutDocumentBuilder);
						OutDocumentBuilder.SetVersionNumber(TargetVersion);
						return true;
					}

					return false;
				}
		};

		/** Versions document from 1.0 to 1.1. */
		class FVersionDocument_1_1 : public FVersionDocumentTransform
		{
			FName Name;
			const FString& Path;

		public:
			virtual ~FVersionDocument_1_1() = default;

			FVersionDocument_1_1(FName InName, const FString& InPath)
			: Name(InName)
			, Path(InPath)
			{
			}

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 1 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				FGraphHandle GraphHandle = InDocument->GetRootGraph();
				TArray<FNodeHandle> FrontendNodes = GraphHandle->GetNodes();
				TArray<FGuid> PageOrder({DefaultPageID}); // pages did not exist at this point in the history of metasound development.

				// Before literals could be stored on node inputs directly, they were stored
				// by creating hidden input nodes. Update the doc by finding all hidden input
				// nodes, placing the literal value of the input node directly on the
				// downstream node's input. Then delete the hidden input node.
				for (FNodeHandle& NodeHandle : FrontendNodes)
				{
					const bool bIsHiddenNode = NodeHandle->GetNodeStyle().Display.Visibility == EMetasoundFrontendNodeStyleDisplayVisibility::Hidden;
					const bool bIsInputNode = EMetasoundFrontendClassType::Input == NodeHandle->GetClassMetadata().GetType();
					const bool bIsHiddenInputNode = bIsHiddenNode && bIsInputNode;

					if (bIsHiddenInputNode)
					{
						// Get literal value from input node.
						const FGuid VertexID = GraphHandle->GetVertexIDForInputVertex(NodeHandle->GetNodeName());
						const FMetasoundFrontendLiteral DefaultLiteral = GraphHandle->GetDefaultInput(VertexID, PageOrder);

						// Apply literal value to downstream node's inputs.
						TArray<FOutputHandle> OutputHandles = NodeHandle->GetOutputs();
						if (ensure(OutputHandles.Num() == 1))
						{
							FOutputHandle OutputHandle = OutputHandles[0];
							TArray<FInputHandle> Inputs = OutputHandle->GetConnectedInputs();
							OutputHandle->Disconnect();

							for (FInputHandle& Input : Inputs)
							{
								if (const FMetasoundFrontendLiteral* Literal = Input->GetClassDefaultLiteral())
								{
									if (!Literal->IsEqual(DefaultLiteral))
									{
										Input->SetLiteral(DefaultLiteral);
									}
								}
								else
								{
									Input->SetLiteral(DefaultLiteral);
								}
							}
						}
						GraphHandle->RemoveNode(*NodeHandle);
					}
				}
	#else
				METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.1 to 1.2. */
		class FVersionDocument_1_2 : public FVersionDocumentTransform
		{
		private:
			const FName Name;
			const FString& Path;

		public:
			FVersionDocument_1_2(const FName InName, const FString& InPath)
				: Name(InName)
				, Path(InPath)
			{
			}
			virtual ~FVersionDocument_1_2() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 2 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				const FMetasoundFrontendGraphClass& GraphClass = InDocument->GetRootGraphClass();
				FMetasoundFrontendClassMetadata Metadata = GraphClass.Metadata;

				Metadata.SetClassName({ "GraphAsset", Name, *Path });
				Metadata.SetDisplayName(FText::FromString(Name.ToString()));
				InDocument->GetRootGraph()->SetGraphMetadata(Metadata);
	#else
				METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.2 to 1.3. */
		class FVersionDocument_1_3 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_3() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return {1, 3};
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
				const FMetasoundFrontendGraphClass& GraphClass = InDocument->GetRootGraphClass();
				FMetasoundFrontendClassMetadata Metadata = GraphClass.Metadata;

				FMetasoundFrontendDocument* Document = InDocument->GetDocumentPtr().Get();
				check(Document);

				Metadata.SetClassName(FMetasoundFrontendClassName { FName(), *FDocumentIDGenerator::Get().CreateClassID(*Document).ToString(), FName() });
				InDocument->GetRootGraph()->SetGraphMetadata(Metadata);
			}
		};

		/** Versions document from 1.3 to 1.4. */
		class FVersionDocument_1_4 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_4() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return {1, 4};
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
				FMetasoundFrontendDocumentMetadata* Metadata = InDocument->GetMetadata();
				check(Metadata);
				check(Metadata->Version.Number.Major == 1);
				check(Metadata->Version.Number.Minor == 3);

				const TSet<FMetasoundFrontendVersion>& Interfaces = InDocument->GetInterfaceVersions();

				// Version 1.3 did not have an "InterfaceVersion" property on the
				// document, so any document that is being updated should start off
				// with an "Invalid" interface version.
				if (ensure(Interfaces.IsEmpty()))
				{
					// At the time when version 1.4 of the document was introduced, 
					// these were the only available interfaces. 
					static const FMetasoundFrontendVersion PreexistingInterfaceVersions[] = {
						FMetasoundFrontendVersion{"MetaSound", {1, 0}},
						FMetasoundFrontendVersion{"MonoSource", {1, 0}},
						FMetasoundFrontendVersion{"StereoSource", {1, 0}},
						FMetasoundFrontendVersion{"MonoSource", {1, 1}},
						FMetasoundFrontendVersion{"StereoSource", {1, 1}}
					};
					static const int32 NumPreexistingInterfaceVersions = sizeof(PreexistingInterfaceVersions) / sizeof(PreexistingInterfaceVersions[0]);

					TArray<FMetasoundFrontendInterface> CandidateInterfaces;
					IInterfaceRegistry& InterfaceRegistry = IInterfaceRegistry::Get();
					for (int32 i = 0; i < NumPreexistingInterfaceVersions; i++)
					{
						FMetasoundFrontendInterface Interface;
						if (InterfaceRegistry.FindInterface(PreexistingInterfaceVersions[i], Interface))
						{
							CandidateInterfaces.Add(Interface);
						}
					}

					const FMetasoundFrontendGraphClass& RootGraph = InDocument->GetRootGraphClass();
					const TArray<FMetasoundFrontendClass>& Dependencies = InDocument->GetDependencies();
					const TArray<FMetasoundFrontendGraphClass>& Subgraphs = InDocument->GetSubgraphs();

					if (const FMetasoundFrontendInterface* Interface = FindMostSimilarInterfaceSupportingEnvironment(RootGraph, Dependencies, Subgraphs, CandidateInterfaces))
					{
						METASOUND_VERSIONING_LOG(Display, TEXT("Assigned interface [InterfaceVersion:%s] to document [RootGraphClassName:%s]"),
							*Interface->Metadata.Version.ToString(), *RootGraph.Metadata.GetClassName().ToString());

						InDocument->AddInterfaceVersion(Interface->Metadata.Version);
					}
					else
					{
						METASOUND_VERSIONING_LOG(Warning, TEXT("Failed to find interface for document [RootGraphClassName:%s]"),
							*RootGraph.Metadata.GetClassName().ToString());
					}
				}
			}
		};

		/** Versions document from 1.4 to 1.5. */
		class FVersionDocument_1_5 : public FVersionDocumentTransform
		{
			FName Name;
			const FString& Path;

		public:
			FVersionDocument_1_5(FName InName, const FString& InPath)
				: Name(InName)
				, Path(InPath)
			{
			}
			virtual ~FVersionDocument_1_5() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 5 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				const FMetasoundFrontendClassMetadata& Metadata = InDocument->GetRootGraphClass().Metadata;
				const FText NewAssetName = FText::FromString(Name.ToString());
				if (Metadata.GetDisplayName().CompareTo(NewAssetName) != 0)
				{
					FMetasoundFrontendClassMetadata NewMetadata = Metadata;
					NewMetadata.SetDisplayName(NewAssetName);
					InDocument->GetRootGraph()->SetGraphMetadata(NewMetadata);
				}
	#else
				METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.5 to 1.6. */
		class FVersionDocument_1_6 : public FVersionDocumentTransform
		{
		public:
			FVersionDocument_1_6() = default;
			virtual ~FVersionDocument_1_6() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 6 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
				FMetasoundFrontendDocument* Document = InDocument->GetDocumentPtr().Get();
				check(Document);
				const FGuid NewAssetClassID = FDocumentIDGenerator::Get().CreateClassID(*Document);
				const FMetasoundFrontendGraphClass& Class = InDocument->GetRootGraphClass();
				const_cast<FMetasoundFrontendGraphClass&>(Class).Metadata.SetClassName(FMetasoundFrontendClassName({ }, FName(*NewAssetClassID.ToString()), { }));
			}
		};

		/** Versions document from 1.6 to 1.7. */
		class FVersionDocument_1_7 : public FVersionDocumentTransform
		{
			FName Name;
			const FString& Path;

		public:
			FVersionDocument_1_7(FName InName, const FString& InPath)
				: Name(InName)
				, Path(InPath)
			{
			}
			virtual ~FVersionDocument_1_7() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 7 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				auto RenameTransform = [](FNodeHandle NodeHandle)
				{
					// Required nodes are all (at the point of this transform) providing
					// unique names and customized display names (ex. 'Audio' for both mono &
					// L/R output, On Play, & 'On Finished'), so do not replace them by nulling
					// out the guid as a name and using the converted FName of the FText DisplayName.
					if (!NodeHandle->IsInterfaceMember())
					{
						const FName NewNodeName = *NodeHandle->GetDisplayName().ToString();
						NodeHandle->IterateInputs([&](FInputHandle InputHandle)
						{
							InputHandle->SetName(NewNodeName);
						});

						NodeHandle->IterateOutputs([&](FOutputHandle OutputHandle)
						{
							OutputHandle->SetName(NewNodeName);
						});

						NodeHandle->SetDisplayName(FText());
						NodeHandle->SetNodeName(NewNodeName);
					}
				};

				InDocument->GetRootGraph()->IterateNodes(RenameTransform, EMetasoundFrontendClassType::Input);
				InDocument->GetRootGraph()->IterateNodes(RenameTransform, EMetasoundFrontendClassType::Output);
	#else
				METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.7 to 1.8. */
		class FVersionDocument_1_8 : public FVersionDocumentTransform
		{
			FName Name;
			const FString& Path;

		public:
			FVersionDocument_1_8(FName InName, const FString& InPath)
				: Name(InName)
				, Path(InPath)
			{
			}
			virtual ~FVersionDocument_1_8() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 8 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				// For all class definitions we are going to access the default interface instead of inspecting the
				// interface override. This is safe here because the class interface override did not exist in this
				// version of the document. 
				checkf(InDocument->GetMetadata()->Version.Number <= FMetasoundFrontendVersionNumber(1, 14), TEXT("Migration of page properties needs to happen before the introduction of node configuration to the document"));

				// Do not serialize MetaData text for dependencies as
				// CacheRegistryData dynamically provides this.
				InDocument->IterateDependencies([](FMetasoundFrontendClass& Dependency)
				{
					constexpr bool bSerializeText = false;
					Dependency.Metadata.SetSerializeText(bSerializeText);

					for (FMetasoundFrontendClassInput& Input : Dependency.GetDefaultInterface().Inputs)
					{
						Input.Metadata.SetSerializeText(false);
					}

					for (FMetasoundFrontendClassOutput& Output : Dependency.GetDefaultInterface().Outputs)
					{
						Output.Metadata.SetSerializeText(false);
					}
				});

				const TSet<FMetasoundFrontendVersion>& InterfaceVersions = InDocument->GetInterfaceVersions();

				using FNameDataTypePair = TPair<FName, FName>;
				TSet<FNameDataTypePair> InterfaceInputs;
				TSet<FNameDataTypePair> InterfaceOutputs;

				for (const FMetasoundFrontendVersion& Version : InterfaceVersions)
				{
					FMetasoundFrontendInterface Interface;
					const bool bFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
					if (ensure(bFound))
					{
						Algo::Transform(Interface.Inputs, InterfaceInputs, [](const FMetasoundFrontendClassInput& Input)
						{
							return FNameDataTypePair(Input.Name, Input.TypeName);
						});

						Algo::Transform(Interface.Outputs, InterfaceOutputs, [](const FMetasoundFrontendClassOutput& Output)
						{
							return FNameDataTypePair(Output.Name, Output.TypeName);
						});
					}
				}

				// Only serialize MetaData text for inputs owned by the graph (not by interfaces)
				FMetasoundFrontendGraphClass RootGraphClass = InDocument->GetRootGraphClass();
				for (FMetasoundFrontendClassInput& Input : RootGraphClass.GetDefaultInterface().Inputs)
				{
					const bool bSerializeText = !InterfaceInputs.Contains(FNameDataTypePair(Input.Name, Input.TypeName));
					Input.Metadata.SetSerializeText(bSerializeText);
				}

				// Only serialize MetaData text for outputs owned by the graph (not by interfaces)
				for (FMetasoundFrontendClassOutput& Output : RootGraphClass.GetDefaultInterface().Outputs)
				{
					const bool bSerializeText = !InterfaceOutputs.Contains(FNameDataTypePair(Output.Name, Output.TypeName));
					Output.Metadata.SetSerializeText(bSerializeText);
				}

				InDocument->SetRootGraphClass(MoveTemp(RootGraphClass));
	#else
			METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.8 to 1.9. */
		class FVersionDocument_1_9 : public FVersionDocumentTransform
		{
			FName Name;
			const FString& Path;

		public:
			FVersionDocument_1_9(FName InName, const FString& InPath)
				: Name(InName)
				, Path(InPath)
			{
			}
			virtual ~FVersionDocument_1_9() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 9 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
	#if WITH_EDITOR
				// Display name text is no longer copied at this versioning point for assets
				// from the asset's FName to avoid FText warnings regarding generation from
				// an FString.  It also avoids desync if asset gets moved.
				FMetasoundFrontendGraphClass RootGraphClass = InDocument->GetRootGraphClass();
				RootGraphClass.Metadata.SetDisplayName(FText());
				InDocument->SetRootGraphClass(MoveTemp(RootGraphClass));
	#else
				METASOUND_VERSIONING_LOG(Error, TEXT("Asset '%s' at '%s' must be saved with editor enabled in order to version document to target version '%s'."), *Name.ToString(), *Path, *GetTargetVersion().ToString());
	#endif // !WITH_EDITOR
			}
		};

		/** Versions document from 1.9 to 1.10. */
		class FVersionDocument_1_10 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_10() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 10 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
				FMetasoundFrontendGraphClass Class = InDocument->GetRootGraphClass();
				FMetasoundFrontendGraphClassPresetOptions PresetOptions = Class.PresetOptions;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Class.PresetOptions.bIsPreset = Class.Metadata.GetAndClearAutoUpdateManagesInterface_Deprecated();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				InDocument->SetRootGraphClass(MoveTemp(Class));
			}
		};

		/** Versions document from 1.10 to 1.11. */
		class FVersionDocument_1_11 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_11() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 11 };
			}

			void TransformInternal(FDocumentHandle InDocument) const override
			{
				// Clear object literals on inputs that are connected 
				// to prevent referencing assets that are not used in the graph
				InDocument->GetRootGraph()->IterateNodes([](FNodeHandle NodeHandle)
				{
					TArray<FInputHandle> NodeInputs = NodeHandle->GetInputs();
					for (FInputHandle NodeInput : NodeInputs)
					{
						NodeInput->ClearConnectedObjectLiterals();
					}
				});
			}
		};

		/** Versions document from 1.11 to 1.12. */
		class FVersionDocument_1_12 : public FVersionDocumentTransform
		{
			const FName Name;
			const FSoftObjectPath* Path = nullptr;

		public:
			FVersionDocument_1_12(FName InName, const FSoftObjectPath& InAssetPath)
				: Name(InName)
				, Path(&InAssetPath)
			{
			}
			virtual ~FVersionDocument_1_12() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 12 };
			}

			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				using namespace VersioningPrivate;

				if (IsRunningCookCommandlet())
				{
					if (DocumentTransform::GetVersioningLoggingEnabled())
					{
						METASOUND_VERSIONING_LOG(Display, TEXT("Resave recommended: Asset '%s' at '%s' skipped migrated editor data/creation of input template nodes during cook to target document version '%s'."), *Name.ToString(), *Path->ToString(), *GetTargetVersion().ToString());
					}
				}
				else
				{
					// This used to migrate page properties, but this has since just been moved
					// to property migration at the head of versioning call to keep API simple and
					// consistent throughout process.
					// FMigratePagePropertiesTransform().Transform(OutBuilder);
					OutBuilder.GetMetasoundAsset().MigrateEditorGraph(OutBuilder);
					METASOUND_VERSIONING_LOG(Display, TEXT("Resave recommended: Asset '%s' at '%s' successfully migrated editor data in target document version '%s'."), *Name.ToString(), *Path->ToString(), *GetTargetVersion().ToString());
				}
			}
		};

		/** Versions document from 1.12 to 1.13. */
		class FVersionDocument_1_13 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_13() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 13 };
			}

			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				// This used to migrate page properties, but this has since just been moved
				// to property migration at the head of versioning call to keep API simple and
				// consistent throughout process.
				// FMigratePagePropertiesTransform().Transform(OutBuilder);
			}
		};

		/** Versions document from 1.13 to 1.14. */
		class FVersionDocument_1_14 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_14() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 14 };
			}

			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				// Between 1.13 and 1.14, it was possible to add multiple default input page values
				// due to missing versioning logic. This fixes that issue if any data was serialized
				// to a MetaSound Asset by removing any extraneous default data (early values in the
				// array were stale).
				FMetasoundFrontendDocument& Document = const_cast<FMetasoundFrontendDocument&>(OutBuilder.GetConstDocumentChecked());

				// For all class definitions we are going to access the default interface instead of inspecting the
				// interface override. This is safe here because the class interface override did not exist in this
				// version of the document. 
				checkf(Document.Metadata.Version.Number <= FMetasoundFrontendVersionNumber(1, 14), TEXT("Migration of page properties needs to happen before the introduction of node configuration to the document"));

				for (FMetasoundFrontendClassInput& Input : Document.RootGraph.GetDefaultInterface().Inputs)
				{
					int32 PageIDIndex = INDEX_NONE;
					TArray<FMetasoundFrontendClassInputDefault>& Defaults = const_cast<TArray<FMetasoundFrontendClassInputDefault>&>(Input.GetDefaults());
					for (int32 Index = 0; Index < Defaults.Num(); ++Index)
					{
						FMetasoundFrontendClassInputDefault& Default = Defaults[Index];
						const bool bIsDefault = Default.PageID == Frontend::DefaultPageID;
						if (bIsDefault)
						{
							if (PageIDIndex == INDEX_NONE)
							{
								PageIDIndex = Index;
							}
							else
							{
								Defaults.RemoveAt(PageIDIndex);
								break;
							}
						}
					}
				}

				// Safeguards against prior fix-up corrupting any cached data.
				// Safe to do without providing delegates as versioning is done
				// with unregistered builder and prior to delegate registration.
				OutBuilder.Reload();
			}
		};

		/** Versions document from 1.14 to 1.15. */
		class FVersionDocument_1_15 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_15() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 15 };
			}

			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				// Some old assets have multiple copies of the same dependency 
				// which causes issues with the builder's cache which relies on 
				// a 1 to 1 relationship between IDs and dependencies.
				// This removes duplicate entries, keeping the highest version dependencies. 
				FMetasoundFrontendDocument& Document = const_cast<FMetasoundFrontendDocument&>(OutBuilder.GetConstDocumentChecked());

				auto CompareForUnique = [&](const FMetasoundFrontendClass& InLHS, const FMetasoundFrontendClass& InRHS)
				{
					return InLHS.ID == InRHS.ID;
				};

				auto CompareForSort = [&](const FMetasoundFrontendClass& InLHS, const FMetasoundFrontendClass& InRHS)
				{
					// Sort by ID
					if (InLHS.ID < InRHS.ID)
					{
						return true;
					}
					else if (InRHS.ID < InLHS.ID)
					{
						return false;
					}
					// If IDs are equal, sort by version number descending
					else if (InLHS.Metadata.GetVersion() > InRHS.Metadata.GetVersion())
					{
						return true;
					}
					else if (InLHS.Metadata.GetVersion() < InRHS.Metadata.GetVersion())
					{
						return false;
					}
					else
					{
						// if IDs and version numbers are equal, sort by number of inputs & outputs descending
						const int32 NumLHSVertices = InLHS.GetDefaultInterface().Inputs.Num() + InLHS.GetDefaultInterface().Outputs.Num();
						const int32 NumRHSVertices = InRHS.GetDefaultInterface().Inputs.Num() + InRHS.GetDefaultInterface().Outputs.Num();

						return NumLHSVertices > NumRHSVertices;
					}
				};
				
				Algo::Sort(Document.Dependencies, CompareForSort);
				Document.Dependencies.SetNum(Algo::Unique(Document.Dependencies, CompareForUnique));

				// Safeguards against prior fix-up corrupting any cached data.
				// Safe to do without providing delegates as versioning is done
				// with unregistered builder and prior to delegate registration.
				OutBuilder.Reload();
			}
		};

		/** Versions document from 1.15 to 1.16. */
		class FVersionDocument_1_16 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_16() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 16 };
			}

			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				const FMetasoundFrontendDocument& Document = OutBuilder.GetConstDocumentChecked();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (Document.RootGraph.Metadata.GetIsDeprecated())
				{
					OutBuilder.AddAccessFlags(EMetasoundFrontendClassAccessFlags::Deprecated);
					OutBuilder.RemoveAccessFlags(EMetasoundFrontendClassAccessFlags::Referenceable);
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		};

		/** Versions document from 1.16 to 1.17. */
		class FVersionDocument_1_17 : public FVersionDocumentTransform
		{
		public:
			virtual ~FVersionDocument_1_17() = default;

			FMetasoundFrontendVersionNumber GetTargetVersion() const override
			{
				return { 1, 17 };
			}

		private:
			static TScriptInterface<IMetaSoundDocumentInterface> FindReferencedMetaSound_Legacy(const FMetaSoundFrontendDocumentBuilder& Builder)
			{
				// Find the single external node which is the parent preset asset,
				// and find the asset with its registry key
				FMetaSoundAssetKey NodeAssetKey;
				auto FindExternalNode = [&Builder, &NodeAssetKey](const FMetasoundFrontendNode& Node)
				{
					const FMetasoundFrontendClass* Class = Builder.FindDependency(Node.ClassID);
					check(Class);
					if (Class->Metadata.GetType() == EMetasoundFrontendClassType::External)
					{
						NodeAssetKey = FMetaSoundAssetKey(Class->Metadata);
						return true;
					}

					return false;
				};

				const FMetasoundFrontendNode* Node = Builder.FindConstBuildGraphChecked().Nodes.FindByPredicate(FindExternalNode);
				if (Node != nullptr)
				{
					const TArray<FMetasoundAssetBase*> ReferencedAssets = Builder.GetMetasoundAsset().GetReferencedAssets();

					if (ReferencedAssets.Num() != 1)
					{
						METASOUND_VERSIONING_LOG(Warning, TEXT("More than one asset reference found when migrating preset to configuration for asset '%s'"), *Builder.GetDebugName());
					}

					for (FMetasoundAssetBase* RefAsset : ReferencedAssets)
					{
						TScriptInterface<IMetaSoundDocumentInterface> RefDocInterface = RefAsset->GetOwningAsset();
						if (RefDocInterface.GetObject() != nullptr)
						{
							const FMetaSoundAssetKey AssetKey(RefDocInterface->GetConstDocument().RootGraph.Metadata);
							if (AssetKey == NodeAssetKey)
							{
								return RefDocInterface;
							}
						}
					}
				}

				METASOUND_VERSIONING_LOG(Error, TEXT("Failed to find MetaSound parent for preset with key '%s' on asset '%s'"), *NodeAssetKey.ToString(), *Builder.GetDebugName());
				return { };
			}

		public:
			void TransformInternal(FMetaSoundFrontendDocumentBuilder& OutBuilder) const override
			{
				const FMetasoundFrontendDocument& Document = OutBuilder.GetConstDocumentChecked();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				const FMetasoundFrontendGraphClassPresetOptions& PresetOptions = Document.RootGraph.PresetOptions;
				bool& bIsPreset = const_cast<bool&>(PresetOptions.bIsPreset);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (bIsPreset)
				{
					bIsPreset = false;

					using FPresetConfigStruct = TInstancedStruct<FMetaSoundFrontendPresetTemplate>;
					FPresetConfigStruct PresetTemplate = FPresetConfigStruct::Make<FMetaSoundFrontendPresetTemplate>();
					PresetTemplate.GetMutable().Parent = FindReferencedMetaSound_Legacy(OutBuilder);

					for (const FMetasoundFrontendClassInput& Input : Document.RootGraph.GetDefaultInterface().Inputs)
					{
						// Some inputs don't support inheriting a default (ex. triggers), so ignore if no metadata is added.
						if (FPresetVertexMetadata* VertexMetadata = PresetTemplate.GetMutable().AddVertexMetadata<FPresetVertexMetadata>(OutBuilder, Input))
						{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
							const bool bOverrideInheritedDefault = !PresetOptions.InputsInheritingDefault.Contains(Input.Name);
							VertexMetadata->bOverrideInheritedDefault = bOverrideInheritedDefault;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
						}
					}

					{
						using FTemplateStruct = TInstancedStruct<FMetaSoundFrontendDocumentTemplate>;
						FTemplateStruct& TemplateRef = const_cast<FTemplateStruct&>(Document.Template);
						TemplateRef = MoveTemp(PresetTemplate);
					}
				}

				// There was a bug in the past where inputs inheriting defaults wasn't getting cleared when converting from preset,
				// so just always clear the deprecated set here either way.
				const_cast<TSet<FName>&>(PresetOptions.InputsInheritingDefault).Empty();
			}
		};

		bool VersionBuilderDocument(FMetaSoundFrontendDocumentBuilder& Builder)
		{
			UObject& DocObject = Builder.CastDocumentObjectChecked<UObject>();
			const FName Name = DocObject.GetFName();
			const FString Path = DocObject.GetPathName();

			bool bWasUpdated = false;
			bWasUpdated |= FVersionDocument_1_12(Name, Path).Transform(Builder);
			bWasUpdated |= FVersionDocument_1_13().Transform(Builder);
			bWasUpdated |= FVersionDocument_1_14().Transform(Builder);
			bWasUpdated |= FVersionDocument_1_15().Transform(Builder);
			bWasUpdated |= FVersionDocument_1_16().Transform(Builder);
			bWasUpdated |= FVersionDocument_1_17().Transform(Builder);

			return bWasUpdated;
		}
	} // namespace VersioningPrivate

	FVersioningManager::~FVersioningManager()
	{
		using namespace UE::Tasks;

		TArray<FTask> PendingTasks;
		Algo::Transform(ActiveVersionTaskData, PendingTasks, [](const TPair<FUniqueId, FTaskData>& Pair) { return Pair.Value.Task; });
		UE::Tasks::Wait(PendingTasks);
		ActiveVersionTaskData.Reset();
	}

	FVersioningManager& FVersioningManager::Get()
	{
		using namespace VersioningPrivate;

		// Its possible some commandlets load MetaSounds prior
		// to the MetaSoundEngine module loading (ex. generating
		// asset manifests), so attempt to generate one on the fly
		// when accessing here.
		if (!VersioningManager.IsValid())
		{
			VersioningManager = MakeUnique<FVersioningManager>();
		}
		return *VersioningManager;
	}

	void FVersioningManager::Initialize()
	{
		using namespace VersioningPrivate;
		if (!VersioningManager.IsValid())
		{
			VersioningManager = MakeUnique<FVersioningManager>();
		}
	}

	void FVersioningManager::Shutdown()
	{
		using namespace VersioningPrivate;
		VersioningManager.Reset();
	}

	bool FVersioningManager::ChangeIDComparisonEnabledInAutoUpdate()
	{
		return VersioningPrivate::MetaSoundAutoUpdateUseAssetChangeIDsCVar != 0;
	}

	FMetasoundFrontendVersionNumber FVersioningManager::GetMaxDocumentVersion()
	{
		return FMetasoundFrontendVersionNumber { 1, 17 };
	}

	FMetasoundFrontendVersionNumber FVersioningManager::GetPageMigrationVersion()
	{
		return FMetasoundFrontendVersionNumber { 1, 13 };
	}

	void FVersioningManager::VersionAssetAsync(UObject& InMetaSound, bool bIsDeterministic)
	{
		TScriptInterface<IMetaSoundDocumentInterface> DocInterface(&InMetaSound);
		if (!ensureAlwaysMsgf(DocInterface.GetObject(), TEXT("Failed to version MetaSound '%s': Object does not implement MetaSound Document Interface."), *InMetaSound.GetName()))
		{
			return;
		}

		if (!ensureAlwaysMsgf(InMetaSound.IsAsset(), TEXT("Failed to version MetaSound '%s': Only serialized MetaSound assets require versioning."), *InMetaSound.GetName()))
		{
			return;
		}

		const FUniqueId AssetId = InMetaSound.GetUniqueID();

		{
			FScopeLock Lock(&ActiveVersionCritSection);
			if (ActiveVersionTaskData.Contains(AssetId))
			{
				return;
			}
		}

		// References have to be up-to-date before versioning the referencing asset to protect
		// against accessing the referenced assets while being mutated asyncronously.
		WaitUntilVersioningReferencesComplete(InMetaSound);

		check(DocInterface.GetObject());
		const FMetasoundFrontendDocument& Document = DocInterface->GetConstDocument();
		const FMetasoundFrontendClassName& ClassName = Document.RootGraph.Metadata.GetClassName();
		const FTopLevelAssetPath AssetPath = DocInterface->GetAssetPathChecked();

		// Ensure nothing is currently holding on to a builder and use a local builder for versioning. All other callsites
		// (such as a request to register and grabbing a new builder) should wait until versioning is complete for safety.
		// In theory, this should never happen. This log line helps determine if something somehow registered a builder
		// prior to the asset being loaded and registered properly.
		if (const FMetaSoundFrontendDocumentBuilder* ExistingBuilder = IDocumentBuilderRegistry::GetChecked().FindBuilder(ClassName, AssetPath))
		{
			METASOUND_VERSIONING_LOG(Warning,
				TEXT("Existing builder for asset '%s' currently active prior to launching MetaSound versioning. Forcing builder to finish. "
				"This indicates a system is erroneously initializing a builder prior to completing versioning."), *ExistingBuilder->GetDebugName());
			IDocumentBuilderRegistry::GetChecked().FinishBuilding(ClassName, AssetPath);
		}

		{
			FScopeLock Lock(&ActiveVersionCritSection);
			
			if (ActiveVersionTaskData.Contains(AssetId))
			{
				// Early out if an async task has already begun
				return;
			}

			UE_LOGF(LogMetaSound, Verbose, "Beginning async versioning for %ls", *AssetPath.ToString());
			InMetaSound.SetInternalFlags(EInternalObjectFlags::Async);
			FTaskData NewTaskData
			{
				.AssetPtr = TStrongObjectPtr<UObject>(&InMetaSound),
				.Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, AssetId, bIsDeterministic]()
				{
					FTaskData TaskData;
					{
						FScopeLock VersionLock(&ActiveVersionCritSection);
						if (const FTaskData* TaskEntry = ActiveVersionTaskData.Find(AssetId))
						{
							TaskData = *TaskEntry;
						}
						else
						{
							return;
						}
					}

					{
						FGCScopeGuard ScopeGuard;

						if (UObject* VersioningMetaSound = TaskData.AssetPtr.Get())
						{
							FDocumentIDGenerator::FScopeDeterminism DeterminismScope(bIsDeterministic);
							FMetaSoundFrontendDocumentBuilder DocBuilder(VersioningMetaSound);
							const bool bVersionedDoc = VersionDocument(DocBuilder);
							const bool bVersionedInterfaces = VersionInterfacesInternal(DocBuilder);

							if (bVersionedDoc || bVersionedInterfaces)
							{
								DocBuilder.GetMetasoundAsset().SetVersionedOnLoad();
							}

							ForEachObjectWithOuter(VersioningMetaSound, [](UObject* Object)
							{
								if (Object->HasAnyInternalFlags(EInternalObjectFlags::Async))
								{
									METASOUND_VERSIONING_LOG(Verbose, TEXT("Clearing object '%s' async flag (object likely generated while MetaSound versioning)"), *Object->GetName());
									Object->ClearInternalFlags(EInternalObjectFlags::Async);
								}
							});
							VersioningMetaSound->ClearInternalFlags(EInternalObjectFlags::Async);
						}
					}

					{
						FScopeLock VersionLock(&ActiveVersionCritSection);
						ActiveVersionTaskData.Remove(AssetId);
					}
				})
			};

			ActiveVersionTaskData.Add(AssetId, MoveTemp(NewTaskData));
		}
	}

	bool FVersioningManager::VersionInterfaces(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
	{
		WaitUntilVersioningComplete(DocBuilder.CastDocumentObjectChecked<UObject>());
		return VersionInterfacesInternal(DocBuilder);
	}

	bool FVersioningManager::VersionInterfacesInternal(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
	{
		using namespace Metasound;
		using namespace Metasound::Frontend;

		bool bDidEdit = false;
		{
			const FMetasoundFrontendDocument& Document = DocBuilder.GetConstDocumentChecked();
			bool bInterfaceUpdated = false;
			bool bPassUpdated = true;

			// Has to be re-run until no pass reports an update in case versions
			// fork (ex. an interface splits into two newly named interfaces).
			while (bPassUpdated)
			{
				bPassUpdated = false;

				const TArray<FMetasoundFrontendVersion> Versions = Document.Interfaces.Array();

				for (const FMetasoundFrontendVersion& Version : Versions)
				{
					bPassUpdated |= VersioningPrivate::TryUpdateInterfaceFromVersion(Version, DocBuilder);
				}

				bInterfaceUpdated |= bPassUpdated;
			}

			if (bInterfaceUpdated)
			{
				DocBuilder.ConformObjectToDocument();
			}
			bDidEdit |= bInterfaceUpdated;

			return bDidEdit;
		}
	}

	bool FVersioningManager::VersionDocument(FMetaSoundFrontendDocumentBuilder& DocBuilder) const
	{
		using namespace VersioningPrivate;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FVersioningManager::VersionDocument);

		// Applies all property migration for data prior to document
		// versioning. Only returns true if data has yet to be migrated.
		bool bWasUpdated = FMigratePropertiesTransform().Transform(DocBuilder);

		UObject& MetaSoundObject = DocBuilder.CastDocumentObjectChecked<UObject>();
		const FName Name(*MetaSoundObject.GetName());
		const FString Path = MetaSoundObject.GetPathName();

		// Copied as value will be mutated with each applicable transform below
		const FMetasoundFrontendVersionNumber InitVersionNumber = DocBuilder.GetConstDocumentChecked().Metadata.Version.Number;

		using namespace Metasound::Frontend;
		if (InitVersionNumber < GetMaxDocumentVersion())
		{
			// Controller (Soft Deprecated) Transforms
			if (InitVersionNumber.Major == 1 && InitVersionNumber.Minor < 12)
			{
				// Builder is not registered during versioning, so don't attempt to reload builder through builder
				// registry indirectly here (which can cause delegates to be fired in invalid scenarios like
				// reloading assets from disk in editor).
				constexpr bool bReloadRegistryBuilder = false;
				FDocumentHandle DocHandle = DocBuilder.GetMetasoundAsset().GetDocumentHandle(bReloadRegistryBuilder);

				bWasUpdated |= FVersionDocument_1_1(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_2(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_3().Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_4().Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_5(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_6().Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_7(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_8(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_9(Name, Path).Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_10().Transform(DocHandle);
				bWasUpdated |= FVersionDocument_1_11().Transform(DocHandle);
				// No longer supported, new versions should go in VersioningPrivate::VersionBuilderDocument
			}

			// Reload builder directly here in case cache is left in invalid state from deprecated handle versioning above.
			DocBuilder.Reload();
			bWasUpdated |= VersionBuilderDocument(DocBuilder);
			if (bWasUpdated)
			{
				const FMetasoundFrontendVersionNumber& NewVersionNumber = DocBuilder.GetConstDocumentChecked().Metadata.Version.Number;
				METASOUND_VERSIONING_LOG(Verbose, TEXT("MetaSound at '%s' Document Versioned: '%s' --> '%s'"), *Path, *InitVersionNumber.ToString(), *NewVersionNumber.ToString());
			}
		}

		return bWasUpdated;
	}

	void FVersioningManager::WaitUntilVersioningReferencesComplete(const UObject& InMetaSound) const
	{
		using namespace UE::Tasks;

		// Async reference load must be waited for prior to versioning wait as this is a
		// required step to kick off versioning for older assets with soft object references.
		if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
		{
			if (const FMetasoundAssetBase* MetaSoundAsset = AssetManager->GetAsAsset(InMetaSound))
			{
				TArray<FMetasoundAssetBase*> ReferencedAssets = MetaSoundAsset->GetReferencedAssets();
				for (FMetasoundAssetBase* Reference : ReferencedAssets)
				{
					AssetManager->WaitUntilAsyncLoadReferencedAssetsComplete(*Reference);

					const UObject* RefOwner = Reference->GetOwningAsset();
					check(RefOwner);

					FTask TaskToWaitFor;
					{
						FScopeLock VersionLock(&ActiveVersionCritSection);
						if (const FTaskData* TaskData = ActiveVersionTaskData.Find(RefOwner->GetUniqueID()))
						{
							TaskToWaitFor = TaskData->Task;
						}
						else
						{
							METASOUND_VERSIONING_LOG(Verbose, TEXT("Skipping wait for %s to versioning reference in async task: no task executing"), *InMetaSound.GetFullName());
							continue;
						}
					}

					METASOUND_VERSIONING_LOG(Verbose, TEXT("Waiting for %s to finish versioning on async task..."), *InMetaSound.GetFullName());
					TaskToWaitFor.Wait();
				}
			}
		}
	}

	void FVersioningManager::WaitUntilVersioningComplete(const UObject& InMetaSound) const
	{
		using namespace UE::Tasks;

		// Async reference load must be waited for prior to versioning wait as this is a
		// required step to kick off versioning for older assets with soft object references.
		if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
		{
			if (const FMetasoundAssetBase* MetaSoundAsset = AssetManager->GetAsAsset(InMetaSound))
			{
				AssetManager->WaitUntilAsyncLoadReferencedAssetsComplete(*MetaSoundAsset);
			}
		}

		FTask TaskToWaitFor;
		{
			FScopeLock VersionLock(&ActiveVersionCritSection);
			if (const FTaskData* TaskData = ActiveVersionTaskData.Find(InMetaSound.GetUniqueID()))
			{
				TaskToWaitFor = TaskData->Task;
			}
			else
			{
				METASOUND_VERSIONING_LOG(Verbose, TEXT("Skipping wait for %s to version on async task: no task executing"), *InMetaSound.GetFullName());
				return;
			}
		}

		METASOUND_VERSIONING_LOG(Verbose, TEXT("Waiting for %s to finish versioning on async task..."), *InMetaSound.GetFullName());
		TaskToWaitFor.Wait();
	}
} // namespace Metasound::Frontend
#endif // WITH_EDITORONLY_DATA
