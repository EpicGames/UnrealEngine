// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendGraph.h"

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontend.h"
#include "MetasoundGraph.h"
#include "MetasoundNodeInterface.h"

namespace Metasound
{
	FFrontendGraph::FFrontendGraph(const FString& InInstanceName, const FGuid& InInstanceID)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	:	FGraph(FTopLevelAssetPath(InInstanceName), InInstanceID)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FFrontendGraph::FFrontendGraph(const FTopLevelAssetPath& InAssetPath, const FGuid& InInstanceID)
		: FGraph(InAssetPath, InInstanceID)
	{
	}

	void FFrontendGraph::AddInputNode(FGuid InDependencyId, int32 InIndex, const FVertexName& InVertexName, TSharedPtr<const INode> InNode)
	{
		if (InNode.IsValid())
		{
			// There shouldn't be duplicate IDs. 
			check(!InputNodes.Contains(InIndex));

			// Input nodes need an extra Index value to keep track of their position in the graph's inputs.
			InputNodes.Add(InIndex, InNode.Get());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AddInputNode(InDependencyId, InVertexName, MoveTemp(InNode));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	void FFrontendGraph::AddOutputNode(FGuid InNodeID, int32 InIndex, const FVertexName& InVertexName, TSharedPtr<const INode> InNode)
	{
		if (InNode.IsValid())
		{
			// There shouldn't be duplicate IDs. 
			check(!OutputNodes.Contains(InIndex));

			// Output nodes need an extra Index value to keep track of their position in the graph's inputs.
			OutputNodes.Add(InIndex, InNode.Get());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AddOutputNode(InNodeID, InVertexName, MoveTemp(InNode));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	const INode* FFrontendGraph::FindInputNode(int32 InIndex) const
	{
		const INode* const* NodePtr = InputNodes.Find(InIndex);

		if (nullptr != NodePtr)
		{
			return *NodePtr;
		}

		return nullptr;
	}

	const INode* FFrontendGraph::FindOutputNode(int32 InIndex) const
	{
		const INode* const* NodePtr = OutputNodes.Find(InIndex);

		if (nullptr != NodePtr)
		{
			return *NodePtr;
		}

		return nullptr;
	}
}
