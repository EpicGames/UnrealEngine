// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendGraphBuilder.h"

#include "MetasoundFrontendGraph.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/TopologicalSort.h"
#include "Algo/Transform.h"
#include "CoreGlobals.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundGraph.h"
#include "MetasoundGraphNode.h"
#include "MetasoundLiteralNode.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "UObject/TopLevelAssetPath.h"

namespace Metasound::Frontend
{
	namespace GraphBuilderPrivate
	{
		// Map of Input VertexID to variable data required to construct and connect default variable
		using FNodeIDVertexID = TTuple<FGuid, FGuid>;
		using FDependencyByIDMap = TMap<FGuid, const FMetasoundFrontendClass*>;
		using FSharedNodeByIDMap = TMap<FGuid, TSharedPtr<const IGraph>>;

		// Context used throughout entire graph build process
		// (for both a root and nested subgraphs)
		struct FBuildContext
		{
			const FTopLevelAssetPath& AssetPath;
			const Frontend::IDataTypeRegistry& DataTypeRegistry;
			const Frontend::FProxyDataCache* ProxyDataCache;
			TArrayView<const FGuid> PageOrder;
		};

		// Context related to the document being built. 
		struct FBuildDocumentContext
		{
			FDependencyByIDMap FrontendClasses;
			FSharedNodeByIDMap Graphs;
		};

		// Transient context used for building a specific graph
		struct FBuildGraphContext
		{
			TUniquePtr<FFrontendGraph> Graph;
			const FMetasoundFrontendGraphClass& GraphClass;
			const FMetasoundFrontendGraph& PagedGraph;
			const FBuildContext& BuildContext;
			const FBuildDocumentContext& BuildDocumentContext;
		};

		bool InterfacesHaveEqualSize(const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendNodeInterface& InNodeInterface)
		{
			return (InClassInterface.Inputs.Num() == InNodeInterface.Inputs.Num()) && (InClassInterface.Outputs.Num() == InNodeInterface.Outputs.Num());
		}

		const FMetasoundFrontendLiteral* FindLiteralForInputVertex(const FBuildContext& InContext, const FVertexName& InVertexName, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInput& InNodeClassInput)
		{
			using namespace Frontend;

			// Default value priority is:
			// 1. A value set directly on the node
			// 2. A default value of the owning graph
			// 3. A default value on the input node class.

			const FMetasoundFrontendLiteral* Literal = nullptr;

			// Check for default value directly on node.
			if (InNode.InputLiterals.Num())
			{
				const FMetasoundFrontendVertex* InputVertex = InNode.Interface.Inputs.FindByPredicate([&InVertexName](const FMetasoundFrontendVertex& Vertex) { return Vertex.Name == InVertexName; });
				if (ensure(InputVertex))
				{
					const FMetasoundFrontendVertexLiteral* VertexLiteral = InNode.InputLiterals.FindByPredicate([&InputVertex](const FMetasoundFrontendVertexLiteral& InVertexLiteral) { return InVertexLiteral.VertexID == InputVertex->VertexID; });
					if (VertexLiteral)
					{
						Literal = &VertexLiteral->Value;
					}
				}
			}

			// Check for default value on node class
			if (nullptr == Literal)
			{
				const FMetasoundFrontendClassInputDefault* InputDefault= FindPreferredPage(InNodeClassInput.GetDefaults(), InContext.PageOrder);
				if (ensure(InputDefault))
				{
					if (InputDefault->Literal.IsValid())
					{
						Literal = &InputDefault->Literal;
					}
				}
			}

			return Literal;
		}

		const FMetasoundFrontendLiteral* FindInputLiteralForInputNode(const FBuildContext& InContext, const FMetasoundFrontendNode& InInputNode, const FMetasoundFrontendClassInterface& InInputNodeClassInterface, const FMetasoundFrontendClassInput& InOwningGraphClassInput)
		{
			using namespace Frontend;

			// Default value priority is:
			// 1. A value set directly on the node
			// 2. A default value of the owning graph
			// 3. A default value on the input node class.

			const FMetasoundFrontendLiteral* Literal = nullptr;

			// Check for default value directly on node.
			if (ensure(InInputNode.Interface.Inputs.Num() == 1))
			{
				const FMetasoundFrontendVertex& InputVertex = InInputNode.Interface.Inputs[0];

				// Find input literal matching VertexID
				const FMetasoundFrontendVertexLiteral* VertexLiteral = InInputNode.InputLiterals.FindByPredicate(
					[&](const FMetasoundFrontendVertexLiteral& InVertexLiteral)
					{
						return InVertexLiteral.VertexID == InputVertex.VertexID;
					}
				);

				if (nullptr != VertexLiteral)
				{
					Literal = &VertexLiteral->Value;
				}
			}

			// Check for default value on owning graph.
			if (nullptr == Literal)
			{
				// Find Class Default that is not invalid
				const FMetasoundFrontendClassInputDefault* InputDefault = FindPreferredPage(InOwningGraphClassInput.GetDefaults(), InContext.PageOrder);
				if (ensure(InputDefault))
				{
					if (InputDefault->Literal.IsValid())
					{
						Literal = &InputDefault->Literal;
					}
				}
			}

			// Check for default value on input node class
			if (nullptr == Literal && ensure(InInputNodeClassInterface.Inputs.Num() == 1))
			{
				const FMetasoundFrontendClassInput& InputNodeClassInput = InInputNodeClassInterface.Inputs.Last();
				const FMetasoundFrontendClassInputDefault* InputDefault = FindPreferredPage(InputNodeClassInput.GetDefaults(), InContext.PageOrder);
				if (ensure(InputDefault))
				{
					if (InputDefault->Literal.IsValid())
					{
						Literal = &InputDefault->Literal;
					}
				}
			}

			return Literal;
		}


		FEnvironmentVertexInterface CreateEnvironmentVertexInterface(const FMetasoundFrontendNodeInterface& InNodeInterface)
		{
			if (InNodeInterface.Environment.Num())
			{
				auto Convert = [](const FMetasoundFrontendVertex& InFrontendVertex) -> FEnvironmentVertex 
				{
					return FEnvironmentVertex{InFrontendVertex.Name, FText::GetEmpty()};
				};

				TArray<FEnvironmentVertex> Vertices;
				Vertices.Reserve(InNodeInterface.Environment.Num());
				Algo::Transform(InNodeInterface.Environment, Vertices, Convert);

				return FEnvironmentVertexInterface{MoveTemp(Vertices)};
			}
			else
			{
				return FEnvironmentVertexInterface{};
			}
		}

		// Helper: Build an FInputDataVertex from frontend data, stripping metadata.
		FInputDataVertex MakeInputVertex(const FBuildContext& InContext, const FVertexName& Name, const FName& DataTypeName, EMetasoundFrontendVertexAccessType InFrontendAccessType, const FMetasoundFrontendLiteral* InFrontendLiteral)
		{
			EVertexAccessType AccessType = FrontendVertexAccessTypeToCoreVertexAccessType(InFrontendAccessType);
			if (InFrontendLiteral)
			{
				FLiteral CoreLiteral = InFrontendLiteral->ToLiteral(DataTypeName, &InContext.DataTypeRegistry, InContext.ProxyDataCache);
				return FInputDataVertex{Name, DataTypeName, FDataVertexMetadata{}, AccessType, MoveTemp(CoreLiteral)};
			}
			return FInputDataVertex{Name, DataTypeName, FDataVertexMetadata{}, AccessType};
		}

		// Helper: Apply per-vertex AccessType overlay from the frontend class interface
		// onto a FVertexInterface produced by FClassInterface::CreateVertexInterface().
		// For sub-interface expanded vertices where names may not match 1:1 with
		// InClassInterface entries, the prototype's AccessType serves as the default.
		void OverlayAccessTypes(FVertexInterface& VertexInterface, const FMetasoundFrontendClassInterface& InClassInterface)
		{
			for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
			{
				if (FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find(ClassInput.Name))
				{
					InputVertex->AccessType = FrontendVertexAccessTypeToCoreVertexAccessType(ClassInput.AccessType);
				}
			}
			for (const FMetasoundFrontendClassOutput& ClassOutput : InClassInterface.Outputs)
			{
				if (FOutputDataVertex* OutputVertex = VertexInterface.GetOutputInterface().Find(ClassOutput.Name))
				{
					OutputVertex->AccessType = FrontendVertexAccessTypeToCoreVertexAccessType(ClassOutput.AccessType);
				}
			}
		}

		// Helper: Apply per-vertex literal defaults from the frontend node onto
		// a FVertexInterface produced by FClassInterface::CreateVertexInterface().
		void OverlayLiteralDefaults(FVertexInterface& VertexInterface, const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInterface& InClassInterface)
		{
			for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
			{
				const FMetasoundFrontendLiteral* FrontendLiteral = FindLiteralForInputVertex(InContext, ClassInput.Name, InNode, ClassInput);
				if (FrontendLiteral)
				{
					if (FInputDataVertex* InputVertex = VertexInterface.GetInputInterface().Find(ClassInput.Name))
					{
						FLiteral CoreLiteral = FrontendLiteral->ToLiteral(ClassInput.TypeName, &InContext.DataTypeRegistry, InContext.ProxyDataCache);
						InputVertex->SetDefaultLiteral(MoveTemp(CoreLiteral));
					}
				}
			}
		}

		FVertexInterface CreateInputNodeVertexInterface(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassInput& InOwningGraphClassInput)
		{
			if (InNode.Interface.Inputs.Num() != 1 || InNode.Interface.Outputs.Num() != 1)
			{
				// Edge case: malformed input node (e.g. legacy TInputNode<FStereoAudio>).
				// Auto-update should have removed these. Log and return empty interface.
				UE_LOGF(LogMetaSound, Error, "The MetaSound %ls has a malformed input node %ls. Please replace the input node and resave.", *InContext.AssetPath.ToString(), *InNode.Name.ToString());
				return {};
			}

			// Input nodes use the owning graph class input for name, access type, and literal.
			// NOTE: The access type on the class vertex may be stale; use InOwningGraphClassInput.
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));
			const FMetasoundFrontendVertex& NodeInputVertex = InNode.Interface.Inputs[0];
			const FMetasoundFrontendLiteral* FrontendLiteral = FindInputLiteralForInputNode(InContext, InNode, InClassInterface, InOwningGraphClassInput);
			FInputVertexInterface InputInterface(
				MakeInputVertex(InContext, NodeInputVertex.Name, NodeInputVertex.TypeName, InOwningGraphClassInput.AccessType, FrontendLiteral));

			const FMetasoundFrontendVertex& NodeOutputVertex = InNode.Interface.Outputs[0];
			FOutputVertexInterface OutputInterface(FOutputDataVertex{
				NodeOutputVertex.Name,
				NodeOutputVertex.TypeName,
				FDataVertexMetadata{},
				FrontendVertexAccessTypeToCoreVertexAccessType(InOwningGraphClassInput.AccessType)});

			FEnvironmentVertexInterface EnvironmentInterface = CreateEnvironmentVertexInterface(InNode.Interface);

			return FVertexInterface(MoveTemp(InputInterface), MoveTemp(OutputInterface), MoveTemp(EnvironmentInterface));
		}

		FVertexInterface CreateOutputNodeVertexInterface(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassOutput& InOwningGraphClassOutput)
		{
			if (InNode.Interface.Inputs.Num() != 1 || InNode.Interface.Outputs.Num() != 1)
			{
				// Edge case: malformed output node (e.g. legacy TOutputNode<FStereoAudio>).
				// Auto-update should have removed these. Log and return empty interface.
				UE_LOGF(LogMetaSound, Error, "The MetaSound %ls has a malformed output node %ls. Please replace the output node and resave.", *InContext.AssetPath.ToString(), *InNode.Name.ToString());
				return {};
			}

			// Output nodes use the owning graph class output for access type.
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));
			const FMetasoundFrontendVertex& NodeInputVertex = InNode.Interface.Inputs[0];
			const FMetasoundFrontendClassInput& ClassInputVertex = InClassInterface.Inputs[0];
			const FMetasoundFrontendLiteral* FrontendLiteral = FindLiteralForInputVertex(InContext, NodeInputVertex.Name, InNode, ClassInputVertex);
			FInputVertexInterface InputInterface(
				MakeInputVertex(InContext, NodeInputVertex.Name, NodeInputVertex.TypeName, InOwningGraphClassOutput.AccessType, FrontendLiteral));

			const FMetasoundFrontendVertex& NodeOutputVertex = InNode.Interface.Outputs[0];
			FOutputVertexInterface OutputInterface(FOutputDataVertex{
				NodeOutputVertex.Name,
				NodeOutputVertex.TypeName,
				FDataVertexMetadata{},
				FrontendVertexAccessTypeToCoreVertexAccessType(InOwningGraphClassOutput.AccessType)});

			FEnvironmentVertexInterface EnvironmentInterface = CreateEnvironmentVertexInterface(InNode.Interface);

			return FVertexInterface(MoveTemp(InputInterface), MoveTemp(OutputInterface), MoveTemp(EnvironmentInterface));
		}

		FVertexInterface CreateConfiguredVertexInterface(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInterface& InClassInterface, FClassInterface& ClassInterface)
		{
			check(ClassInterface.ContainsSubInterfaces() || ClassInterface.ContainsVariants());

			// Extract configuration from the node's FMetaSoundDynamicInterfaceNodeConfiguration
			TMap<FName, uint32> SubInterfaceCounts;
			TMap<FName, FName> VariantSelections;

			if (InNode.Configuration.IsValid())
			{
				if (const FMetaSoundDynamicInterfaceNodeConfiguration* BaseConfig = InNode.Configuration.GetPtr<FMetaSoundDynamicInterfaceNodeConfiguration>())
				{
					SubInterfaceCounts = BaseConfig->SubInterfaceCounts;
					VariantSelections = BaseConfig->VariantSelections;
				}
			}

			// If no sub-interface configuration is provided, use defaults from the class interface
			if (SubInterfaceCounts.IsEmpty() && ClassInterface.ContainsSubInterfaces())
			{
				for (const FSubInterfaceDescription& Desc : ClassInterface.GetSubInterfaceDescriptions())
				{
					SubInterfaceCounts.Add(Desc.SubInterfaceName, Desc.NumDefault);
				}
			}

			FVertexInterface VertexInterface = ClassInterface.CreateVertexInterface(SubInterfaceCounts, VariantSelections);

			VertexInterface.GetEnvironmentInterface() = CreateEnvironmentVertexInterface(InNode.Interface);

			// Validate vertex counts against frontend document
			if (VertexInterface.GetInputInterface().Num() != InNode.Interface.Inputs.Num()
				|| VertexInterface.GetOutputInterface().Num() != InNode.Interface.Outputs.Num())
			{
				UE_LOGF(LogMetaSound, Warning, "Vertex count mismatch for node %ls: produced %d/%d inputs/outputs, document has %d/%d. Document may be stale.",
					*InNode.Name.ToString(),
					VertexInterface.GetInputInterface().Num(), VertexInterface.GetOutputInterface().Num(),
					InNode.Interface.Inputs.Num(), InNode.Interface.Outputs.Num());
			}

			// Overlay per-vertex AccessType from frontend class interface
			OverlayAccessTypes(VertexInterface, InClassInterface);

			// Apply per-vertex literal defaults from the frontend node
			OverlayLiteralDefaults(VertexInterface, InContext, InNode, InClassInterface);

			return VertexInterface;
		}

		FVertexInterface CreateFlatVertexInterface(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInterface& InClassInterface)
		{
			check(InClassInterface.Inputs.Num() == InNode.Interface.Inputs.Num());
			check(InClassInterface.Outputs.Num() == InNode.Interface.Outputs.Num());

			// Build inputs
			TArray<FInputDataVertex> InputVertices;
			InputVertices.Reserve(InClassInterface.Inputs.Num());
			for (const FMetasoundFrontendClassInput& ClassInput : InClassInterface.Inputs)
			{
				const FMetasoundFrontendLiteral* FrontendLiteral = FindLiteralForInputVertex(InContext, ClassInput.Name, InNode, ClassInput);
				InputVertices.Add(MakeInputVertex(InContext, ClassInput.Name, ClassInput.TypeName, ClassInput.AccessType, FrontendLiteral));
			}

			// Build outputs
			TArray<FOutputDataVertex> OutputVertices;
			OutputVertices.Reserve(InClassInterface.Outputs.Num());
			for (const FMetasoundFrontendClassOutput& ClassOutput : InClassInterface.Outputs)
			{
				OutputVertices.Add(FOutputDataVertex{ClassOutput.Name, ClassOutput.TypeName, FDataVertexMetadata{}, FrontendVertexAccessTypeToCoreVertexAccessType(ClassOutput.AccessType)});
			}

			// Build environment
			FEnvironmentVertexInterface EnvironmentInterface = CreateEnvironmentVertexInterface(InNode.Interface);

			return FVertexInterface(
				FInputVertexInterface{MoveTemp(InputVertices), TArray<VertexPrivate::FSubInterfaceLayout>()},
				FOutputVertexInterface{MoveTemp(OutputVertices), TArray<VertexPrivate::FSubInterfaceLayout>()},
				MoveTemp(EnvironmentInterface));
		}

		FVertexInterface CreateVertexInterface(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassInput* InOwningGraphClassInput, const FMetasoundFrontendClassOutput* InOwningGraphClassOutput)
		{
			switch (InClass.Metadata.GetType())
			{
				case EMetasoundFrontendClassType::Input:
				{
					check(InOwningGraphClassInput != nullptr);
					return CreateInputNodeVertexInterface(InContext, InNode, InClassInterface, *InOwningGraphClassInput);
				}

				case EMetasoundFrontendClassType::Output:
				{
					check(InOwningGraphClassOutput != nullptr);
					return CreateOutputNodeVertexInterface(InContext, InNode, InClassInterface, *InOwningGraphClassOutput);
				}

				case EMetasoundFrontendClassType::Graph:
				{
					// This should only be used to support subgraphs. Subgraphs
					// are not registered node classes and therefore cannot be
					// looked up in the INodeClassRegistry (see "default:" case implementation below)
					return CreateFlatVertexInterface(InContext, InNode, InClassInterface);
				}

				default:
				{
					const FNodeClassRegistryKey NodeClassKey(InClass.Metadata);
					FClassInterface ClassInterface;
					if (INodeClassRegistry::GetChecked().FindClassInterface(NodeClassKey, ClassInterface)
						&& (ClassInterface.ContainsSubInterfaces() || ClassInterface.ContainsVariants()))
					{
						return CreateConfiguredVertexInterface(InContext, InNode, InClassInterface, ClassInterface);
					}

					return CreateFlatVertexInterface(InContext, InNode, InClassInterface);
				}
			}
		}

		const FName& GetNodeName(const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClassInput* InOwningGraphClassInput=nullptr, const FMetasoundFrontendClassOutput* InOwningGraphClassOutput=nullptr)
		{
			// Make sure the node's name is correct when it is an input or output node. 
			// The node name is used when adding the node to the FGraph as an input.
			if (InOwningGraphClassInput)
			{
				return InOwningGraphClassInput->Name;
			}
			else if (InOwningGraphClassOutput)
			{
				return InOwningGraphClassOutput->Name;
			}
			else
			{
				return InNode.Name;
			}
		}

		FNodeData CreateNodeData(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassInput* InOwningGraphClassInput=nullptr, const FMetasoundFrontendClassOutput* InOwningGraphClassOutput=nullptr)
		{
			TSharedPtr<const IOperatorData> Config;
			if (InNode.Configuration.IsValid())
			{
				Config = InNode.Configuration.Get().GetOperatorData();
			}

			const FName& NodeName = GetNodeName(InNode, InOwningGraphClassInput, InOwningGraphClassOutput);

			return FNodeData(NodeName, InNode.GetID(), CreateVertexInterface(InContext, InNode, InClass, InClassInterface, InOwningGraphClassInput, InOwningGraphClassOutput), MoveTemp(Config));
		}

		const FMetasoundFrontendVariable* FindVariableForVariableNode(const FMetasoundFrontendNode& InVariableNode, const FMetasoundFrontendGraph& InGraph)
		{
			const FGuid& DesiredID = InVariableNode.GetID();
			return InGraph.Variables.FindByPredicate([&](const FMetasoundFrontendVariable& InVar) { return InVar.VariableNodeID == DesiredID; });
		}

		TUniquePtr<INode> CreateVariableNode(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendGraph& InGraph)
		{
			check(InClass.Metadata.GetType() == EMetasoundFrontendClassType::Variable);
			check(InNode.ClassID == InClass.ID);
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));

			// Find the variable object associated with the node.
			const FMetasoundFrontendVariable* FrontendVariable = FindVariableForVariableNode(InNode, InGraph);

			if (nullptr != FrontendVariable)
			{
				const IDataTypeRegistry& DataTypeRegistry = InContext.DataTypeRegistry;
				const bool IsLiteralParsableByDataType = DataTypeRegistry.IsLiteralTypeSupported(FrontendVariable->TypeName, FrontendVariable->Literal.GetType());

				if (IsLiteralParsableByDataType)
				{
					FLiteral Literal = FrontendVariable->Literal.ToLiteral(FrontendVariable->TypeName, &InContext.DataTypeRegistry, InContext.ProxyDataCache);
					return DataTypeRegistry.CreateVariableNode(FrontendVariable->TypeName, MoveTemp(Literal), CreateNodeData(InContext, InNode, InClass, InClassInterface));
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot create variable node [NodeID:%ls]. [Variable:%ls] cannot be constructed with the provided literal type.", *InNode.GetID().ToString(), *FrontendVariable->Name.ToString());
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Cannot create variable node [NodeID:%ls]. No variable found for variable node.", *InNode.GetID().ToString());
			}

			return TUniquePtr<INode>(nullptr);
		}


		TUniquePtr<INode> CreateInputNode(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassInput& InOwningGraphClassInput)
		{
			check(InClass.Metadata.GetType() == EMetasoundFrontendClassType::Input);
			check(InNode.ClassID == InClass.ID);
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));

			// Check that the frontend node and class is correct.
			if (InNode.Interface.Inputs.Num() != 1)
			{
				UE_LOGF(LogMetaSound, Error, "MetaSound %ls contains invalid number of inputs (%d) on input node %ls", *InContext.AssetPath.ToString(), InNode.Interface.Inputs.Num(), *InNode.Name.ToString());
				return TUniquePtr<INode>(nullptr);
			}

			// Create the input node
			const FMetasoundFrontendVertex& InputVertex = InNode.Interface.Inputs[0];

			if (InOwningGraphClassInput.AccessType == EMetasoundFrontendVertexAccessType::Reference)
			{
				return InContext.DataTypeRegistry.CreateInputNode(InputVertex.TypeName, CreateNodeData(InContext, InNode, InClass, InClassInterface, &InOwningGraphClassInput));
			}
			else if (InOwningGraphClassInput.AccessType == EMetasoundFrontendVertexAccessType::Value)
			{
				return InContext.DataTypeRegistry.CreateConstructorInputNode(InputVertex.TypeName, CreateNodeData(InContext, InNode, InClass, InClassInterface, &InOwningGraphClassInput));
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "MetaSound %ls contains invalid input access type on input %ls", *InContext.AssetPath.ToString(), *InNode.Name.ToString());
			}

			return TUniquePtr<INode>(nullptr);
		}

		TUniquePtr<INode> CreateOutputNode(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface, const FMetasoundFrontendClassOutput& InOwningGraphClassOutput)
		{
			check(InClass.Metadata.GetType() == EMetasoundFrontendClassType::Output);
			check(InNode.ClassID == InClass.ID);
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));

			if (InNode.Interface.Outputs.Num() != 1)
			{
				UE_LOGF(LogMetaSound, Error, "MetaSound %ls contains invalid number of outputs (%d) on output node %ls", *InContext.AssetPath.ToString(), InNode.Interface.Outputs.Num(), *InNode.Name.ToString());
				return TUniquePtr<INode>(nullptr);
			}

			const FMetasoundFrontendVertex& OutputVertex = InNode.Interface.Outputs[0];

			if (InClassInterface.Outputs[0].AccessType == EMetasoundFrontendVertexAccessType::Reference)
			{
				return InContext.DataTypeRegistry.CreateOutputNode(OutputVertex.TypeName, CreateNodeData(InContext, InNode, InClass, InClassInterface, nullptr, &InOwningGraphClassOutput));
			}
			else if (InClassInterface.Outputs[0].AccessType == EMetasoundFrontendVertexAccessType::Value)
			{
				return InContext.DataTypeRegistry.CreateConstructorOutputNode(OutputVertex.TypeName, CreateNodeData(InContext, InNode, InClass, InClassInterface, nullptr, &InOwningGraphClassOutput));
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "MetaSound %ls contains invalid output access type on output %ls", *InContext.AssetPath.ToString(), *InNode.Name.ToString());
			}

			return TUniquePtr<INode>(nullptr);
		}

		TUniquePtr<INode> CreateExternalNode(const FBuildContext& InContext, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface)
		{
			using namespace Frontend;

			check(InNode.ClassID == InClass.ID);
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));

			const FNodeRegistryKey Key = FNodeRegistryKey(InClass.Metadata);
			return INodeClassRegistry::Get()->CreateNode(Key, CreateNodeData(InContext, InNode, InClass, InClassInterface));
		}

		TUniquePtr<INode> CreateSubgraphNode(const FBuildContext& InContext, const TSharedRef<const IGraph>& InSubgraph, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InClass, const FMetasoundFrontendClassInterface& InClassInterface)
		{
			using namespace Frontend;

			check(InNode.ClassID == InClass.ID);
			check(InterfacesHaveEqualSize(InClassInterface, InNode.Interface));

			return MakeUnique<FGraphNode>(CreateNodeData(InContext, InNode, InClass, InClassInterface), InSubgraph);
		}

		const FMetasoundFrontendClassInput* FindClassInputForInputNode(const FMetasoundFrontendGraphClass& InOwningGraph, const FMetasoundFrontendNode& InInputNode)
		{
			// Input nodes should have exactly one input.
			if (ensure(InInputNode.Interface.Inputs.Num() == 1))
			{
				const FName& TypeName = InInputNode.Interface.Inputs[0].TypeName;

				auto IsMatchingInput = [&](const FMetasoundFrontendClassInput& GraphInput)
				{
					return (InInputNode.GetID() == GraphInput.NodeID);
				};

				return InOwningGraph.GetDefaultInterface().Inputs.FindByPredicate(IsMatchingInput);
			}
			return nullptr;
		}

		const FMetasoundFrontendClassOutput* FindClassOutputForOutputNode(const FMetasoundFrontendGraphClass& InOwningGraph, const FMetasoundFrontendNode& InOutputNode)
		{
			// Output nodes should have exactly one output
			if (ensure(InOutputNode.Interface.Outputs.Num() == 1))
			{
				const FName& TypeName = InOutputNode.Interface.Outputs[0].TypeName;

				auto IsMatchingOutput = [&](const FMetasoundFrontendClassOutput& GraphOutput)
				{
					return (InOutputNode.GetID() == GraphOutput.NodeID);
				};

				return InOwningGraph.GetDefaultInterface().Outputs.FindByPredicate(IsMatchingOutput);
			}
			return nullptr;
		}


		void LogFailedToCreateNode(const FString& InAssetName, const FMetasoundFrontendNode& InNode, const FMetasoundFrontendClass& InNodeClass)
		{
			UE_LOGF(LogMetaSound, Error, "Metasound '%ls': Failed to create node [NodeID:%ls, NodeName:%ls] from registry [Class: %ls %ls]", *InAssetName, *InNode.GetID().ToString(), *InNode.Name.ToString(), *InNodeClass.Metadata.GetClassName().ToString(), *InNodeClass.Metadata.GetVersion().ToString());
		}

		bool AddNodesToGraph(FBuildGraphContext& InGraphContext)
		{
			for (const FMetasoundFrontendNode& FrontendNode : InGraphContext.PagedGraph.Nodes)
			{
				const FMetasoundFrontendClass* FrontendNodeClass = InGraphContext.BuildDocumentContext.FrontendClasses.FindRef(FrontendNode.ClassID);

				if (ensure(nullptr != FrontendNodeClass))
				{
					const FGraphBuilder::FCreateNodeParams CreateNodeParams
					{
						.FrontendNode = FrontendNode,
						.FrontendNodeClass = *FrontendNodeClass,
						.OwningFrontendGraph = InGraphContext.PagedGraph,
						.OwningFrontendGraphClass = InGraphContext.GraphClass,
						.ProxyDataCache = InGraphContext.BuildContext.ProxyDataCache,
						.DataTypeRegistry= &InGraphContext.BuildContext.DataTypeRegistry,
						.Subgraphs = &InGraphContext.BuildDocumentContext.Graphs,
						.PageOrder = InGraphContext.BuildContext.PageOrder,
						.OwningAssetPath = InGraphContext.BuildContext.AssetPath
					};
					TUniquePtr<INode> Node = FGraphBuilder::CreateNode(CreateNodeParams);

					if (!Node.IsValid())
					{
						LogFailedToCreateNode(InGraphContext.BuildContext.AssetPath.ToString(), FrontendNode, *FrontendNodeClass);
						continue;
					}

					switch (FrontendNodeClass->Metadata.GetType())
					{
						case EMetasoundFrontendClassType::Input:
							{
								const FName& InputName = Node->GetInstanceName();
								InGraphContext.Graph->AddInputNode(FrontendNode.GetID(), InputName, MoveTemp(Node));
							}
						break;

						case EMetasoundFrontendClassType::Output:
							{
								const FName& OutputName = Node->GetInstanceName();
								InGraphContext.Graph->AddOutputNode(FrontendNode.GetID(), OutputName, MoveTemp(Node));
							}
						break;

						case EMetasoundFrontendClassType::Literal:
						case EMetasoundFrontendClassType::Graph:
						case EMetasoundFrontendClassType::Variable:
						case EMetasoundFrontendClassType::Template:
						case EMetasoundFrontendClassType::VariableAccessor:
						case EMetasoundFrontendClassType::VariableDeferredAccessor:
						case EMetasoundFrontendClassType::VariableMutator:
						case EMetasoundFrontendClassType::External:
						default:
							InGraphContext.Graph->AddNode(FrontendNode.GetID(), MoveTemp(Node));
						break;
					}
				}
			}

			return true;
		}

		bool AddEdgesToGraph(FBuildGraphContext& InGraphContext)
		{
			// Pair of frontend node and core node. The frontend node can be one of
			// several types.
			struct FCoreNodeAndFrontendVertex
			{
				const INode* Node = nullptr;
				const FMetasoundFrontendVertex* Vertex = nullptr;
			};

			TMap<FNodeIDVertexID, FCoreNodeAndFrontendVertex> NodeSourcesByID;
			TMap<FNodeIDVertexID, FCoreNodeAndFrontendVertex> NodeDestinationsByID;

			// Add nodes to NodeID/VertexID map
			for (const FMetasoundFrontendNode& Node : InGraphContext.PagedGraph.Nodes)
			{
				const INode* CoreNode = InGraphContext.Graph->FindNode(Node.GetID());
				if (nullptr == CoreNode)
				{
					UE_LOGF(LogMetaSound, Warning, "Metasound '%ls': Could not find referenced node [Name:%ls, NodeID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Node.Name.ToString(), *Node.GetID().ToString());
					return false;
				}

				for (const FMetasoundFrontendVertex& Vertex : Node.Interface.Inputs)
				{
					NodeDestinationsByID.Add(FNodeIDVertexID(Node.GetID(), Vertex.VertexID), FCoreNodeAndFrontendVertex({CoreNode, &Vertex}));
				}

				for (const FMetasoundFrontendVertex& Vertex : Node.Interface.Outputs)
				{
					NodeSourcesByID.Add(FNodeIDVertexID(Node.GetID(), Vertex.VertexID), FCoreNodeAndFrontendVertex({CoreNode, &Vertex}));
				}
			};

			for (const FMetasoundFrontendEdge& Edge : InGraphContext.PagedGraph.Edges)
			{
				const FNodeIDVertexID DestinationKey(Edge.ToNodeID, Edge.ToVertexID);
				const FCoreNodeAndFrontendVertex* DestinationNodeAndVertex = NodeDestinationsByID.Find(DestinationKey);

				if (nullptr == DestinationNodeAndVertex)
				{
					UE_LOGF(LogMetaSound, Error, "MetaSound '%ls': Failed to add edge. Could not find destination [NodeID:%ls, VertexID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Edge.ToNodeID.ToString(), *Edge.ToVertexID.ToString());
					return false;
				}

				if (nullptr == DestinationNodeAndVertex->Node)
				{
					UE_LOGF(LogMetaSound, Warning, "MetaSound '%ls': 'Failed to add edge. Null destination node [NodeID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Edge.ToNodeID.ToString());
					return false;
				}

				const FNodeIDVertexID SourceKey(Edge.FromNodeID, Edge.FromVertexID);
				const FCoreNodeAndFrontendVertex* SourceNodeAndVertex = NodeSourcesByID.Find(SourceKey);

				if (nullptr == SourceNodeAndVertex)
				{
					UE_LOGF(LogMetaSound, Error, "MetaSound '%ls': Failed to add edge. Could not find source [NodeID:%ls, VertexID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Edge.FromNodeID.ToString(), *Edge.FromVertexID.ToString());
					return false;
				}

				if (nullptr == SourceNodeAndVertex->Node)
				{
					UE_LOGF(LogMetaSound, Warning, "MetaSound '%ls': Skipping edge. Null source node [NodeID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Edge.FromNodeID.ToString());
					return false;
				}

				const INode* FromNode = SourceNodeAndVertex->Node;
				const FVertexName FromVertexKey = SourceNodeAndVertex->Vertex->Name;

				const INode* ToNode = DestinationNodeAndVertex->Node;
				const FVertexName ToVertexKey = DestinationNodeAndVertex->Vertex->Name;

				bool bSuccess = InGraphContext.Graph->AddDataEdge(*FromNode, FromVertexKey,  *ToNode, ToVertexKey);

				// If succeeded, remove input as viable vertex to construct default variable, as it has been superceded by a connection.
				if (!bSuccess)
				{
					UE_LOGF(LogMetaSound, Error, "MetaSound '%ls': Failed to connect edge from [NodeID:%ls, VertexID:%ls] to [NodeID:%ls, VertexID:%ls]", *InGraphContext.BuildContext.AssetPath.ToString(), *Edge.FromNodeID.ToString(), *Edge.FromVertexID.ToString(), *Edge.ToNodeID.ToString(), *Edge.ToVertexID.ToString());
					return false;
				}
			}

			return true;
		}

		bool SortSubgraphDependencies(TArray<const FMetasoundFrontendGraphClass*>& Subgraphs)
		{
			// Helper for caching and querying subgraph dependencies
			struct FSubgraphDependencyLookup
			{
				FSubgraphDependencyLookup(TArrayView<const FMetasoundFrontendGraphClass*> InGraphs)
				{
					// Map ClassID to graph pointer. 
					TMap<FGuid, const FMetasoundFrontendGraphClass*> ClassIDAndGraph;
					for (const FMetasoundFrontendGraphClass* Graph: InGraphs)
					{
						ClassIDAndGraph.Add(Graph->ID, Graph);
					}

					// Cache subgraph dependencies.
					for (const FMetasoundFrontendGraphClass* GraphClass : InGraphs)
					{
						for (const FMetasoundFrontendNode& Node : GraphClass->GetConstDefaultGraph().Nodes)
						{
							if (ClassIDAndGraph.Contains(Node.ClassID))
							{
								DependencyMap.Add(GraphClass, ClassIDAndGraph[Node.ClassID]);
							}
						}
					}
				}

				TArray<const FMetasoundFrontendGraphClass*> operator()(const FMetasoundFrontendGraphClass* InParent) const
				{
					TArray<const FMetasoundFrontendGraphClass*> Dependencies;
					DependencyMap.MultiFind(InParent, Dependencies);
					return Dependencies;
				}

			private:

				TMultiMap<const FMetasoundFrontendGraphClass*, const FMetasoundFrontendGraphClass*> DependencyMap;
			};

			bool bSuccess = Algo::TopologicalSort(Subgraphs, FSubgraphDependencyLookup(Subgraphs));
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Error, "Failed to topologically sort subgraphs. Possible recursive subgraph dependency");
			}

			return bSuccess;
		}

		TUniquePtr<FFrontendGraph> CreateGraphInternal(FBuildContext& InContext, const FBuildDocumentContext& InDocumentContext, const FMetasoundFrontendGraphClass& InGraphClass, const FGuid& InGraphId)
		{
			const FMetasoundFrontendGraph* PageGraph = FindPreferredPage(InGraphClass.GetConstGraphPages(), InContext.PageOrder);
			if (!PageGraph)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot create FrontendGraph because could not find preferred graph page.");
				return nullptr;
			}

			FBuildGraphContext BuildGraphContext
			{
				MakeUnique<FFrontendGraph>(InContext.AssetPath, InGraphId),
				InGraphClass,
				*PageGraph,
				InContext,
				InDocumentContext
			};

			bool bSuccess = AddNodesToGraph(BuildGraphContext);

			if (bSuccess)
			{
				bSuccess = AddEdgesToGraph(BuildGraphContext);
			}

			if (bSuccess)
			{
				return MoveTemp(BuildGraphContext.Graph);
			}
			else
			{
				return TUniquePtr<FFrontendGraph>(nullptr);
			}
		}
	} // namespace GraphBuilderPrivate
	 
	TUniquePtr<INode> FGraphBuilder::CreateNode(const FCreateNodeParams& InParams)
	{
		using namespace GraphBuilderPrivate;

		checkf(InParams.ProxyDataCache != nullptr || IsInGameThread(), TEXT("An FProxyDataCache must be used if creating nodes on a thread other than the game thread"));


		TUniquePtr<INode> Node;

		const FMetasoundFrontendClassInterface& ClassInterface = InParams.FrontendNodeClass.GetInterfaceForNode(InParams.FrontendNode);

		if (ClassInterface.Inputs.Num() != InParams.FrontendNode.Interface.Inputs.Num())
		{
			UE_LOGF(LogMetaSound, Error,
				"Cannot create node. MetaSound %ls contains mismatched number of inputs (%d / %d) on node %ls class %ls",
				*InParams.OwningAssetPath.ToString(),
				InParams.FrontendNode.Interface.Inputs.Num(),
				ClassInterface.Inputs.Num(),
				*InParams.FrontendNode.Name.ToString(),
				*InParams.FrontendNodeClass.Metadata.GetClassName().ToString());
			return Node;
		}
		else if (ClassInterface.Outputs.Num() != InParams.FrontendNode.Interface.Outputs.Num())
		{
			UE_LOGF(LogMetaSound, Error,
				"Cannot create node. MetaSound %ls contains mismatched number of outputs (%d / %d) on node %ls class %ls",
				*InParams.OwningAssetPath.ToString(),
				InParams.FrontendNode.Interface.Outputs.Num(),
				ClassInterface.Outputs.Num(),
				*InParams.FrontendNode.Name.ToString(),
				*InParams.FrontendNodeClass.Metadata.GetClassName().ToString());
			return Node;
		}

		FBuildContext BuildContext
		{
			InParams.OwningAssetPath,
			InParams.DataTypeRegistry ? *InParams.DataTypeRegistry : IDataTypeRegistry::Get(),
			InParams.ProxyDataCache,
			InParams.PageOrder
		};

		switch (InParams.FrontendNodeClass.Metadata.GetType())
		{
			case EMetasoundFrontendClassType::Input:
			{
				const FMetasoundFrontendClassInput* ClassInput = FindClassInputForInputNode(InParams.OwningFrontendGraphClass, InParams.FrontendNode);

				if (nullptr != ClassInput)
				{
					Node = CreateInputNode(BuildContext, InParams.FrontendNode, InParams.FrontendNodeClass, ClassInterface, *ClassInput);
				}
				else
				{
					const FString GraphClassIDString = InParams.OwningFrontendGraphClass.ID.ToString();
					UE_LOGF(LogMetaSound, Error,
						"MetaSound '%ls': Failed to match input node [NodeID:%ls, NodeName:%ls] to owning graph [ClassID:%ls] input.",
						*InParams.OwningAssetPath.ToString(),
						*InParams.FrontendNode.GetID().ToString(),
						*InParams.FrontendNode.Name.ToString(),
						*GraphClassIDString);
				}
			}
			break;

			case EMetasoundFrontendClassType::Output:
			{
				const FMetasoundFrontendClassOutput* ClassOutput = FindClassOutputForOutputNode(InParams.OwningFrontendGraphClass, InParams.FrontendNode);
				if (nullptr != ClassOutput)
				{
					Node = CreateOutputNode(BuildContext, InParams.FrontendNode, InParams.FrontendNodeClass, ClassInterface, *ClassOutput);
				}
				else
				{
					const FString GraphClassIDString = InParams.OwningFrontendGraphClass.ID.ToString();
					UE_LOGF(LogMetaSound, Error,
						"MetaSound '%ls': Failed to match output node [NodeID:%ls, NodeName:%ls] to owning graph [ClassID:%ls] output.",
						*InParams.OwningAssetPath.ToString(),
						*InParams.FrontendNode.GetID().ToString(),
						*InParams.FrontendNode.Name.ToString(),
						*GraphClassIDString);
				}
			}
			break;

			case EMetasoundFrontendClassType::Graph:
			{
				if (InParams.Subgraphs == nullptr)
				{
					UE_LOGF(LogMetaSound, Error,
						"MetaSound '%ls': Failed to find subgraph for node [NodeID:%ls, NodeName:%ls, ClassID:%ls] because of missing subgraph map",
						*InParams.OwningAssetPath.ToString(),
						*InParams.FrontendNode.GetID().ToString(),
						*InParams.FrontendNode.Name.ToString(),
						*InParams.FrontendNode.ClassID.ToString());
				}
				else
				{
					TSharedPtr<const IGraph> SubgraphPtr = InParams.Subgraphs->FindRef(InParams.FrontendNode.ClassID);
					if (SubgraphPtr.IsValid())
					{
						Node = CreateSubgraphNode(BuildContext, SubgraphPtr.ToSharedRef(), InParams.FrontendNode, InParams.FrontendNodeClass, ClassInterface);
					}
					else
					{
						UE_LOGF(LogMetaSound, Error,
							"MetaSound '%ls': Found invalid subgraph for node [NodeID:%ls, NodeName:%ls, ClassID:%ls]",
							*InParams.OwningAssetPath.ToString(),
							*InParams.FrontendNode.GetID().ToString(),
							*InParams.FrontendNode.Name.ToString(),
							*InParams.FrontendNode.ClassID.ToString());

					}
				}
			}
			break;

			case EMetasoundFrontendClassType::Literal:
			{
				UE_LOGF(LogMetaSound, Warning,
					"MetaSound '%ls': Adding literal nodes is being deprecated. Please set literal values on the connected node interface directly.",
					*InParams.OwningAssetPath.ToString());
				const FName& DataTypeName = InParams.FrontendNodeClass.Metadata.GetClassName().Name;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FDefaultLiteralNodeConstructorParams InitData { InParams.FrontendNode.Name, InParams.FrontendNode.GetID(), BuildContext.DataTypeRegistry.CreateDefaultLiteral(DataTypeName)};
				Node = BuildContext.DataTypeRegistry.CreateLiteralNode(DataTypeName, MoveTemp(InitData));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				break;
			}

			case EMetasoundFrontendClassType::Variable:
			{
				Node = CreateVariableNode(BuildContext, InParams.FrontendNode, InParams.FrontendNodeClass, ClassInterface, InParams.OwningFrontendGraph);
			}
			break;

			// Templates, variable accessors and mutators are
			// constructed with the same parameters as external nodes.
			case EMetasoundFrontendClassType::Template:
			case EMetasoundFrontendClassType::VariableAccessor:
			case EMetasoundFrontendClassType::VariableDeferredAccessor:
			case EMetasoundFrontendClassType::VariableMutator:
			case EMetasoundFrontendClassType::External:
			default:
			{
				Node = CreateExternalNode(BuildContext, InParams.FrontendNode, InParams.FrontendNodeClass, ClassInterface);
			}
			break;
		}

		return Node;
	}


	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendGraphClass& InGraph,
		const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
		const TArray<FMetasoundFrontendClass>& InDependencies,
		const Frontend::FProxyDataCache& InProxyDataCache,
		const FTopLevelAssetPath& InAssetPath,
		const FGuid InGraphId,
		TArrayView<const FGuid> InPageOrder)
	{
		using namespace GraphBuilderPrivate;

		FBuildContext Context
		{
			InAssetPath,
			Frontend::IDataTypeRegistry::Get(),
			&InProxyDataCache,
			InPageOrder
		};

		FBuildDocumentContext DocumentContext;

		// Gather all references to node classes from external dependencies and subgraphs.
		for (const FMetasoundFrontendClass& ExtClass : InDependencies)
		{
			DocumentContext.FrontendClasses.Add(ExtClass.ID, &ExtClass);
		}
		for (const FMetasoundFrontendClass& ExtClass : InSubgraphs)
		{
			DocumentContext.FrontendClasses.Add(ExtClass.ID, &ExtClass);
		}

		// Sort subgraphs so that dependent subgraphs are created in correct order.
		TArray<const FMetasoundFrontendGraphClass*> FrontendSubgraphPtrs;
		Algo::Transform(InSubgraphs, FrontendSubgraphPtrs, [](const FMetasoundFrontendGraphClass& InClass) { return &InClass; });

		bool bSuccess = SortSubgraphDependencies(FrontendSubgraphPtrs);
		if (!bSuccess)
		{
			UE_LOGF(LogMetaSound, Error, "Failed to create graph due to failed subgraph ordering in asset '%ls'.", *InAssetPath.ToString());
			return TUniquePtr<FFrontendGraph>(nullptr);
		}

		// Create each subgraph.
		for (const FMetasoundFrontendGraphClass* FrontendSubgraphPtr : FrontendSubgraphPtrs)
		{
			TSharedPtr<const IGraph> Subgraph(CreateGraphInternal(Context, DocumentContext, *FrontendSubgraphPtr, FrontendSubgraphPtr->ID).Release());
			if (!Subgraph.IsValid())
			{
				UE_LOGF(LogMetaSound, Warning, "Failed to create subgraph [SubgraphName: %ls] in asset '%ls'", *FrontendSubgraphPtr->Metadata.GetClassName().ToString(), *InAssetPath.ToString());
			}
			else
			{
				// Add subgraphs to context so they are accessible for subsequent graphs.
				DocumentContext.Graphs.Add(FrontendSubgraphPtr->ID, Subgraph);
			}
		}

		// Create parent graph.
		return CreateGraphInternal(Context, DocumentContext, InGraph, InGraphId);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendDocument& InDocument,
		const FTopLevelAssetPath& InAssetPath,
		TArrayView<const FGuid> InPageOrder)
	{
		// Create proxies before creating graph.
		Frontend::FProxyDataCache ProxyDataCache;
		ProxyDataCache.CreateAndCacheProxies(InDocument, InPageOrder);
		
		return CreateGraph(InDocument, ProxyDataCache, InAssetPath, Frontend::CreateLocallyUniqueId(), InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendGraphClass& InGraph,
		const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
		const TArray<FMetasoundFrontendClass>& InDependencies,
		const FTopLevelAssetPath& InAssetPath,
		TArrayView<const FGuid> InPageOrder)
	{
		// Create proxies before building graph
		Frontend::FProxyDataCache ProxyDataCache;
		ProxyDataCache.CreateAndCacheProxies(InGraph, InPageOrder);

		for (const FMetasoundFrontendGraphClass& SubgraphClass : InSubgraphs)
		{
			ProxyDataCache.CreateAndCacheProxies(SubgraphClass, InPageOrder);
		}

		for (const FMetasoundFrontendClass& DependencyClass : InDependencies)
		{
			ProxyDataCache.CreateAndCacheProxies(DependencyClass, InPageOrder);
		}

		return CreateGraph(InGraph, InSubgraphs, InDependencies, ProxyDataCache, InAssetPath, Frontend::CreateLocallyUniqueId(), InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendDocument& InDocument,
		const Frontend::FProxyDataCache& InProxyDataCache,
		const FTopLevelAssetPath& InAssetPath,
		const FGuid InGraphId,
		TArrayView<const FGuid> InPageOrder)
	{
		return CreateGraph(InDocument.RootGraph, InDocument.Subgraphs, InDocument.Dependencies, InProxyDataCache, InAssetPath, InGraphId, InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(const FMetasoundFrontendDocument& InDocument, const FString& InDebugAssetName, TArrayView<const FGuid> InPageOrder)
	{
		return CreateGraph(InDocument, FTopLevelAssetPath(InDebugAssetName), InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendDocument& InDocument,
		const Frontend::FProxyDataCache& InProxies,
		const FString& InDebugAssetName,
		const FGuid InGraphId,
		TArrayView<const FGuid> InPageOrder)
	{
		return CreateGraph(InDocument, InProxies, FTopLevelAssetPath(InDebugAssetName), InGraphId, InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendGraphClass& InGraph,
		const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
		const TArray<FMetasoundFrontendClass>& InDependencies,
		const FString& InDebugAssetName,
		TArrayView<const FGuid> InPageOrder)
	{
		return CreateGraph(InGraph, InSubgraphs, InDependencies, FTopLevelAssetPath(InDebugAssetName), InPageOrder);
	}

	TUniquePtr<FFrontendGraph> FGraphBuilder::CreateGraph(
		const FMetasoundFrontendGraphClass& InGraph,
		const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
		const TArray<FMetasoundFrontendClass>& InDependencies,
		const Frontend::FProxyDataCache& InProxyDataCache,
		const FString& InDebugAssetName,
		const FGuid InGraphId,
		TArrayView<const FGuid> InPageOrder)
	{
		return CreateGraph(InGraph, InSubgraphs, InDependencies, InProxyDataCache, FTopLevelAssetPath(InDebugAssetName), InGraphId, InPageOrder);
	}
}

