// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendGraphController.h"

#include "Algo/NoneOf.h"
#include "Internationalization/Text.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeController.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundFrontendSubgraphNodeController.h"
#include "MetasoundFrontendInvalidController.h"
#include "MetasoundFrontendVariableController.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundVariableNodes.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontendGraphController"

namespace Metasound
{
	namespace Frontend
	{
		const FMetasoundFrontendGraph& FindConstBuildGraphChecked(const FMetasoundFrontendGraphClass& InGraphClass)
		{
			// Registry can be null in test builds for legacy controller implementation, so doesn't use GetChecked
			if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
			{
				if (const FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(InGraphClass.Metadata.GetClassName(), { }))
				{
					return Builder->FindConstBuildGraphChecked();
				}
			}

			return InGraphClass.GetConstDefaultGraph();
		}

		FMetasoundFrontendGraph& FindBuildGraphChecked(FMetasoundFrontendGraphClass& InGraphClass)
		{
			// Registry can be null in test builds for legacy controller implementation, so doesn't use GetChecked
			if (IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
			{
				if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(InGraphClass.Metadata.GetClassName(), { }))
				{
					// const cast to dissuade mutable accessor for external API as the controller API is actively deprecated across engine releases.
					return const_cast<FMetasoundFrontendGraph&>(Builder->FindConstBuildGraphChecked());
				}
			}

			return InGraphClass.GetDefaultGraph();
		}

		// 
		// FGraphController
		//
		FGraphController::FGraphController(EPrivateToken InToken, const FGraphController::FInitParams& InParams)
		: GraphClassPtr(InParams.GraphClassPtr)
		, OwningDocument(InParams.OwningDocument)
		{
		}

		FGraphHandle FGraphController::CreateGraphHandle(const FGraphController::FInitParams& InParams)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = InParams.GraphClassPtr.Get())
			{
				if (GraphClass->Metadata.GetType() == EMetasoundFrontendClassType::Graph)
				{
					return MakeShared<FGraphController>(EPrivateToken::Token, InParams);
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to make graph controller [ClassID:%ls]. Class must be EMeatsoundFrontendClassType::Graph.", *GraphClass->ID.ToString())
				}
			}
			return IGraphController::GetInvalidHandle();
		}

		FConstGraphHandle FGraphController::CreateConstGraphHandle(const FGraphController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = InParams.GraphClassPtr.Get())
			{
				if (GraphClass->Metadata.GetType() == EMetasoundFrontendClassType::Graph)
				{
					return MakeShared<FGraphController>(EPrivateToken::Token, InParams);
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to make graph controller [ClassID:%ls]. Class must be EMeatsoundFrontendClassType::Graph.", *GraphClass->ID.ToString())
				}
			}
			return IGraphController::GetInvalidHandle();
		}

		bool FGraphController::IsValid() const
		{
			return (nullptr != GraphClassPtr.Get()) && OwningDocument->IsValid();
		}

		FGuid FGraphController::GetClassID() const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->ID;
			}

			return Metasound::FrontendInvalidID;
		}

#if WITH_EDITOR
		FText FGraphController::GetDisplayName() const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}
#endif // WITH_EDITOR

		TArray<FVertexName> FGraphController::GetInputVertexNames() const
		{
			TArray<FVertexName> Names;

			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				for (const FMetasoundFrontendClassInput& Input : GraphClass->GetDefaultInterface().Inputs)
				{
					Names.Add(Input.Name);
				}
			}

			return Names;
		}

		TArray<FVertexName> FGraphController::GetOutputVertexNames() const
		{
			TArray<FVertexName> Names;

			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				for (const FMetasoundFrontendClassOutput& Output : GraphClass->GetDefaultInterface().Outputs)
				{
					Names.Add(Output.Name);
				}
			}

			return Names;
		}

		FConstClassInputAccessPtr FGraphController::FindClassInputWithName(const FVertexName& InName) const
		{
			return GraphClassPtr.GetInputWithName(InName);
		}

		FConstClassOutputAccessPtr FGraphController::FindClassOutputWithName(const FVertexName& InName) const
		{
			return GraphClassPtr.GetOutputWithName(InName);
		}

		FGuid FGraphController::GetVertexIDForInputVertex(const FVertexName& InInputName) const
		{
			if (const FMetasoundFrontendClassInput* Input = FindClassInputWithName(InInputName).Get())
			{
				return Input->VertexID;
			}
			return Metasound::FrontendInvalidID;
		}

		FGuid FGraphController::GetVertexIDForOutputVertex(const FVertexName& InOutputName) const
		{
			if (const FMetasoundFrontendClassOutput* Output = FindClassOutputWithName(InOutputName).Get())
			{
				return Output->VertexID;
			}
			return Metasound::FrontendInvalidID;
		}

		TArray<FNodeHandle> FGraphController::GetNodes()
		{
			return GetNodeHandles(GetNodesAndClasses());
		}

		TArray<FConstNodeHandle> FGraphController::GetConstNodes() const
		{
			return GetNodeHandles(GetNodesAndClasses());
		}

		FConstNodeHandle FGraphController::GetNodeWithID(FGuid InNodeID) const
		{
			auto IsNodeWithSameID = [&](const FMetasoundFrontendClass& NodeClas, const FMetasoundFrontendNode& Node) 
			{ 
				return Node.GetID() == InNodeID;
			};
			return GetNodeByPredicate(IsNodeWithSameID);
		}

		FNodeHandle FGraphController::GetNodeWithID(FGuid InNodeID)
		{
			auto IsNodeWithSameID = [&](const FMetasoundFrontendClass& NodeClas, const FMetasoundFrontendNode& Node) 
			{ 
				return Node.GetID() == InNodeID;
			};
			return GetNodeByPredicate(IsNodeWithSameID);
		}

#if WITH_EDITOR
		const FMetasoundFrontendGraphStyle& FGraphController::GetGraphStyle() const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return FindConstBuildGraphChecked(*GraphClass).Style;
			}

			return Invalid::GetInvalidGraphStyle();
		}

		const FMetasoundFrontendInterfaceStyle& FGraphController::GetInputStyle() const
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().GetInputStyle();
			}

			return Invalid::GetInvalidInterfaceStyle();
		}

		const FMetasoundFrontendInterfaceStyle& FGraphController::GetOutputStyle() const
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().GetOutputStyle();
			}

			return Invalid::GetInvalidInterfaceStyle();
		}

		void FGraphController::SetGraphStyle(FMetasoundFrontendGraphStyle Style)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FindBuildGraphChecked(*GraphClass).Style = MoveTemp(Style);
			}
		}

		void FGraphController::SetInputStyle(FMetasoundFrontendInterfaceStyle Style)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				TArray<FMetasoundFrontendClassInput>& Inputs = GraphClass->GetDefaultInterface().Inputs;
				Style.DefaultSortOrder.SetNumZeroed(Inputs.Num());

				for (int32 i = 0; i < Inputs.Num(); ++i)
				{
					Inputs[i].Metadata.SortOrderIndex = Style.DefaultSortOrder[i];
				}
				GraphClass->GetDefaultInterface().SetInputStyle(MoveTemp(Style));
			}
		}

		void FGraphController::SetOutputStyle(FMetasoundFrontendInterfaceStyle Style)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				TArray<FMetasoundFrontendClassOutput>& Outputs = GraphClass->GetDefaultInterface().Outputs;
				Style.DefaultSortOrder.SetNumZeroed(Outputs.Num());

				for (int32 i = 0; i < Outputs.Num(); ++i)
				{
					Outputs[i].Metadata.SortOrderIndex = Style.DefaultSortOrder[i];
				}
				GraphClass->GetDefaultInterface().SetOutputStyle(MoveTemp(Style));
			}
		}
#endif // WITH_EDITOR

		TArray<FNodeHandle> FGraphController::GetOutputNodes()
		{
			auto IsOutputNode = [](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node) 
			{
				return NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Output;
			};
			return GetNodesByPredicate(IsOutputNode);
		}

		TArray<FNodeHandle> FGraphController::GetInputNodes()
		{
			auto IsInputNode = [](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node)
			{
				return NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Input;
			};
			return GetNodesByPredicate(IsInputNode);
		}

		FVariableHandle FGraphController::AddVariable(const FName& InDataType)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const IDataTypeRegistry& Registry = IDataTypeRegistry::Get();
				FDataTypeRegistryInfo Info;
				if (ensure(Registry.GetDataTypeInfo(InDataType, Info)))
				{
					FGuid VariableID = FGuid::NewGuid();

					FMetasoundFrontendVariable Variable;
#if WITH_EDITORONLY_DATA
					Variable.DisplayName = Info.DataTypeDisplayText;
#endif // WITH_EDITORONLY_DATA
					Variable.TypeName = Info.DataTypeName;
					Variable.Literal.SetFromLiteral(Registry.CreateDefaultLiteral(InDataType));
					Variable.ID = VariableID;
#if WITH_EDITOR
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddMemberIDModified(Variable.ID);
					}
#endif // WITH_EDITOR

					FMetasoundFrontendClass VariableNodeClass;
					if (IDataTypeRegistry::Get().GetFrontendVariableClass(Variable.TypeName, VariableNodeClass))
					{
						FNodeHandle InitNode = AddNode(VariableNodeClass.Metadata, FGuid::NewGuid());
						if (InitNode->IsValid())
						{
							Variable.VariableNodeID = InitNode->GetID();
							FindBuildGraphChecked(*GraphClass).Variables.Add(MoveTemp(Variable));
						}
					}

					return FindVariable(VariableID);
				}
			}

			return IVariableController::GetInvalidHandle();
		}

		FVariableHandle FGraphController::FindVariable(const FGuid& InVariableID)
		{
			FVariableAccessPtr VariablePtr = GraphClassPtr.GetVariableWithID(InVariableID);
			return MakeShared<FVariableController>(FVariableController::FInitParams{VariablePtr, this->AsShared()});
		}

		FConstVariableHandle FGraphController::FindVariable(const FGuid& InVariableID) const
		{
			FConstVariableAccessPtr ConstVariablePtr = GraphClassPtr.GetVariableWithID(InVariableID);
			return MakeShared<FVariableController>(FVariableController::FInitParams{ConstCastAccessPtr<FVariableAccessPtr>(ConstVariablePtr), ConstCastSharedRef<IGraphController>(this->AsShared())});
		}

		FVariableHandle FGraphController::FindVariableContainingNode(const FGuid& InNodeID)
		{
			FGuid VariableID = FindVariableIDOfVariableContainingNode(InNodeID);
			if (VariableID.IsValid())
			{
				return FindVariable(VariableID);
			}
			return IVariableController::GetInvalidHandle();
		}

		FConstVariableHandle FGraphController::FindVariableContainingNode(const FGuid& InNodeID) const
		{
			FGuid VariableID = FindVariableIDOfVariableContainingNode(InNodeID);
			if (VariableID.IsValid())
			{
				return FindVariable(VariableID);
			}
			return IVariableController::GetInvalidHandle();
		}

		bool FGraphController::RemoveVariable(const FGuid& InVariableID)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
				{
					FNodeHandle NodeHandle = GetNodeWithID(Variable->VariableNodeID);
					RemoveNode(*NodeHandle);

					NodeHandle = GetNodeWithID(Variable->MutatorNodeID);
					RemoveNode(*NodeHandle);

					// Copy ids as node removal will update variable node IDs
					TArray<FGuid> AccessorNodeIDs = Variable->AccessorNodeIDs;
					for (const FGuid& NodeID : AccessorNodeIDs)
					{
						NodeHandle = GetNodeWithID(NodeID);
						RemoveNode(*NodeHandle);
					}

					// Copy ids as node removal will update variable node IDs
					TArray<FGuid> DeferredAccessorNodeIDs = Variable->DeferredAccessorNodeIDs;
					for (const FGuid& NodeID : DeferredAccessorNodeIDs)
					{
						NodeHandle = GetNodeWithID(NodeID);
						RemoveNode(*NodeHandle);
					}

#if WITH_EDITOR
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddMemberIDModified(InVariableID);
					}
#endif // WITH_EDITOR

					auto IsVariableWithID = [&](const FMetasoundFrontendVariable& InVariable)
					{
						return InVariable.ID == InVariableID;
					};
					FindBuildGraphChecked(*GraphClass).Variables.RemoveAllSwap(IsVariableWithID);
					return true;
				}
			}

			return false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		TArray<FVariableHandle> FGraphController::GetVariables()
		{
			TArray<FVariableHandle> VariableHandles;

			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendVariable>& Variables = FindConstBuildGraphChecked(*GraphClass).Variables;
				for (const FMetasoundFrontendVariable& Variable : Variables)
				{
					VariableHandles.Add(FindVariable(Variable.ID));
				}
			}

			return VariableHandles;
		}

		TArray<FConstVariableHandle> FGraphController::GetVariables() const
		{
			TArray<FConstVariableHandle> VariableHandles;

			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendVariable>& Variables = FindConstBuildGraphChecked(*GraphClass).Variables;
				for (const FMetasoundFrontendVariable& Variable : Variables)
				{
					VariableHandles.Add(FindVariable(Variable.ID));
				}
			}

			return VariableHandles;
		}

		FNodeHandle FGraphController::FindOrAddVariableMutatorNode(const FGuid& InVariableID)
		{
			using namespace VariableNames; 

			if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
			{
				FNodeHandle SetNode = GetNodeWithID(Variable->MutatorNodeID);
				if (!SetNode->IsValid())
				{
					FMetasoundFrontendClass SetNodeClass;
					if (IDataTypeRegistry::Get().GetFrontendVariableMutatorClass(Variable->TypeName, SetNodeClass))
					{
						SetNode = AddNode(SetNodeClass.Metadata, FGuid::NewGuid());
						if (SetNode->IsValid())
						{
							// Initialize set default literal value to that of the variable
							FInputHandle InputHandle = SetNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputData));
							if (ensure(InputHandle->IsValid()))
							{
								InputHandle->SetLiteral(Variable->Literal);
							}

							Variable->MutatorNodeID = SetNode->GetID();
							FGuid SourceVariableNodeID = Variable->VariableNodeID;

							// Connect last delayed getter in variable stack.
							if (Variable->DeferredAccessorNodeIDs.Num() > 0)
							{
								SourceVariableNodeID = Variable->DeferredAccessorNodeIDs.Last();
							}
							FNodeHandle SourceVariableNode = GetNodeWithID(SourceVariableNodeID);

							if (ensure(SourceVariableNode->IsValid()))
							{
								FInputHandle SetNodeInput = SetNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));
								FOutputHandle SourceVariableNodeOutput = SourceVariableNode->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));

								ensure(SetNodeInput->Connect(*SourceVariableNodeOutput));
							}

							// Connect to first inline getter in variable stack
							if (Variable->AccessorNodeIDs.Num() > 0)
							{
								FNodeHandle HeadGetNode = GetNodeWithID(Variable->AccessorNodeIDs[0]);
								if (HeadGetNode->IsValid())
								{
									FOutputHandle SetNodeOutput = SetNode->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));
									FInputHandle GetNodeInput = HeadGetNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));

									ensure(SetNodeOutput->Connect(*GetNodeInput));
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Could not find registered \"set variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
					}
				}
				return SetNode;
			}
			
			return INodeController::GetInvalidHandle();
		}

		FNodeHandle FGraphController::AddVariableAccessorNode(const FGuid& InVariableID)
		{
			using namespace VariableNames;
			
			if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
			{
				FMetasoundFrontendClass NodeClass;
				if (IDataTypeRegistry::Get().GetFrontendVariableAccessorClass(Variable->TypeName, NodeClass))
				{
					FNodeHandle NewNode = AddNode(NodeClass.Metadata, FGuid::NewGuid());

					if (NewNode->IsValid())
					{
						// Connect new node.
						FInputHandle NewInput = NewNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));
						FNodeHandle TailNode = FindTailNodeInVariableStack(InVariableID);
						
						if (!TailNode->IsValid())
						{
							// variable stack is empty. Use connect to varialbe init node.
							TailNode = GetNodeWithID(Variable->VariableNodeID);
						}

						if (ensure(TailNode->IsValid()))
						{
							// connect new node to the last "get" node.
							FOutputHandle TailNodeOutput = TailNode->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));
							check(!TailNodeOutput->IsConnected());
							const bool bSuccess = TailNodeOutput->Connect(*NewInput);
							check(bSuccess);
						}

						// Add node ID to variable after connecting since the array
						// order of node ids is used to determine whether a node 
						// is the tail node.
						Variable->AccessorNodeIDs.Add(NewNode->GetID());
					}

					return NewNode;
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Could not find registered \"get variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FNodeHandle FGraphController::AddVariableDeferredAccessorNode(const FGuid& InVariableID)
		{
			using namespace VariableNames;

			if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
			{
				FMetasoundFrontendClass NodeClass;
				if (IDataTypeRegistry::Get().GetFrontendVariableDeferredAccessorClass(Variable->TypeName, NodeClass))
				{
					FNodeHandle NewNode = AddNode(NodeClass.Metadata, FGuid::NewGuid());

					if (NewNode->IsValid())
					{
						// Connect new node.
						FOutputHandle NewNodeOutput = NewNode->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));
						FNodeHandle HeadNode = FindHeadNodeInVariableStack(InVariableID);
						if (HeadNode->IsValid())
						{
							FInputHandle HeadNodeInput = HeadNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));
							const bool bSuccess = HeadNodeInput->Connect(*NewNodeOutput);
							check(bSuccess);
						}

						FInputHandle NewNodeInput = NewNode->GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));
						FNodeHandle VariableNode = GetNodeWithID(Variable->VariableNodeID);
						if (ensure(VariableNode->IsValid()))
						{
							FOutputHandle VariableNodeOutput = VariableNode->GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));
							const bool bSuccess = VariableNodeOutput->Connect(*NewNodeInput);
							check(bSuccess);
						}

						// Add node ID to variable after connecting since the array
						// order of node ids is used to determine whether a node 
						// is the tail node.
						Variable->DeferredAccessorNodeIDs.Add(NewNode->GetID());
					}

					return NewNode;
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Could not find registered \"get variable\" node class for data type \"%ls\"", *Variable->TypeName.ToString());
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FMetasoundFrontendVariable* FGraphController::FindFrontendVariable(const FGuid& InVariableID)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsVariableWithID = [&](const FMetasoundFrontendVariable& InVariable)
				{
					return InVariable.ID == InVariableID;
				};
				return FindBuildGraphChecked(*GraphClass).Variables.FindByPredicate(IsVariableWithID);
			}
			return nullptr;
		}

		const FMetasoundFrontendVariable* FGraphController::FindFrontendVariable(const FGuid& InVariableID) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsVariableWithID = [&](const FMetasoundFrontendVariable& InVariable)
				{
					return InVariable.ID == InVariableID;
				};
				const TArray<FMetasoundFrontendVariable>& Variables = FindConstBuildGraphChecked(*GraphClass).Variables;
				return Variables.FindByPredicate(IsVariableWithID);
			}
			return nullptr;
		}

		FGuid FGraphController::FindVariableIDOfVariableContainingNode(const FGuid& InNodeID) const
		{
			if (InNodeID.IsValid())
			{
				if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					auto ContainsNodeWithID = [&InNodeID](const FMetasoundFrontendVariable& InVariable)
					{
						return (InNodeID == InVariable.VariableNodeID) || (InNodeID == InVariable.MutatorNodeID) || InVariable.AccessorNodeIDs.Contains(InNodeID) || InVariable.DeferredAccessorNodeIDs.Contains(InNodeID);

					};
					const TArray<FMetasoundFrontendVariable>& Variables = FindConstBuildGraphChecked(*GraphClass).Variables;
					if (const FMetasoundFrontendVariable* Variable = Variables.FindByPredicate(ContainsNodeWithID))
					{
						return Variable->ID;
					}
				}
			}

			return FGuid();
		}

		FNodeHandle FGraphController::FindHeadNodeInVariableStack(const FGuid& InVariableID)
		{
			// The variable "stack" is [GetDelayedNodes, SetNode, GetNodes].
			if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
			{
				if (Variable->DeferredAccessorNodeIDs.Num() > 0)
				{
					return GetNodeWithID(Variable->DeferredAccessorNodeIDs[0]);
				}

				if (FrontendInvalidID != Variable->MutatorNodeID)
				{
					return GetNodeWithID(Variable->MutatorNodeID);
				}

				if (Variable->AccessorNodeIDs.Num() > 0)
				{
					return GetNodeWithID(Variable->AccessorNodeIDs[0]);
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FNodeHandle FGraphController::FindTailNodeInVariableStack(const FGuid& InVariableID)
		{
			// The variable "stack" is [GetDelayedNodes, SetNode, GetNodes].
			if (FMetasoundFrontendVariable* Variable = FindFrontendVariable(InVariableID))
			{
				if (Variable->AccessorNodeIDs.Num() > 0)
				{
					return GetNodeWithID(Variable->AccessorNodeIDs.Last());
				}

				if (FrontendInvalidID != Variable->MutatorNodeID)
				{
					return GetNodeWithID(Variable->MutatorNodeID);
				}

				if (Variable->DeferredAccessorNodeIDs.Num() > 0)
				{
					return GetNodeWithID(Variable->DeferredAccessorNodeIDs.Last());
				}
			}

			return INodeController::GetInvalidHandle();
		}

		void FGraphController::RemoveNodeIDFromAssociatedVariable(const INodeController& InNode)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const FGuid NodeID = InNode.GetID();
				
				for (FMetasoundFrontendVariable& Variable : FindBuildGraphChecked(*GraphClass).Variables)
				{
					if (NodeID == Variable.VariableNodeID)
					{
						Variable.VariableNodeID = FGuid();
						break;
					}

					if (NodeID == Variable.MutatorNodeID)
					{
						Variable.MutatorNodeID = FGuid();
					}

					if (Variable.AccessorNodeIDs.Remove(NodeID) > 0)
					{
						break;
					}

					if (Variable.DeferredAccessorNodeIDs.Remove(NodeID) > 0)
					{
						break;
					}
				}
			}
		}

		void FGraphController::SpliceVariableNodeFromVariableStack(INodeController& InNode)
		{
			// Variable nodes are organized in a stack to ensure that variables are
			// accessed in a consistent manner at runtime. A single variable object is 
			// shared amongst all nodes associated with a single variable. The variable
			// object is shared by daisy-chaining the variable from one node to the
			// next. 
			//
			// If a node is removed, that daisy-chain must be preserved. This function 
			// removes a node while maintaining the daisy-chain.
			check(InNode.IsValid());

			using namespace VariableNames;

			FInputHandle InputToSpliceOut = InNode.GetInputWithVertexName(METASOUND_GET_PARAM_NAME(InputVariable));
			FOutputHandle OutputToReroute = InputToSpliceOut->GetConnectedOutput();

			if (OutputToReroute->IsValid())
			{
				FOutputHandle OutputToSpliceOut = InNode.GetOutputWithVertexName(METASOUND_GET_PARAM_NAME(OutputVariable));
				for (const FInputHandle& InputToReroute : OutputToSpliceOut->GetConnectedInputs())
				{
					ensure(InputToReroute->Connect(*OutputToReroute));
				}
			}
		}

		void FGraphController::ClearGraph()
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendGraph& Graph = FindBuildGraphChecked(*GraphClass);
				Graph.Nodes.Reset();
				Graph.Edges.Reset();
				GraphClass->GetDefaultInterface().Inputs.Reset();
				GraphClass->GetDefaultInterface().Outputs.Reset();

				OwningDocument->ClearInterfaceVersions();
				OwningDocument->RemoveUnreferencedDependencies();

#if WITH_EDITORONLY_DATA
				FDocumentAccessPtr DocumentPtr = OwningDocument->GetDocumentPtr();
				FMetasoundFrontendDocument* Document = DocumentPtr.Get();
				check(Document);
				Document->Metadata.MemberMetadata.Reset();

				Graph.Style.EdgeStyles.Reset();
#endif // WITH_EDITORONLY_DATA
			}
		}

		void FGraphController::IterateNodes(TFunctionRef<void(FNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendNode>& Nodes = FindConstBuildGraphChecked(*GraphClass).Nodes;
				for (const FMetasoundFrontendNode& Node : Nodes)
				{
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);
					if (const FMetasoundFrontendClass* NodeClass = NodeClassPtr.Get())
					{
						if (InClassType == EMetasoundFrontendClassType::Invalid || NodeClass->Metadata.GetType() == InClassType)
						{
							FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							FNodeHandle NodeHandle = GetNodeHandle(FGraphController::FNodeAndClass{ NodePtr, NodeClassPtr });
							InFunction(NodeHandle);
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}
		}

		void FGraphController::IterateConstNodes(TFunctionRef<void(FConstNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType) const
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendNode>& Nodes = FindConstBuildGraphChecked(*GraphClass).Nodes;
				for (const FMetasoundFrontendNode& Node : Nodes)
				{
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);
					if (const FMetasoundFrontendClass* NodeClass = NodeClassPtr.Get())
					{
						if (InClassType == EMetasoundFrontendClassType::Invalid || NodeClass->Metadata.GetType() == InClassType)
						{
							FConstNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							FConstNodeHandle NodeHandle = GetNodeHandle(FGraphController::FConstNodeAndClass{ NodePtr, NodeClassPtr });
							InFunction(NodeHandle);
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}
		}

		TArray<FConstNodeHandle> FGraphController::GetConstOutputNodes() const
		{
			auto IsOutputNode = [](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node) 
			{
				return NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Output;
			};
			return GetNodesByPredicate(IsOutputNode);
		}

		TArray<FConstNodeHandle> FGraphController::GetConstInputNodes() const
		{
			auto IsInputNode = [](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node) 
			{
				return NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Input;
			};
			return GetNodesByPredicate(IsInputNode);
		}

		bool FGraphController::ContainsOutputVertex(const FVertexName& InName, const FName& InTypeName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsOutputVertexWithSameNameAndType = [&](const FMetasoundFrontendClassOutput& ClassOutput)
				{
					return ClassOutput.Name == InName && ClassOutput.TypeName == InTypeName;
				};
				return GraphClass->GetDefaultInterface().Outputs.ContainsByPredicate(IsOutputVertexWithSameNameAndType);
			}
			return false;
		}

		bool FGraphController::ContainsOutputVertexWithName(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsOutputVertexWithSameName = [&](const FMetasoundFrontendClassOutput& ClassOutput)
				{
					return ClassOutput.Name == InName;
				};
				return GraphClass->GetDefaultInterface().Outputs.ContainsByPredicate(IsOutputVertexWithSameName);
			}
			return false;
		}

		bool FGraphController::ContainsInputVertex(const FVertexName& InName, const FName& InTypeName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsInputVertexWithSameNameAndType = [&](const FMetasoundFrontendClassInput& ClassInput)
				{
					return ClassInput.Name == InName && ClassInput.TypeName == InTypeName;
				};
				return GraphClass->GetDefaultInterface().Inputs.ContainsByPredicate(IsInputVertexWithSameNameAndType);
			}
			return false;
		}

		bool FGraphController::ContainsInputVertexWithName(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsInputVertexWithSameName = [&](const FMetasoundFrontendClassInput& ClassInput)
				{ 
					return ClassInput.Name == InName;
				};
				return GraphClass->GetDefaultInterface().Inputs.ContainsByPredicate(IsInputVertexWithSameName);
			}
			return false;
		}

		FConstNodeHandle FGraphController::GetOutputNodeWithName(const FVertexName& InName) const
		{
			auto IsOutputNodeWithSameName = [&](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node)
			{
				return (NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Output) && (Node.Name == InName);
			};
			return GetNodeByPredicate(IsOutputNodeWithSameName);
		}

		FConstNodeHandle FGraphController::GetInputNodeWithName(const FVertexName& InName) const
		{
			auto IsInputNodeWithSameName = [&](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node)
			{
				return (NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Input) && (Node.Name == InName);
			};
			return GetNodeByPredicate(IsInputNodeWithSameName);
		}

		FNodeHandle FGraphController::GetOutputNodeWithName(const FVertexName& InName)
		{
			auto IsOutputNodeWithSameName = [&](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node)
			{
				return (NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Output) && (Node.Name == InName);
			};
			return GetNodeByPredicate(IsOutputNodeWithSameName);
		}

		FNodeHandle FGraphController::GetInputNodeWithName(const FVertexName& InName)
		{
			auto IsInputNodeWithSameName = [&](const FMetasoundFrontendClass& NodeClass, const FMetasoundFrontendNode& Node)
			{
				return (NodeClass.Metadata.GetType() == EMetasoundFrontendClassType::Input) && (Node.Name == InName);
			};
			return GetNodeByPredicate(IsInputNodeWithSameName);
		}

		FNodeHandle FGraphController::AddInputVertex(const FMetasoundFrontendClassInput& InClassInput)
		{
			using FRegistry = INodeClassRegistry;

			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsInputWithSameName = [&](const FMetasoundFrontendClassInput& ExistingDesc) { return ExistingDesc.Name == InClassInput.Name; };
				if (Algo::NoneOf(GraphClass->GetDefaultInterface().Inputs, IsInputWithSameName))
				{
					FNodeRegistryKey Key;
					if (FRegistry::GetInputNodeRegistryKeyForDataType(InClassInput.TypeName, InClassInput.AccessType, Key))
					{
						FConstClassAccessPtr InputClassPtr = OwningDocument->FindOrAddClass(Key);
						if (const FMetasoundFrontendClass* InputClass = InputClassPtr.Get())
						{
							const FVertexName NewName = InClassInput.Name;

							TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration = FRegistry::Get()->CreateFrontendNodeConfiguration(Key);
							// Setup input node
							FMetasoundFrontendNode& Node = FindBuildGraphChecked(*GraphClass).Nodes.Emplace_GetRef(*InputClass, MoveTemp(Configuration));
							Node.Name = NewName;

							FGuid NodeID = InClassInput.NodeID;
							if (!NodeID.IsValid())
							{
								FDocumentAccessPtr DocumentPtr = OwningDocument->GetDocumentPtr();
								const FMetasoundFrontendDocument* Document = DocumentPtr.Get();
								check(Document);
								NodeID = FDocumentIDGenerator::Get().CreateNodeID(*Document);
							}
							Node.UpdateID(NodeID);

							// Set name on related vertices of input node
							auto IsVertexWithTypeName = [&](FMetasoundFrontendVertex& Vertex) { return Vertex.TypeName == InClassInput.TypeName; };
							if (FMetasoundFrontendVertex* InputVertex = Node.Interface.Inputs.FindByPredicate(IsVertexWithTypeName))
							{
								InputVertex->Name = NewName;
							}
							else
							{
								UE_LOGF(LogMetaSound, Error, "Input node [TypeName:%ls] does not contain input vertex with type [TypeName:%ls]", *InClassInput.TypeName.ToString(), *InClassInput.TypeName.ToString());
							}

							if (FMetasoundFrontendVertex* OutputVertex = Node.Interface.Outputs.FindByPredicate(IsVertexWithTypeName))
							{
								OutputVertex->Name = NewName;
							}
							else
							{
								UE_LOGF(LogMetaSound, Error, "Input node [TypeName:%ls] does not contain output vertex with type [TypeName:%ls]", *InClassInput.TypeName.ToString(), *InClassInput.TypeName.ToString());
							}

							// Add input to this graph class interface
							FMetasoundFrontendClassInput& NewInput = GraphClass->GetDefaultInterface().Inputs.Add_GetRef(InClassInput);

							NewInput.NodeID = Node.GetID();
							if (!NewInput.VertexID.IsValid())
							{
								// Create a new guid if there wasn't a valid guid attached
								// to input.
								FDocumentAccessPtr DocumentPtr = OwningDocument->GetDocumentPtr();
								const FMetasoundFrontendDocument* Document = DocumentPtr.Get();
								check(Document);
								NewInput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(*Document);
							}

#if WITH_EDITOR
							if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
							{
								DocMetadata->ModifyContext.AddMemberIDModified(NewInput.NodeID);
							}
#endif // WITH_EDITOR
							GraphClass->GetDefaultInterface().UpdateChangeID();

							FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							return GetNodeHandle(FGraphController::FNodeAndClass { NodePtr, InputClassPtr });
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Display, "Failed to add input. No input node registered for data type [TypeName:%ls]", *InClassInput.TypeName.ToString());
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Display, "Failed to add input. Input with same name \"%ls\" exists in class [ClassID:%ls]", *InClassInput.Name.ToString(), *GraphClass->ID.ToString());
				}
			}
			return INodeController::GetInvalidHandle();
		}

		bool FGraphController::RemoveInputVertex(const FVertexName& InName)
		{
			auto IsInputNodeWithSameName = [&](const FMetasoundFrontendClass& InClass, const FMetasoundFrontendNode& InNode)
			{
				return InClass.Metadata.GetType() == EMetasoundFrontendClassType::Input && InNode.Name == InName;
			};
			const TArray<FGraphController::FNodeAndClass> NodeAndClassPairs = GetNodesAndClassesByPredicate(IsInputNodeWithSameName);

			for (const FNodeAndClass& NodeAndClass : NodeAndClassPairs)
			{
				if (const FMetasoundFrontendNode* Node = NodeAndClass.Node.Get())
				{
					return RemoveInput(*Node);
				}
			}

			return false;
		}

		FNodeHandle FGraphController::AddOutputVertex(const FVertexName& InName, const FName InTypeName)
		{
			const FGuid VertexID = FGuid::NewGuid();

			FMetasoundFrontendClassOutput Description;

			Description.Name = InName;
			Description.TypeName = InTypeName;
			Description.VertexID = VertexID;

			return AddOutputVertex(Description);
		}

		FNodeHandle FGraphController::AddOutputVertex(const FMetasoundFrontendClassOutput& InClassOutput)
		{
			using FRegistry = INodeClassRegistry;

			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsOutputWithSameName = [&](const FMetasoundFrontendClassOutput& ExistingDesc) { return ExistingDesc.Name == InClassOutput.Name; };
				if (Algo::NoneOf(GraphClass->GetDefaultInterface().Outputs, IsOutputWithSameName))
				{
					FNodeRegistryKey Key;
					if (FRegistry::GetOutputNodeRegistryKeyForDataType(InClassOutput.TypeName, InClassOutput.AccessType, Key))
					{
						FConstClassAccessPtr OutputClassPtr = OwningDocument->FindOrAddClass(Key);
						if (const FMetasoundFrontendClass* OutputClass = OutputClassPtr.Get())
						{
							const FVertexName NewName = InClassOutput.Name;

							TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration = FRegistry::Get()->CreateFrontendNodeConfiguration(Key);

							FMetasoundFrontendNode& Node = FindBuildGraphChecked(*GraphClass).Nodes.Emplace_GetRef(*OutputClass, MoveTemp(Configuration));
							Node.Name = NewName;

							FGuid NodeID = InClassOutput.NodeID;
							if (!NodeID.IsValid())
							{
								FDocumentAccessPtr DocumentPtr = OwningDocument->GetDocumentPtr();
								const FMetasoundFrontendDocument* Document = DocumentPtr.Get();
								check(Document);
								NodeID = FDocumentIDGenerator::Get().CreateNodeID(*Document);
							}
							Node.UpdateID(NodeID);

							// Set vertex name on output node
							auto IsVertexWithTypeName = [&](FMetasoundFrontendVertex& Vertex) { return Vertex.TypeName == InClassOutput.TypeName; };
							if (FMetasoundFrontendVertex* InputVertex = Node.Interface.Inputs.FindByPredicate(IsVertexWithTypeName))
							{
								InputVertex->Name = NewName;
							}
							else
							{
								UE_LOGF(LogMetaSound, Error, "Output node [TypeName:%ls] does not contain input vertex with type [TypeName:%ls]", *InClassOutput.TypeName.ToString(), *InClassOutput.TypeName.ToString());
							}

							if (FMetasoundFrontendVertex* OutputVertex = Node.Interface.Outputs.FindByPredicate(IsVertexWithTypeName))
							{
								OutputVertex->Name = NewName;
							}
							else
							{
								UE_LOGF(LogMetaSound, Error, "Output node [TypeName:%ls] does not contain output vertex with type [TypeName:%ls]", *InClassOutput.TypeName.ToString(), *InClassOutput.TypeName.ToString());
							}

							// Add output to graph interface
							FMetasoundFrontendClassOutput& NewOutput = GraphClass->GetDefaultInterface().Outputs.Add_GetRef(InClassOutput);
							
							// Setup new output
							NewOutput.NodeID = Node.GetID();
							if (!NewOutput.VertexID.IsValid())
							{
								// Create a new guid if there wasn't a valid guid attached
								// to output.
								FDocumentAccessPtr DocumentPtr = OwningDocument->GetDocumentPtr();
								const FMetasoundFrontendDocument* Document = DocumentPtr.Get();
								check(Document);
								NewOutput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(*Document);
							}

#if WITH_EDITOR
							if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
							{
								DocMetadata->ModifyContext.AddMemberIDModified(NewOutput.NodeID);
							}
#endif // WITH_EDITOR

							// Mark interface as changed.
							GraphClass->GetDefaultInterface().UpdateChangeID();

							FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							return GetNodeHandle(FGraphController::FNodeAndClass { NodePtr, OutputClassPtr });
						}
					}
					else 
					{
						UE_LOGF(LogMetaSound, Display, "Failed to add output. No output node registered for data type [TypeName:%ls]", *InClassOutput.TypeName.ToString());
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Display, "Failed to add output. Output with same name \"%ls\" exists in class [ClassID:%ls]", *InClassOutput.Name.ToString(), *GraphClass->ID.ToString());
				}
			}

			return INodeController::GetInvalidHandle();
		}

		bool FGraphController::RemoveOutputVertex(const FVertexName& InName)
		{
			auto IsOutputNodeWithSameName = [&](const FMetasoundFrontendClass& InClass, const FMetasoundFrontendNode& InNode)
			{
				return InClass.Metadata.GetType() == EMetasoundFrontendClassType::Output
					&& InNode.Name == InName;
			};
			const TArray<FGraphController::FNodeAndClass> NodeAndClassPairs = GetNodesAndClassesByPredicate(IsOutputNodeWithSameName);

			for (const FNodeAndClass& NodeAndClass : NodeAndClassPairs)
			{
				if (const FMetasoundFrontendNode* Node = NodeAndClass.Node.Get())
				{
					return RemoveOutput(*Node);
				}
			}

			return false;
		}

		// This can be used to determine what kind of property editor we should use for the data type of a given input.
		// Will return Invalid if the input couldn't be found, or if the input doesn't support any kind of literals.
		ELiteralType FGraphController::GetPreferredLiteralTypeForInputVertex(const FVertexName& InInputName) const
		{
			if (const FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InInputName))
			{
				return IDataTypeRegistry::Get().GetDesiredLiteralType(Desc->TypeName);
			}
			return ELiteralType::Invalid;
		}

		// For inputs whose preferred literal type is UObject or UObjectArray, this can be used to determine the UObject corresponding to that input's datatype.
		UClass* FGraphController::GetSupportedClassForInputVertex(const FVertexName& InInputName)
		{
			if (const FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InInputName))
			{
				return IDataTypeRegistry::Get().GetUClassForDataType(Desc->TypeName);
			}
			return nullptr;
		}

		FMetasoundFrontendLiteral FGraphController::GetDefaultInput(const FGuid& InVertexID) const
		{
			TArray<FGuid> PageOrder;
			if (IDocumentBuilderRegistry* DocRegistry = IDocumentBuilderRegistry::Get())
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				PageOrder = DocRegistry->GetPageOrder();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			return GetDefaultInput(InVertexID, PageOrder);
		}

		FMetasoundFrontendLiteral FGraphController::GetDefaultInput(const FGuid& InVertexID, TArrayView<const FGuid> InPageOrder) const
		{
			if (const FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithVertexID(InVertexID))
			{
				if (const FMetasoundFrontendClassInputDefault* Default = FindPreferredPage(Desc->GetDefaults(), InPageOrder))
				{
					return Default->Literal;
				}
			}
			return FMetasoundFrontendLiteral{};
		}

		bool FGraphController::SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral)
		{
			TArray<FGuid> PageOrder;
			if (IDocumentBuilderRegistry* DocRegistry = IDocumentBuilderRegistry::Get())
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				PageOrder = DocRegistry->GetPageOrder();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			return SetDefaultInput(InVertexID, InLiteral, PageOrder);
		}

		bool FGraphController::SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, TArrayView<const FGuid> InPageOrder)
		{
			if (FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithVertexID(InVertexID))
			{
				if (ensure(IDataTypeRegistry::Get().IsLiteralTypeSupported(Desc->TypeName, InLiteral.GetType())))
				{
					if (const FMetasoundFrontendClassInputDefault* Default = FindPreferredPage(Desc->GetDefaults(), InPageOrder))
					{
						Desc->FindDefaultChecked(Default->PageID) = InLiteral;
						return true;
					}
				}
				else
				{
					SetDefaultInputToDefaultLiteralOfType(InVertexID);
				}
			}

			return false;
		}

		bool FGraphController::SetDefaultInputToDefaultLiteralOfType(const FGuid& InVertexID)
		{
			if (FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithVertexID(InVertexID))
			{
				Metasound::FLiteral Literal = IDataTypeRegistry::Get().CreateDefaultLiteral(Desc->TypeName);
				FMetasoundFrontendLiteral& DefaultLiteral = Desc->FindDefaultChecked(Frontend::DefaultPageID);
				DefaultLiteral.SetFromLiteral(Literal);
				return DefaultLiteral.IsValid();
			}

			return false;
		}

#if WITH_EDITOR
		const FText& FGraphController::GetInputDescription(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InName))
			{
				return Desc->Metadata.GetDescription();
			}

			return FText::GetEmpty();
		}

		const FText& FGraphController::GetOutputDescription(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendClassOutput* Desc = FindOutputDescriptionWithName(InName))
			{
				return Desc->Metadata.GetDescription();
			}

			return FText::GetEmpty();
		}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
		int32 FGraphController::GetSortOrderIndexForInput(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InName))
			{
				return Desc->Metadata.SortOrderIndex;
			}

			return 0;
		}

		int32 FGraphController::GetSortOrderIndexForOutput(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendClassOutput* Desc = FindOutputDescriptionWithName(InName))
			{
				return Desc->Metadata.SortOrderIndex;
			}

			return 0;
		}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

		void FGraphController::SetInputDisplayName(const FVertexName& InName, const FText& InDisplayName)
		{
			if (FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InName))
			{
				Desc->Metadata.SetDisplayName(InDisplayName);
				if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					for (const FMetasoundFrontendClassInput& Input : GraphClass->GetDefaultInterface().Inputs)
					{
						if (Input.Name == InName)
						{
							if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
							{
								if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
								{
									const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs.Find(Input);
									FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
									DocumentDelegates.InterfaceDelegates.OnInputDisplayNameChanged.Broadcast(Index);
								}
							}
						}
					}
				}
			}
		}

		void FGraphController::SetOutputDisplayName(const FVertexName& InName, const FText& InDisplayName)
		{
			if (FMetasoundFrontendClassOutput* Desc = FindOutputDescriptionWithName(InName))
			{
				Desc->Metadata.SetDisplayName(InDisplayName);
				if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					for (const FMetasoundFrontendClassOutput& Output : GraphClass->GetDefaultInterface().Outputs)
					{
						if (Output.Name == InName)
						{
							if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
							{
								if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
								{
									const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Outputs.Find(Output);
									FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
									DocumentDelegates.InterfaceDelegates.OnOutputDisplayNameChanged.Broadcast(Index);
								}
							}
						}
					}
				}
			}
		}

		void FGraphController::SetInputDescription(const FVertexName& InName, const FText& InDescription)
		{
			if (FMetasoundFrontendClassInput* Desc = FindInputDescriptionWithName(InName))
			{
				Desc->Metadata.SetDescription(InDescription);
				if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					for (const FMetasoundFrontendClassInput& Input : GraphClass->GetDefaultInterface().Inputs)
					{
						if (Input.Name == InName)
						{
							if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
							{
								if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
								{
									const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs.Find(Input);
									FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
									DocumentDelegates.InterfaceDelegates.OnInputDescriptionChanged.Broadcast(Index);
								}
							}
						}
					}
				}
			}
		}

		void FGraphController::SetOutputDescription(const FVertexName& InName, const FText& InDescription)
		{
			if (FMetasoundFrontendClassOutput* Desc = FindOutputDescriptionWithName(InName))
			{
				Desc->Metadata.SetDescription(InDescription);
				if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					for (const FMetasoundFrontendClassOutput& Output : GraphClass->GetDefaultInterface().Outputs)
					{
						if (Output.Name == InName)
						{
							if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
							{
								if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
								{
									const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Outputs.Find(Output);
									FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
									DocumentDelegates.InterfaceDelegates.OnOutputDescriptionChanged.Broadcast(Index);
								}
							}
						}
					}
				}
			}
		}

		void FGraphController::SetSortOrderIndexForInput(const FVertexName& InName, int32 InSortOrderIndex)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendInterfaceStyle Style = GraphClass->GetDefaultInterface().GetInputStyle();
				Style.DefaultSortOrder.Reset();
				for (FMetasoundFrontendClassInput& Input : GraphClass->GetDefaultInterface().Inputs)
				{
					if (Input.Name == InName)
					{
						Input.Metadata.SortOrderIndex = InSortOrderIndex;
						if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
						{
							if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
							{
								const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs.Find(Input);
								FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
								DocumentDelegates.InterfaceDelegates.OnInputSortOrderIndexChanged.Broadcast(Index);
							}
						}
					}
					Style.DefaultSortOrder.Add(Input.Metadata.SortOrderIndex);
				}
				GraphClass->GetDefaultInterface().SetInputStyle(Style);
			}
		}

		void FGraphController::SetSortOrderIndexForOutput(const FVertexName& InName, int32 InSortOrderIndex)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendInterfaceStyle Style = GraphClass->GetDefaultInterface().GetOutputStyle();
				Style.DefaultSortOrder.Reset();
				for (FMetasoundFrontendClassOutput& Output : GraphClass->GetDefaultInterface().Outputs)
				{
					if (Output.Name == InName)
					{
						Output.Metadata.SortOrderIndex = InSortOrderIndex;
						if (const IDocumentBuilderRegistry* BuilderRegistry = IDocumentBuilderRegistry::Get())
						{
							if (FMetaSoundFrontendDocumentBuilder* Builder = BuilderRegistry->FindBuilder(GraphClass->Metadata.GetClassName(), { }))
							{
								const int32 Index = Builder->GetConstDocumentChecked().RootGraph.GetDefaultInterface().Outputs.Find(Output);
								FDocumentModifyDelegates& DocumentDelegates = Builder->GetDocumentDelegates();
								DocumentDelegates.InterfaceDelegates.OnOutputSortOrderIndexChanged.Broadcast(Index);
							}
						}
					}
					Style.DefaultSortOrder.Add(Output.Metadata.SortOrderIndex);
				}
				GraphClass->GetDefaultInterface().SetOutputStyle(MoveTemp(Style));
			}
		}
#endif // WITH_EDITOR

		FNodeHandle FGraphController::AddNode(const FNodeRegistryKey& InKey, FGuid InNodeGuid)
		{
			FConstClassAccessPtr Class = OwningDocument->FindOrAddClass(InKey);
			const bool bIsValidClass = (nullptr != Class.Get());

			if (bIsValidClass)
			{
				return AddNode(Class, InNodeGuid);
			}

			UE_LOGF(LogMetaSound, Warning, "Failed to find or add node class info with registry key [Key:%ls]", *InKey.ToString());
			return INodeController::GetInvalidHandle();
		}

		FNodeHandle FGraphController::AddNode(const FMetasoundFrontendClassMetadata& InClassMetadata, FGuid InNodeGuid)
		{
			ensureAlwaysMsgf(InClassMetadata.GetType() != EMetasoundFrontendClassType::Template,
				TEXT("Cannot implement '%s' template node using 'AddNode'. Template nodes must always "
				"be added using AddTemplateNode function and supply the interface to be implemented"),
				*InClassMetadata.GetClassName().ToString());
			return AddNode(FNodeRegistryKey(InClassMetadata), InNodeGuid);
		}

		FNodeHandle FGraphController::AddTemplateNode(const INodeTemplate& InTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeGuid)
		{
			const FNodeRegistryKey Key(InTemplate.GetFrontendClass().Metadata);
			FConstClassAccessPtr Class = OwningDocument->FindOrAddClass(Key);
			const bool bIsValidClass = (nullptr != Class.Get());

			if (bIsValidClass)
			{
				if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
				{
					if (const FMetasoundFrontendClass* NodeClass = Class.Get())
					{
						FMetasoundFrontendNode& Node = FindBuildGraphChecked(*GraphClass).Nodes.Emplace_GetRef(*NodeClass, InTemplate.CreateFrontendTemplateNodeConfiguration());
						Node.UpdateID(InNodeGuid);
						Node.Interface = InTemplate.GenerateNodeInterface(MoveTemp(Params));
						FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
						return GetNodeHandle(FGraphController::FNodeAndClass{ NodePtr, Class });
					}
				}
			}

			UE_LOGF(LogMetaSound, Warning, "Failed to find or add node template class info with registry key [Key:%ls]", *Key.ToString());
			return INodeController::GetInvalidHandle();
		}

		FNodeHandle FGraphController::AddDuplicateNode(const INodeController& InNode)
		{
			// TODO: will need to copy node interface when dynamic pins exist.
			const FMetasoundFrontendClassMetadata& ClassMetadata = InNode.GetClassMetadata();

			FConstClassAccessPtr ClassPtr;

			if (EMetasoundFrontendClassType::Graph == ClassMetadata.GetType())
			{
				// Add subgraph and dependencies if needed
				ClassPtr = OwningDocument->FindClass(ClassMetadata);
				const bool bIsClassMissing = (nullptr == ClassPtr.Get());

				if (bIsClassMissing)
				{
					// Class does not exist, need to add the subgraph
					OwningDocument->AddDuplicateSubgraph(*(InNode.AsGraph()));
					ClassPtr = OwningDocument->FindClass(ClassMetadata);
				}
			}
			else
			{
				ClassPtr = OwningDocument->FindOrAddClass(ClassMetadata);
			}

			return AddNode(ClassPtr, FGuid::NewGuid());
		}

		// Remove the node corresponding to this node handle.
		// On success, invalidates the received node handle.
		bool FGraphController::RemoveNode(INodeController& InNode)
		{
			const EMetasoundFrontendClassType NodeClassType = InNode.GetClassMetadata().GetType();
			const bool bIsVariableNodeClassType = (NodeClassType == EMetasoundFrontendClassType::Variable) 
				|| (NodeClassType == EMetasoundFrontendClassType::VariableAccessor) 
				|| (NodeClassType == EMetasoundFrontendClassType::VariableDeferredAccessor) 
				|| (NodeClassType == EMetasoundFrontendClassType::VariableMutator);

			if (bIsVariableNodeClassType)
			{
				// Variables hold on to related node IDs. These need to be removed
				// from the variable definition. 
				RemoveNodeIDFromAssociatedVariable(InNode);

				// Variable nodes of the same variable are connected serially. 
				// Special care is taken to ensure the stack is connected when
				// removing a node from the stack.
				SpliceVariableNodeFromVariableStack(InNode);

			}
			
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FGuid NodeID = InNode.GetID();
				auto IsNodeWithSameID = [&](const FMetasoundFrontendNode& InFrontendNode) { return InFrontendNode.GetID() == NodeID; };
				const TArray<FMetasoundFrontendNode>& Nodes = FindConstBuildGraphChecked(*GraphClass).Nodes;
				if (const FMetasoundFrontendNode* FrontendNode = Nodes.FindByPredicate(IsNodeWithSameID))
				{
					switch(NodeClassType)
					{
						case EMetasoundFrontendClassType::Input:
						{
							return RemoveInput(*FrontendNode);
						}

						case EMetasoundFrontendClassType::Output:
						{
							return RemoveOutput(*FrontendNode);
						}

						case EMetasoundFrontendClassType::Variable:
						case EMetasoundFrontendClassType::VariableAccessor:
						case EMetasoundFrontendClassType::VariableDeferredAccessor:
						case EMetasoundFrontendClassType::VariableMutator:
							// TODO: remove node from variable.7
						case EMetasoundFrontendClassType::Literal:
						case EMetasoundFrontendClassType::External:
						case EMetasoundFrontendClassType::Template:
						case EMetasoundFrontendClassType::Graph:
						{
							return RemoveNode(*FrontendNode);
						}

						default:
						case EMetasoundFrontendClassType::Invalid:
						{
							static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing switch case coverage for EMetasoundFrontendClassType.");
							checkNoEntry();
						}
					}
				}
			}

			return false;
		}

		const FMetasoundFrontendClassMetadata& FGraphController::GetGraphMetadata() const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->Metadata;
			}
			return Invalid::GetInvalidClassMetadata();
		}

		void FGraphController::SetGraphMetadata(const FMetasoundFrontendClassMetadata& InMetadata)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				GraphClass->Metadata = InMetadata;
			}
		}

		FNodeHandle FGraphController::CreateEmptySubgraph(const FMetasoundFrontendClassMetadata& InMetadata)
		{
			if (InMetadata.GetType() == EMetasoundFrontendClassType::Graph)
			{
				if (const FMetasoundFrontendClass* ExistingDependency = OwningDocument->FindClass(InMetadata).Get())
				{
					UE_LOGF(LogMetaSound, Error, "Cannot add new subgraph. Metasound class already exists with matching metadata Name: \"%ls\", Version %d.%d", *(ExistingDependency->Metadata.GetClassName().ToString()), ExistingDependency->Metadata.GetVersion().Major, ExistingDependency->Metadata.GetVersion().Minor);
				}
				else 
				{
					return AddNode(OwningDocument->FindOrAddClass(InMetadata), FGuid::NewGuid());
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Incompatible Metasound NodeType encountered when attempting to create an empty subgraph.  NodeType must equal EMetasoundFrontendClassType::Graph");
			}
			
			return INodeController::GetInvalidHandle();
		}

		FDocumentHandle FGraphController::GetOwningDocument()
		{
			return OwningDocument;
		}

		FConstDocumentHandle FGraphController::GetOwningDocument() const
		{
			return OwningDocument;
		}

		FNodeHandle FGraphController::AddNode(FConstClassAccessPtr InExistingDependency, FGuid InNodeGuid)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				if (const FMetasoundFrontendClass* NodeClass = InExistingDependency.Get())
				{
					TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration;
					if (NodeClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
					{
						if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FNodeClassRegistryKey(NodeClass->Metadata)))
						{
							Configuration = Template->CreateFrontendTemplateNodeConfiguration();
						}
					}
					else if (NodeClass->Metadata.GetType() != EMetasoundFrontendClassType::Graph) // Subgraphs do not have extensions
					{
						Configuration = INodeClassRegistry::Get()->CreateFrontendNodeConfiguration(FNodeClassRegistryKey(NodeClass->Metadata));
					}
					FMetasoundFrontendNode& Node = FindBuildGraphChecked(*GraphClass).Nodes.Emplace_GetRef(*NodeClass, MoveTemp(Configuration));

					// Cache the asset name on the node if it node is reference to asset-defined graph.
					// AssetManager may not exist if this is called from a build that does not load the
					// engine module (ex. unit test builds which only load frontend and core), or any build
					// where the AssetManager more generally has not been implemented.
					if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
					{
						if (NodeClass->Metadata.GetType() == EMetasoundFrontendClassType::External)
						{
							const FNodeRegistryKey RegistryKey = FNodeRegistryKey(NodeClass->Metadata);
							const FTopLevelAssetPath Path = AssetManager->FindAssetPath(RegistryKey);
							if (Path.IsValid())
							{
								Node.Name = Path.GetAssetName();
							}
						}
					}

					Node.UpdateID(InNodeGuid);
#if WITH_EDITOR
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddNodeIDModified(InNodeGuid);
					}
#endif // WITH_EDITOR
					FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
					return GetNodeHandle(FGraphController::FNodeAndClass { NodePtr, InExistingDependency });
				}
			}

			return INodeController::GetInvalidHandle();
		}

		bool FGraphController::RemoveNode(const FMetasoundFrontendNode& InDesc)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				// Remove any reference connections
				FMetasoundFrontendGraph& Graph = FindBuildGraphChecked(*GraphClass);

#if WITH_EDITORONLY_DATA
				{
					auto IsStyleForThisNode = [&](const FMetasoundFrontendEdgeStyle& EdgeStyle) { return (EdgeStyle.NodeID == InDesc.GetID()); };
					Graph.Style.EdgeStyles.RemoveAllSwap(IsStyleForThisNode);
				}
#endif // WITH_EDITORONLY_DATA

				auto IsEdgeForThisNode = [&](const FMetasoundFrontendEdge& ConDesc) { return (ConDesc.FromNodeID == InDesc.GetID()) || (ConDesc.ToNodeID == InDesc.GetID()); };
				int32 NumRemoved = Graph.Edges.RemoveAll(IsEdgeForThisNode);

				auto IsNodeWithID = [&](const FMetasoundFrontendNode& Desc) { return InDesc.GetID() == Desc.GetID(); };

				NumRemoved += Graph.Nodes.RemoveAll(IsNodeWithID);
				OwningDocument->RemoveUnreferencedDependencies();

#if WITH_EDITOR
				if (NumRemoved > 0)
				{
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddNodeIDModified(InDesc.GetID());
					}
				}	
#endif // WITH_EDITOR

				return (NumRemoved > 0);
			}
			return false;
		}

		bool FGraphController::RemoveInput(const FMetasoundFrontendNode& InNode)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsInputWithSameNodeID = [&](const FMetasoundFrontendClassInput& ClassInput) { return ClassInput.NodeID == InNode.GetID(); };

				int32 NumInputsRemoved = GraphClass->GetDefaultInterface().Inputs.RemoveAll(IsInputWithSameNodeID);

				if (NumInputsRemoved > 0)
				{
#if WITH_EDITOR
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddMemberIDModified(InNode.GetID());
					}
#endif // WITH_EDITOR
					GraphClass->GetDefaultInterface().UpdateChangeID();
				}

				bool bDidRemoveNode = RemoveNode(InNode);

				return NumInputsRemoved > 0 || bDidRemoveNode;
			}

			return false;
		}

		bool FGraphController::RemoveOutput(const FMetasoundFrontendNode& InNode)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				auto IsOutputWithSameNodeID = [&](const FMetasoundFrontendClassOutput& ClassOutput) { return ClassOutput.NodeID == InNode.GetID(); };

				int32 NumOutputsRemoved = GraphClass->GetDefaultInterface().Outputs.RemoveAll(IsOutputWithSameNodeID);

				if (NumOutputsRemoved)
				{
#if WITH_EDITOR
					if (FMetasoundFrontendDocumentMetadata* DocMetadata = OwningDocument->GetMetadata())
					{
						DocMetadata->ModifyContext.AddMemberIDModified(InNode.GetID());
					}
#endif // WITH_EDITOR
					GraphClass->GetDefaultInterface().UpdateChangeID();
				}

				bool bDidRemoveNode = RemoveNode(InNode);

				return (NumOutputsRemoved > 0) || bDidRemoveNode;
			}

			return false;
		}

		void FGraphController::UpdateInterfaceChangeID()
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				GraphClass->GetDefaultInterface().UpdateChangeID();
			}
		}

		bool FGraphController::ContainsNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				for (FMetasoundFrontendNode& Node : FindBuildGraphChecked(*GraphClass).Nodes)
				{
					const FMetasoundFrontendClass* NodeClass = OwningDocument->FindClassWithID(Node.ClassID).Get();
					if (nullptr != NodeClass)
					{
						if (InPredicate(*NodeClass, Node))
						{
							return true;
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}

			return false;
		}

		TArray<FGraphController::FNodeAndClass> FGraphController::GetNodesAndClasses()
		{
			TArray<FNodeAndClass> NodesAndClasses;

			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				for (FMetasoundFrontendNode& Node : FindBuildGraphChecked(*GraphClass).Nodes)
				{
					FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);

					const bool bIsValidNodePtr = (nullptr != NodePtr.Get());
					const bool bIsValidNodeClassPtr = (nullptr != NodeClassPtr.Get());

					if (bIsValidNodePtr && bIsValidNodeClassPtr)
					{
						NodesAndClasses.Add(FGraphController::FNodeAndClass{ NodePtr, NodeClassPtr });
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}

			return NodesAndClasses;
		}

		TArray<FGraphController::FConstNodeAndClass> FGraphController::GetNodesAndClasses() const
		{
			TArray<FConstNodeAndClass> NodesAndClasses;

			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendNode>& Nodes = FindConstBuildGraphChecked(*GraphClass).Nodes;
				for (const FMetasoundFrontendNode& Node : Nodes)
				{
					FConstNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);

					const bool bIsValidNodePtr = (nullptr != NodePtr.Get());
					const bool bIsValidNodeClassPtr = (nullptr != NodeClassPtr.Get());

					if (bIsValidNodePtr && bIsValidNodeClassPtr)
					{
						NodesAndClasses.Add(FGraphController::FConstNodeAndClass{ NodePtr, NodeClassPtr });
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}

			return NodesAndClasses;
		}

		TArray<FGraphController::FNodeAndClass> FGraphController::GetNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate)
		{
			TArray<FNodeAndClass> NodesAndClasses;

			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				for (FMetasoundFrontendNode& Node : FindBuildGraphChecked(*GraphClass).Nodes)
				{
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);
					if (const FMetasoundFrontendClass* NodeClass = NodeClassPtr.Get())
					{
						if (InPredicate(*NodeClass, Node))
						{
							FNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							NodesAndClasses.Add(FGraphController::FNodeAndClass{ NodePtr, NodeClassPtr });
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}

			return NodesAndClasses;
		}

		TArray<FGraphController::FConstNodeAndClass> FGraphController::GetNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const
		{
			TArray<FConstNodeAndClass> NodesAndClasses;

			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				const TArray<FMetasoundFrontendNode>& Nodes = FindConstBuildGraphChecked(*GraphClass).Nodes;
				for (const FMetasoundFrontendNode& Node : Nodes)
				{
					FConstClassAccessPtr NodeClassPtr = OwningDocument->FindClassWithID(Node.ClassID);
					if (const FMetasoundFrontendClass* NodeClass = NodeClassPtr.Get())
					{
						if (InPredicate(*NodeClass, Node))
						{
							FConstNodeAccessPtr NodePtr = GraphClassPtr.GetNodeWithNodeID(Node.GetID());
							NodesAndClasses.Add(FGraphController::FConstNodeAndClass{ NodePtr, NodeClassPtr });
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to find class for node [NodeID:%ls, ClassID:%ls]", *Node.GetID().ToString(), *Node.ClassID.ToString());
					}
				}
			}

			return NodesAndClasses;
		}

		FNodeHandle FGraphController::GetNodeByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate)
		{
			TArray<FNodeAndClass> NodeAndClass = GetNodesAndClassesByPredicate(InPredicate);
			if (NodeAndClass.Num() > 0)
			{
				return GetNodeHandle(NodeAndClass[0]);
			}

			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FGraphController::GetNodeByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const
		{
			TArray<FConstNodeAndClass> NodeAndClass = GetNodesAndClassesByPredicate(InPredicate);
			if (NodeAndClass.Num() > 0)
			{
				return GetNodeHandle(NodeAndClass[0]);
			}

			return INodeController::GetInvalidHandle();
		}

		TArray<FNodeHandle> FGraphController::GetNodesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InFilterFunc)
		{
			return GetNodeHandles(GetNodesAndClassesByPredicate(InFilterFunc));
		}

		TArray<FConstNodeHandle> FGraphController::GetNodesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InFilterFunc) const
		{
			return GetNodeHandles(GetNodesAndClassesByPredicate(InFilterFunc));
		}

		TArray<FNodeHandle> FGraphController::GetNodeHandles(TArrayView<const FGraphController::FNodeAndClass> InNodesAndClasses)
		{
			TArray<FNodeHandle> Nodes;

			for (const FNodeAndClass& NodeAndClass: InNodesAndClasses)
			{
				FNodeHandle NodeController = GetNodeHandle(NodeAndClass);
				if (NodeController->IsValid())
				{
					Nodes.Add(NodeController);
				}
			}

			return Nodes;
		}

		TArray<FConstNodeHandle> FGraphController::GetNodeHandles(TArrayView<const FGraphController::FConstNodeAndClass> InNodesAndClasses) const
		{
			TArray<FConstNodeHandle> Nodes;

			for (const FConstNodeAndClass& NodeAndClass : InNodesAndClasses)
			{
				FConstNodeHandle NodeController = GetNodeHandle(NodeAndClass);
				if (NodeController->IsValid())
				{
					Nodes.Add(NodeController);
				}
			}

			return Nodes;
		}

		FNodeHandle FGraphController::GetNodeHandle(const FGraphController::FNodeAndClass& InNodeAndClass)
		{
			FMetasoundFrontendNode* Node = InNodeAndClass.Node.Get();
			const FMetasoundFrontendClass* NodeClass = InNodeAndClass.Class.Get();
			
			if ((nullptr != Node) && (nullptr != NodeClass))
			{
				FGraphHandle OwningGraph = this->AsShared();
				FGraphAccessPtr GraphPtr = GraphClassPtr.GetGraph();

				switch (NodeClass->Metadata.GetType())
				{
					case EMetasoundFrontendClassType::Input:
						{
							FClassInputAccessPtr OwningGraphClassInputPtr = FindInputDescriptionWithNodeID(Node->GetID());
							if (nullptr != OwningGraphClassInputPtr.Get())
							{
								FInputNodeController::FInitParams InitParams
								{
									InNodeAndClass.Node,
									InNodeAndClass.Class,
									OwningGraphClassInputPtr,
									GraphPtr,
									OwningGraph
								};
								return FInputNodeController::CreateInputNodeHandle(InitParams);
							}
							else
							{
								// TODO: This supports input nodes introduced during subgraph inflation. Input nodes
								// should be replaced with value nodes once they are implemented. 
								FNodeController::FInitParams InitParams
								{
									InNodeAndClass.Node,
									InNodeAndClass.Class,
									GraphPtr,
									OwningGraph
								};
								return FNodeController::CreateNodeHandle(InitParams);
							}
						}

					case EMetasoundFrontendClassType::Output:
						{
							FClassOutputAccessPtr OwningGraphClassOutputPtr = FindOutputDescriptionWithNodeID(Node->GetID());
							if (nullptr != OwningGraphClassOutputPtr.Get())
							{
								FOutputNodeController::FInitParams InitParams
								{
									InNodeAndClass.Node,
									InNodeAndClass.Class,
									OwningGraphClassOutputPtr,
									GraphPtr,
									OwningGraph
								};
								return FOutputNodeController::CreateOutputNodeHandle(InitParams);
							}
							else
							{
								// TODO: This supports output nodes introduced during subgraph inflation. Output nodes
								// should be replaced with value nodes once they are implemented. 
								FNodeController::FInitParams InitParams
								{
									InNodeAndClass.Node,
									InNodeAndClass.Class,
									GraphPtr,
									OwningGraph
								};
								return FNodeController::CreateNodeHandle(InitParams);
							}
						}

					case EMetasoundFrontendClassType::Variable:
					case EMetasoundFrontendClassType::VariableAccessor:
					case EMetasoundFrontendClassType::VariableDeferredAccessor:
					case EMetasoundFrontendClassType::VariableMutator:
						{
							FVariableNodeController::FInitParams InitParams
							{
								InNodeAndClass.Node,
								InNodeAndClass.Class,
								GraphPtr,
								OwningGraph
							};
							return FVariableNodeController::CreateNodeHandle(InitParams);
						}

					case EMetasoundFrontendClassType::External:
					case EMetasoundFrontendClassType::Template:
						{
							FNodeController::FInitParams InitParams
							{
								InNodeAndClass.Node,
								InNodeAndClass.Class,
								GraphPtr,
								OwningGraph
							};
							return FNodeController::CreateNodeHandle(InitParams);
						}

					case EMetasoundFrontendClassType::Graph:
						{
							FSubgraphNodeController::FInitParams InitParams
							{
								InNodeAndClass.Node,
								InNodeAndClass.Class,
								GraphPtr,
								OwningGraph
							};
							return FSubgraphNodeController::CreateNodeHandle(InitParams);
						}

					default:
						checkNoEntry();
						static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing switch case coverage for EMetasoundFrontendClassType.");
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FGraphController::GetNodeHandle(const FGraphController::FConstNodeAndClass& InNodeAndClass) const
		{
			const FMetasoundFrontendNode* Node = InNodeAndClass.Node.Get();
			const FMetasoundFrontendClass* NodeClass = InNodeAndClass.Class.Get();
			
			if ((nullptr != Node) && (nullptr != NodeClass))
			{
				FConstGraphHandle OwningGraph = this->AsShared();
				FConstGraphAccessPtr GraphPtr = GraphClassPtr.GetGraph();

				switch (NodeClass->Metadata.GetType())
				{
					case EMetasoundFrontendClassType::Input:
						{
							FConstClassInputAccessPtr OwningGraphClassInputPtr = FindInputDescriptionWithNodeID(Node->GetID());
							if (nullptr != OwningGraphClassInputPtr.Get())
							{
								FInputNodeController::FInitParams InitParams
								{
									ConstCastAccessPtr<FNodeAccessPtr>(InNodeAndClass.Node),
									InNodeAndClass.Class,
									ConstCastAccessPtr<FClassInputAccessPtr>(OwningGraphClassInputPtr),
									ConstCastAccessPtr<FGraphAccessPtr>(GraphPtr),
									ConstCastSharedRef<IGraphController>(OwningGraph)
								};
								return FInputNodeController::CreateConstInputNodeHandle(InitParams);
							}
						}
						break;

					case EMetasoundFrontendClassType::Output:
						{
							FConstClassOutputAccessPtr OwningGraphClassOutputPtr = FindOutputDescriptionWithNodeID(Node->GetID());
							if (nullptr != OwningGraphClassOutputPtr.Get())
							{
								FOutputNodeController::FInitParams InitParams
								{
									ConstCastAccessPtr<FNodeAccessPtr>(InNodeAndClass.Node),
									InNodeAndClass.Class,
									ConstCastAccessPtr<FClassOutputAccessPtr>(OwningGraphClassOutputPtr),
									ConstCastAccessPtr<FGraphAccessPtr>(GraphPtr),
									ConstCastSharedRef<IGraphController>(OwningGraph)
								};
								return FOutputNodeController::CreateConstOutputNodeHandle(InitParams);
							}
						}
						break;

					case EMetasoundFrontendClassType::Variable:
					case EMetasoundFrontendClassType::VariableAccessor:
					case EMetasoundFrontendClassType::VariableDeferredAccessor:
					case EMetasoundFrontendClassType::VariableMutator:
						{
							FVariableNodeController::FInitParams InitParams
							{
								ConstCastAccessPtr<FNodeAccessPtr>(InNodeAndClass.Node),
								InNodeAndClass.Class,
								ConstCastAccessPtr<FGraphAccessPtr>(GraphPtr),
								ConstCastSharedRef<IGraphController>(OwningGraph)
							};
							return FVariableNodeController::CreateConstNodeHandle(InitParams);
						}

					case EMetasoundFrontendClassType::External:
					case EMetasoundFrontendClassType::Template:
						{
							FNodeController::FInitParams InitParams
							{
								ConstCastAccessPtr<FNodeAccessPtr>(InNodeAndClass.Node),
								InNodeAndClass.Class,
								ConstCastAccessPtr<FGraphAccessPtr>(GraphPtr),
								ConstCastSharedRef<IGraphController>(OwningGraph)
							};
							return FNodeController::CreateConstNodeHandle(InitParams);
						}

					case EMetasoundFrontendClassType::Graph:
						{
							FSubgraphNodeController::FInitParams InitParams
							{
								ConstCastAccessPtr<FNodeAccessPtr>(InNodeAndClass.Node),
								InNodeAndClass.Class,
								ConstCastAccessPtr<FGraphAccessPtr>(GraphPtr),
								ConstCastSharedRef<IGraphController>(OwningGraph)
							};
							return FSubgraphNodeController::CreateConstNodeHandle(InitParams);
						}

					default:
						checkNoEntry();
						static_assert(static_cast<int32>(EMetasoundFrontendClassType::Invalid) == 10, "Possible missing switch case coverage for EMetasoundFrontendClassType.");
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FMetasoundFrontendClassInput* FGraphController::FindInputDescriptionWithName(const FVertexName& InName)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendClassInput* ClassInput = GraphClass->GetDefaultInterface().Inputs.FindByPredicate([&](const FMetasoundFrontendClassInput& Desc) { return Desc.Name == InName; });
				if (ClassInput)
				{
					// TODO: This assumes the class input is being mutated due to the adjacent const correct call not being utilized.
					// Make this more explicit rather than risking whether or not the caller is using proper const correctness.
					GraphClass->GetDefaultInterface().UpdateChangeID();
					return ClassInput;
				}
			}
			return nullptr;
		}

		const FMetasoundFrontendClassInput* FGraphController::FindInputDescriptionWithName(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().Inputs.FindByPredicate([&](const FMetasoundFrontendClassInput& Desc) { return Desc.Name == InName; });
			}
			return nullptr;
		}

		FMetasoundFrontendClassOutput* FGraphController::FindOutputDescriptionWithName(const FVertexName& InName)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendClassOutput* ClassOutput = GraphClass->GetDefaultInterface().Outputs.FindByPredicate([&](const FMetasoundFrontendClassOutput& Desc) { return Desc.Name == InName; });
				if (ClassOutput)
				{
					// TODO: This assumes the class input is being mutated due to the adjacent const correct call not being utilized.
					// Make this more explicit rather than risking whether or not the caller is using proper const correctness.
					GraphClass->GetDefaultInterface().UpdateChangeID();
					return ClassOutput;
				}
			}
			return nullptr;
		}

		const FMetasoundFrontendClassOutput* FGraphController::FindOutputDescriptionWithName(const FVertexName& InName) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().Outputs.FindByPredicate([&](const FMetasoundFrontendClassOutput& Desc) { return Desc.Name == InName; });
			}
			return nullptr;
		}

		FMetasoundFrontendClassInput* FGraphController::FindInputDescriptionWithVertexID(const FGuid& InVertexID)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendClassInput* ClassInput = GraphClass->GetDefaultInterface().Inputs.FindByPredicate([&](const FMetasoundFrontendClassInput& Desc) { return Desc.VertexID == InVertexID; });
				if (ClassInput)
				{
					// TODO: This assumes the class input is being mutated due to the adjacent const correct call not being utilized.
					// Make this more explicit rather than risking whether or not the caller is using proper const correctness.
					GraphClass->GetDefaultInterface().UpdateChangeID();
					return ClassInput;
				}
			}
			return nullptr;
		}

		const FMetasoundFrontendClassInput* FGraphController::FindInputDescriptionWithVertexID(const FGuid& InVertexID) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().Inputs.FindByPredicate([&](const FMetasoundFrontendClassInput& Desc) { return Desc.VertexID == InVertexID; });
			}
			return nullptr;
		}

		FMetasoundFrontendClassOutput* FGraphController::FindOutputDescriptionWithVertexID(const FGuid& InVertexID)
		{
			if (FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				FMetasoundFrontendClassOutput* ClassOutput = GraphClass->GetDefaultInterface().Outputs.FindByPredicate([&](const FMetasoundFrontendClassOutput& Desc) { return Desc.VertexID == InVertexID; });
				if (ClassOutput)
				{
					// TODO: This assumes the class input is being mutated due to the adjacent const correct call not being utilized.
					// Make this more explicit rather than risking whether or not the caller is using proper const correctness.
					GraphClass->GetDefaultInterface().UpdateChangeID();
					return ClassOutput;
				}
			}
			return nullptr;
		}

		const FMetasoundFrontendClassOutput* FGraphController::FindOutputDescriptionWithVertexID(const FGuid& InVertexID) const
		{
			if (const FMetasoundFrontendGraphClass* GraphClass = GraphClassPtr.Get())
			{
				return GraphClass->GetDefaultInterface().Outputs.FindByPredicate([&](const FMetasoundFrontendClassOutput& Desc) { return Desc.VertexID == InVertexID; });
			}
			return nullptr;
		}

		FClassInputAccessPtr FGraphController::FindInputDescriptionWithNodeID(FGuid InNodeID)
		{
			return GraphClassPtr.GetInputWithNodeID(InNodeID);
		}

		FConstClassInputAccessPtr FGraphController::FindInputDescriptionWithNodeID(FGuid InNodeID) const
		{
			return GraphClassPtr.GetInputWithNodeID(InNodeID);
		}

		FClassOutputAccessPtr FGraphController::FindOutputDescriptionWithNodeID(FGuid InNodeID)
		{
			return GraphClassPtr.GetOutputWithNodeID(InNodeID);
		}

		FConstClassOutputAccessPtr FGraphController::FindOutputDescriptionWithNodeID(FGuid InNodeID) const
		{
			return GraphClassPtr.GetOutputWithNodeID(InNodeID);
		}

		FDocumentAccess FGraphController::ShareAccess() 
		{
			FDocumentAccess Access;

			Access.GraphClass = GraphClassPtr;
			Access.ConstGraphClass = GraphClassPtr;

			return Access;
		}

		FConstDocumentAccess FGraphController::ShareAccess() const
		{
			FConstDocumentAccess Access;

			Access.ConstGraphClass = GraphClassPtr;

			return Access;
		}
	}
}
#undef LOCTEXT_NAMESPACE
