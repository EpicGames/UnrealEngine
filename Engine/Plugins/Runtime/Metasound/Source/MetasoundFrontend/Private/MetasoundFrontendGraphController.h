// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"


// Forward Declarations
class UClass;

namespace Metasound
{
	// Forward Declarations
	class INodeTemplate;

	namespace Frontend
	{
		// Convenience functions used for finding targeted graph to mutate as the transition away from controllers
		// is on-going, however documents can now contain multiple graph topologies (i.e. paged graphs) to select from.
		const FMetasoundFrontendGraph& FindConstBuildGraphChecked(const FMetasoundFrontendGraphClass& InGraphClass);
		FMetasoundFrontendGraph& FindBuildGraphChecked(FMetasoundFrontendGraphClass& InGraphClass);

		/** FGraphController represents a Metasound graph class. */
		class FGraphController : public IGraphController
		{
			// Private token only allows members or friends to call constructor.
			enum EPrivateToken { Token };

		public:

			struct FInitParams
			{
				FGraphClassAccessPtr GraphClassPtr;
				FDocumentHandle OwningDocument;
			};

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FGraphController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a graph handle. 
			 *
			 * @return A graph handle. On error, an invalid handle is returned. 
			 */
			static FGraphHandle CreateGraphHandle(const FInitParams& InParams);

			/** Create a graph handle. 
			 *
			 * @return A graph handle. On error, an invalid handle is returned. 
			 */
			static FConstGraphHandle CreateConstGraphHandle(const FInitParams& InParams);

			virtual ~FGraphController() = default;

			bool IsValid() const override;

			FGuid GetClassID() const override;

#if WITH_EDITOR
			FText GetDisplayName() const override;
#endif // WITH_EDITOR

			TArray<FVertexName> GetInputVertexNames() const override;
			TArray<FVertexName> GetOutputVertexNames() const override;
			FConstClassInputAccessPtr FindClassInputWithName(const FVertexName& InName) const override;
			FConstClassOutputAccessPtr FindClassOutputWithName(const FVertexName& InName) const override;
			FGuid GetVertexIDForInputVertex(const FVertexName& InInputName) const override;
			FGuid GetVertexIDForOutputVertex(const FVertexName& InOutputName) const override;

			TArray<FNodeHandle> GetNodes() override;
			TArray<FConstNodeHandle> GetConstNodes() const override;

			FConstNodeHandle GetNodeWithID(FGuid InNodeID) const override;
			FNodeHandle GetNodeWithID(FGuid InNodeID) override;

			TArray<FNodeHandle> GetInputNodes() override;
			TArray<FConstNodeHandle> GetConstInputNodes() const override;

			virtual FVariableHandle AddVariable(const FName& InDataType) override;
			virtual FVariableHandle FindVariable(const FGuid& InVariableID) override;
			virtual FConstVariableHandle FindVariable(const FGuid& InVariableID) const override;
			virtual FVariableHandle FindVariableContainingNode(const FGuid& InNodeID) override;
			virtual FConstVariableHandle FindVariableContainingNode(const FGuid& InNodeID) const override;

			UE_DEPRECATED(5.8, "Use the MetaSound Builder API")
			virtual bool RemoveVariable(const FGuid& InVariableID) override;

			virtual TArray<FVariableHandle> GetVariables() override;
			virtual TArray<FConstVariableHandle> GetVariables() const override;
			virtual FNodeHandle FindOrAddVariableMutatorNode(const FGuid& InVariableID) override;
			virtual FNodeHandle AddVariableAccessorNode(const FGuid& InVariableID) override;
			virtual FNodeHandle AddVariableDeferredAccessorNode(const FGuid& InVariableID) override;

			void ClearGraph() override;

			void IterateNodes(TFunctionRef<void(FNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType) override;
			void IterateConstNodes(TFunctionRef<void(FConstNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType) const override;

			TArray<FNodeHandle> GetOutputNodes() override;
			TArray<FConstNodeHandle> GetConstOutputNodes() const override;

#if WITH_EDITOR
			virtual const FMetasoundFrontendGraphStyle& GetGraphStyle() const override;
			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const override;
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const override;
			virtual void SetGraphStyle(FMetasoundFrontendGraphStyle Style) override;
			virtual void SetInputStyle(FMetasoundFrontendInterfaceStyle Style) override;
			virtual void SetOutputStyle(FMetasoundFrontendInterfaceStyle Style) override;
#endif // WITH_EDITOR

			bool ContainsInputVertex(const FVertexName& InName, const FName& InTypeName) const override;
			bool ContainsInputVertexWithName(const FVertexName& InName) const override;
			bool ContainsOutputVertex(const FVertexName& InName, const FName& InTypeName) const override;
			bool ContainsOutputVertexWithName(const FVertexName& InName) const override;

			FConstNodeHandle GetInputNodeWithName(const FVertexName& InName) const override;
			FConstNodeHandle GetOutputNodeWithName(const FVertexName& InName) const override;

			FNodeHandle GetInputNodeWithName(const FVertexName& InName) override;
			FNodeHandle GetOutputNodeWithName(const FVertexName& InName) override;

			FNodeHandle AddInputVertex(const FMetasoundFrontendClassInput& InDescription) override;
			bool RemoveInputVertex(const FVertexName& InName) override;

			FNodeHandle AddOutputVertex(const FMetasoundFrontendClassOutput& InDescription) override;
			FNodeHandle AddOutputVertex(const FVertexName& InName, const FName InTypeName) override;
			bool RemoveOutputVertex(const FVertexName& InName) override;

			void UpdateInterfaceChangeID() override;

			// This can be used to determine what kind of property editor we should use for the data type of a given input.
			// Will return Invalid if the input couldn't be found, or if the input doesn't support any kind of literals.
			ELiteralType GetPreferredLiteralTypeForInputVertex(const FVertexName& InInputName) const override;

			// For inputs whose preferred literal type is UObject or UObjectArray, this can be used to determine the UObject corresponding to that input's datatype.
			UClass* GetSupportedClassForInputVertex(const FVertexName& InInputName) override;

			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID) const override;

			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID, TArrayView<const FGuid> InPageOrder) const override;

			// These can be used to set the default value for a given input on this graph.
			// @returns false if the input name couldn't be found, or if the literal type was incompatible with the Data Type of this input.

			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral) override;

			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, TArrayView<const FGuid> InPageOrder) override;
			bool SetDefaultInputToDefaultLiteralOfType(const FGuid& InVertexID) override;

#if WITH_EDITOR
			// Get the description for the input with the given name.
			const FText& GetInputDescription(const FVertexName& InName) const override;

			// Get the description for the output with the given name.
			const FText& GetOutputDescription(const FVertexName& InName) const override;

			// Set the description for the input with the given name
			void SetInputDescription(const FVertexName& InName, const FText& InDescription) override;

			// Set the description for the input with the given name
			void SetOutputDescription(const FVertexName& InName, const FText& InDescription) override;

			// Set the display name for the input with the given name
			void SetInputDisplayName(const FVertexName& InName, const FText& InDisplayName) override;

			// Set the display name for the output with the given name
			void SetOutputDisplayName(const FVertexName& InName, const FText& InDisplayName) override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
			int32 GetSortOrderIndexForInput(const FVertexName& InName) const override;
			int32 GetSortOrderIndexForOutput(const FVertexName& InName) const override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
			void SetSortOrderIndexForInput(const FVertexName& InName, int32 InSortOrderIndex) override;
			void SetSortOrderIndexForOutput(const FVertexName& InName, int32 InSortOrderIndex) override;
#endif // WITH_EDITOR

			FNodeHandle AddNode(const FNodeRegistryKey& InNodeClass, FGuid InNodeGuid) override;
			FNodeHandle AddNode(const FMetasoundFrontendClassMetadata& InClassMetadata, FGuid InNodeGuid) override;
			FNodeHandle AddDuplicateNode(const INodeController& InNode) override;
			FNodeHandle AddTemplateNode(const INodeTemplate& InNodeTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeGuid = FGuid::NewGuid()) override;

			// Remove the node corresponding to this node handle.
			// On success, invalidates the received node handle.

			UE_DEPRECATED(5.8, "Use the MetaSound Builder API")
			bool RemoveNode(INodeController& InNode) override;

			const FMetasoundFrontendClassMetadata& GetGraphMetadata() const override;
			void SetGraphMetadata(const FMetasoundFrontendClassMetadata& InClassMetadata) override;

			FNodeHandle CreateEmptySubgraph(const FMetasoundFrontendClassMetadata& InInfo) override;

			FDocumentHandle GetOwningDocument() override;
			FConstDocumentHandle GetOwningDocument() const override;

			// Exposed to aid in transition of controller API to Document Builder API
			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithName(const FVertexName& InName) const;
			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithVertexID(const FGuid& InVertexID) const;
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithName(const FVertexName& InName) const;
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithVertexID(const FGuid& InVertexID) const;

		protected:

			FDocumentAccess ShareAccess() override;
			FConstDocumentAccess ShareAccess() const override;

		private:

			FGuid FindVariableIDOfVariableContainingNode(const FGuid& InNodeID) const;
			FMetasoundFrontendVariable* FindFrontendVariable(const FGuid& InVariableID);
			const FMetasoundFrontendVariable* FindFrontendVariable(const FGuid& InVariableID) const;
			FNodeHandle FindHeadNodeInVariableStack(const FGuid& InVariableID);
			FNodeHandle FindTailNodeInVariableStack(const FGuid& InVariableID);

			void RemoveNodeIDFromAssociatedVariable(const INodeController& InNode);
			// Remove variable node from variable stack, and reconnect variable to remaining nodes.
			void SpliceVariableNodeFromVariableStack(INodeController& InNode);


			FNodeHandle AddNode(FConstClassAccessPtr InExistingDependency, FGuid InNodeGuid);
			bool RemoveNode(const FMetasoundFrontendNode& InDesc);
			bool RemoveInput(const FMetasoundFrontendNode& InNode);
			bool RemoveOutput(const FMetasoundFrontendNode& InNode);

			FNodeHandle GetNodeByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate);
			FConstNodeHandle GetNodeByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const;

			TArray<FNodeHandle> GetNodesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InFilterFunc);
			TArray<FConstNodeHandle> GetNodesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InFilterFunc) const;


			struct FNodeAndClass
			{
				FNodeAccessPtr Node;
				FConstClassAccessPtr Class;
			};

			struct FConstNodeAndClass
			{
				FConstNodeAccessPtr Node;
				FConstClassAccessPtr Class;
			};

			bool ContainsNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const;
			TArray<FNodeAndClass> GetNodesAndClasses();
			TArray<FConstNodeAndClass> GetNodesAndClasses() const;

			TArray<FNodeAndClass> GetNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate);
			TArray<FConstNodeAndClass> GetNodesAndClassesByPredicate(TFunctionRef<bool (const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)> InPredicate) const;


			TArray<FNodeHandle> GetNodeHandles(TArrayView<const FNodeAndClass> InNodesAndClasses);
			TArray<FConstNodeHandle> GetNodeHandles(TArrayView<const FConstNodeAndClass> InNodesAndClasses) const;

			FNodeHandle GetNodeHandle(const FNodeAndClass& InNodeAndClass);
			FConstNodeHandle GetNodeHandle(const FConstNodeAndClass& InNodeAndClass) const;

			FMetasoundFrontendClassInput* FindInputDescriptionWithName(const FVertexName& InName);

			FMetasoundFrontendClassInput* FindInputDescriptionWithVertexID(const FGuid& InVertexID);

			FMetasoundFrontendClassOutput* FindOutputDescriptionWithName(const FVertexName& InName);

			FMetasoundFrontendClassOutput* FindOutputDescriptionWithVertexID(const FGuid& InVertexID);

			FClassInputAccessPtr FindInputDescriptionWithNodeID(FGuid InNodeID);
			FConstClassInputAccessPtr FindInputDescriptionWithNodeID(FGuid InNodeID) const;

			FClassOutputAccessPtr FindOutputDescriptionWithNodeID(FGuid InNodeID);
			FConstClassOutputAccessPtr FindOutputDescriptionWithNodeID(FGuid InNodeID) const;

			FGraphClassAccessPtr GraphClassPtr;
			FDocumentHandle OwningDocument;
		};
	}
}
