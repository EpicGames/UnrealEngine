// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendGraphLinter.h"

#include "MetasoundFrontendGraphController.h"


namespace Metasound
{
	namespace Frontend
	{
		bool FGraphLinter::DoesConnectionCauseLoop(const IInputController& InInputController, const IOutputController& InOutputController)
		{
			bool bCausesLoop = false;

			FConstNodeHandle OutputNode = InOutputController.GetOwningNode();
			FConstNodeHandle InputNode = InInputController.GetOwningNode();

			const FGuid InputNodeID = InputNode->GetID();
			const FGuid OutputNodeID = OutputNode->GetID();
			const FGuid InputVertexID = InInputController.GetID();

			// Sets bCausesLoop if the input node already has a path to the output node
			FGraphLinter::DepthFirstTraversal(*InputNode, [&](const INodeController& Node) -> TSet<FGuid>
				{
					TSet<FGuid> Children;

					if (OutputNodeID == Node.GetID())
					{
						// If the input node can already reach the output node, then this 
						// connection will cause a loop.
						bCausesLoop = true;
					}

					if (!bCausesLoop)
					{
						// Only produce children if no loop exists to avoid wasting unnecessary CPU
						if (Node.IsValid())
						{
							TArray<FConstOutputHandle> NodeOutputs = Node.GetConstOutputs();
							for (const FConstOutputHandle& Output : NodeOutputs)
							{
								TArray<FConstInputHandle> ConnectedInputs = Output->GetConstConnectedInputs();
								for (const FConstInputHandle& Input : ConnectedInputs)
								{
									Children.Add(Input->GetOwningNode()->GetID());
								}
							}
							
						}
					}

					return Children;
				}
			);
			
			return bCausesLoop;
		}


		bool FGraphLinter::IsReachableDownstream(const INodeController& InFromNode, const INodeController& InToNode)
		{
			bool bIsReachable = false;
			const FGuid ToNodeID = InToNode.GetID();

			// Sets bCausesLoop if the FromNode has a path to the ToNode
			FGraphLinter::DepthFirstTraversal(InFromNode, [&](const INodeController& Node) -> TSet<FGuid>
			{
				TSet<FGuid> Children;

				if (ToNodeID == Node.GetID())
				{
					bIsReachable = true;
				}

				if (!bIsReachable)
				{
					// Only produce children if bIsReachbable is not set to avoid wasting CPU. 
					if (Node.IsValid())
					{
						TArray<FConstOutputHandle> NodeOutputs = Node.GetConstOutputs();
						for (const FConstOutputHandle& Output : NodeOutputs)
						{
							TArray<FConstInputHandle> ConnectedInputs = Output->GetConstConnectedInputs();
							for (const FConstInputHandle& Input : ConnectedInputs)
							{
								Children.Add(Input->GetOwningNode()->GetID());
							}
						}
						
					}
				}

				return Children;
			});
			
			return bIsReachable;
		}

		bool FGraphLinter::IsReachableUpstream(const INodeController& InFromNode, const INodeController& InToNode)
		{
			// Searching upstream is equivalent to swapping the order of the nodes 
			// and searching downstream.
			return IsReachableDownstream(InToNode, InFromNode);
		}

		void FGraphLinter::DepthFirstTraversal(const INodeController& Node, FDepthFirstVisitFunction Visit)
		{
			// Non recursive depth first traversal.
			TArray<FGuid> Stack({Node.GetID()});
			TSet<FGuid> Visited;
			FConstGraphHandle Graph = Node.GetOwningGraph();

			while (Stack.Num() > 0)
			{
				FGuid CurrentNodeID = Stack.Pop();
				if (Visited.Contains(CurrentNodeID))
				{
					// Do not revisit a node that has already been visited. 
					continue;
				}

				FConstNodeHandle CurrentNode = Graph->GetNodeWithID(CurrentNodeID);

				TArray<FGuid> Children = Visit(*CurrentNode).Array();
				Stack.Append(Children);

				Visited.Add(CurrentNodeID);
			}
		}
	}
}


