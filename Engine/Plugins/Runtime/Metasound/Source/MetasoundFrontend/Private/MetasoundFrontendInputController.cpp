// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendInputController.h"

#include "Internationalization/Text.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendGraphLinter.h"
#include "MetasoundFrontendInvalidController.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "Misc/Guid.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontendInputController"

namespace Metasound
{
	namespace Frontend
	{
		//
		// FBaseInputController
		// 
		FBaseInputController::FBaseInputController(const FBaseInputController::FInitParams& InParams)
		: ID(InParams.ID)
		, NodeVertexPtr(InParams.NodeVertexPtr)
		, ClassInputPtr(InParams.ClassInputPtr)
		, GraphPtr(InParams.GraphPtr)
		, OwningNode(InParams.OwningNode)
		{
		}

		bool FBaseInputController::IsValid() const 
		{
			return OwningNode->IsValid() && (nullptr != NodeVertexPtr.Get()) &&  (nullptr != GraphPtr.Get());
		}

		FGuid FBaseInputController::GetID() const
		{
			return ID;
		}

		const FName& FBaseInputController::GetDataType() const
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				return Vertex->TypeName;
			}
			
			return Invalid::GetInvalidName();
		}

		const FVertexName& FBaseInputController::GetName() const
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				return Vertex->Name;
			}
			
			return Invalid::GetInvalidName();
		}

		EMetasoundFrontendVertexAccessType FBaseInputController::GetVertexAccessType() const
		{
			EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Unset;
			bool bIsRerouted = false;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Frontend::IterateReroutedInputs(AsShared(), [this, &bIsRerouted, &AccessType](const FConstInputHandle& ReroutedInput)
			{
				bIsRerouted = true;

				if (AccessType != EMetasoundFrontendVertexAccessType::Value)
				{
					if (ReroutedInput->IsValid())
					{
						// If ReroutedInput is top-level controller, iterator function is returning self, so just report if set to value.
						if (ReroutedInput->GetID() == GetID() && ReroutedInput->GetOwningNodeID() == GetOwningNodeID())
						{
							if (const FMetasoundFrontendClassVertex* ClassInput = ClassInputPtr.Get())
							{
								AccessType = ClassInput->AccessType;
								return;
							}

							// Likely template node with no set class input interface, so valid to return unset
							AccessType = EMetasoundFrontendVertexAccessType::Unset;
							return;
						}

						const EMetasoundFrontendVertexAccessType RerouteAccessType = ReroutedInput->GetVertexAccessType();
						if (RerouteAccessType == EMetasoundFrontendVertexAccessType::Value)
						{
							AccessType = RerouteAccessType;
						}
					}
				}
			});
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			return bIsRerouted ? AccessType : EMetasoundFrontendVertexAccessType::Reference;
		}

#if WITH_EDITOR
		FText FBaseInputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassInput* ClassInput = ClassInputPtr.Get())
			{
				return ClassInput->Metadata.GetDisplayName();
			}
			
			return Invalid::GetInvalidText();
		}
#endif // WITH_EDITOR

		bool FBaseInputController::ClearLiteral()
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				OwningNode->ClearInputLiteral(Vertex->VertexID);
			}

			return false;
		}

		const FMetasoundFrontendLiteral* FBaseInputController::GetLiteral() const
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				return OwningNode->GetInputLiteral(Vertex->VertexID);
			}

			return nullptr;
		}

		void FBaseInputController::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				OwningNode->SetInputLiteral(FMetasoundFrontendVertexLiteral { Vertex->VertexID, InLiteral });
			}
		}

		const FMetasoundFrontendLiteral* FBaseInputController::GetClassDefaultLiteral() const
		{
			if (const FMetasoundFrontendClassInput* ClassInput = ClassInputPtr.Get())
			{
				return ClassInput->FindConstDefault(Frontend::DefaultPageID);
			}
			return nullptr;
		}

#if WITH_EDITOR
		const FText& FBaseInputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassInput* ClassInput = ClassInputPtr.Get())
			{
				return ClassInput->Metadata.GetDescription();
			}
			
			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FBaseInputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassInput* ClassInput = ClassInputPtr.Get())
			{
				return ClassInput->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		bool FBaseInputController::IsConnected() const 
		{
			return (nullptr != FindEdge());
		}

		FGuid FBaseInputController::GetOwningNodeID() const
		{
			return OwningNode->GetID();
		}

		FNodeHandle FBaseInputController::GetOwningNode()
		{
			return OwningNode;
		}

		FConstNodeHandle FBaseInputController::GetOwningNode() const
		{
			return OwningNode;
		}

		FOutputHandle FBaseInputController::GetConnectedOutput()
		{
			if (const FMetasoundFrontendEdge* Edge = FindEdge())
			{
				// Create output handle from output node.
				FGraphHandle Graph = OwningNode->GetOwningGraph();
				FNodeHandle OutputNode = Graph->GetNodeWithID(Edge->FromNodeID);
				return OutputNode->GetOutputWithID(Edge->FromVertexID);
			}

			return IOutputController::GetInvalidHandle();
		}

		FConstOutputHandle FBaseInputController::GetConnectedOutput() const
		{
			if (const FMetasoundFrontendEdge* Edge = FindEdge())
			{
				// Create output handle from output node.
				FConstGraphHandle Graph = OwningNode->GetOwningGraph();
				FConstNodeHandle OutputNode = Graph->GetNodeWithID(Edge->FromNodeID);
				return OutputNode->GetOutputWithID(Edge->FromVertexID);
			}

			return IOutputController::GetInvalidHandle();
		}

		bool FBaseInputController::IsConnectionUserModifiable() const
		{
			FConstNodeHandle Owner = GetOwningNode();
			if (Owner->GetClassMetadata().GetType() == EMetasoundFrontendClassType::Template)
			{
				const FNodeRegistryKey Key(GetOwningNode()->GetClassMetadata());
				const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
				if (ensure(Template))
				{
					return Template->IsInputConnectionUserModifiable();
				}
			}

			return true;
		}

		FConnectability FBaseInputController::CanConnectTo(const IOutputController& InController) const
		{
			FConnectability OutConnectability;
			OutConnectability.Connectable = FConnectability::EConnectable::No;
			OutConnectability.Reason = FConnectability::EReason::None;

			const FName& DataType = GetDataType();
			const FName& OtherDataType = InController.GetDataType();

			if (DataType == Invalid::GetInvalidName())
			{
				OutConnectability.Connectable = FConnectability::EConnectable::No;
				OutConnectability.Reason = FConnectability::EReason::IncompatibleDataTypes;
			}
			else if (!FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(InController.GetVertexAccessType(), GetVertexAccessType()))
			{
				OutConnectability.Connectable = FConnectability::EConnectable::No;
				OutConnectability.Reason = FConnectability::EReason::IncompatibleAccessTypes;
			}
			else if (IsCastable(OtherDataType, DataType))
			{
				// If data types are equal, connection can happen.
				OutConnectability.Connectable = FConnectability::EConnectable::Yes;
				OutConnectability.Reason = FConnectability::EReason::None;
			}
			else
			{
				// If data types are not equal, check for converter nodes which could
				// convert data type.
				OutConnectability.PossibleConverterNodeClasses = FRegistry::Get()->GetPossibleConverterNodes(OtherDataType, DataType);

				if (OutConnectability.PossibleConverterNodeClasses.Num() > 0)
				{
					OutConnectability.Connectable = FConnectability::EConnectable::YesWithConverterNode;
				}
			}

			// If data types are connectable, check if causes loop.
			if (FConnectability::EConnectable::No != OutConnectability.Connectable)
			{
				if (FGraphLinter::DoesConnectionCauseLoop(*this, InController))
				{
					OutConnectability.Connectable = FConnectability::EConnectable::No;
					OutConnectability.Reason = FConnectability::EReason::CausesLoop;
				}
			}

			return OutConnectability;
		}

		bool FBaseInputController::Connect(IOutputController& InController)
		{
			const FName& DataType = GetDataType();
			const FName& OtherDataType = InController.GetDataType();

			if (DataType == Invalid::GetInvalidName())
			{
				return false;
			}
			

			if (FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				if (IsCastable(OtherDataType, DataType))
				{
					if (FMetasoundFrontendClassVertex::CanConnectVertexAccessTypes(InController.GetVertexAccessType(), GetVertexAccessType()))
					{
						// Overwrite an existing connection if it exists.
						FMetasoundFrontendEdge* Edge = FindEdge();

						if (!Edge)
						{
							Edge = &Graph->Edges.AddDefaulted_GetRef();
							Edge->ToNodeID = GetOwningNodeID();
							Edge->ToVertexID = GetID();
						}

						Edge->FromNodeID = InController.GetOwningNodeID();
						Edge->FromVertexID = InController.GetID();

						ClearConnectedObjectLiterals();

						return true;
					}
					else
					{
						UE_LOGF(LogMetaSound, Error, "Cannot connect incompatible vertex access types (Input)%ls and (Output)%ls.", LexToString(GetVertexAccessType()), LexToString(InController.GetVertexAccessType()));
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot connect incompatible data types %ls and %ls.", *DataType.ToString(), *OtherDataType.ToString());
				}
			}

			return false;
		}

		bool FBaseInputController::ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InConverterInfo)
		{
			FGraphHandle OwningGraph = OwningNode->GetOwningGraph();

			// Generate the converter node.
			FNodeHandle ConverterNode = OwningGraph->AddNode(InConverterInfo.NodeKey);

			FInputHandle ConverterInput = ConverterNode->GetInputWithVertexName(InConverterInfo.PreferredConverterInputPin);
			FOutputHandle ConverterOutput = ConverterNode->GetOutputWithVertexName(InConverterInfo.PreferredConverterOutputPin);

			if (!ConverterInput->IsValid())
			{
				UE_LOGF(LogMetaSound, Warning, "Converter node [Name: %ls] does not support preferred input vertex [Vertex: %ls]", *ConverterNode->GetNodeName().ToString(), *InConverterInfo.PreferredConverterInputPin.ToString());
				return false;
			}

			if (!ConverterOutput->IsValid())
			{
				UE_LOGF(LogMetaSound, Warning, "Converter node [Name: %ls] does not support preferred output vertex [Vertex: %ls]", *ConverterNode->GetNodeName().ToString(), *InConverterInfo.PreferredConverterOutputPin.ToString());
				return false;
			}

			// Connect the output InController to the converter, than connect the converter to this input.
			if (ConverterInput->Connect(InController) && Connect(*ConverterOutput))
			{
				return true;
			}

			return false;
		}

		bool FBaseInputController::Disconnect(IOutputController& InController)
		{
			if (FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				FGuid FromNodeID = InController.GetOwningNodeID();
				FGuid FromVertexID = InController.GetID();
				FGuid ToNodeID = GetOwningNodeID();
				FGuid ToVertexID = GetID();

				auto IsMatchingEdge = [&](const FMetasoundFrontendEdge& Edge)
				{
					return (Edge.FromNodeID == FromNodeID) && (Edge.FromVertexID == FromVertexID) && (Edge.ToNodeID == ToNodeID) && (Edge.ToVertexID == ToVertexID);
				};

				const int32 NumRemoved = Graph->Edges.RemoveAllSwap(IsMatchingEdge);

#if WITH_EDITOR
				auto IsMatchingStyle = [&](const FMetasoundFrontendEdgeStyle& EdgeStyle)
				{
					return EdgeStyle.NodeID == FromNodeID && InController.GetName() == EdgeStyle.OutputName;
				};
				Graph->Style.EdgeStyles.RemoveAllSwap(IsMatchingStyle);
#endif // WITH_EDITOR

				return NumRemoved > 0;
			}

			return false;
		}

		bool FBaseInputController::Disconnect()
		{
			if (FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				const FGuid NodeID = GetOwningNodeID();

#if WITH_EDITORONLY_DATA
				{
					const FName OutputName = GetConnectedOutput()->GetName();
					auto IsStyleForThisNode = [&](const FMetasoundFrontendEdgeStyle& EdgeStyle) { return EdgeStyle.NodeID == NodeID && OutputName == EdgeStyle.OutputName; };
					Graph->Style.EdgeStyles.RemoveAllSwap(IsStyleForThisNode);
				}
#endif // WITH_EDITORONLY_DATA

				FGuid VertexID = GetID();

				auto EdgeHasMatchingDestination = [&](const FMetasoundFrontendEdge& Edge)
				{
					return (Edge.ToNodeID == NodeID) && (Edge.ToVertexID == VertexID);
				};
				const int32 NumRemoved = Graph->Edges.RemoveAllSwap(EdgeHasMatchingDestination);
				return NumRemoved > 0;
			}

			return false;
		}
		
		void FBaseInputController::ClearConnectedObjectLiterals()
		{
			FDataTypeRegistryInfo DataTypeInfo;
			if (IsConnected())
			{
				if (IDataTypeRegistry::Get().GetDataTypeInfo(GetDataType(), DataTypeInfo))
				{
					if (DataTypeInfo.IsDataTypeProxyParsable())
					{
						ClearLiteral();
					}
				}
			}
		}

		const FMetasoundFrontendEdge* FBaseInputController::FindEdge() const
		{
			if (const FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				const FGuid NodeID = GetOwningNodeID();
				FGuid VertexID = GetID();

				auto EdgeHasMatchingDestination = [&](const FMetasoundFrontendEdge& Edge)
				{
					return (Edge.ToNodeID == NodeID) && (Edge.ToVertexID == VertexID);
				};

				return Graph->Edges.FindByPredicate(EdgeHasMatchingDestination);
			}

			return nullptr;
		}

		FMetasoundFrontendEdge* FBaseInputController::FindEdge()
		{
			if (FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				const FGuid NodeID = GetOwningNodeID();
				FGuid VertexID = GetID();

				auto EdgeHasMatchingDestination = [&](const FMetasoundFrontendEdge& Edge)
				{
					return (Edge.ToNodeID == NodeID) && (Edge.ToVertexID == VertexID);
				};

				return Graph->Edges.FindByPredicate(EdgeHasMatchingDestination);
			}

			return nullptr;
		}

		FDocumentAccess FBaseInputController::ShareAccess() 
		{
			FDocumentAccess Access;

			Access.ConstVertex = NodeVertexPtr;
			Access.ConstClassInput = ClassInputPtr;
			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		FConstDocumentAccess FBaseInputController::ShareAccess() const
		{
			FConstDocumentAccess Access;

			Access.ConstVertex = NodeVertexPtr;
			Access.ConstClassInput = ClassInputPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		//
		// FOutputNodeInputController
		//
		FOutputNodeInputController::FOutputNodeInputController(const FOutputNodeInputController::FInitParams& InParams)
		: FBaseInputController({InParams.ID, InParams.NodeVertexPtr, InParams.ClassInputPtr, InParams.GraphPtr, InParams.OwningNode})
		, OwningGraphClassOutputPtr(InParams.OwningGraphClassOutputPtr)
		{
		}

		bool FOutputNodeInputController::IsValid() const
		{
			return FBaseInputController::IsValid() && (nullptr != OwningGraphClassOutputPtr.Get());
		}

#if WITH_EDITOR
		FText FOutputNodeInputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				if (const FMetasoundFrontendClassInput* ClassInput = ClassInputPtr.Get())
				{
					// If there the ClassInput exists, combine the variable name and class input name.
					// of the variable should be added to the names of the vertices.
					CachedDisplayName = FText::Format(LOCTEXT("OutputNodeInputControllerFormat", "{1} {0}"), OwningOutput->Metadata.GetDisplayName(), ClassInput->Metadata.GetDisplayName());
				}
				else
				{
					// If there is not ClassInput, then use the variable name.
					CachedDisplayName = OwningOutput->Metadata.GetDisplayName();
				}
			}

			return CachedDisplayName;
		}

		const FText& FOutputNodeInputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata.GetDescription();
			}
			
			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FOutputNodeInputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		void FOutputNodeInputController::SetName(const FVertexName& InName)
		{
			if (FMetasoundFrontendVertex* Vertex = ConstCastAccessPtr<FVertexAccessPtr>(NodeVertexPtr).Get())
			{
				Vertex->Name = InName;
			}
		}

		EMetasoundFrontendVertexAccessType FOutputNodeInputController::GetVertexAccessType() const
		{
			return OwningGraphClassOutputPtr.Get()->AccessType;
		}

		FDocumentAccess FOutputNodeInputController::ShareAccess() 
		{
			FDocumentAccess Access = FBaseInputController::ShareAccess();

			Access.ConstClassOutput = OwningGraphClassOutputPtr;

			return Access;
		}

		FConstDocumentAccess FOutputNodeInputController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseInputController::ShareAccess();

			Access.ConstClassOutput = OwningGraphClassOutputPtr;

			return Access;
		}


		//
		// FInputNodeInputController
		//
		FInputNodeInputController::FInputNodeInputController(const FInputNodeInputController::FInitParams& InParams)
		: FBaseInputController({InParams.ID, InParams.NodeVertexPtr, InParams.ClassInputPtr, InParams.GraphPtr, InParams.OwningNode})
		, OwningGraphClassInputPtr(InParams.OwningGraphClassInputPtr)
		{
		}

		bool FInputNodeInputController::IsValid() const
		{
			return FBaseInputController::IsValid() && (nullptr != OwningGraphClassInputPtr.Get());
		}

#if WITH_EDITOR
		FText FInputNodeInputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				return OwningInput->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		const FText& FInputNodeInputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				return OwningInput->Metadata.GetDescription();
			}
			
			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FInputNodeInputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				return OwningInput->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		void FInputNodeInputController::SetName(const FVertexName& InName)
		{
			if (FMetasoundFrontendVertex* Vertex = ConstCastAccessPtr<FVertexAccessPtr>(NodeVertexPtr).Get())
			{
				Vertex->Name = InName;
			}
		}
		
		EMetasoundFrontendVertexAccessType FInputNodeInputController::GetVertexAccessType() const
		{
			return OwningGraphClassInputPtr.Get()->AccessType;
		}

		bool FInputNodeInputController::IsConnectionUserModifiable() const
		{
			// Inputs to input nodes on a graph cannot be connected by the user
			// because they must be exposed externally from the graph.
			return false;
		}

		FConnectability FInputNodeInputController::CanConnectTo(const IOutputController& InController) const 
		{
			static const FConnectability Connectability = {FConnectability::EConnectable::No};
			return Connectability;
		}

		bool FInputNodeInputController::Connect(IOutputController& InController) 
		{
			return false;
		}

		bool FInputNodeInputController::ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InNodeClassName)
		{
			return false;
		}


		FVariableInputController::FVariableInputController(const FInitParams& InParams)
		: FBaseInputController(InParams)
		{
		}

		bool FVariableInputController::IsConnectionUserModifiable() const
		{
			// Variable connections are managed by the graph and cannot be modified
			// by the user.
			return false;
		}
	}
}
#undef LOCTEXT_NAMESPACE
