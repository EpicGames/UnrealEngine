// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "Misc/Guid.h"

namespace Metasound
{
	namespace Frontend
	{
		/** FBaseInputController provides common functionality for multiple derived
		 * input controllers.
		 */
		class FBaseInputController : public IInputController 
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:

			struct FInitParams
			{
				FGuid ID; 
				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassInputAccessPtr ClassInputPtr;
				FGraphAccessPtr GraphPtr; 
				FNodeHandle OwningNode;
			};

			/** Construct the input controller base. */
			FBaseInputController(const FInitParams& InParams);

			virtual ~FBaseInputController() = default;

			virtual bool IsValid() const override;

			virtual FGuid GetID() const override;
			virtual const FName& GetDataType() const override;
			virtual const FVertexName& GetName() const override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;
			virtual bool ClearLiteral() override;
			virtual const FMetasoundFrontendLiteral* GetLiteral() const override;
			virtual void SetLiteral(const FMetasoundFrontendLiteral& InLiteral) override;

			virtual const FMetasoundFrontendLiteral* GetClassDefaultLiteral() const override;

			// This only exists to allow for transform fix-ups to easily cleanup input/output
			// vertex names & should not be used for typical edit or runtime callsites.
			virtual void SetName(const FVertexName& InName) override { checkNoEntry(); }

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Input metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			// Owning node info
			virtual FGuid GetOwningNodeID() const override;
			virtual FNodeHandle GetOwningNode() override;
			virtual FConstNodeHandle GetOwningNode() const override;

			// Connection info
			virtual bool IsConnectionUserModifiable() const override;
			virtual bool IsConnected() const override;
			virtual FOutputHandle GetConnectedOutput() override;
			virtual FConstOutputHandle GetConnectedOutput() const override;

			virtual FConnectability CanConnectTo(const IOutputController& InController) const override;
			virtual bool Connect(IOutputController& InController) override;

			// Connection controls.
			virtual bool ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InNodeClassName) override;

			virtual bool Disconnect(IOutputController& InController) override;
			virtual bool Disconnect() override;

			virtual void ClearConnectedObjectLiterals() override;

		protected:


			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			const FMetasoundFrontendEdge* FindEdge() const;
			FMetasoundFrontendEdge* FindEdge();

			FGuid ID;
			FConstVertexAccessPtr NodeVertexPtr;
			FConstClassInputAccessPtr ClassInputPtr;
			FGraphAccessPtr GraphPtr;
			FNodeHandle OwningNode;
		};

		/** FOutputNodeInputController represents the input vertex of an output 
		 * node. 
		 *
		 * FOutputNodeInputController is largely to represent outputs exposed from
		 * a graph. 
		 */
		class FOutputNodeInputController : public FBaseInputController 
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:
			struct FInitParams
			{
				FGuid ID; 
				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassInputAccessPtr ClassInputPtr;
				FConstClassOutputAccessPtr OwningGraphClassOutputPtr;
				FGraphAccessPtr GraphPtr; 
				FNodeHandle OwningNode;
			};

			/** Constructs the input controller. */
			FOutputNodeInputController(const FInitParams& InParams);

			virtual bool IsValid() const override;

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Input metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			virtual void SetName(const FVertexName& InName) override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

		protected:

			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

		private:

			mutable FText CachedDisplayName;

			FConstClassOutputAccessPtr OwningGraphClassOutputPtr;
		};

		/** FInputNodeInputController represents the input vertex of an output 
		 * node. 
		 *
		 * FInputNodeInputController is largely to represent outputs exposed from
		 * a graph. 
		 */
		class FInputNodeInputController : public FBaseInputController 
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:
			struct FInitParams
			{
				FGuid ID; 
				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassInputAccessPtr ClassInputPtr;
				FConstClassInputAccessPtr OwningGraphClassInputPtr;
				FGraphAccessPtr GraphPtr; 
				FNodeHandle OwningNode;
			};

			/** Constructs the input controller. */
			FInputNodeInputController(const FInitParams& InParams);

			virtual bool IsValid() const override;

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Input metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			virtual void SetName(const FVertexName& InName) override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

			virtual bool IsConnectionUserModifiable() const override;
			virtual FConnectability CanConnectTo(const IOutputController& InController) const override;
			virtual bool Connect(IOutputController& InController) override;

			// Connection controls.
			virtual bool ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InNodeClassName) override;

		private:
			FConstClassInputAccessPtr OwningGraphClassInputPtr;
		};

		/** Input controller for variable data type. */
		class FVariableInputController : public FBaseInputController
		{
		public:
			using FInitParams = FBaseInputController::FInitParams;

			FVariableInputController(const FInitParams& InParams);
			virtual ~FVariableInputController() = default;

			/** Variable data type connections are not modifiable by users */
			virtual bool IsConnectionUserModifiable() const override;
		};
	}
}
