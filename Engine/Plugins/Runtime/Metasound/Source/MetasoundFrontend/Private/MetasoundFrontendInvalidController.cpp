// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendInvalidController.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace Invalid
		{
			const FText& GetInvalidText()
			{
				return FText::GetEmpty();
			}

			const FName& GetInvalidName()
			{
				static const FName Invalid;
				return Invalid;
			}

			const FString& GetInvalidString()
			{
				static const FString Invalid;
				return Invalid;
			}

			const FMetasoundFrontendLiteral& GetInvalidLiteral()
			{
				auto CreateInvalidLiteral = []() ->FMetasoundFrontendLiteral
				{
					FMetasoundFrontendLiteral Literal;
					Literal.SetType(EMetasoundFrontendLiteralType::Invalid);
					return Literal;
				};

				static const FMetasoundFrontendLiteral Literal = CreateInvalidLiteral();
				return Literal;
			}

			const FMetasoundFrontendClassInterface& GetInvalidClassInterface()
			{
				static const FMetasoundFrontendClassInterface Invalid;
				return Invalid;
			}

			const FMetasoundFrontendClassMetadata& GetInvalidClassMetadata()
			{
				static const FMetasoundFrontendClassMetadata Invalid;
				return Invalid;
			}

			const FMetasoundFrontendNodeInterface& GetInvalidNodeInterface()
			{
				static const FMetasoundFrontendNodeInterface Invalid;
				return Invalid;
			}

			const FMetasoundFrontendGraphClassPresetOptions& GetInvalidGraphClassPresetOptions()
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				static const FMetasoundFrontendGraphClassPresetOptions Invalid;
				return Invalid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

#if WITH_EDITOR
			const FMetasoundFrontendVertexMetadata& GetInvalidVertexMetadata()
			{
				static const FMetasoundFrontendVertexMetadata Invalid;
				return Invalid;
			}

			const FMetasoundFrontendInterfaceStyle& GetInvalidInterfaceStyle()
			{
				static const FMetasoundFrontendInterfaceStyle Invalid;
				return Invalid;
			}

			const FMetasoundFrontendClassStyle& GetInvalidClassStyle()
			{
				static const FMetasoundFrontendClassStyle Invalid;
				return Invalid;
			}

			const FMetasoundFrontendGraphStyle& GetInvalidGraphStyle() 
			{
				static const FMetasoundFrontendGraphStyle Invalid;
				return Invalid;
			}
#endif // WITH_EDITOR

			const FMetasoundFrontendGraphClass& GetInvalidGraphClass() 
			{ 
				static const FMetasoundFrontendGraphClass Invalid;
				return Invalid;
			}

			const TArray<FMetasoundFrontendClass>& GetInvalidClassArray() 
			{ 
				static const TArray<FMetasoundFrontendClass> Invalid;
				return Invalid;
			}

			const TArray<FMetasoundFrontendGraphClass>& GetInvalidGraphClassArray() 
			{ 
				static const TArray<FMetasoundFrontendGraphClass> Invalid;
				return Invalid;
			}

			const TSet<FName>& GetInvalidNameSet()
			{
				static TSet<FName> Invalid;
				return Invalid;
			}

			const FMetasoundFrontendDocumentMetadata& GetInvalidDocumentMetadata()
			{
				static const FMetasoundFrontendDocumentMetadata Invalid;
				return Invalid;
			}
		}

		TSharedRef<INodeController> FInvalidOutputController::GetOwningNode()
		{
			return INodeController::GetInvalidHandle();
		}

		TSharedRef<const INodeController> FInvalidOutputController::GetOwningNode() const
		{
			return INodeController::GetInvalidHandle();
		}

		TSharedRef<INodeController> FInvalidInputController::GetOwningNode()
		{
			return INodeController::GetInvalidHandle();
		}

		TSharedRef<const INodeController> FInvalidInputController::GetOwningNode() const
		{
			return INodeController::GetInvalidHandle();
		}

		TSharedRef<IGraphController> FInvalidNodeController::AsGraph()
		{
			return IGraphController::GetInvalidHandle();
		}

		TSharedRef<const IGraphController> FInvalidNodeController::AsGraph() const
		{
			return IGraphController::GetInvalidHandle();
		}

		TSharedRef<IGraphController> FInvalidNodeController::GetOwningGraph()
		{
			return IGraphController::GetInvalidHandle();
		}

		TSharedRef<const IGraphController> FInvalidNodeController::GetOwningGraph() const
		{
			return IGraphController::GetInvalidHandle();
		}

		TSharedRef<IDocumentController> FInvalidGraphController::GetOwningDocument()
		{
			return IDocumentController::GetInvalidHandle();
		}

		TSharedRef<const IDocumentController> FInvalidGraphController::GetOwningDocument() const
		{
			return IDocumentController::GetInvalidHandle();
		}

		TSharedRef<IGraphController> FInvalidDocumentController::GetRootGraph()
		{
			return IGraphController::GetInvalidHandle();
		}

		TSharedRef<const IGraphController> FInvalidDocumentController::GetRootGraph() const
		{
			return IGraphController::GetInvalidHandle();
		}

		FDocumentAccessPtr FInvalidDocumentController::GetDocumentPtr()
		{
			static FDocumentAccessPtr Invalid;
			return Invalid;
		}

		const FDocumentAccessPtr FInvalidDocumentController::GetDocumentPtr() const
		{
			static const FDocumentAccessPtr Invalid;
			return Invalid;
		}
	}
}
