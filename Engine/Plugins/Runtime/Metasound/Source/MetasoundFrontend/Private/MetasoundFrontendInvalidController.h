// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "MetasoundAssetBase.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundGraph.h"
#include "MetasoundVertex.h"

#define UE_API METASOUNDFRONTEND_API

class FText;
class FName;
class FString;

namespace Metasound
{
	namespace Frontend
	{
		namespace Invalid
		{
			const FText& GetInvalidText();
			const FName& GetInvalidName();
			const FString& GetInvalidString();

#if WITH_EDITOR
			const FMetasoundFrontendVertexMetadata& GetInvalidVertexMetadata();
			const FMetasoundFrontendInterfaceStyle& GetInvalidInterfaceStyle();
			const FMetasoundFrontendClassStyle& GetInvalidClassStyle();
			const FMetasoundFrontendGraphStyle& GetInvalidGraphStyle();
#endif // WITH_EDITOR

			const FMetasoundFrontendLiteral& GetInvalidLiteral();
			const FMetasoundFrontendClassInterface& GetInvalidClassInterface();
			const FMetasoundFrontendNodeInterface& GetInvalidNodeInterface();
			const FMetasoundFrontendClassMetadata& GetInvalidClassMetadata();

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			const FMetasoundFrontendGraphClassPresetOptions& GetInvalidGraphClassPresetOptions();

			const FMetasoundFrontendGraphClass& GetInvalidGraphClass();
			const TArray<FMetasoundFrontendClass>& GetInvalidClassArray();
			const TArray<FMetasoundFrontendGraphClass>& GetInvalidGraphClassArray();
			const TSet<FName>& GetInvalidNameSet();
			const FMetasoundFrontendDocumentMetadata& GetInvalidDocumentMetadata();
		}

		/** FInvalidOutputController is a controller which is always invalid. 
		 *
		 *  All methods return defaults and return error flags or invalid values.
		 */
		class FInvalidOutputController : public IOutputController
		{
		public:
			FInvalidOutputController() = default;

			virtual ~FInvalidOutputController() = default;


			virtual bool IsValid() const override { return false; }
			virtual FGuid GetID() const override { return Metasound::FrontendInvalidID; }
			virtual const FName& GetDataType() const override { return Invalid::GetInvalidName(); }
			virtual const FVertexName& GetName() const override { return Invalid::GetInvalidName(); }
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override { return EMetasoundFrontendVertexAccessType::Unset; }

#if WITH_EDITOR
			virtual FText GetDisplayName() const override { return FText::GetEmpty(); }
			virtual const FText& GetTooltip() const override { return FText::GetEmpty(); }
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override { return Invalid::GetInvalidVertexMetadata(); }
#endif // WITH_EDITOR

			virtual FGuid GetOwningNodeID() const override { return Metasound::FrontendInvalidID; }
			UE_API virtual FNodeHandle GetOwningNode() override;
			UE_API virtual FConstNodeHandle GetOwningNode() const override;
			virtual void SetName(const FVertexName& InName) override { }
			virtual bool IsConnected() const override { return false; }
			virtual TArray<FInputHandle> GetConnectedInputs() override { return TArray<FInputHandle>(); }
			virtual TArray<FConstInputHandle> GetConstConnectedInputs() const override { return TArray<FConstInputHandle>(); }
			virtual bool Disconnect() override { return false; }

			virtual bool IsConnectionUserModifiable() const override { return false; }
			virtual FConnectability CanConnectTo(const IInputController& InController) const override { return FConnectability(); }
			virtual bool Connect(IInputController& InController) override { return false; }
			virtual bool ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName) override { return false; }
			virtual bool Disconnect(IInputController& InController) override { return false; }

		protected:
			virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
			virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }

		};

		/** FInvalidInputController is a controller which is always invalid. 
		 *
		 *  All methods return defaults and return error flags or invalid values.
		 */
		class FInvalidInputController : public IInputController 
		{
		public:
			FInvalidInputController() = default;
			virtual ~FInvalidInputController() = default;

			virtual bool IsValid() const override { return false; }
			virtual FGuid GetID() const override { return Metasound::FrontendInvalidID; }
			virtual bool IsConnected() const override { return false; }
			virtual const FName& GetDataType() const override { return Invalid::GetInvalidName(); }
			virtual const FVertexName& GetName() const override { return Invalid::GetInvalidName(); }
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override { return EMetasoundFrontendVertexAccessType::Reference; }

#if WITH_EDITOR
			virtual FText GetDisplayName() const override { return Invalid::GetInvalidText(); }
			virtual const FText& GetTooltip() const override { return Invalid::GetInvalidText(); }
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override { return Invalid::GetInvalidVertexMetadata(); }
#endif // WITH_EDITOR

			virtual bool ClearLiteral() override { return false; }
			virtual const FMetasoundFrontendLiteral* GetLiteral() const override { return nullptr; }
			virtual void SetLiteral(const FMetasoundFrontendLiteral& InLiteral) { };
			virtual const FMetasoundFrontendLiteral* GetClassDefaultLiteral() const override { return nullptr; }
			virtual FGuid GetOwningNodeID() const override { return Metasound::FrontendInvalidID; }
			UE_API virtual FNodeHandle GetOwningNode() override;
			UE_API virtual FConstNodeHandle GetOwningNode() const override;

			virtual bool IsConnectionUserModifiable() const override { return false; }
			virtual FOutputHandle GetConnectedOutput() override { return IOutputController::GetInvalidHandle(); }
			virtual FConstOutputHandle GetConnectedOutput() const override { return IOutputController::GetInvalidHandle(); }
			virtual bool Disconnect() override { return false; }

			virtual void SetName(const FVertexName& InName) override { }

			virtual FConnectability CanConnectTo(const IOutputController& InController) const override { return FConnectability(); }
			virtual bool Connect(IOutputController& InController) override { return false; }
			virtual bool ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InNodeClassName) override { return false; }
			virtual bool Disconnect(IOutputController& InController) override { return false; }

			virtual void ClearConnectedObjectLiterals() override { }
		protected:
			virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
			virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }
		};

		class FInvalidVariableController : public IVariableController
		{
		public:
			FInvalidVariableController() = default;
			virtual ~FInvalidVariableController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const override { return false; }

			virtual FGuid GetID() const override { return Metasound::FrontendInvalidID; }
			
			/** Returns the data type name associated with this output. */
			virtual const FName& GetDataType() const override { return Invalid::GetInvalidName(); }
			
			/** Returns the name associated with this variable. */
			virtual const FName& GetName() const override { return Invalid::GetInvalidName(); }
			virtual void SetName(const FName&) override { }

#if WITH_EDITOR
			/** Returns the human readable name associated with this output. */
			virtual FText GetDisplayName() const override { return Invalid::GetInvalidText(); }
			virtual void SetDisplayName(const FText&) override { }
			virtual FText GetDescription() const override { return Invalid::GetInvalidText(); }
			virtual void SetDescription(const FText&) override { }
#endif // WITH_EDITOR

			virtual FNodeHandle FindMutatorNode() override { return INodeController::GetInvalidHandle(); }
			virtual FConstNodeHandle FindMutatorNode() const override { return INodeController::GetInvalidHandle(); }
			virtual TArray<FNodeHandle> FindAccessorNodes() override { return TArray<FNodeHandle>(); }
			virtual TArray<FConstNodeHandle> FindAccessorNodes() const override { return TArray<FConstNodeHandle>(); }
			virtual TArray<FNodeHandle> FindDeferredAccessorNodes() override { return TArray<FNodeHandle>(); }
			virtual TArray<FConstNodeHandle> FindDeferredAccessorNodes() const override { return TArray<FConstNodeHandle>(); }
			
			/** Returns a FGraphHandle to the node which owns this output. */
			virtual FGraphHandle GetOwningGraph() override { return IGraphController::GetInvalidHandle(); }
			
			/** Returns a FConstGraphHandle to the node which owns this output. */
			virtual FConstGraphHandle GetOwningGraph() const override { return IGraphController::GetInvalidHandle(); }

			/** Returns the value for the given variable instance if set. */
			virtual const FMetasoundFrontendLiteral& GetLiteral() const override { return Invalid::GetInvalidLiteral(); }

			/** Sets the value for the given variable instance */
			virtual bool SetLiteral(const FMetasoundFrontendLiteral& InLiteral) override { return false; }
		protected:
			virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
			virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }
		};


		/** FInvalidNodeController is a controller which is always invalid. 
		 *
		 *  All methods return defaults and return error flags or invalid values.
		 */
		class FInvalidNodeController : public INodeController 
		{

		public:
			FInvalidNodeController() = default;
			virtual ~FInvalidNodeController() = default;

			virtual bool IsValid() const override { return false; }

			virtual TArray<FInputHandle> GetInputs() override { return TArray<FInputHandle>(); }
			virtual TArray<FOutputHandle> GetOutputs() override { return TArray<FOutputHandle>(); }
			virtual TArray<FConstInputHandle> GetConstInputs() const override { return TArray<FConstInputHandle>(); }
			virtual TArray<FConstOutputHandle> GetConstOutputs() const override { return TArray<FConstOutputHandle>(); }

			virtual FInputHandle GetInputWithVertexName(const FVertexName& InName) override { return IInputController::GetInvalidHandle(); }
			virtual FConstInputHandle GetConstInputWithVertexName(const FVertexName& InName) const override { return IInputController::GetInvalidHandle(); }
			virtual FOutputHandle GetOutputWithVertexName(const FVertexName& InName) override { return IOutputController::GetInvalidHandle(); }
			virtual FConstOutputHandle GetConstOutputWithVertexName(const FVertexName& InName) const override { return IOutputController::GetInvalidHandle(); }
			virtual FInputHandle GetInputWithID(FGuid InVertexID) override { return IInputController::GetInvalidHandle(); }
			virtual FOutputHandle GetOutputWithID(FGuid InVertexID) override { return IOutputController::GetInvalidHandle(); }
			virtual FConstInputHandle GetInputWithID(FGuid InVertexID) const override { return IInputController::GetInvalidHandle(); }
			virtual FConstOutputHandle GetOutputWithID(FGuid InVertexID) const override { return IOutputController::GetInvalidHandle(); }

#if WITH_EDITORONLY_DATA
			virtual const FMetasoundFrontendNodeStyle& GetNodeStyle() const override { static const FMetasoundFrontendNodeStyle Invalid; return Invalid; }
			virtual void SetNodeStyle(const FMetasoundFrontendNodeStyle& InNodeStyle) override { }
#endif // WITH_EDITOR

			virtual void SetNodeName(const FVertexName& InName) override { }

			virtual FNodeHandle ReplaceWithVersion(const FMetasoundFrontendVersionNumber& InNewVersion, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs) override { return INodeController::GetInvalidHandle(); }


			virtual bool CanAddInput(const FVertexName& InVertexName) const override { return false; }
			virtual FInputHandle AddInput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) override { return IInputController::GetInvalidHandle(); }
			virtual bool RemoveInput(FGuid InVertexID) override { return false; }

			virtual bool CanAddOutput(const FVertexName& InVertexName) const override { return false; }
			virtual FInputHandle AddOutput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) override { return IInputController::GetInvalidHandle(); }
			virtual bool RemoveOutput(FGuid InVertexID) override { return false; }

			virtual bool ClearInputLiteral(FGuid InVertexID) override { return false; };
			virtual const FMetasoundFrontendLiteral* GetInputLiteral(const FGuid& InVertexID) const override { return nullptr; }
			virtual void SetInputLiteral(const FMetasoundFrontendVertexLiteral& InVertexLiteral) override { }

			virtual const FMetasoundFrontendClassInterface& GetClassInterface() const override { return Invalid::GetInvalidClassInterface(); }
			virtual const FMetasoundFrontendClassMetadata& GetClassMetadata() const override { return Invalid::GetInvalidClassMetadata(); }

			virtual const FMetasoundFrontendNodeInterface& GetNodeInterface() const override { return Invalid::GetInvalidNodeInterface(); }

#if WITH_EDITOR
			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const override { return Invalid::GetInvalidInterfaceStyle(); }
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const override { return Invalid::GetInvalidInterfaceStyle(); }
			virtual const FMetasoundFrontendClassStyle& GetClassStyle() const override { return Invalid::GetInvalidClassStyle(); }

			virtual const FText& GetDescription() const override { return Invalid::GetInvalidText(); }
#endif // WITH_EDITOR

			UE_DEPRECATED(5.8, "Controllers are being actively deprecated, and this call was not thread safe in certain contexts")
			virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const { return FMetasoundFrontendVersion::GetInvalid(); }

			UE_DEPRECATED(5.8, "Controllers are being actively deprecated, and this call was not thread safe in certain contexts")
			virtual bool IsInterfaceMember() const override { return false; }

#if WITH_EDITORONLY_DATA
			virtual bool DiffAgainstRegistryInterface(FClassInterfaceUpdates& OutInterfaceUpdates, bool bInUseHighestMinorVersion, bool bInForceRegeneratedClassInterfaceOverride) const override { return false; }
			virtual bool CanAutoUpdate(FClassInterfaceUpdates& OutInterfaceUpdates) const override { OutInterfaceUpdates = { }; return false; }
#endif // WITH_EDITORONLY_DATA

			UE_API virtual TSharedRef<IGraphController> AsGraph() override;
			UE_API virtual TSharedRef<const IGraphController> AsGraph() const override;

			virtual FGuid GetID() const override { return Metasound::FrontendInvalidID; }
			virtual FGuid GetClassID() const override { return Metasound::FrontendInvalidID; }

			virtual FGuid GetOwningGraphClassID() const override { return Metasound::FrontendInvalidID; }
			UE_API virtual TSharedRef<IGraphController> GetOwningGraph() override;
			UE_API virtual TSharedRef<const IGraphController> GetOwningGraph() const override;

			virtual void IterateInputs(TUniqueFunction<void(FInputHandle)> InFunction) override { }
			virtual void IterateConstInputs(TUniqueFunction<void(FConstInputHandle)> InFunction) const override { }

			virtual void IterateOutputs(TUniqueFunction<void(FOutputHandle)> InFunction) override { }
			virtual void IterateConstOutputs(TUniqueFunction<void(FConstOutputHandle)> InFunction) const override { }

			virtual int32 GetNumInputs() const override { return 0; }
			virtual int32 GetNumOutputs() const override { return 0; }

			virtual const FVertexName& GetNodeName() const override { return Invalid::GetInvalidName(); }

			virtual void SetNodeConfiguration(TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfig) override {}
			virtual TConstStructView<FMetaSoundFrontendNodeConfiguration> GetNodeConfiguration() const override { return TConstStructView<FMetaSoundFrontendNodeConfiguration>{}; }

#if WITH_EDITOR
			virtual FText GetDisplayName() const override { return Invalid::GetInvalidText(); }
			virtual const FText& GetDisplayTitle() const override { return Invalid::GetInvalidText(); }
			virtual void SetDescription(const FText& InDescription) override { }
			virtual void SetDisplayName(const FText& InText) override { }
#endif // WITH_EDITOR

		protected:
			virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
			virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }
		};

		/** FInvalidGraphController is a controller which is always invalid. 
		 *
		 *  All methods return defaults and return error flags or invalid values.
		 */
		class FInvalidGraphController : public IGraphController 
		{
			public:
			FInvalidGraphController() = default;
			virtual ~FInvalidGraphController() = default;

			virtual bool IsValid() const override { return false; }
			virtual FGuid GetClassID() const override { return Metasound::FrontendInvalidID; }

#if WITH_EDITOR
			virtual FText GetDisplayName() const override { return Invalid::GetInvalidText(); }
#endif // WITH_EDITOR

			virtual TArray<FVertexName> GetInputVertexNames() const override { return TArray<FVertexName>(); }
			virtual TArray<FVertexName> GetOutputVertexNames() const override { return TArray<FVertexName>(); }

			virtual TArray<FNodeHandle> GetNodes() override { return TArray<FNodeHandle>(); }
			virtual TArray<FConstNodeHandle> GetConstNodes() const override { return TArray<FConstNodeHandle>(); }

			virtual FConstNodeHandle GetNodeWithID(FGuid InNodeID) const override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle GetNodeWithID(FGuid InNodeID) override { return INodeController::GetInvalidHandle(); }

			virtual TArray<FNodeHandle> GetOutputNodes() override { return TArray<FNodeHandle>(); }
			virtual TArray<FNodeHandle> GetInputNodes() override { return TArray<FNodeHandle>(); }
			virtual TArray<FConstNodeHandle> GetConstOutputNodes() const override { return TArray<FConstNodeHandle>(); }
			virtual TArray<FConstNodeHandle> GetConstInputNodes() const override { return TArray<FConstNodeHandle>(); }

			virtual FVariableHandle AddVariable(const FName& InDataTypeName) override { return IVariableController::GetInvalidHandle(); }
			virtual FVariableHandle FindVariable(const FGuid& InVariableID) override { return IVariableController::GetInvalidHandle(); }
			virtual FConstVariableHandle FindVariable(const FGuid& InVariableID) const { return IVariableController::GetInvalidHandle(); }
			virtual FVariableHandle FindVariableContainingNode(const FGuid& InNodeID) override { return IVariableController::GetInvalidHandle(); }
			virtual FConstVariableHandle FindVariableContainingNode(const FGuid& InNodeID) const override { return IVariableController::GetInvalidHandle(); }
			virtual TArray<FVariableHandle> GetVariables() override { return TArray<FVariableHandle>(); }
			virtual TArray<FConstVariableHandle> GetVariables() const override { return TArray<FConstVariableHandle>(); }
			virtual FNodeHandle FindOrAddVariableMutatorNode(const FGuid& InVariableID) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddVariableAccessorNode(const FGuid& InVariableID) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddVariableDeferredAccessorNode(const FGuid& InVariableID) override{ return INodeController::GetInvalidHandle(); }

#if WITH_EDITOR
			virtual const FMetasoundFrontendGraphStyle& GetGraphStyle() const override { return Invalid::GetInvalidGraphStyle(); }
			virtual void SetGraphStyle(FMetasoundFrontendGraphStyle InStyle) override { }
			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const override { return Invalid::GetInvalidInterfaceStyle(); }
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const override { return Invalid::GetInvalidInterfaceStyle(); }
			virtual void SetInputStyle(FMetasoundFrontendInterfaceStyle InStyle) override { }
			virtual void SetOutputStyle(FMetasoundFrontendInterfaceStyle InStyle) override { }
#endif // WITH_EDITOR

			virtual void ClearGraph() override { };

			virtual void IterateConstNodes(TFunctionRef<void(FConstNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType /* = EMetasoundFrontendClassType::Invalid */) const override { }
			virtual void IterateNodes(TFunctionRef<void(FNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType /* = EMetasoundFrontendClassType::Invalid */) override { }

			virtual bool ContainsOutputVertex(const FVertexName& InName, const FName& InTypeName) const override { return false; }
			virtual bool ContainsOutputVertexWithName(const FVertexName& InName) const override { return false; }
			virtual bool ContainsInputVertex(const FVertexName& InName, const FName& InTypeName) const override { return false; }
			virtual bool ContainsInputVertexWithName(const FVertexName& InName) const override { return false; }

			virtual FConstNodeHandle GetOutputNodeWithName(const FVertexName& InName) const override { return INodeController::GetInvalidHandle(); }
			virtual FConstNodeHandle GetInputNodeWithName(const FVertexName& InName) const override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle GetOutputNodeWithName(const FVertexName& InName) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle GetInputNodeWithName(const FVertexName& InName) override { return INodeController::GetInvalidHandle(); }

			virtual FConstClassInputAccessPtr FindClassInputWithName(const FVertexName& InName) const override { return FConstClassInputAccessPtr(); }
			virtual FConstClassOutputAccessPtr FindClassOutputWithName(const FVertexName& InName) const override { return FConstClassOutputAccessPtr(); }

			virtual FNodeHandle AddInputVertex(const FMetasoundFrontendClassInput& InDescription) override { return INodeController::GetInvalidHandle(); }
			virtual bool RemoveInputVertex(const FVertexName& InputName) override { return false; }

			virtual FNodeHandle AddOutputVertex(const FMetasoundFrontendClassOutput& InDescription) override { return INodeController::GetInvalidHandle(); }
			virtual TSharedRef<INodeController> AddOutputVertex(const FVertexName& InName, const FName InTypeName) override { return INodeController::GetInvalidHandle(); }
			virtual bool RemoveOutputVertex(const FVertexName& OutputName) override { return false; }

			// This can be used to determine what kind of property editor we should use for the data type of a given input.
			// Will return Invalid if the input couldn't be found, or if the input doesn't support any kind of literals.
			virtual ELiteralType GetPreferredLiteralTypeForInputVertex(const FVertexName& InInputName) const override { return ELiteralType::Invalid; }

			// For inputs whose preferred literal type is UObject or UObjectArray, this can be used to determine the UObject corresponding to that input's datatype.
			virtual UClass* GetSupportedClassForInputVertex(const FVertexName& InInputName) override { return nullptr; }

			virtual FGuid GetVertexIDForInputVertex(const FVertexName& InInputName) const override { return Metasound::FrontendInvalidID; }
			virtual FGuid GetVertexIDForOutputVertex(const FVertexName& InOutputName) const override { return Metasound::FrontendInvalidID; }
			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID) const override { return FMetasoundFrontendLiteral{}; }

			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID, TArrayView<const FGuid> InPageOrder) const override { return FMetasoundFrontendLiteral{}; }

			// These can be used to set the default value for a given input on this graph.
			// @returns false if the input name couldn't be found, or if the literal type was incompatible with the Data Type of this input.

			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral) override { return false; }

			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, TArrayView<const FGuid> InPageOrder) override { return false; }

			virtual bool SetDefaultInputToDefaultLiteralOfType(const FGuid& InVertexID) override { return false; }

#if WITH_EDITOR
			virtual const FText& GetInputDescription(const FVertexName& InName) const override { return FText::GetEmpty(); }
			virtual const FText& GetOutputDescription(const FVertexName& InName) const override { return FText::GetEmpty(); }

			virtual void SetInputDescription(const FVertexName& InName, const FText& InDescription) override { }
			virtual void SetOutputDescription(const FVertexName& InName, const FText& InDescription) override { }
			virtual void SetInputDisplayName(const FVertexName& InName, const FText& InDisplayName) override { }
			virtual void SetOutputDisplayName(const FVertexName& InName, const FText& InDisplayName) override { }
#endif // WITH_EDITOR
#if WITH_EDITORONLY_DATA
			virtual int32 GetSortOrderIndexForInput(const FVertexName& InName) const { return 0; }
			virtual int32 GetSortOrderIndexForOutput(const FVertexName& InName) const { return 0; }
#endif // WITH_EDITORONLY_DATA
#if WITH_EDITOR
			virtual void SetSortOrderIndexForInput(const FVertexName& InName, int32 InSortOrderIndex) { }
			virtual void SetSortOrderIndexForOutput(const FVertexName& InName, int32 InSortOrderIndex) { }
#endif // WITH_EDITOR

			virtual FNodeHandle AddNode(const FNodeRegistryKey& InNodeClass, FGuid InNodeGuid) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddNode(const FMetasoundFrontendClassMetadata& InNodeClass, FGuid InNodeGuid) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddDuplicateNode(const INodeController& InNode) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddTemplateNode(const FNodeRegistryKey& InNodeClass, FMetasoundFrontendNodeInterface&& InNodeInterface, FGuid InNodeGuid) override { return INodeController::GetInvalidHandle(); }
			virtual FNodeHandle AddTemplateNode(const INodeTemplate& InNodeTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeGuid = FGuid::NewGuid()) override { return INodeController::GetInvalidHandle(); }

			UE_DEPRECATED(5.8, "Use the MetaSound Builder API")
			virtual bool RemoveNode(INodeController& InNode) override { return false; }

			// Returns the metadata for the current graph, including the name, description and author.
			virtual const FMetasoundFrontendClassMetadata& GetGraphMetadata() const override { return Invalid::GetInvalidClassMetadata(); }

			virtual void SetGraphMetadata(const FMetasoundFrontendClassMetadata& InMetadata) override { }

			virtual FNodeHandle CreateEmptySubgraph(const FMetasoundFrontendClassMetadata& InInfo) override { return INodeController::GetInvalidHandle(); }

			UE_API virtual FDocumentHandle GetOwningDocument() override;
			UE_API virtual FConstDocumentHandle GetOwningDocument() const override;

			virtual void UpdateInterfaceChangeID() override { }

			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithName(const FVertexName& InName) const override { return nullptr; }
			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithVertexID(const FGuid& InVertexID) const override { return nullptr; }
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithName(const FVertexName& InName) const override { return nullptr; }
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithVertexID(const FGuid& InVertexID) const override { return nullptr; }

		protected:
			virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
			virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }
		};

		/** FInvalidDocumentController is a controller which is always invalid. 
		 *
		 *  All methods return defaults and return error flags or invalid values.
		 */
		class FInvalidDocumentController : public IDocumentController 
		{
			public:
				FInvalidDocumentController() = default;
				virtual ~FInvalidDocumentController() = default;

				virtual bool IsValid() const override { return false; }

				virtual const TArray<FMetasoundFrontendClass>& GetDependencies() const override { return Invalid::GetInvalidClassArray(); }
				virtual void IterateDependencies(TFunctionRef<void(FMetasoundFrontendClass&)> InFunction) override { }
				virtual void IterateDependencies(TFunctionRef<void(const FMetasoundFrontendClass&)> InFunction) const override { }
				virtual const TArray<FMetasoundFrontendGraphClass>& GetSubgraphs() const override { return Invalid::GetInvalidGraphClassArray(); }
				virtual const FMetasoundFrontendGraphClass& GetRootGraphClass() const override { return Invalid::GetInvalidGraphClass(); }
				virtual void SetRootGraphClass(FMetasoundFrontendGraphClass&& InClass) override { }

				virtual FConstClassAccessPtr FindDependencyWithID(FGuid InClassID) const override { return FConstClassAccessPtr(); }
				virtual FConstGraphClassAccessPtr FindSubgraphWithID(FGuid InClassID) const override { return FConstGraphClassAccessPtr(); }
				virtual FConstClassAccessPtr FindClassWithID(FGuid InClassID) const override { return FConstClassAccessPtr(); }

				virtual FConstClassAccessPtr FindClass(const FNodeRegistryKey& InKey) const override { return FConstClassAccessPtr(); }
				virtual FConstClassAccessPtr FindOrAddClass(const FNodeRegistryKey& InKey, bool bInRefreshFromRegistry) override { return FConstClassAccessPtr(); }
				virtual FConstClassAccessPtr FindClass(const FMetasoundFrontendClassMetadata& InMetadata) const override{ return FConstClassAccessPtr(); }
				virtual FConstClassAccessPtr FindOrAddClass(const FMetasoundFrontendClassMetadata& InMetadata) override{ return FConstClassAccessPtr(); }
				virtual FGraphHandle AddDuplicateSubgraph(const IGraphController& InGraph) override { return IGraphController::GetInvalidHandle(); }

				virtual const TSet<FMetasoundFrontendVersion>& GetInterfaceVersions() const override { static const TSet<FMetasoundFrontendVersion> EmptyArray; return EmptyArray; }
				virtual void AddInterfaceVersion(const FMetasoundFrontendVersion& InVersion) override { }
				virtual void RemoveInterfaceVersion(const FMetasoundFrontendVersion& InVersion) override { }
				virtual void ClearInterfaceVersions() override { }

				virtual void SetMetadata(const FMetasoundFrontendDocumentMetadata& InMetadata) override { }
				virtual const FMetasoundFrontendDocumentMetadata& GetMetadata() const override { return Invalid::GetInvalidDocumentMetadata(); }
				virtual FMetasoundFrontendDocumentMetadata* GetMetadata() override { return nullptr; }

				virtual void RemoveUnreferencedDependencies() override { }
				virtual TArray<FConstClassAccessPtr> SynchronizeDependencyMetadata() override { return { }; }

				virtual TArray<FGraphHandle> GetSubgraphHandles() override { return TArray<FGraphHandle>(); }

				virtual TArray<FConstGraphHandle> GetSubgraphHandles() const override { return TArray<FConstGraphHandle>(); }

				virtual FGraphHandle GetSubgraphWithClassID(FGuid InClassID) override { return IGraphController::GetInvalidHandle(); }

				virtual FConstGraphHandle GetSubgraphWithClassID(FGuid InClassID) const override { return IGraphController::GetInvalidHandle(); }

				UE_API virtual TSharedRef<IGraphController> GetRootGraph() override;
				UE_API virtual TSharedRef<const IGraphController> GetRootGraph() const override;
				UE_API virtual FDocumentAccessPtr GetDocumentPtr() override;
				UE_API virtual const FDocumentAccessPtr GetDocumentPtr() const override;

				virtual bool ExportToJSONAsset(const FString& InAbsolutePath) const override { return false; }
				virtual FString ExportToJSON() const override { return FString(); }

			protected:

				virtual FDocumentAccess ShareAccess() override { return FDocumentAccess(); }
				virtual FConstDocumentAccess ShareAccess() const override { return FConstDocumentAccess(); }
		};
	}
}

#undef UE_API
