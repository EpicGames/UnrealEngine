// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendLiteral.h"

#include "CoreMinimal.h"
#include "MetasoundLog.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundFrontendRegistries.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendLiteral)

FString LexToString(Metasound::FLiteral::FNone InValue)
{
	return FString(TEXT(""));
}

namespace MetasoundFrontendLiteralPrivate
{
	// Default size of string for string builder stack allocation. Heap allocation is used if the string exceeds this size. 
	constexpr int32 ArrayStringAllocatedBytes = 256;

	// Defaults to calling LexToString for most literal data types.
	template<typename Type>
	struct TLiteralValueToStringHelper
	{
		static FString Convert(const Type& InType)
		{
			return LexToString(InType);
		}
	};

	// String conversion specialization for UObject*
	template<>
	struct TLiteralValueToStringHelper<TObjectPtr<UObject>>
	{
		static FString Convert(const TObjectPtr<UObject> InObject)
		{
			// Use empty string for null object as this is recognized
			// as empty value in editor context (the string "nullptr"
			// by contrast is not).
			if (!InObject)
			{
				return FString();
			}
			return InObject->GetPathName();
		}
	};

	// String conversion specialization for TArray<>
	template<typename ElementType>
	struct TLiteralValueToStringHelper<TArray<ElementType>>
	{
		using FArrayType = TArray<ElementType>;
		
		static FString Convert(const FArrayType& InArray)
		{
			TStringBuilder<ArrayStringAllocatedBytes> Builder;
			Builder << TEXT("[");

			for (int32 i = 0; i < InArray.Num(); i++)
			{
				if (i> 0)
				{
					Builder << TEXT(", ");
				}
				Builder << TLiteralValueToStringHelper<ElementType>::Convert(InArray[i]);
			}

			Builder << TEXT("]");

			return FString(Builder);
		}
	};

	template<typename Type>
	FString TLiteralValueToString(const Type& InType)
	{
		// Uses a helper struct to allow for partial specialization. 
		// (c++ does not allow partial specialization of functions)
		return TLiteralValueToStringHelper<Type>::Convert(InType);
	}
}

FMetasoundFrontendLiteral::FMetasoundFrontendLiteral(const Metasound::FLiteral& InLiteral)
{
	this->SetFromLiteral(InLiteral);
}

FMetasoundFrontendLiteral::FMetasoundFrontendLiteral(const FAudioParameter& InParameter)
{
	switch (InParameter.ParamType)
	{
		case EAudioParameterType::Trigger:
		case EAudioParameterType::Boolean:
		{
			Set(InParameter.BoolParam);
		}
		break;

		case EAudioParameterType::BooleanArray:
		{
			Set(InParameter.ArrayBoolParam);
		}
		break;

		case EAudioParameterType::Float:
		{
			Set(InParameter.FloatParam);
		}
		break;

		case EAudioParameterType::FloatArray:
		{
			Set(InParameter.ArrayFloatParam);
		}
		break;

		case EAudioParameterType::Integer:
		{
			Set(InParameter.IntParam);
		}
		break;

		case EAudioParameterType::IntegerArray:
		{
			Set(InParameter.ArrayIntParam);
		}
		break;

		case EAudioParameterType::None:
		{
			Set(FMetasoundFrontendLiteral::FDefault());
		}
		break;

		case EAudioParameterType::NoneArray:
		{
			Set(FMetasoundFrontendLiteral::FDefaultArray{ InParameter.IntParam });
		}
		break;

		case EAudioParameterType::Object:
		{
			Set(InParameter.ObjectParam);
		}
		break;

		case EAudioParameterType::ObjectArray:
		{
			Set(InParameter.ArrayObjectParam);
		}
		break;

		case EAudioParameterType::String:
		{
			Set(InParameter.StringParam);
		}
		break;

		case EAudioParameterType::StringArray:
		{
			Set(InParameter.ArrayStringParam);
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EAudioParameterType::COUNT) == 13, "Possible missing switch case coverage");
			checkNoEntry();
		}
	}
}


bool FMetasoundFrontendLiteral::IsArray() const
{
	switch (Type)
	{
		case EMetasoundFrontendLiteralType::NoneArray:
		case EMetasoundFrontendLiteralType::BooleanArray:
		case EMetasoundFrontendLiteralType::IntegerArray:
		case EMetasoundFrontendLiteralType::FloatArray:
		case EMetasoundFrontendLiteralType::StringArray:
		case EMetasoundFrontendLiteralType::UObjectArray:
			return true;

		case EMetasoundFrontendLiteralType::None:
		case EMetasoundFrontendLiteralType::Boolean:
		case EMetasoundFrontendLiteralType::Integer:
		case EMetasoundFrontendLiteralType::Float:
		case EMetasoundFrontendLiteralType::String:
		case EMetasoundFrontendLiteralType::UObject:
		case EMetasoundFrontendLiteralType::Invalid:
			return false;
			
		default:
			static_assert(static_cast<int32>(EMetasoundFrontendLiteralType::Invalid) == 12, "Possible missing EMetasoundFrontendLiteralType case coverage");
			{
				checkNoEntry();
				return false;
			}
	}
}

bool FMetasoundFrontendLiteral::IsEqual(const FMetasoundFrontendLiteral& InOther) const
{
	if (InOther.GetType() != GetType())
	{
		return false;
	}

	switch (Type)
	{
		case EMetasoundFrontendLiteralType::Boolean:
		case EMetasoundFrontendLiteralType::BooleanArray:
			return AsBoolean == InOther.AsBoolean;

		case EMetasoundFrontendLiteralType::Float:
		case EMetasoundFrontendLiteralType::FloatArray:
			return AsFloat == InOther.AsFloat;

		case EMetasoundFrontendLiteralType::Integer:
		case EMetasoundFrontendLiteralType::IntegerArray:
			return AsInteger == InOther.AsInteger;

		case EMetasoundFrontendLiteralType::None:
		case EMetasoundFrontendLiteralType::NoneArray:
			return AsNumDefault == InOther.AsNumDefault;

		case EMetasoundFrontendLiteralType::String:
		case EMetasoundFrontendLiteralType::StringArray:
			return AsString == InOther.AsString;

		case EMetasoundFrontendLiteralType::UObject:
		case EMetasoundFrontendLiteralType::UObjectArray:
			return AsUObject == InOther.AsUObject;

		case EMetasoundFrontendLiteralType::Invalid:
		default:
			return true;
	}
}

bool FMetasoundFrontendLiteral::IsValid() const
{
	return (Type != EMetasoundFrontendLiteralType::Invalid);
}

void FMetasoundFrontendLiteral::Set(FDefault InValue)
{
	Empty();

	Type = EMetasoundFrontendLiteralType::None;

	AsNumDefault = 1;
}

void FMetasoundFrontendLiteral::Set(FDefaultArray InValue)
{
	AsBoolean.Empty();
	AsInteger.Empty();
	AsFloat.Empty();
	AsString.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::NoneArray;
	AsNumDefault = FMath::Max(0, InValue.Num);
}

void FMetasoundFrontendLiteral::Set(bool InValue)
{
	Empty();

	Type = EMetasoundFrontendLiteralType::Boolean;

	AsBoolean.Add(InValue);
}

void FMetasoundFrontendLiteral::Set(TArray<bool> InValue)
{
	AsNumDefault = 0;
	AsInteger.Empty();
	AsFloat.Empty();
	AsString.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::BooleanArray;
	AsBoolean = MoveTemp(InValue);
}

void FMetasoundFrontendLiteral::Set(int32 InValue)
{
	Empty();

	Type = EMetasoundFrontendLiteralType::Integer;

	AsInteger.Add(InValue);
}

void FMetasoundFrontendLiteral::Set(TArray<int32> InValue)
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsFloat.Empty();
	AsString.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::IntegerArray;
	AsInteger = MoveTemp(InValue);
}

void FMetasoundFrontendLiteral::Set(float InValue)
{
	Empty();

	Type = EMetasoundFrontendLiteralType::Float;

	AsFloat.Add(InValue);
}

void FMetasoundFrontendLiteral::Set(TArray<float> InValue)
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsInteger.Empty();
	AsString.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::FloatArray;
	AsFloat = MoveTemp(InValue);
}

void FMetasoundFrontendLiteral::Set(FString InValue)
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsInteger.Empty();
	AsFloat.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::String;
	AsString = { MoveTemp(InValue) };
}

void FMetasoundFrontendLiteral::Set(TArray<FString> InValue)
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsInteger.Empty();
	AsFloat.Empty();
	AsUObject.Empty();

	Type = EMetasoundFrontendLiteralType::StringArray;
	AsString = MoveTemp(InValue);
}

void FMetasoundFrontendLiteral::Set(UObject* InValue)
{
	Empty();

	Type = EMetasoundFrontendLiteralType::UObject;

	AsUObject.Add(InValue);
}

void FMetasoundFrontendLiteral::Set(TArray<UObject*> InValue)
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsInteger.Empty();
	AsFloat.Empty();
	AsString.Empty();

	Type = EMetasoundFrontendLiteralType::UObjectArray;
	AsUObject = MoveTemp(InValue);
}

void FMetasoundFrontendLiteral::SetFromLiteral(const Metasound::FLiteral& InLiteral)
{
	using namespace Metasound;

	Clear();
	switch (InLiteral.GetType())
	{
		case ELiteralType::None:
		{
			Set(FMetasoundFrontendLiteral::FDefault{});
		}
		break;

		case ELiteralType::Boolean:
		{
			Set(InLiteral.Value.Get<bool>());
		}
		break;

		case ELiteralType::Float:
		{
			Set(InLiteral.Value.Get<float>());
		}
		break;

		case ELiteralType::Integer:
		{
			Set(InLiteral.Value.Get<int32>());
		}
		break;

		case ELiteralType::String:
		{
			Set(InLiteral.Value.Get<FString>());
		}
		break;

		case ELiteralType::UObjectProxy:
		{
			// Only error if attempting to retrieve valid UObject from ProxyDataPtr
			// as this function can safely is used to initialize from defaults (which
			// is valid as a null proxy can safely correspond to a null UObject ptr).
			if (InLiteral.Value.Get<TSharedPtr<Audio::IProxyData>>().IsValid())
			{
				UE_LOGF(LogMetaSound, Error, "Cannot set UObjectProxy from Metasound::FLiteral");
			}
			Set(static_cast<UObject*>(nullptr));
		}
		break;

		case ELiteralType::NoneArray:
		{
			int32 Num = InLiteral.Value.Get<TArray<FLiteral::FNone>>().Num(); 
			Set(FMetasoundFrontendLiteral::FDefaultArray{ Num });
		}
		break;

		case ELiteralType::BooleanArray:
		{
			Set(InLiteral.Value.Get<TArray<bool>>());
		}
		break;

		case ELiteralType::IntegerArray:
		{
			Set(InLiteral.Value.Get<TArray<int32>>());
		}
		break;

		case ELiteralType::FloatArray:
		{
			Set(InLiteral.Value.Get<TArray<float>>());
		}
		break;

		case ELiteralType::StringArray:
		{
			Set(InLiteral.Value.Get<TArray<FString>>());
		}
		break;

		case ELiteralType::UObjectProxyArray:
		{
			// Only error if attempting to retrieve valid UObject from ProxyDataPtr
			// as this function can safely is used to initialize from defaults (which
			// is valid as a null proxy can safely correspond to a null UObject ptr).
			if (!InLiteral.Value.Get<TArray<TSharedPtr<Audio::IProxyData>>>().IsEmpty())
			{
				UE_LOGF(LogMetaSound, Error, "Cannot set UObjectProxy from Metasound::FLiteral");
			}
			Set(TArray<UObject*>());
		}
		break;

		case ELiteralType::Invalid:
		default:
		{
			static_assert(static_cast<int32>(ELiteralType::Invalid) == 12, "Possible missing literal type switch coverage");
		}
	}
}

void FMetasoundFrontendLiteral::SetType(EMetasoundFrontendLiteralType InType)
{
	switch (InType)
	{
		case EMetasoundFrontendLiteralType::NoneArray:
		{
			Set(FDefaultArray());
		}
		break;

		case EMetasoundFrontendLiteralType::BooleanArray:
		{
			Set(TArray<bool>());
		}
		break;

		case EMetasoundFrontendLiteralType::IntegerArray:
		{
			Set(TArray<int32>());
		}
		break;

		case EMetasoundFrontendLiteralType::FloatArray:
		{
			Set(TArray<float>());
		}
		break;

		case EMetasoundFrontendLiteralType::StringArray:
		{
			Set(TArray<FString>());
		}
		break;

		case EMetasoundFrontendLiteralType::UObjectArray:
		{
			Set(TArray<UObject*>());
		}
		break;
		
		case EMetasoundFrontendLiteralType::None:
		{
			Set(FDefault());
		}
		break;

		case EMetasoundFrontendLiteralType::Boolean:
		{
			Set(false);
		}
		break;

		case EMetasoundFrontendLiteralType::Integer:
		{
			Set(0);
		}
		break;

		case EMetasoundFrontendLiteralType::Float:
		{
			Set(0.f);
		}
		break;
	
		case EMetasoundFrontendLiteralType::String:
		{
			Set(FString());
		}
		break;
		
		case EMetasoundFrontendLiteralType::UObject:
		{
			Set(static_cast<UObject*>(nullptr));
		}
		break;
		
		case EMetasoundFrontendLiteralType::Invalid:
		{
			Set(FDefault());
		}
		break;
		
		default:
		static_assert(static_cast<int32>(EMetasoundFrontendLiteralType::Invalid) == 12, "Possible missing EMetasoundFrontendLiteralType case coverage");
		{
			checkNoEntry();
		}
		break;
	}
}

bool FMetasoundFrontendLiteral::TryGet(UObject*& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::UObject)
	{
		if (ensure(!AsUObject.IsEmpty()))
		{
			OutValue = AsUObject[0];
			return true;
		}
	}

	return false;
}

bool FMetasoundFrontendLiteral::TryGet(TArray<UObject*>& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::UObjectArray)
	{
		OutValue = AsUObject;
		return true;
	}

	return false;
}

bool FMetasoundFrontendLiteral::TryGet(bool& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::Boolean)
	{
		OutValue = AsBoolean[0];
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(TArray<bool>& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::BooleanArray)
	{
		OutValue = AsBoolean;
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(int32& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::Integer)
	{
		OutValue = AsInteger[0];
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(TArray<int32>& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::IntegerArray)
	{
		OutValue = AsInteger;
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(float& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::Float)
	{
		OutValue = AsFloat[0];
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(TArray<float>& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::FloatArray)
	{
		OutValue = AsFloat;
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(FString& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::String)
	{
		OutValue = AsString[0];
		return true;
	}
	return false;
}

bool FMetasoundFrontendLiteral::TryGet(TArray<FString>& OutValue) const
{
	if (Type == EMetasoundFrontendLiteralType::StringArray)
	{
		OutValue = AsString;
		return true;
	}
	return false;
}

Metasound::FLiteral FMetasoundFrontendLiteral::ToLiteral(const FName& InMetasoundDataTypeName, const Metasound::Frontend::IDataTypeRegistry* InDataTypeRegistry, const Metasound::Frontend::FProxyDataCache* InProxyDataCache) const
{
	using namespace Metasound;
	using namespace Metasound::Frontend;

	if (EMetasoundFrontendLiteralType::UObject == Type)
	{
		if (ensure(AsUObject.Num() > 0))
		{
			if (InProxyDataCache)
			{
				if (const TSharedPtr<Audio::IProxyData>* ProxyData = InProxyDataCache->FindProxy(AsUObject[0]))
				{
					return FLiteral(*ProxyData);
				}
				else
				{
					return FLiteral(TSharedPtr<Audio::IProxyData>());
				}
			}
			else
			{
				if (!InDataTypeRegistry)
				{
					InDataTypeRegistry = &IDataTypeRegistry::Get();
				}

				// UObject proxies must go through the registry. The registry contains the information
				// needed to convert UObjects to proxies. 
				return InDataTypeRegistry->CreateLiteralFromUObject(InMetasoundDataTypeName, AsUObject[0]);
			}
		}

		return FLiteral(TSharedPtr<Audio::IProxyData>());
	}
	else if (EMetasoundFrontendLiteralType::UObjectArray == Type)
	{
		if (InProxyDataCache)
		{
			TArray<TSharedPtr<Audio::IProxyData>> Proxies;
			for (const UObject* Object : AsUObject)
			{
				if (const TSharedPtr<Audio::IProxyData>* ProxyData = InProxyDataCache->FindProxy(Object))
				{
					Proxies.Add(*ProxyData);
				}
				else
				{
					Proxies.Add(TSharedPtr<Audio::IProxyData>());
				}
			}
			return FLiteral(Proxies);
		}
		else
		{
			if (!InDataTypeRegistry)
			{
				InDataTypeRegistry = &IDataTypeRegistry::Get();
			}
			// UObject proxies must go through the registry. The registry contains the information
			// needed to convert UObjects to proxies. 
			return InDataTypeRegistry->CreateLiteralFromUObjectArray(InMetasoundDataTypeName, AsUObject);
		}
	}
	else 
	{
		// Use default conversions for core literal types.
		return ToLiteralNoProxy();
	}
}

Metasound::FLiteral FMetasoundFrontendLiteral::ToLiteralNoProxy() const
{
	using namespace Metasound;

	if ((EMetasoundFrontendLiteralType::UObject == Type) || (EMetasoundFrontendLiteralType::UObjectArray == Type))
	{
		UE_LOGF(LogMetaSound, Error, "Cannot convert FMetasoundFrontendLiteral to Metasound::Literal without data type name [Literal:%ls]", *ToString());
		return FLiteral::CreateInvalid();
	}

	switch (Type)
	{
		case EMetasoundFrontendLiteralType::None:
			return FLiteral(FLiteral::FNone{});

		case EMetasoundFrontendLiteralType::Boolean:
			return FLiteral(AsBoolean[0]);

		case EMetasoundFrontendLiteralType::Integer:
			return FLiteral(AsInteger[0]);
			
		case EMetasoundFrontendLiteralType::Float:
			return FLiteral(AsFloat[0]);

		case EMetasoundFrontendLiteralType::String:
			return FLiteral(AsString[0]);

		case EMetasoundFrontendLiteralType::NoneArray:
		{
			TArray<FLiteral::FNone> Nones;
			Nones.SetNum(AsNumDefault);
			return FLiteral(Nones);
		}

		case EMetasoundFrontendLiteralType::BooleanArray:
			return FLiteral(AsBoolean);

		case EMetasoundFrontendLiteralType::IntegerArray:
			return FLiteral(AsInteger);

		case EMetasoundFrontendLiteralType::FloatArray:
			return FLiteral(AsFloat);

		case EMetasoundFrontendLiteralType::StringArray:
			return FLiteral(AsString);

		case EMetasoundFrontendLiteralType::UObject:
		case EMetasoundFrontendLiteralType::UObjectArray:
		case EMetasoundFrontendLiteralType::Invalid:
			return FLiteral::CreateInvalid();

		default:
			static_assert(static_cast<int32>(EMetasoundFrontendLiteralType::Invalid) == 12, "Possible missing literal type switch coverage");
			return FLiteral::CreateInvalid();
	}
}

FString FMetasoundFrontendLiteral::ToString() const
{
	using namespace Metasound;
	using namespace MetasoundFrontendLiteralPrivate;

	switch (Type)
	{
		case EMetasoundFrontendLiteralType::None:
			return TLiteralValueToString(FLiteral::FNone{});

		case EMetasoundFrontendLiteralType::Boolean:
			return TLiteralValueToString(AsBoolean[0]);

		case EMetasoundFrontendLiteralType::Integer:
			return TLiteralValueToString(AsInteger[0]);

		case EMetasoundFrontendLiteralType::Float:
			return TLiteralValueToString(AsFloat[0]);

		case EMetasoundFrontendLiteralType::String:
			return TLiteralValueToString(AsString[0]);

		case EMetasoundFrontendLiteralType::UObject:
			return TLiteralValueToString(AsUObject[0]);

		case EMetasoundFrontendLiteralType::NoneArray:
		{
			TArray<FLiteral::FNone> Nones;
			Nones.SetNum(AsNumDefault);
			return TLiteralValueToString(Nones);
		}

		case EMetasoundFrontendLiteralType::BooleanArray:
			return TLiteralValueToString(AsBoolean);

		case EMetasoundFrontendLiteralType::IntegerArray:
			return TLiteralValueToString(AsInteger);

		case EMetasoundFrontendLiteralType::FloatArray:
			return TLiteralValueToString(AsFloat);

		case EMetasoundFrontendLiteralType::StringArray:
			return TLiteralValueToString(AsString);

		case EMetasoundFrontendLiteralType::UObjectArray:
			return TLiteralValueToString(AsUObject);

		case EMetasoundFrontendLiteralType::Invalid:
		default:
			static_assert(static_cast<int32>(EMetasoundFrontendLiteralType::Invalid) == 12, "Possible missing literal type switch coverage");
			return FString();
	}
}

int32 FMetasoundFrontendLiteral::GetArrayNum() const
{
	switch (Type)
	{
	case EMetasoundFrontendLiteralType::None:
	case EMetasoundFrontendLiteralType::Boolean:
	case EMetasoundFrontendLiteralType::Integer:
	case EMetasoundFrontendLiteralType::Float:
	case EMetasoundFrontendLiteralType::String:
	case EMetasoundFrontendLiteralType::UObject:
		return INDEX_NONE;

	case EMetasoundFrontendLiteralType::NoneArray:
		return AsNumDefault;

	case EMetasoundFrontendLiteralType::BooleanArray:
		return AsBoolean.Num();

	case EMetasoundFrontendLiteralType::IntegerArray:
		return AsInteger.Num();

	case EMetasoundFrontendLiteralType::FloatArray:
		return AsFloat.Num();

	case EMetasoundFrontendLiteralType::StringArray:
		return AsString.Num();

	case EMetasoundFrontendLiteralType::UObjectArray:
		return AsUObject.Num();

	case EMetasoundFrontendLiteralType::Invalid:
	default:
		static_assert(static_cast<int32>(EMetasoundFrontendLiteralType::Invalid) == 12, "Possible missing literal type switch coverage");
		return INDEX_NONE;
	}
}

EMetasoundFrontendLiteralType FMetasoundFrontendLiteral::GetType() const
{
	return Type;
}

void FMetasoundFrontendLiteral::Clear()
{
	Empty();
	Type = EMetasoundFrontendLiteralType::None;
	AsNumDefault = 1;
}

FMetasoundFrontendLiteral FMetasoundFrontendLiteral::GetInvalid()
{
	FMetasoundFrontendLiteral InvalidValue;
	InvalidValue.Type = EMetasoundFrontendLiteralType::Invalid;
	return InvalidValue;
}

void FMetasoundFrontendLiteral::Empty()
{
	AsNumDefault = 0;
	AsBoolean.Empty();
	AsInteger.Empty();
	AsFloat.Empty();
	AsString.Empty();
	AsUObject.Empty();
}

namespace Metasound::Frontend
{
	// Convenience function to convert Metasound::ELiteralType to EMetasoundFrontendLiteralType, the UEnum used for metasound documents.
	EMetasoundFrontendLiteralType GetMetasoundFrontendLiteralType(ELiteralType InLiteralType)
	{
		static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::Invalid) == static_cast<uint8>(ELiteralType::Invalid), "Possible literal type enum mismatch");
		return static_cast<EMetasoundFrontendLiteralType>(InLiteralType);
	}

	// Convenience function to convert Metasound::ELiteralType to EMetasoundFrontendLiteralType, the UEnum used for metasound documents.
	ELiteralType GetMetasoundLiteralType(EMetasoundFrontendLiteralType InLiteralType)
	{
		static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::Invalid) == static_cast<uint8>(ELiteralType::Invalid), "Possible literal type enum mismatch");
		return static_cast<ELiteralType>(InLiteralType);
	}
} // namespace Metasound::Frontend
