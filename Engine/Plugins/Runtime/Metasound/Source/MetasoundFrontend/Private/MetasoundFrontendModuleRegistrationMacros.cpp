// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"

namespace Metasound::Frontend::RegistrationPrivate
{
	FModuleInfo FGlobalRegistrationList::ModuleInfo = 
	{
#if WITH_EDITORONLY_DATA
		"UnknownPlugin",
		"UnknownModule"
#endif // if WITH_EDITORONLY_DATA
	};

	FRegistrationAction::FRegistrationAction(FRegistrationAction*& InOutHead, FRegistrationFunction InRegister, FRegistrationFunction InUnregister)
	: Register(InRegister)
	, Unregister(InUnregister)
	{
		Next = InOutHead;
		InOutHead = this;
	}

	void FRegistrationListBase::RegisterAll(const Metasound::Frontend::FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head)
	{
		METASOUND_LLM_SCOPE 

#if WITH_EDITORONLY_DATA
		UE_LOGF(LogMetaSound, Verbose, "Running registration actions for module %ls in plugin %ls", *InOwningModuleInfo.ModuleName.ToString(), *InOwningModuleInfo.PluginName.ToString());
#endif // if WITH_EDITORONLY_DATA
		for (FRegistrationAction* Action = Head; Action != nullptr; Action = Action->Next)
		{
#if WITH_EDITORONLY_DATA
			UE_LOGF(LogMetaSound, VeryVerbose, "Running Register:%ls for module %ls in plugin %ls", Action->Name, *InOwningModuleInfo.ModuleName.ToString(), *InOwningModuleInfo.PluginName.ToString());
#endif
			if (Action->Register)
			{
				Action->Register(InOwningModuleInfo);
			}
		}
	}

	void FRegistrationListBase::UnregisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head)
	{
		METASOUND_LLM_SCOPE 

#if WITH_EDITORONLY_DATA
		UE_LOGF(LogMetaSound, Verbose, "Running unregistration actions for module %ls in plugin %ls", *InOwningModuleInfo.ModuleName.ToString(), *InOwningModuleInfo.PluginName.ToString());
#endif // if WITH_EDITORONLY_DATA
		for (FRegistrationAction* Action = Head; Action != nullptr; Action = Action->Next)
		{
#if WITH_EDITORONLY_DATA
			UE_LOGF(LogMetaSound, VeryVerbose, "Running Unregister:%ls for module %ls in plugin %ls", Action->Name, *InOwningModuleInfo.ModuleName.ToString(), *InOwningModuleInfo.PluginName.ToString());
#endif
			if (Action->Unregister)
			{
				Action->Unregister(InOwningModuleInfo);
			}
		}
	}

	FRegistrationAction*& FGlobalRegistrationList::GetHeadAction()
	{
		static FRegistrationAction* Head = nullptr;
		return Head;
	}

	void FGlobalRegistrationList::RegisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head)
	{
		METASOUND_LLM_SCOPE 

#if WITH_EDITORONLY_DATA
		UE_LOGF(LogMetaSound, Verbose, "Running pending registration actions for global registration list");
#endif // if WITH_EDITORONLY_DATA
	  
		// Global registration needs to handle the scenario where it is called
		// multiple times from different modules. It has to protect against double
		// registration of nodes. 
		
		FRegistrationAction* End = GetRegisterTailAction();
		
		for (FRegistrationAction* Action = Head; (Action != End) && (Action != nullptr); Action = Action->Next)
		{
#if WITH_EDITORONLY_DATA
			UE_LOGF(LogMetaSound, VeryVerbose, "Running Register:%ls for global registration list", Action->Name);
#endif
			if (Action->Register)
			{
				Action->Register(InOwningModuleInfo);
			}
		}

		// Track nodes that have been registered
		GetRegisterTailAction() = Head;
	}

	void FGlobalRegistrationList::UnregisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head)
	{
		// Unregister on the global registration list is not supported. 
		checkNoEntry();
	}

	FRegistrationAction*& FGlobalRegistrationList::GetRegisterTailAction()
	{
		static FRegistrationAction* Tail = nullptr;
		return Tail;
	}
}

