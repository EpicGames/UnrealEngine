// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeClassRegistryPrivate.h"

#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "HAL/PlatformTime.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "StructUtils/InstancedStruct.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendGraph.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendNodeUpdateTransform.h"
#include "MetasoundFrontendPages.h"
#include "MetasoundFrontendProxyDataCache.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundGraphNode.h"
#include "MetasoundLog.h"
#include "MetasoundRouter.h"
#include "MetasoundTrace.h"

#ifndef UE_METASOUND_DISABLE_5_6_NODE_REGISTRATION_DEPRECATION_WARNINGS
#define UE_METASOUND_DISABLE_5_6_NODE_REGISTRATION_DEPRECATION_WARNINGS (0)
#endif

namespace Metasound::Frontend
{
	namespace ConsoleVariables
	{
		static bool bDisableAsyncGraphRegistration = false;
		FAutoConsoleVariableRef CVarMetaSoundDisableAsyncGraphRegistration(
			TEXT("au.MetaSound.DisableAsyncGraphRegistration"),
			Metasound::Frontend::ConsoleVariables::bDisableAsyncGraphRegistration,
			TEXT("Disables async registration of MetaSound graphs\n")
			TEXT("Default: false"),
			ECVF_Default);
	} // namespace ConsoleVariables

	namespace RegistryPrivate
	{
		TScriptInterface<IMetaSoundDocumentInterface> BuildRegistryDocument(TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface, bool bAsync)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::BuildRegistryDocument);

			UObject* DocObject = DocumentInterface.GetObject();
			check(DocObject);
			const FMetasoundFrontendDocument& Document = DocumentInterface->GetConstDocument();

#if WITH_EDITOR
			DocumentInterface = &UMetaSoundBuilderDocument::Create(*DocumentInterface.GetInterface());
			FMetaSoundFrontendDocumentBuilder Builder(DocumentInterface);
			Builder.TransformTemplateNodes();
			return DocumentInterface;
#else
			const bool bIsBuilding = DocumentInterface->IsActivelyBuilding();
			const bool bForceCopy = bIsBuilding && bAsync;

	#if !NO_LOGGING
		// Force a copy if async registration is enabled and we need to protect against race conditions from external modifications.

		#if WITH_EDITORONLY_DATA
			// Only assets require template node processing and support document attachment
			if (DocObject->IsAsset())
			{
				const FMetaSoundFrontendDocumentBuilder& OriginalDocBuilder = IDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(DocumentInterface);
				const bool bContainsTemplateDependency = OriginalDocBuilder.ContainsDependencyOfType(EMetasoundFrontendClassType::Template);
				if (bContainsTemplateDependency)
				{
					UE_LOGF(LogMetaSound, Error,
						"Template node processing disabled but provided asset class at '%ls' to register contains template nodes. Runtime graph will fail to build.",
						*OriginalDocBuilder.GetDebugName());
				}

				// Destroy builder if one didn't exist before running template check to ensure
				// that builder existence doesn't inadvertently cause potential future re-registration
				// calls to perform unnecessary document copy below.
				if (!bIsBuilding)
				{
					const FMetasoundFrontendClassName& ClassName = Document.RootGraph.Metadata.GetClassName();
					IDocumentBuilderRegistry::GetChecked().FinishBuilding(ClassName, OriginalDocBuilder.GetHintPath());
				}
			}
		#endif // WITH_EDITORONLY_DATA
	#endif // !NO_LOGGING

			if (bForceCopy)
			{
				return &UMetaSoundBuilderDocument::Create(*DocumentInterface.GetInterface());
			}
			else
			{
				return DocumentInterface;
			}
#endif // WITH_EDITOR
		}

		// FDocumentNodeClassRegistryEntry encapsulates a node registry entry for a FGraph
		class FDocumentNodeClassRegistryEntry : public INodeClassRegistryEntry
		{
		public:
			FDocumentNodeClassRegistryEntry(
				const FMetasoundFrontendGraphClass& InGraphClass,
				const TSet<FMetasoundFrontendVersion>& InInterfaces,
				TSharedPtr<const IGraph> InGraph,
				FTopLevelAssetPath InAssetPath)
			: FrontendClass(InGraphClass)
			, Interfaces(InInterfaces)
			, Graph(InGraph)
			, AssetPath(InAssetPath)
			{
				FrontendClass.Metadata.SetType(EMetasoundFrontendClassType::External);
			}

			FDocumentNodeClassRegistryEntry(const FDocumentNodeClassRegistryEntry&) = default;

			virtual ~FDocumentNodeClassRegistryEntry() = default;

			virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
			{
				if (Graph.IsValid())
				{
					return MakeUnique<FGraphNode>(MoveTemp(InNodeData), Graph.ToSharedRef());
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot create MetaSound node from asset %ls due to prior failure to build graph", *AssetPath.ToString());
					return TUniquePtr<INode>();
				}
			}

			virtual const FMetasoundFrontendClass& GetFrontendClass() const override
			{
				return FrontendClass;
			}

			virtual const TSet<FMetasoundFrontendVersion>* GetImplementedInterfaces() const override
			{
				return &Interfaces;
			}

			virtual FVertexInterface GetDefaultVertexInterface() const override
			{
				if (ensure(Graph.IsValid()))
				{
					return FVertexInterface(Graph->GetMetadata().DefaultInterface);
				}
				else
				{
					return FVertexInterface();
				}
			}

			virtual const FClassInterface& GetClassInterface() const override
			{
				if (ensure(Graph.IsValid()))
				{
					return Graph->GetMetadata().DefaultInterface;
				}
				else
				{
					static const FClassInterface EmptyClassInterface;
					return EmptyClassInterface;
				}
			}

			virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const override final
			{
				// Document based nodes do not support node configuration because
				// many MetaSound systems assume that a graph defined in a FMetasoundFrontendDocument
				// only supplies a default interface. The use of class interface
				// overrides in FMetasoundFrontendDocument based nodes is unsupported. 
				return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
			}

			virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override
			{
				// No node configuration supported for this node type, so only compatible if setting to invalid (null) configuration
				return !InNodeConfiguration.IsValid();
			}

		private:
			FMetasoundFrontendClass FrontendClass;
			TSet<FMetasoundFrontendVersion> Interfaces;
			TSharedPtr<const IGraph> Graph;
			FTopLevelAssetPath AssetPath;
		};
	} // namespace RegistryPrivate
	
	const FNodeClassInfo& INodeTemplateRegistryEntry::GetClassInfo() const
	{
		return FNodeClassInfo::GetInvalid();
	}

#if !UE_METASOUND_PURE_VIRTUAL_CREATE_FRONTEND_NODE_EXTENSION
	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> INodeClassRegistryEntry::CreateFrontendNodeConfiguration() const
	{
		static bool bDidWarn = false;
		if (!bDidWarn)
		{
			UE_LOGF(LogMetaSound, Warning,
				"Please implement INodeClassRegistryEntry::CreateFrontendNodeConfiguration for the registry entry class representing node %ls. "
					"This method will become pure virtual in future releases. Define UE_METASOUND_PURE_VIRTUAL_CREATE_FRONTEND_NODE_EXTENSION in order "
					"to build with this method as a pure virtual on the interface.",
				*GetFrontendClass().Metadata.GetClassName().ToString());
		}
		return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
	}
#endif

#if WITH_EDITORONLY_DATA
		FName INodeClassRegistryEntry::GetPluginName() const
		{
			// Default implementation returns default FName
			return FName();
		}

		FName INodeClassRegistryEntry::GetModuleName() const
		{
			// Default implementation returns default FName
			return FName();
		}
#endif

	FNodeClassRegistryTransaction::FNodeClassRegistryTransaction(ETransactionType InType, const FNodeClassInfo& InNodeClassInfo, FNodeClassRegistryTransaction::FTimeType InTimestamp)
		: Type(InType)
		, RegistryKey()
		, Timestamp(InTimestamp)
	{
	}

	FNodeClassRegistryTransaction::FNodeClassRegistryTransaction(ETransactionType InType, FNodeClassRegistryKey InRegistryKey, FNodeClassRegistryTransaction::FTimeType InTimestamp)
	: Type(InType)
	, RegistryKey(MoveTemp(InRegistryKey))
	, Timestamp(InTimestamp)
	{
	}

	FNodeClassRegistryTransaction::ETransactionType FNodeClassRegistryTransaction::GetTransactionType() const
	{
		return Type;
	}

	const FNodeClassInfo& FNodeClassRegistryTransaction::GetNodeClassInfo() const
	{
		return FNodeClassInfo::GetInvalid();
	}

	FNodeClassRegistryKey FNodeClassRegistryTransaction::GetNodeRegistryKey() const
	{
		return RegistryKey;
	}

	FNodeClassRegistryTransaction::FTimeType FNodeClassRegistryTransaction::GetTimestamp() const
	{
		return Timestamp;
	}

	namespace NodeClassRegistryKey
	{
		FNodeClassRegistryKey CreateKey(EMetasoundFrontendClassType InType, const FString& InFullClassName, int32 InMajorVersion, int32 InMinorVersion)
		{
			if (InType == EMetasoundFrontendClassType::Graph)
			{
				// No graphs are registered. Any registered graph should be registered as an external node.
				InType = EMetasoundFrontendClassType::External;
			}

			FMetasoundFrontendClassName ClassName;
			FMetasoundFrontendClassName::Parse(InFullClassName, ClassName);
			return FNodeClassRegistryKey(InType, ClassName, InMajorVersion, InMinorVersion);
		}

		const FNodeClassRegistryKey& GetInvalid()
		{
			return FNodeClassRegistryKey::GetInvalid();
		}

		bool IsValid(const FNodeClassRegistryKey& InKey)
		{
			return InKey.IsValid();
		}

		bool IsEqual(const FNodeClassRegistryKey& InLHS, const FNodeClassRegistryKey& InRHS)
		{
			return InLHS == InRHS;
		}

		bool IsEqual(const FMetasoundFrontendClassMetadata& InLHS, const FMetasoundFrontendClassMetadata& InRHS)
		{
			if (InLHS.GetClassName() == InRHS.GetClassName())
			{
				if (InLHS.GetType() == InRHS.GetType())
				{
					if (InLHS.GetVersion() == InRHS.GetVersion())
					{
						return true;
					}
				}
			}
			return false;
		}

		bool IsEqual(const FNodeClassInfo& InLHS, const FMetasoundFrontendClassMetadata& InRHS)
		{
			if (InLHS.ClassName == InRHS.GetClassName())
			{
				if (InLHS.Type == InRHS.GetType())
				{
					if (InLHS.Version == InRHS.GetVersion())
					{
						return true;
					}
				}
			}
			return false;
		}

		FNodeClassRegistryKey CreateKey(const FNodeClassMetadata& InNodeMetadata)
		{
			return FNodeClassRegistryKey(InNodeMetadata);
		}

		FNodeClassRegistryKey CreateKey(const FMetasoundFrontendClassMetadata& InNodeMetadata)
		{
			checkf(InNodeMetadata.GetType() != EMetasoundFrontendClassType::Graph, TEXT("Cannot create key from 'graph' type. Likely meant to use CreateKey overload that is provided FMetasoundFrontendGraphClass"));
			return FNodeClassRegistryKey(InNodeMetadata);
		}

		FNodeClassRegistryKey CreateKey(const FMetasoundFrontendGraphClass& InGraphClass)
		{
			return FNodeClassRegistryKey(InGraphClass);
		}
	} // namespace NodeClassRegistryKey
	  

	void FNodeClassRegistry::BuildAndRegisterGraphFromDocument(const FMetasoundFrontendDocument& InDocument, const FProxyDataCache& InProxyDataCache, const FGraphRegistryKey& InGraphRegistryKey, TArrayView<const FGuid> InPageOrder)
	{
		using namespace RegistryPrivate;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FNodeClassRegistry::BuildAndRegisterGraphFromDocument);
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Metasound::FNodeClassRegistry::BuildAndRegisterGraphFromDocument asset %s"), *InGraphRegistryKey.AssetPath.GetAssetName().ToString()));

		FGuid AssetClassID;
		if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
		{
			ensureAlways(AssetManager->TryGetAssetIDFromClassName(InGraphRegistryKey.NodeKey.ClassName, AssetClassID));
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "No AssetManager registered, registering bespoke AssetClassID '%ls' for asset '%ls'", *AssetClassID.ToString(), *InGraphRegistryKey.AssetPath.ToString());
			AssetClassID = FGuid::NewGuid();
		}

		// Use the asset class id for the graph id because it should be locally unique per asset.
		TUniquePtr<FFrontendGraph> FrontendGraph = Frontend::FGraphBuilder::CreateGraph(InDocument, InProxyDataCache, InGraphRegistryKey.AssetPath, /*GraphId=*/ AssetClassID, InPageOrder);
		if (!FrontendGraph.IsValid())
		{
			UE_LOGF(LogMetaSound, Error, "Failed to build MetaSound graph in asset '%ls'", *InGraphRegistryKey.AssetPath.ToString());
		}

		TSharedPtr<const FGraph> GraphToRegister(FrontendGraph.Release());
		TUniquePtr<INodeClassRegistryEntry> RegistryEntry = MakeUnique<FDocumentNodeClassRegistryEntry>(
			InDocument.RootGraph,
			InDocument.Interfaces,
			GraphToRegister,
			InGraphRegistryKey.AssetPath);

		const FNodeClassRegistryKey NodeRegistryKey = RegisterNodeInternal(MoveTemp(RegistryEntry));
		checkf(NodeRegistryKey == InGraphRegistryKey.NodeKey, TEXT("Mismatched between expected node registry key for registered MetaSound graph. Expected %s but found %s"), *InGraphRegistryKey.NodeKey.ToString(), *NodeRegistryKey.ToString());

		// Key must use the graph registry key provided to this function and *NOT* one created 
		// from the document's owning DocumentInterface object, as that may have been 
		// built/optimized using a transient object with a different, transient asset path 
		// and different UObject UniqueID.
		RegisterGraphInternal(InGraphRegistryKey, GraphToRegister);
	}

	FNodeClassRegistry* FNodeClassRegistry::LazySingleton = nullptr;

	FNodeClassRegistry& FNodeClassRegistry::Get()
	{
		if (!LazySingleton)
		{
			LazySingleton = new Metasound::Frontend::FNodeClassRegistry();
		}

		return *LazySingleton;
	}

	void FNodeClassRegistry::Shutdown()
	{
		if (nullptr != LazySingleton)
		{
			delete LazySingleton;
			LazySingleton = nullptr;
		}
	}

	FNodeClassRegistry::FNodeClassRegistry()
	: TransactionBuffer(MakeShared<FNodeClassRegistryTransactionBuffer>())
	, AsyncRegistrationPipe( UE_SOURCE_LOCATION )
	{
	}

	void FNodeClassRegistry::RegisterPendingNodes()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(metasound::FNodeClassRegistry::RegisterPendingNodes);
		{
			UE::TScopeLock ScopeLock(LazyInitCommandCritSection);

			for (TUniqueFunction<void()>& Command : LazyInitCommands)
			{
				Command();
			}

			LazyInitCommands.Empty();
		}

		// For backwards compatibility, also register global registration list
		// to mimic behavior before the introduction of the module specific registration
		// lists.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RegistrationPrivate::FGlobalRegistrationList::RegisterAll(FModuleInfo{}, RegistrationPrivate::FGlobalRegistrationList::GetHeadAction());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!IsRunningCommandlet())
		{
			// Prime search engine after bulk registration.
			ISearchEngine::Get().Prime();
		}
	}

	bool FNodeClassRegistry::EnqueueInitCommand(TUniqueFunction<void()>&& InFunc)
	{

		UE::TScopeLock ScopeLock(LazyInitCommandCritSection);
		if (LazyInitCommands.Num() >= MaxNumNodesAndDatatypesToInitialize)
		{
			UE_LOGF(LogMetaSound, Warning, "Registering more that %d nodes and datatypes for metasounds! Consider increasing MetasoundFrontendRegistryContainer::MaxNumNodesAndDatatypesToInitialize.", MaxNumNodesAndDatatypesToInitialize);
		}

		LazyInitCommands.Add(MoveTemp(InFunc));
		return true;
	}

	void FNodeClassRegistry::SetObjectReferencer(TUniquePtr<IObjectReferencer> InReferencer)
	{
		UE::TScopeLock LockActiveReg(ActiveRegistrationTasksCriticalSection);
		checkf(ActiveRegistrationTasks.IsEmpty(), TEXT("Object Referencer cannot be set while registry is actively being manipulated"));
		ObjectReferencer = MoveTemp(InReferencer);
	}

	TUniquePtr<Metasound::INode> FNodeClassRegistry::CreateNode(const FNodeClassRegistryKey& InKey, Metasound::FNodeData InNodeData) const
	{
		TUniquePtr<INode> Node;
		auto CreateNodeLambda = [&Node, &InNodeData](const INodeClassRegistryEntry& Entry) mutable
		{ 
			Node = Entry.CreateNode(MoveTemp(InNodeData)); 
		};

		if (!AccessNodeEntryThreadSafe(InKey, CreateNodeLambda))
		{
			// Creation of external nodes can rely on assets being unavailable due to errors in loading order, asset(s)
			// missing, etc. 
			UE_LOGF(LogMetaSound, Error, "Could not find node [RegistryKey:%ls]", *InKey.ToString());
		}

		return MoveTemp(Node);
	}

	TArray<::Metasound::Frontend::FConverterNodeInfo> FNodeClassRegistry::GetPossibleConverterNodes(const FName& FromDataType, const FName& ToDataType)
	{
		FConverterNodeClassRegistryKey InKey = { FromDataType, ToDataType };
		if (!ConverterNodeClassRegistry.Contains(InKey))
		{
			return TArray<FConverterNodeInfo>();
		}
		else
		{
			return ConverterNodeClassRegistry[InKey].PotentialConverterNodes;
		}
	}

	TUniquePtr<FNodeClassRegistryTransactionStream> FNodeClassRegistry::CreateTransactionStream()
	{
		return MakeUnique<FNodeClassRegistryTransactionStream>(TransactionBuffer);
	}

	FGraphRegistryKey FNodeClassRegistry::RegisterGraph(const TScriptInterface<IMetaSoundDocumentInterface>& InDocumentInterface, TArrayView<const FGuid> InPageOrder)
	{
		using namespace UE;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FNodeClassRegistry::RegisterGraph);

		check(InDocumentInterface);
		check(IsInGameThread());

		uint32 OwningObjUniqueID = (uint32)INDEX_NONE;
		if (const UObject* OwningObj = InDocumentInterface.GetObject())
		{
			OwningObjUniqueID = OwningObj->GetUniqueID();
		}
		const FMetasoundFrontendDocument& Document = InDocumentInterface->GetConstDocument();
		const FTopLevelAssetPath AssetPath = InDocumentInterface->GetAssetPathChecked();
		const FGraphRegistryKey RegistryKey { FNodeClassRegistryKey(Document.RootGraph), AssetPath, OwningObjUniqueID };

		if (!RegistryKey.IsValid())
		{
			// Do not attempt to build and register a MetaSound with an invalid registry key
			UE_LOGF(LogMetaSound, Warning, "Registry key is invalid when attempting to register graph for asset %ls, key %ls", *AssetPath.ToString(), *RegistryKey.ToString());
			return RegistryKey;
		}

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FNodeClassRegistry::RegisterGraph key:%s, asset %s"), *RegistryKey.ToString(), *AssetPath.ToString()));

		// Wait for any async tasks that are in flight which correspond to the same graph prior to building, even if this is a synchronous call.
		WaitForAsyncGraphRegistration(RegistryKey);

		const bool bAsync = !ConsoleVariables::bDisableAsyncGraphRegistration;

		// Use the asset path of the provided document interface object for identification, *NOT* the
		// built version as the build process may in fact create a new object with a transient path.
		const TScriptInterface<IMetaSoundDocumentInterface> RegistryDocInterface = RegistryPrivate::BuildRegistryDocument(InDocumentInterface, bAsync);


		// Proxies are created synchronously to avoid creating proxies in async tasks. Proxies
		// are created from UObjects which need to be protected from GC and non-GT access.
		FProxyDataCache ProxyDataCache;
		ProxyDataCache.CreateAndCacheProxies(Document, InPageOrder);

#if !NO_LOGGING
		if (UE_LOG_ACTIVE(LogMetaSound, Verbose))
		{
			FGuid GraphPageID = DefaultPageID;
			if (const FMetasoundFrontendGraph* GraphPage = FindPreferredPage(Document.RootGraph.GetConstGraphPages(), InPageOrder))
			{
				GraphPageID = GraphPage->PageID;
			}

			const bool bContainsMultipleGraphs = Document.RootGraph.GetConstGraphPages().Num() > 1;
			if (bContainsMultipleGraphs || GraphPageID != Metasound::Frontend::DefaultPageID)
			{
				UE_LOGF(LogMetaSound, Verbose, "Registered MetaSound '%ls' Graph Page with PageID '%ls'.",
					*AssetPath.GetAssetName().ToString(),
					*GraphPageID.ToString());
				if (bContainsMultipleGraphs)
				{
					UE_LOGF(LogMetaSound, Verbose, "Graphs found with following PageIDs Implemented:");
					Document.RootGraph.IterateGraphPages([](const FMetasoundFrontendGraph& Graph)
					{
						UE_LOGF(LogMetaSound, Verbose, "    - %ls'", *Graph.PageID.ToString());
					});
				}
			}
		}
#endif // !NO_LOGGING

		// Store update to newly registered node in history so nodes
		// can be queried by transaction ID
		{
			FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeRegistration, RegistryKey.NodeKey, Timestamp));
		}

		if (bAsync)
		{
			// Keep track of exact registration task to ensure correct task instance is removed upon task completion. 
			int32 RegistrationTaskID = ++RegistrationTaskCntr;
			TSharedRef<Tasks::FCancellationToken> CancellationToken = MakeShared<Tasks::FCancellationToken>();
			TArray<FGuid> PageOrder(InPageOrder.GetData(), InPageOrder.Num());
			Tasks::FTask BuildAndRegisterTask = AsyncRegistrationPipe.Launch(
				UE_SOURCE_LOCATION,
				[RegistryKey, CancellationToken, RegistryDocInterface, ProxyDataCache = MoveTemp(ProxyDataCache), PageOrder = MoveTemp(PageOrder), RegistrationTaskID]() mutable
				{
					FNodeClassRegistry& Registry = FNodeClassRegistry::Get();
					if (!CancellationToken->IsCanceled())
					{
						// Unregister the graph before re-registering
						Registry.UnregisterGraphInternal(RegistryKey);
						Registry.BuildAndRegisterGraphFromDocument(RegistryDocInterface->GetConstDocument(), ProxyDataCache, RegistryKey, PageOrder);
					}

					Registry.RemoveRegistrationTask(RegistryKey, RegistrationTaskID, FNodeClassRegistryTransaction::ETransactionType::NodeRegistration);
					Registry.RemoveDocumentReference(RegistryDocInterface);
				}
			);

			AddDocumentReference(RegistryDocInterface);
			AddRegistrationTask(RegistryKey, FActiveRegistrationTaskInfo
			{
				FNodeClassRegistryTransaction::ETransactionType::NodeRegistration,
				BuildAndRegisterTask,
				AssetPath,
				RegistrationTaskID,
				RegistryDocInterface,
				CancellationToken.ToSharedPtr()
			});
		}
		else
		{
			UnregisterGraphInternal(RegistryKey);

			// Build and register graph synchronously
			BuildAndRegisterGraphFromDocument(RegistryDocInterface->GetConstDocument(), ProxyDataCache, RegistryKey, InPageOrder);
		}

		return RegistryKey;
	}

	void FNodeClassRegistry::AddRegistrationTask(const FGraphRegistryKey& InKey, FActiveRegistrationTaskInfo&& TaskInfo)
	{
		UE::TScopeLock LockActiveReg(ActiveRegistrationTasksCriticalSection);
		ActiveRegistrationTasks.FindOrAdd(InKey.NodeKey).Add(MoveTemp(TaskInfo));
	}

	void FNodeClassRegistry::RemoveRegistrationTask(const FGraphRegistryKey& InKey, int32 InRegistrationTaskID, FNodeClassRegistryTransaction::ETransactionType TransactionType)
	{
		UE::TScopeLock LockActiveReg(ActiveRegistrationTasksCriticalSection);

		int32 NumRemoved = 0;
		if (TArray<FActiveRegistrationTaskInfo>* TaskInfos = ActiveRegistrationTasks.Find(InKey.NodeKey))
		{
			auto MatchesEntryInTask = [&TransactionType, InRegistrationTaskID](const FActiveRegistrationTaskInfo& Info)
			{
				const bool bIsTransactionType = Info.TransactionType == TransactionType;
				const bool bIsSameRegistrationTaskID = Info.RegistrationTaskID == InRegistrationTaskID;
				return bIsTransactionType && bIsSameRegistrationTaskID;
			};

			NumRemoved = TaskInfos->RemoveAllSwap(MatchesEntryInTask, EAllowShrinking::No);
			if (TaskInfos->IsEmpty())
			{
				ActiveRegistrationTasks.Remove(InKey.NodeKey);
			}
		}

		if (NumRemoved == 0)
		{
			const bool bIsCooking = IsRunningCookCommandlet();
			if (ensureMsgf(!bIsCooking,
				TEXT("Failed to find active %s tasks for the graph '%s': Async registration is not supported while cooking"),
				*FNodeClassRegistryTransaction::LexToString(TransactionType),
				*InKey.ToString()))
			{
				UE_LOGF(LogMetaSound, Verbose,
					"Failed to find active %ls tasks for the graph '%ls'.",
					*FNodeClassRegistryTransaction::LexToString(TransactionType),
					*InKey.ToString());
			}
		}
	}

	void FNodeClassRegistry::AddDocumentReference(TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface)
	{
		UE::TScopeLock LockActiveReg(ObjectReferencerCriticalSection);
		if (UObject* Object = DocumentInterface.GetObject())
		{
			if (ObjectReferencer)
			{
				ObjectReferencer->AddObject(Object);
			}
		}
	}

	void FNodeClassRegistry::RemoveDocumentReference(TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface)
	{
		UE::TScopeLock LockActiveReg(ObjectReferencerCriticalSection);
		if (UObject* Object = DocumentInterface.GetObject())
		{
			if (ObjectReferencer)
			{
				ObjectReferencer->RemoveObject(Object);
			}
		}
	}

	void FNodeClassRegistry::RegisterGraphInternal(const FGraphRegistryKey& InKey, TSharedPtr<const FGraph> InGraph)
	{
		using namespace RegistryPrivate;

		UE::TScopeLock Lock(RegistryMapsCriticalSection);

#if !NO_LOGGING
		if (RegisteredGraphs.Contains(InKey))
		{
			UE_LOGF(LogMetaSound, Warning, "Graph is already registered with the same registry key '%ls'. The existing registered graph will be replaced with the new graph.", *InKey.ToString());
		}
#endif // !NO_LOGGING

		RegisteredGraphs.Add(InKey, InGraph);
	}

	bool FNodeClassRegistry::UnregisterGraphInternal(const FGraphRegistryKey& InKey)
	{
		using namespace RegistryPrivate;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*InKey.ToString(TEXT("FNodeClassRegistry::UnregisterGraphInternal")));

		UE::TScopeLock Lock(RegistryMapsCriticalSection);
		{
			if (!RegisteredGraphs.Contains(InKey))
			{
				return false;
			}

			const int32 GraphUnregistered = RegisteredGraphs.Remove(InKey) > 0;
			const bool bNodeUnregistered = UnregisterNodeInternal(InKey.NodeKey);

#if !NO_LOGGING
			if (GraphUnregistered)
			{
				UE_LOGF(LogMetaSound, VeryVerbose, "Unregistered graph with key '%ls'", *InKey.ToString());
			}
			else
			{
				// Avoid warning if in cook as we always expect a graph to not get registered/
				// unregistered while cooking (as its unnecessary for serialization).
				if (bNodeUnregistered && !IsRunningCookCommandlet())
				{
					UE_LOGF(LogMetaSound, Warning, "Graph '%ls' was not found, but analogous registered node class was when unregistering.", *InKey.ToString());
				}
			}
#endif // !NO_LOGGING

			return bNodeUnregistered;
		}
	}

	bool FNodeClassRegistry::UnregisterGraph(const FGraphRegistryKey& InRegistryKey, const TScriptInterface<IMetaSoundDocumentInterface>& InDocumentInterface)
	{
		using namespace UE;
		using namespace RegistryPrivate;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FNodeClassRegistry::UnregisterGraph);

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*InRegistryKey.ToString(TEXT("FNodeClassRegistry::UnregisterGraph")));
#endif // (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)

		// Do not attempt to unregister a MetaSound with an invalid registry key
		if (!InRegistryKey.IsValid())
		{
			UE_LOGF(LogMetaSound, Warning, "Registry key is invalid when attempting to unregister graph (%ls)", *InRegistryKey.ToString());
			return false;
		}

		// Store update to unregistered node in history so nodes can be queried by transaction ID
		{
			FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration, InRegistryKey.NodeKey, Timestamp));
		}

		// Async registration is only available if:
		// 1. The IMetaSoundDocumentInterface is not actively modified by a builder
		//    (built graph must be released synchronously to avoid a race condition on
		//    reading/writing the IMetaSoundDocumentInterface on the Game Thread)
		// 2. Async registration is globally disabled via console variable.
		const bool bAsync = !(InDocumentInterface->IsActivelyBuilding() || ConsoleVariables::bDisableAsyncGraphRegistration);
		if (bAsync)
		{
			// Keep track of exact registration task to ensure correct task instance is removed upon task completion. 
			int32 RegistrationTaskID = ++RegistrationTaskCntr;

			// Wait for any async tasks that are in flight which correspond to the same graph
			WaitForAsyncGraphRegistration(InRegistryKey);

			Tasks::FTask UnregisterTask = AsyncRegistrationPipe.Launch(UE_SOURCE_LOCATION, [RegistryKey = InRegistryKey, RegistrationTaskID]()
			{
				FNodeClassRegistry& Registry = FNodeClassRegistry::Get();
				Registry.UnregisterGraphInternal(RegistryKey);
				Registry.RemoveRegistrationTask(RegistryKey, RegistrationTaskID, FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration);
			});

			AddRegistrationTask(InRegistryKey, FActiveRegistrationTaskInfo
			{
				FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration,
				UnregisterTask,
				InRegistryKey.AssetPath,
				RegistrationTaskID
			});
		}
		else
		{
			UnregisterGraphInternal(InRegistryKey);
		}

		return true;
	}

	TSharedPtr<const Metasound::FGraph> FNodeClassRegistry::GetGraph(const FGraphRegistryKey& InKey) const
	{
		WaitForAsyncGraphRegistration(InKey);

		TSharedPtr<const FGraph> Graph;
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			if (const TSharedPtr<const FGraph>* RegisteredGraph = RegisteredGraphs.Find(InKey))
			{
				Graph = *RegisteredGraph;
			}
		}

		if (!Graph)
		{
			UE_LOGF(LogMetaSound, Error, "Could not find graph with registry graph key '%ls'.", *InKey.ToString());
		}

		return Graph;
	}

	FNodeClassRegistryKey FNodeClassRegistry::RegisterNodeInternal(TUniquePtr<INodeClassRegistryEntry>&& InEntry)
	{
		using namespace RegistryPrivate;

		METASOUND_LLM_SCOPE;

		if (!InEntry.IsValid())
		{
			return { };
		}

		const FNodeClassRegistryKey Key = FNodeClassRegistryKey(InEntry->GetFrontendClass().Metadata);

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Key.ToString(TEXT("FNodeClassRegistry::RegisterNodeInternal")))
#else // !(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE("FNodeClassRegistry::RegisterNodeInternal")
#endif // !(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)

#if !NO_LOGGING
		TArray<TSharedRef<INodeClassRegistryEntry>> Entries;
#endif // !NO_LOGGING

		TSharedRef<INodeClassRegistryEntry, ESPMode::ThreadSafe> Entry(InEntry.Release());
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			// check to see if an identical node was already registered, and log if necessary
			// Store registry elements in map so nodes can be queried using registry key.
			RegisteredNodes.Add(Key, Entry);

#if !NO_LOGGING
			RegisteredNodes.MultiFind(Key, Entries);
#endif // !NO_LOGGING
		}

#if !NO_LOGGING 
		if (Entries.Num() > 1)
		{
			if (IMetaSoundAssetManager* AssetManager = IMetaSoundAssetManager::Get())
			{
				if (AssetManager->IsAssetClass(Entry->GetFrontendClass().Metadata))
				{
					const TArray<FTopLevelAssetPath> AssetPaths = AssetManager->FindAssetPaths(FMetaSoundAssetKey(Key));
					TArray<FString> ExistingPaths;
					Algo::Transform(AssetPaths, ExistingPaths, [](const FTopLevelAssetPath& AssetPath) { return AssetPath.ToString(); });
					const FString ExistingAssetPaths = FString::Join(ExistingPaths, TEXT("\n"));
					UE_LOGF(LogMetaSound, Error,
						"Multiple node classes with key '%ls' registered. Assets currently registered with class name:\n%ls\nReassign the class name of colliding assets by using the Reassign Asset Class Guid asset action in the right click -> Asset Actions menu of the MetaSound.",
							*Key.ToString(),
							*ExistingAssetPaths
					);
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Multiple node classes with key '%ls' registered.",*Key.ToString());
				}
			}
		}
#endif // !NO_LOGGING

		return Key;
	}

	FNodeClassRegistryKey FNodeClassRegistry::RegisterNode(TUniquePtr<INodeClassRegistryEntry>&& InEntry)
	{
		const FNodeClassRegistryKey Key = RegisterNodeInternal(MoveTemp(InEntry));

		if (Key.IsValid())
		{
			// Store update to newly registered node in history so nodes
			// can be queried by transaction ID
			const FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeRegistration, Key, Timestamp));
		}

		return Key;
	}

#if WITH_EDITORONLY_DATA
	bool FNodeClassRegistry::RegisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo)
	{
		const FNodeClassRegistryKey Key(
			EMetasoundFrontendClassType::External,
			InMigrationInfo.ClassName,
			FMetasoundFrontendVersionNumber
			{
				.Major = InMigrationInfo.MajorVersion,
				.Minor = InMigrationInfo.MinorVersion
			}
		);

		// Attempts to register the exact same migration info twice will result in bSuccess=false
		bool bSuccess = false;

		// Add to map of migrations
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			if (nullptr == NodeMigrations.FindPair(Key, InMigrationInfo))
			{
				NodeMigrations.Add(Key, InMigrationInfo);
				bSuccess = true;
			}
		}

		if (bSuccess)
		{
			// Add to transactions so that the search engine knows it exists. 
			const FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeMigrationRegistration, Key, Timestamp));
		}

		UE_CLOGF(!bSuccess, LogMetaSound, Error, "Invalid attempt to register node migration more than once: %ls. Please ensure node migration info is unique.", *InMigrationInfo.ToString());


		return bSuccess;
	}

	bool FNodeClassRegistry::UnregisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo)
	{
		const FNodeClassRegistryKey Key(
			EMetasoundFrontendClassType::External,
			InMigrationInfo.ClassName,
			FMetasoundFrontendVersionNumber
			{
				.Major = InMigrationInfo.MajorVersion,
				.Minor = InMigrationInfo.MinorVersion
			}
		);

		// Attempts to unregister a migration that does not exist will result in bSuccess=false
		bool bSuccess = false;

		// remove from map of migrations
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			int32 NumRemoved = NodeMigrations.Remove(Key, InMigrationInfo);
			if (NumRemoved > 0)
			{
				bSuccess = true;
			}
		}

		if (bSuccess)
		{
			// Add to transactions so that the search engine knows it exists. 
			const FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeMigrationUnregistration, Key, Timestamp));
		}

		UE_CLOGF(!bSuccess, LogMetaSound, Error, "Invalid attempt to unregister node migration more than once: %ls. Please ensure node migration info is unique.", *InMigrationInfo.ToString());

		return bSuccess;
	}

	FName FNodeClassRegistry::GetOwningPluginName(const FNodeClassRegistryKey& InKey) const
	{
		FName PluginName;
		auto GetPluginName = [&PluginName](const INodeClassRegistryEntry& Entry)
		{
			PluginName = Entry.GetPluginName();
		};

		AccessNodeEntryThreadSafe(InKey, GetPluginName);
		return PluginName;
	}

	FName FNodeClassRegistry::GetOwningModuleName(const FNodeClassRegistryKey& InKey) const
	{
		FName ModuleName;
		auto GetModuleName = [&ModuleName](const INodeClassRegistryEntry& Entry)
		{
			ModuleName = Entry.GetModuleName();
		};

		AccessNodeEntryThreadSafe(InKey, GetModuleName);
		return ModuleName;
	}

	bool FNodeClassRegistry::IsCustomNodeUpdateTransformWellFormed(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, const TSharedRef<const FBaseNodeUpdateTransform>& InTransform)
	{
		// Cannot register multiple node transforms for the same key
		if (CustomNodeUpdateTransforms.Find(FMajorNodeUpdateTransformKey(InClassName, InMajorVersion)))
		{
			return false;
		}
		// A custom version update must transform to an entirely new node class or the next major version increment of the same node class
		const FNodeClassRegistryKey& NewNodeClassKey = InTransform->GetNewNodeClassKey();
		return (InClassName != NewNodeClassKey.ClassName) || (NewNodeClassKey.Version.Major == InMajorVersion + 1);
	}
	
	void FNodeClassRegistry::RegisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion, const TSharedRef<const FBaseNodeUpdateTransform>& InTransform)
	{
		if (ensureMsgf(IsCustomNodeUpdateTransformWellFormed(ClassName, MajorVersion, InTransform),
			TEXT("MetaSound custom major node update transform for class name %s version %i could not be registered. Transform already already exists and or is invalid (does not transform to an entirely new node class or the next major version increment of the same node class)."), *ClassName.ToString(), MajorVersion))
		{
			UE_LOG(LogMetaSound, Verbose, TEXT("Registering custom node update transform for %s v%d"), *ClassName.ToString(), MajorVersion);
			const FMajorNodeUpdateTransformKey TransformKey(ClassName, MajorVersion);
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			CustomNodeUpdateTransforms.Add(TransformKey, InTransform);
		}
	}
	
	const TSharedPtr<const FBaseNodeUpdateTransform> FNodeClassRegistry::FindCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion) const
	{
		const FMajorNodeUpdateTransformKey TransformKey(ClassName, MajorVersion);
		const TSharedRef<const FBaseNodeUpdateTransform, ESPMode::ThreadSafe>* NodeUpdateTransform = CustomNodeUpdateTransforms.Find(TransformKey);
		if (NodeUpdateTransform)
		{
			return *NodeUpdateTransform;
		}
		return nullptr;
	}


	bool FNodeClassRegistry::UnregisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion)
	{
		UE_LOG(LogMetaSound, Verbose, TEXT("Unregistering custom node update transform for %s v%d"), *ClassName.ToString(), MajorVersion);
		const FMajorNodeUpdateTransformKey TransformKey(ClassName, MajorVersion);
		UE::TScopeLock Lock(RegistryMapsCriticalSection);
		return CustomNodeUpdateTransforms.Remove(TransformKey) > 0;
	}

#endif // WITH_EDITORONLY_DATA

	FNodeClassRegistryKey FNodeClassRegistry::RegisterNodeTemplate(TUniquePtr<INodeTemplateRegistryEntry>&& InEntry)
	{
		METASOUND_LLM_SCOPE;

		FNodeClassRegistryKey Key;

		if (InEntry.IsValid())
		{
			TSharedRef<INodeTemplateRegistryEntry, ESPMode::ThreadSafe> Entry(InEntry.Release());

			FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();

			Key = FNodeClassRegistryKey(Entry->GetFrontendClass().Metadata);

			{
				UE::TScopeLock Lock(RegistryMapsCriticalSection);
				// check to see if an identical node was already registered, and log
				ensureAlwaysMsgf(
					!RegisteredNodeTemplates.Contains(Key),
					TEXT("Node template with registry key '%s' already registered. The previously registered node will be overwritten."),
					*Key.ToString());

				// Store registry elements in map so nodes can be queried using registry key.
				RegisteredNodeTemplates.Add(Key, Entry);
			}

			// Store update to newly registered node in history so nodes
			// can be queried by transaction ID

			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeRegistration, Key, Timestamp));
		}

		return Key;
	}

	bool FNodeClassRegistry::UnregisterNodeInternal(const FNodeClassRegistryKey& InKey)
	{
		METASOUND_LLM_SCOPE;

		if (InKey.IsValid())
		{
#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FNodeClassRegistry::UnregisterNodeInternal key %s"), *InKey.ToString()))
#else // !(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TEXT("FNodeClassRegistry::UnregisterNodeInternal"))
#endif // !(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)

			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			if (const TSharedRef<INodeClassRegistryEntry, ESPMode::ThreadSafe>* EntryPtr = RegisteredNodes.Find(InKey))
			{
				const TSharedRef<INodeClassRegistryEntry>& Entry = *EntryPtr;
				const uint32 NumRemoved = RegisteredNodes.RemoveSingle(InKey, Entry);
				if (ensure(NumRemoved == 1))
				{
					return true;
				}
			}
		}

		return false;
	}

	bool FNodeClassRegistry::UnregisterNode(const FNodeClassRegistryKey& InKey)
	{
		if (UnregisterNodeInternal(InKey))
		{
			const FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();
			TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration, InKey, Timestamp));

			return true;
		}

		return false;
	}

	bool FNodeClassRegistry::UnregisterNodeTemplate(const FNodeClassRegistryKey& InKey)
	{
		METASOUND_LLM_SCOPE;

		if (InKey.IsValid())
		{
			if (const INodeTemplateRegistryEntry* Entry = FindNodeTemplateEntry(InKey))
			{
				FNodeClassRegistryTransaction::FTimeType Timestamp = FPlatformTime::Cycles64();

				TransactionBuffer->AddTransaction(FNodeClassRegistryTransaction(FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration, InKey, Timestamp));

				{
					UE::TScopeLock Lock(RegistryMapsCriticalSection);
					RegisteredNodeTemplates.Remove(InKey);
				}
				return true;
			}
		}

		return false;
	}

	bool FNodeClassRegistry::RegisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FConverterNodeInfo& InNodeInfo)
	{
		if (!ConverterNodeClassRegistry.Contains(InConversionKey))
		{
			ConverterNodeClassRegistry.Add(InConversionKey);
		}

		FConverterNodeClassRegistryValue& ConverterNodeList = ConverterNodeClassRegistry[InConversionKey];

		if (ensureAlways(!ConverterNodeList.PotentialConverterNodes.Contains(InNodeInfo)))
		{
			ConverterNodeList.PotentialConverterNodes.Add(InNodeInfo);
			return true;
		}
		else
		{
			// If we hit this, someone attempted to add the same converter node to our list multiple times.
			return false;
		}
	}

	bool FNodeClassRegistry::UnregisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FNodeClassRegistryKey& InNodeKey)
	{
		if (FConverterNodeClassRegistryValue* ConverterNodeList = ConverterNodeClassRegistry.Find(InConversionKey))
		{
			auto HasSameNodeKey = [&InNodeKey](const FConverterNodeClassInfo& InInfo) -> bool
			{
				return InInfo.NodeKey == InNodeKey;
			};
			int32 NumRemoved = ConverterNodeList->PotentialConverterNodes.RemoveAll(HasSameNodeKey);

			return NumRemoved > 0;
		}
		return false;
	}

	bool FNodeClassRegistry::IsNodeRegistered(const FNodeClassRegistryKey& InKey) const
	{
		auto IsNodeRegisteredInternal = [this, &InKey]() -> bool
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			return RegisteredNodes.Contains(InKey) || RegisteredNodeTemplates.Contains(InKey);
		};

		if (IsNodeRegisteredInternal())
		{
			return true;
		}
		else
		{
			WaitForAsyncRegistrationInternal(InKey, nullptr /* InAssetPath */);
			return IsNodeRegisteredInternal();
		}
	}

	bool FNodeClassRegistry::IsGraphRegistered(const FGraphRegistryKey& InKey) const
	{
		WaitForAsyncGraphRegistration(InKey);

		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			return RegisteredGraphs.Contains(InKey);
		}
	}

	bool FNodeClassRegistry::FindDefaultVertexInterface(const FNodeClassRegistryKey& InKey, FVertexInterface& OutVertexInterface) const 
	{
		auto GetDefaultVertexInterface = [&OutVertexInterface](const INodeClassRegistryEntry& Entry)
		{
			OutVertexInterface = Entry.GetDefaultVertexInterface();
		};

		if (AccessNodeEntryThreadSafe(InKey, GetDefaultVertexInterface))
		{
			return true;
		}

		return false;
	}

	bool FNodeClassRegistry::FindClassInterface(const FNodeClassRegistryKey& InKey, FClassInterface& OutClassInterface) const
	{
		auto GetClassInterface = [&OutClassInterface](const INodeClassRegistryEntry& Entry)
		{
			OutClassInterface = Entry.GetClassInterface();
		};

		if (AccessNodeEntryThreadSafe(InKey, GetClassInterface))
		{
			return true;
		}

		return false;
	}

	bool FNodeClassRegistry::FindFrontendClassFromRegistered(const FNodeClassRegistryKey& InKey, FMetasoundFrontendClass& OutClass) const
	{
		auto SetFrontendClass = [&OutClass](const INodeClassRegistryEntry& Entry)
		{
			OutClass = Entry.GetFrontendClass();
		};

		if (AccessNodeEntryThreadSafe(InKey, SetFrontendClass))
		{
			return true;
		}

		if (const INodeTemplateRegistryEntry* Entry = FindNodeTemplateEntry(InKey))
		{
			OutClass = Entry->GetFrontendClass();
			return true;
		}

		return false;
	}

	bool FNodeClassRegistry::IsCompatibleNodeConfiguration(const FNodeClassRegistryKey& InKey, TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const
	{
		bool bIsCompatible = false;
		auto GetNodeConfigurationCompatibility = [&bIsCompatible, &InNodeConfiguration](const INodeClassRegistryEntry& Entry)
		{
			bIsCompatible = Entry.IsCompatibleNodeConfiguration(InNodeConfiguration);
		};

		AccessNodeEntryThreadSafe(InKey, GetNodeConfigurationCompatibility);
		return bIsCompatible;
	}

	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FNodeClassRegistry::CreateFrontendNodeConfiguration(const FNodeClassRegistryKey& InKey) const
	{
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> NodeConfiguration;
		auto SetNodeConfiguration = [&NodeConfiguration](const INodeClassRegistryEntry& Entry)
		{
			NodeConfiguration = Entry.CreateFrontendNodeConfiguration();
		};

		AccessNodeEntryThreadSafe(InKey, SetNodeConfiguration);

		// Currently node configuration on template nodes is not supported. To enable that, the node template registry will need 
		// to provide a creation mechanisms for making related FMetaSoundFrontendNodeConfigurations

		return NodeConfiguration;
	}

	bool FNodeClassRegistry::FindImplementedInterfacesFromRegistered(const Metasound::Frontend::FNodeClassRegistryKey& InKey, TSet<FMetasoundFrontendVersion>& OutInterfaceVersions) const 
	{
		bool bDidCopy = false;

		auto CopyImplementedInterfaces = [&OutInterfaceVersions, &bDidCopy](const INodeClassRegistryEntry& Entry)
		{
			if (const TSet<FMetasoundFrontendVersion>* Interfaces = Entry.GetImplementedInterfaces())
			{
				OutInterfaceVersions = *Interfaces;
				bDidCopy = true;
			}
		};

		AccessNodeEntryThreadSafe(InKey, CopyImplementedInterfaces);

		return bDidCopy;
	}

	bool FNodeClassRegistry::FindInputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey)
	{
		FMetasoundFrontendClass Class;
		switch (InAccessType)
		{
			case EMetasoundFrontendVertexAccessType::Reference:
			{
				if (IDataTypeRegistry::Get().GetFrontendInputClass(InDataTypeName, Class))
				{
					OutKey = FNodeClassRegistryKey(Class.Metadata);
					return true;
				}
			}
			break;

			case EMetasoundFrontendVertexAccessType::Value:
			{
				if (IDataTypeRegistry::Get().GetFrontendConstructorInputClass(InDataTypeName, Class))
				{
					OutKey = FNodeClassRegistryKey(Class.Metadata);
					return true;
				}
			}
			break;

			default:
			case EMetasoundFrontendVertexAccessType::Unset:
			{
				return false;
			}
		}

		return false;
	}

	bool FNodeClassRegistry::FindVariableNodeRegistryKeyForDataType(const FName& InDataTypeName, FNodeClassRegistryKey& OutKey)
	{
		FMetasoundFrontendClass Class;
		if (IDataTypeRegistry::Get().GetFrontendLiteralClass(InDataTypeName, Class))
		{
			OutKey = FNodeClassRegistryKey(Class.Metadata);
			return true;
		}
		return false;
	}

	bool FNodeClassRegistry::FindOutputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey)
	{
		FMetasoundFrontendClass Class;
		switch (InAccessType)
		{
			case EMetasoundFrontendVertexAccessType::Reference:
			{
				if (IDataTypeRegistry::Get().GetFrontendOutputClass(InDataTypeName, Class))
				{
					OutKey = FNodeClassRegistryKey(Class.Metadata);
					return true;
				}
			}
			break;

			case EMetasoundFrontendVertexAccessType::Value:
			{
				if (IDataTypeRegistry::Get().GetFrontendConstructorOutputClass(InDataTypeName, Class))
				{
					OutKey = FNodeClassRegistryKey(Class.Metadata);
					return true;
				}
			}
			break;
		}

		return false;
	}

	void FNodeClassRegistry::IterateRegistry(FIterateMetasoundFrontendClassFunction InIterFunc, EMetasoundFrontendClassType InClassType) const
	{
		UE_LOGF(LogMetaSound, Verbose, "Calling FMetasoundRegistryContainer::IterateRegistry(...) is can be slow and prone to deadlocks because it locks the registry with a critical section. Please use Metasound::Frontend::ISearchEngine instead");

		UE::TScopeLock Lock(RegistryMapsCriticalSection);
		auto WrappedFunc = [&](const TPair<FNodeClassRegistryKey, TSharedPtr<INodeClassRegistryEntry, ESPMode::ThreadSafe>>& Pair)
		{
			InIterFunc(Pair.Value->GetFrontendClass());
		};

		if (EMetasoundFrontendClassType::Invalid == InClassType)
		{
			// Iterate through all classes. 
			Algo::ForEach(RegisteredNodes, WrappedFunc);
		}
		else
		{
			// Only call function on classes of certain type.
			auto IsMatchingClassType = [&](const TPair<FNodeClassRegistryKey, TSharedPtr<INodeClassRegistryEntry, ESPMode::ThreadSafe>>& Pair)
			{
				return Pair.Value->GetFrontendClass().Metadata.GetType() == InClassType;
			};
			Algo::ForEachIf(RegisteredNodes, IsMatchingClassType, WrappedFunc);
		}
	}

	bool FNodeClassRegistry::AccessNodeEntryThreadSafe(const FNodeClassRegistryKey& InKey, TFunctionRef<void(const INodeClassRegistryEntry&)> InFunc) const
	{
		auto TryAccessNodeEntry = [this, &InKey, &InFunc]() -> bool
		{
			UE::TScopeLock Lock(RegistryMapsCriticalSection);
			if (const TSharedRef<INodeClassRegistryEntry, ESPMode::ThreadSafe>* Entry = RegisteredNodes.Find(InKey))
			{
				InFunc(*(*Entry));
				return true;
			}
#if WITH_EDITORONLY_DATA
			else if (const FNodeMigrationInfo* Migration = NodeMigrations.Find(InKey))
			{
				TArray<const FNodeMigrationInfo*> Migrations;
				NodeMigrations.MultiFindPointer(InKey, Migrations);
				for (const FNodeMigrationInfo* Info : Migrations)
				{
					UE_LOGF(LogMetaSound, Error, "Node cannot be found because it has been migrated. %ls. Please update your MetaSound Graph or add plugin %ls as a depedency.", *Info->ToString(), *Info->ToPlugin.ToString());
				}
			}
#endif // if WITH_EDITORONLY_DATA
			return false;
		};

		if (TryAccessNodeEntry())
		{
			return true;
		}
		else
		{
			// Wait for any async registration tasks related to the registry key. 
			WaitForAsyncRegistrationInternal(InKey, nullptr /* InAssetPath */);
			return TryAccessNodeEntry();
		}
	}

	const INodeTemplateRegistryEntry* FNodeClassRegistry::FindNodeTemplateEntry(const FNodeClassRegistryKey& InKey) const
	{
		UE::TScopeLock Lock(RegistryMapsCriticalSection);
		if (const TSharedRef<INodeTemplateRegistryEntry, ESPMode::ThreadSafe>* Entry = RegisteredNodeTemplates.Find(InKey))
		{
			return &Entry->Get();
		}

		return nullptr;
	}

	void FNodeClassRegistry::NotifyPluginUnmounted(IPlugin& Plugin)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FNodeClassRegistry::NotifyPluginUnmounted);

		using namespace UE::Tasks;

		if (!Plugin.CanContainContent())
		{
			return;
		}

		const FString MountPath = Plugin.GetMountedAssetPath();
		TArray<FTask> TasksToWaitFor;
		{
			UE::TScopeLock Lock(ActiveRegistrationTasksCriticalSection);
			FString PackageName;
			for (TPair<FNodeClassRegistryKey, TArray<FActiveRegistrationTaskInfo>>& TaskInfoPair : ActiveRegistrationTasks)
			{
				for (FActiveRegistrationTaskInfo& TaskInfo : TaskInfoPair.Value)
				{
					PackageName = TaskInfo.AssetPath.GetPackageName().ToString();
					if (PackageName.StartsWith(MountPath))
					{
						if (TaskInfo.CancellationToken.IsValid())
						{
							TaskInfo.CancellationToken->Cancel();
							UE_LOGF(LogMetaSound, Display, "Async registration task for plugin '%ls' asset '%ls' canceled on unmount.",
								*Plugin.GetName(),
								*TaskInfo.AssetPath.ToString());
						}

						TasksToWaitFor.Add(TaskInfo.Task);
					}
				}
			}
		}

		Algo::ForEach(TasksToWaitFor, [] (const FTask& Task) { Task.Wait(); });
	}

	void FNodeClassRegistry::WaitForAsyncGraphRegistration(const FGraphRegistryKey& InKey) const
	{
		WaitForAsyncRegistrationInternal(InKey.NodeKey, &InKey.AssetPath);
	}

	void FNodeClassRegistry::WaitForAsyncRegistrationInternal(const FNodeClassRegistryKey& InRegistryKey, const FTopLevelAssetPath* InAssetPath) const
	{
		using namespace UE::Tasks;

		if (AsyncRegistrationPipe.IsInContext())
		{
			// It is not safe to wait for an async registration task from within the async registration pipe because it will result in a deadlock. 
			UE_LOGF(LogMetaSound, Verbose, "Async registration pipe is already in context for registering key %ls. Task will not be waited for.", *InRegistryKey.ToString());
			return;
		}

		TArray<FTask> TasksToWaitFor;
		{
			UE::TScopeLock Lock(ActiveRegistrationTasksCriticalSection);
			if (const TArray<FActiveRegistrationTaskInfo>* FoundTasks = ActiveRegistrationTasks.Find(InRegistryKey))
			{
				// Filter by asset path or ignore if not provided
				Algo::TransformIf(*FoundTasks, TasksToWaitFor,
					[&InAssetPath](const FActiveRegistrationTaskInfo& TaskInfo) { return !InAssetPath || InAssetPath->IsNull() || TaskInfo.AssetPath == *InAssetPath; },
					[](const FActiveRegistrationTaskInfo& TaskInfo) { return TaskInfo.Task; });
			}
		}

		for (const FTask& Task : TasksToWaitFor)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FNodeClassRegistry::WaitForRegistrationTaskToComplete);
			if (Task.IsValid())
			{
				Task.Wait();
			}
		}
	}

	INodeClassRegistry* INodeClassRegistry::Get()
	{
		return &Metasound::Frontend::FNodeClassRegistry::Get();
	}

	INodeClassRegistry& INodeClassRegistry::GetChecked()
	{
		return Metasound::Frontend::FNodeClassRegistry::Get();
	}

	void INodeClassRegistry::ShutdownMetasoundFrontend()
	{
		Metasound::Frontend::FNodeClassRegistry::Shutdown();
	}

	bool INodeClassRegistry::GetFrontendClassFromRegistered(const FNodeClassRegistryKey& InKey, FMetasoundFrontendClass& OutClass)
	{
		INodeClassRegistry* Registry = INodeClassRegistry::Get();

		if (ensure(nullptr != Registry))
		{
			return Registry->FindFrontendClassFromRegistered(InKey, OutClass);
		}

		return false;
	}


	bool INodeClassRegistry::GetInputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey)
	{
		if (INodeClassRegistry* Registry = INodeClassRegistry::Get())
		{
			return Registry->FindInputNodeRegistryKeyForDataType(InDataTypeName, InAccessType, OutKey);
		}
		return false;
	}

	bool INodeClassRegistry::GetVariableNodeRegistryKeyForDataType(const FName& InDataTypeName, FNodeClassRegistryKey& OutKey)
	{
		if (INodeClassRegistry* Registry = INodeClassRegistry::Get())
		{
			return Registry->FindVariableNodeRegistryKeyForDataType(InDataTypeName, OutKey);
		}
		return false;
	}

	bool INodeClassRegistry::GetOutputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InVertexAccessType, FNodeClassRegistryKey& OutKey)
	{
		if (INodeClassRegistry* Registry = INodeClassRegistry::Get())
		{
			return Registry->FindOutputNodeRegistryKeyForDataType(InDataTypeName, InVertexAccessType, OutKey);
		}
		return false;
	}
} // namespace Metasound::Frontend
