// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Interfaces/IPluginManager.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Tasks/Pipe.h"


struct FMetasoundFrontendDocument; 

namespace Metasound
{
	class FProxyDataCache;
	class FGraph;
}

namespace Metasound::Frontend
{
	struct FNodeClassInfo;

	using FNodeClassRegistryTransactionBuffer = TTransactionBuffer<FNodeClassRegistryTransaction>;
	using FNodeClassRegistryTransactionStream = TTransactionStream<FNodeClassRegistryTransaction>; 

	/** INodeClassRegistryTemplateEntry declares the interface for a node registry entry.
	 * Each node class in the registry must satisfy this interface.
	 */
	class INodeTemplateRegistryEntry
	{
	public:
		virtual ~INodeTemplateRegistryEntry() = default;

		UE_DEPRECATED(5.8, "Use GetFrontendClass() instead")
		virtual const FNodeClassInfo& GetClassInfo() const;

		/** Return a FMetasoundFrontendClass which describes the node. */
		virtual const FMetasoundFrontendClass& GetFrontendClass() const = 0;
	};

	// Registry container private implementation.
	class FNodeClassRegistry : public INodeClassRegistry
	{

		FNodeClassRegistry();
	public:

		FNodeClassRegistry(const FNodeClassRegistry&) = delete;
		FNodeClassRegistry& operator=(const FNodeClassRegistry&) = delete;

		static FNodeClassRegistry& Get();
		static void Shutdown();

		virtual ~FNodeClassRegistry() = default;

		// Add a function to the init command array.
		virtual bool EnqueueInitCommand(TUniqueFunction<void()>&& InFunc) override;

		virtual void SetObjectReferencer(TUniquePtr<IObjectReferencer> InReferencer) override;

		// This is called on module startup. This invokes any registration commands enqueued by our registration macros.
		virtual void RegisterPendingNodes() override;

		// Wait for async graph registration to complete for a specific graph
		virtual void WaitForAsyncGraphRegistration(const FGraphRegistryKey& InRegistryKey) const override;

		// Retrieve a registered graph. 
		//
		// If the graph is registered asynchronously, this will wait until the registration task has completed.
		virtual TSharedPtr<const FGraph> GetGraph(const FGraphRegistryKey& InRegistryKey) const override;

		/** Register external node with the frontend.
		 *
		 * @param InEntry - Entry to register.
		 *
		 * @return True on success.
		 */
		virtual FNodeClassRegistryKey RegisterNode(TUniquePtr<Metasound::Frontend::INodeClassRegistryEntry>&& InEntry) override;
		virtual bool UnregisterNode(const FNodeClassRegistryKey& InKey) override;

#if WITH_EDITORONLY_DATA
		virtual bool RegisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo) override;
		virtual bool UnregisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo) override;
		virtual FName GetOwningPluginName(const FNodeClassRegistryKey& InKey) const override;
		virtual FName GetOwningModuleName(const FNodeClassRegistryKey& InKey) const override;

		virtual void RegisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion, const TSharedRef<const FBaseNodeUpdateTransform>& InTransform) override;
	
		virtual bool UnregisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion) override;

		virtual const TSharedPtr<const FBaseNodeUpdateTransform> FindCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion) const override;
#endif // if WITH_EDITORONLY_DATA

		virtual bool IsNodeRegistered(const FNodeClassRegistryKey& InKey) const override;
		virtual bool IsGraphRegistered(const FGraphRegistryKey& InKey) const override;

		virtual bool RegisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FConverterNodeInfo& InNodeInfo) override;
		virtual bool UnregisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FNodeClassRegistryKey& InNodeKey) override;

		UE_DEPRECATED(5.8, "Use Metasound::Frontend::ISearchEngine instead.")
		virtual void IterateRegistry(FIterateMetasoundFrontendClassFunction InIterFunc, EMetasoundFrontendClassType InClassType = EMetasoundFrontendClassType::Invalid) const override;

		// Find Frontend class data.
		virtual bool FindDefaultVertexInterface(const FNodeClassRegistryKey& InKey, FVertexInterface& OutVertexInterface) const override;
		virtual bool FindClassInterface(const FNodeClassRegistryKey& InKey, FClassInterface& OutClassInterface) const override;
		
		virtual bool FindFrontendClassFromRegistered(const FNodeClassRegistryKey& InKey, FMetasoundFrontendClass& OutClass) const override;

		virtual bool IsCompatibleNodeConfiguration(const FNodeClassRegistryKey& InKey, TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override;
		virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration(const FNodeClassRegistryKey& InKey) const override;

		virtual bool FindImplementedInterfacesFromRegistered(const Metasound::Frontend::FNodeClassRegistryKey& InKey, TSet<FMetasoundFrontendVersion>& OutInterfaceVersions) const override;
		
		virtual bool FindInputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey) override;
		virtual bool FindVariableNodeRegistryKeyForDataType(const FName& InDataTypeName, FNodeClassRegistryKey& OutKey) override;
		virtual bool FindOutputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey) override;

		// Create a new instance of a C++ implemented node from the registry.
		virtual TUniquePtr<Metasound::INode> CreateNode(const FNodeClassRegistryKey& InKey, Metasound::FNodeData InNodeData) const override;

		// Returns a list of possible nodes to use to convert from FromDataType to ToDataType.
		// Returns an empty array if none are available.
		virtual TArray<FConverterNodeInfo> GetPossibleConverterNodes(const FName& FromDataType, const FName& ToDataType) override;

		// Cancels async graphs contained in the given directory and waits for all
		// associated unregistration tasks associated with the given plugin to complete
		// such that subsequent call to GC all remaining plugin-specific content is not
		// being blocked by in-flight registration tasks.
		void NotifyPluginUnmounted(IPlugin& Plugin);

		// Register a graph from an IMetaSoundDocumentInterface
		FGraphRegistryKey RegisterGraph(const TScriptInterface<IMetaSoundDocumentInterface>& InDocumentInterface, TArrayView<const FGuid> InPageOrder);

		// Unregister a graph
		bool UnregisterGraph(const FGraphRegistryKey& InRegistryKey, const TScriptInterface<IMetaSoundDocumentInterface>& InDocumentInterface);

		// Private implementation until hardened and used for template nodes other than reroutes.
		FNodeClassRegistryKey RegisterNodeTemplate(TUniquePtr<Metasound::Frontend::INodeTemplateRegistryEntry>&& InEntry);
		bool UnregisterNodeTemplate(const FNodeClassRegistryKey& InKey);

		// Create a transaction stream for any newly transactions
		TUniquePtr<FNodeClassRegistryTransactionStream> CreateTransactionStream();

	private:
		struct FActiveRegistrationTaskInfo
		{
			FNodeClassRegistryTransaction::ETransactionType TransactionType = FNodeClassRegistryTransaction::ETransactionType::NodeRegistration;
			UE::Tasks::FTask Task;
			FTopLevelAssetPath AssetPath;
			int32 RegistrationTaskID = 0;
			const TScriptInterface<IMetaSoundDocumentInterface> RegistryDocInterface;

			TSharedPtr<UE::Tasks::FCancellationToken> CancellationToken;
		};

		void BuildAndRegisterGraphFromDocument(const FMetasoundFrontendDocument& InDocument, const FProxyDataCache& InProxyDataCache, const FGraphRegistryKey& InGraphRegistryKey, TArrayView<const FGuid> InPageOrder);

		void AddRegistrationTask(const FGraphRegistryKey& InKey, FActiveRegistrationTaskInfo&& TaskInfo);
		void RemoveRegistrationTask(const FGraphRegistryKey& InKey, int32 InRegistrationTaskID, FNodeClassRegistryTransaction::ETransactionType TransactionType);

		// Adds reference to document's owning UObject to internal ObjectReferencer,
		// indicating async registration/unregistration task(s) are depending on it.
		void AddDocumentReference(TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface);

		// Removes reference to document's owning UObject from internal ObjectReferencer,
		// indicating async registration/unregistration task(s) are no longer depending on it.
		void RemoveDocumentReference(TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface);

		bool UnregisterNodeInternal(const FNodeClassRegistryKey& InKey);
		FNodeClassRegistryKey RegisterNodeInternal(TUniquePtr<INodeClassRegistryEntry>&& InEntry);

		static FNodeClassRegistry* LazySingleton;

		void WaitForAsyncRegistrationInternal(const FNodeClassRegistryKey& InRegistryKey, const FTopLevelAssetPath* InAssetPath) const;

		void RegisterGraphInternal(const FGraphRegistryKey& InKey, TSharedPtr<const FGraph> InGraph);
		bool UnregisterGraphInternal(const FGraphRegistryKey& InKey);

		// Access a node entry safely. Node entries can be added/removed asynchronously. Functions passed to this method will be
		// executed in a manner where access to the node registry entry is safe from threading issues. 
		//
		// @returns true if a node registry entry was found and the function executed. False if the entry was not 
		//          found and the function not executed. 
		bool AccessNodeEntryThreadSafe(const FNodeClassRegistryKey& InKey, TFunctionRef<void(const INodeClassRegistryEntry&)> InFunc) const;

		const INodeTemplateRegistryEntry* FindNodeTemplateEntry(const FNodeClassRegistryKey& InKey) const;

		bool IsCustomNodeUpdateTransformWellFormed(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion, const TSharedRef<const FBaseNodeUpdateTransform>& InTransform);

		// This buffer is used to enqueue nodes and datatypes to register when the module has been initialized,
		// in order to avoid bad behavior with ensures, logs, etc. on static initialization.
		// The bad news is that TInlineAllocator is the safest allocator to use on static init.
		// The good news is that none of these lambdas typically have captures, so this should have low memory overhead.
		static constexpr int32 MaxNumNodesAndDatatypesToInitialize = 2048;
		TArray<TUniqueFunction<void()>, TInlineAllocator<MaxNumNodesAndDatatypesToInitialize>> LazyInitCommands;
		
		FCriticalSection LazyInitCommandCritSection;

		// Registry in which we keep all information about nodes implemented in C++.
		TMultiMap<FNodeClassRegistryKey, TSharedRef<INodeClassRegistryEntry, ESPMode::ThreadSafe>> RegisteredNodes;

#if WITH_EDITORONLY_DATA
		TMultiMap<FNodeClassRegistryKey, FNodeMigrationInfo> NodeMigrations;
		// Custom node update transforms
		using FMajorNodeUpdateTransformKey = TPair<const FMetasoundFrontendClassName, int32 /*MajorVersion*/>;
		TMap<FMajorNodeUpdateTransformKey, TSharedRef<const FBaseNodeUpdateTransform>> CustomNodeUpdateTransforms;
#endif

		// Registry in which we keep all information about dynamically-generated templated nodes via in C++.
		TMap<FNodeClassRegistryKey, TSharedRef<INodeTemplateRegistryEntry, ESPMode::ThreadSafe>> RegisteredNodeTemplates;

		// Map of all registered graphs
		TMap<FGraphRegistryKey, TSharedPtr<const FGraph>> RegisteredGraphs;

		// Registry in which we keep lists of possible nodes to use to convert between two datatypes
		TMap<FConverterNodeClassRegistryKey, FConverterNodeClassRegistryValue> ConverterNodeClassRegistry;

		TSharedRef<FNodeClassRegistryTransactionBuffer> TransactionBuffer;

		mutable FTransactionallySafeCriticalSection RegistryMapsCriticalSection;
		mutable FTransactionallySafeCriticalSection ActiveRegistrationTasksCriticalSection;
		mutable FTransactionallySafeCriticalSection ObjectReferencerCriticalSection;

		UE::Tasks::FPipe AsyncRegistrationPipe;
		int32 RegistrationTaskCntr = 0;
		TMap<FNodeClassRegistryKey, TArray<FActiveRegistrationTaskInfo>> ActiveRegistrationTasks;
		TUniquePtr<INodeClassRegistry::IObjectReferencer> ObjectReferencer;
	};
}
