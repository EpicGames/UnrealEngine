// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodeController.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendInputController.h"
#include "MetasoundFrontendInvalidController.h"
#include "MetasoundFrontendOutputController.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontendNodeController"

namespace Metasound
{
	namespace Frontend
	{
		//
		// FBaseNodeController
		//
		FBaseNodeController::FBaseNodeController(const FBaseNodeController::FInitParams& InParams)
		: NodePtr(InParams.NodePtr)
		, ClassPtr(InParams.ClassPtr)
		, OwningGraph(InParams.OwningGraph)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
				{
					if (Node->ClassID != Class->ID)
					{
						UE_LOGF(LogMetaSound, Warning, "Changing node's class id from [ClassID:%ls] to [ClassID:%ls]", *Node->ClassID.ToString(), *Class->ID.ToString());
						Node->ClassID = Class->ID;
					}
				}
			}
		}

		bool FBaseNodeController::IsValid() const
		{
			return OwningGraph->IsValid() && (nullptr != NodePtr.Get()) && (nullptr != ClassPtr.Get());
		}

		FGuid FBaseNodeController::GetOwningGraphClassID() const
		{
			return OwningGraph->GetClassID();
		}

		FGraphHandle FBaseNodeController::GetOwningGraph()
		{
			return OwningGraph;
		}

		FConstGraphHandle FBaseNodeController::GetOwningGraph() const
		{
			return OwningGraph;
		}

		FGuid FBaseNodeController::GetID() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->GetID();
			}
			return Metasound::FrontendInvalidID;
		}

		FGuid FBaseNodeController::GetClassID() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->ID;
			}
			return Metasound::FrontendInvalidID;
		}

		const FMetasoundFrontendLiteral* FBaseNodeController::GetInputLiteral(const FGuid& InVertexID) const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				for (const FMetasoundFrontendVertexLiteral& VertexLiteral : Node->InputLiterals)
				{
					if (VertexLiteral.VertexID == InVertexID)
					{
						return &VertexLiteral.Value;
					}
				}
			}

			return nullptr;
		}

		void FBaseNodeController::SetInputLiteral(const FMetasoundFrontendVertexLiteral& InVertexLiteral)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				auto IsInputVertex = [InVertexLiteral] (const FMetasoundFrontendVertex& Vertex)
				{
					return InVertexLiteral.VertexID == Vertex.VertexID;
				};

				FMetasoundFrontendNodeInterface& NodeInterface = Node->Interface;
				if (!ensure(NodeInterface.Inputs.ContainsByPredicate(IsInputVertex)))
				{
					return;
				}

				for (FMetasoundFrontendVertexLiteral& VertexLiteral : Node->InputLiterals)
				{
					if (VertexLiteral.VertexID == InVertexLiteral.VertexID)
					{
						// Type None implies forwarding to the node instance to default construct the literal 
						// so setting to/from that class default literal is an exception to requiring a type match
						bool bSettingToClassDefaultLiteral = false;
						bool bSettingFromClassDefaultLiteralToMatchingType = false;

						auto GetInputWithVertexID = [InVertexID = VertexLiteral.VertexID](const FMetasoundFrontendClassInput& ClassInput)
						{
							return ClassInput.VertexID == InVertexID;
						};

						const FMetasoundFrontendClassInput* ClassInput = nullptr;
						if (const FMetasoundFrontendClassInterface* ClassInterface = GetClassInterfaceForNode())
						{
							ClassInput = ClassInterface->Inputs.FindByPredicate(GetInputWithVertexID);
						}

						if (ClassInput)
						{
							// Check if setting back to class default literal
							const FMetasoundFrontendLiteral& ClassDefaultLiteral = ClassInput->FindConstDefaultChecked(Frontend::DefaultPageID);
							bSettingToClassDefaultLiteral = ClassDefaultLiteral.IsEqual(InVertexLiteral.Value);

							// Check if setting from class default literal (which may have a None type) to an appropriate type 
							FDataTypeRegistryInfo DataTypeInfo;
							IDataTypeRegistry::Get().GetDataTypeInfo(ClassInput->TypeName, DataTypeInfo);
							const EMetasoundFrontendLiteralType ClassInputLiteralType = static_cast<EMetasoundFrontendLiteralType>(DataTypeInfo.PreferredLiteralType);
							bSettingFromClassDefaultLiteralToMatchingType |= ClassDefaultLiteral.IsEqual(VertexLiteral.Value) && ClassInputLiteralType == InVertexLiteral.Value.GetType();
						}

						if (bSettingToClassDefaultLiteral)
						{
							ClearInputLiteral(VertexLiteral.VertexID);
							return;
						}
						
						const bool bIsMatchingType = VertexLiteral.Value.GetType() == InVertexLiteral.Value.GetType();
						if (ensure(bIsMatchingType || bSettingFromClassDefaultLiteralToMatchingType))
						{
							VertexLiteral = InVertexLiteral;
						}
						return;
					}
				}

				Node->InputLiterals.Add(InVertexLiteral);
			}
		}

		bool FBaseNodeController::ClearInputLiteral(FGuid InVertexID)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				auto IsInputVertex = [InVertexID](const FMetasoundFrontendVertexLiteral& VertexLiteral)
				{
					return InVertexID == VertexLiteral.VertexID;
				};

				return Node->InputLiterals.RemoveAllSwap(IsInputVertex, EAllowShrinking::No) > 0;
			}

			return false;
		}

		const FMetasoundFrontendClassInterface& FBaseNodeController::GetClassInterface() const
		{
			if (const FMetasoundFrontendClassInterface* Interface = GetClassInterfaceForNode())
			{
				return *Interface;
			}
			return Invalid::GetInvalidClassInterface();
		}

		const FMetasoundFrontendClassMetadata& FBaseNodeController::GetClassMetadata() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->Metadata;
			}
			return Invalid::GetInvalidClassMetadata();
		}

		const FMetasoundFrontendNodeInterface& FBaseNodeController::GetNodeInterface() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->Interface;
			}
			return Invalid::GetInvalidNodeInterface();
		}

#if WITH_EDITOR
		const FMetasoundFrontendInterfaceStyle& FBaseNodeController::GetInputStyle() const
		{
			if (const FMetasoundFrontendClassInterface* ClassInterface = GetClassInterfaceForNode())
			{
				return ClassInterface->GetInputStyle();
			}

			return Invalid::GetInvalidInterfaceStyle();
		}

		const FMetasoundFrontendInterfaceStyle& FBaseNodeController::GetOutputStyle() const
		{
			if (const FMetasoundFrontendClassInterface* ClassInterface = GetClassInterfaceForNode())
			{
				return ClassInterface->GetOutputStyle();
			}

			return Invalid::GetInvalidInterfaceStyle();
		}

		const FMetasoundFrontendClassStyle& FBaseNodeController::GetClassStyle() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->Style;
			}

			static const FMetasoundFrontendClassStyle Invalid;
			return Invalid;
		}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA

		const FMetasoundFrontendNodeStyle& FBaseNodeController::GetNodeStyle() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->Style;
			}

			static const FMetasoundFrontendNodeStyle Invalid;
			return Invalid;
		}

		void FBaseNodeController::SetNodeStyle(const FMetasoundFrontendNodeStyle& InStyle)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				Node->Style = InStyle;
			}
		}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

		const FText& FBaseNodeController::GetDescription() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->Metadata.GetDescription();
			}
			return Invalid::GetInvalidText();
		}
#endif // WITH_EDITOR

		const FVertexName& FBaseNodeController::GetNodeName() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->Name;
			}
			return Invalid::GetInvalidName();
		}

		bool FBaseNodeController::CanAddInput(const FVertexName& InVertexName) const
		{
			// TODO: not yet supported
			return false;
		}

		FInputHandle FBaseNodeController::AddInput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault)
		{
			checkNoEntry();
			// TODO: not yet supported
			return IInputController::GetInvalidHandle();
		}

		bool FBaseNodeController::RemoveInput(FGuid InVertexID)
		{
			checkNoEntry();
			// TODO: not yet supported
			return false;
		}

		bool FBaseNodeController::CanAddOutput(const FVertexName& InVertexName) const
		{
			// TODO: not yet supported
			return false;
		}

		FInputHandle FBaseNodeController::AddOutput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault)
		{
			checkNoEntry();
			// TODO: not yet supported
			return IInputController::GetInvalidHandle();
		}

		bool FBaseNodeController::RemoveOutput(FGuid InVertexID)
		{
			checkNoEntry();
			// TODO: not yet supported
			return false;
		}

		TArray<FInputHandle> FBaseNodeController::GetInputs()
		{
			TArray<FInputHandle> Inputs;

			FNodeHandle ThisNode = this->AsShared();

			for (const FInputControllerParams& Params : GetInputControllerParams())
			{
				FInputHandle InputHandle = CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
				if (InputHandle->IsValid())
				{
					Inputs.Add(InputHandle);
				}
			}

			return Inputs;
		}

		int32 FBaseNodeController::GetNumInputs() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->Interface.Inputs.Num();
			}

			return 0;
		}

		void FBaseNodeController::IterateInputs(TUniqueFunction<void(FInputHandle)> InFunction)
		{
			for (const FInputControllerParams& Params : GetInputControllerParams())
			{
				FInputHandle InputHandle = CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, AsShared());
				if (InputHandle->IsValid())
				{
					InFunction(InputHandle);
				}
			}
		}

		TArray<FOutputHandle> FBaseNodeController::GetOutputs()
		{
			TArray<FOutputHandle> Outputs;

			FNodeHandle ThisNode = this->AsShared();

			for (const FOutputControllerParams& Params : GetOutputControllerParams())
			{
				FOutputHandle OutputHandle = CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
				if (OutputHandle->IsValid())
				{
					Outputs.Add(OutputHandle);
				}
			}

			return Outputs;
		}

		int32 FBaseNodeController::GetNumOutputs() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				return Node->Interface.Outputs.Num();
			}

			return 0;
		}

		TArray<FConstInputHandle> FBaseNodeController::GetConstInputs() const
		{
			TArray<FConstInputHandle> Inputs;

			// If I had a nickle for every time C++ backed me into a corner, I would be sitting
			// on a tropical beach next to my mansion sipping strawberry daiquiris instead of 
			// trying to code using this guileful language. The const cast is generally safe here
			// because the FConstInputHandle only allows const access to the internal node controller. 
			// Ostensibly, there could have been a INodeController and IConstNodeController
			// which take different types in their constructor, but it starts to become
			// difficult to maintain. So instead of adding 500 lines of nearly duplicate 
			// code, a ConstCastSharedRef is used here. 
			FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());

			for (const FInputControllerParams& Params : GetInputControllerParams())
			{
				FConstInputHandle InputHandle = CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
				if (InputHandle->IsValid())
				{
					Inputs.Add(InputHandle);
				}
			}

			return Inputs;
		}

		void FBaseNodeController::IterateOutputs(TUniqueFunction<void(FOutputHandle)> InFunction)
		{
			for (const FOutputControllerParams& Params : GetOutputControllerParams())
			{
				FOutputHandle OutputHandle = CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, AsShared());
				if (OutputHandle->IsValid())
				{
					InFunction(OutputHandle);
				}
			}
		}

#if WITH_EDITOR
		const FText& FBaseNodeController::GetDisplayTitle() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		FText FBaseNodeController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return Class->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}
#endif // WITH_EDITOR

		void FBaseNodeController::IterateConstInputs(TUniqueFunction<void(FConstInputHandle)> InFunction) const
		{
			// See early use of ConstCastSharedRef in this class for discussion.
			FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());

			for (const FInputControllerParams& Params : GetInputControllerParams())
			{
				FConstInputHandle InputHandle = CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
				if (InputHandle->IsValid())
				{
					InFunction(InputHandle);
				}
			}
		}

		TArray<FConstOutputHandle> FBaseNodeController::GetConstOutputs() const
		{
			TArray<FConstOutputHandle> Outputs;

			// See early use of ConstCastSharedRef in this class for discussion.
			FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());

			for (const FOutputControllerParams& Params : GetOutputControllerParams())
			{
				FConstOutputHandle OutputHandle = CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
				if (OutputHandle->IsValid())
				{
					Outputs.Add(OutputHandle);
				}
			}

			return Outputs;
		}

		void FBaseNodeController::IterateConstOutputs(TUniqueFunction<void(FConstOutputHandle)> InFunction) const
		{
			// See early use of ConstCastSharedRef in this class for discussion.
			FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());

			for (const FOutputControllerParams& Params : GetOutputControllerParams())
			{
				FConstOutputHandle OutputHandle = CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
				if (OutputHandle->IsValid())
				{
					InFunction(OutputHandle);
				}
			}
		}

		FInputHandle FBaseNodeController::GetInputWithVertexName(const FVertexName& InName)
		{
			FInputControllerParams Params;
			if (FindInputControllerParamsWithVertexName(InName, Params))
			{
				FNodeHandle ThisNode = this->AsShared();
				return CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
			}

			return IInputController::GetInvalidHandle();
		}

		FConstInputHandle FBaseNodeController::GetConstInputWithVertexName(const FVertexName& InName) const
		{
			FInputControllerParams Params;
			if (FindInputControllerParamsWithVertexName(InName, Params))
			{
				// See early use of ConstCastSharedRef in this class for discussion.
				FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());
				return CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
			}

			return IInputController::GetInvalidHandle();
		}

		FOutputHandle FBaseNodeController::GetOutputWithVertexName(const FVertexName& InName)
		{
			FOutputControllerParams Params;
			if (FindOutputControllerParamsWithVertexName(InName, Params))
			{
				FNodeHandle ThisNode = this->AsShared();
				return CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
			}

			return IOutputController::GetInvalidHandle();
		}

		FConstOutputHandle FBaseNodeController::GetConstOutputWithVertexName(const FVertexName& InName) const
		{
			FOutputControllerParams Params;
			if (FindOutputControllerParamsWithVertexName(InName, Params))
			{
				// See early use of ConstCastSharedRef in this class for discussion.
				FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());
				return CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
			}

			return IOutputController::GetInvalidHandle();
		}

		FInputHandle FBaseNodeController::GetInputWithID(FGuid InVertexID)
		{
			FInputControllerParams Params;

			if (FindInputControllerParamsWithID(InVertexID, Params))
			{
				FNodeHandle ThisNode = this->AsShared();
				return CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
			}

			return IInputController::GetInvalidHandle();
		}

		bool FBaseNodeController::IsInterfaceMember() const
		{
			return GetInterfaceVersion() != FMetasoundFrontendVersion::GetInvalid();
		}

		const FMetasoundFrontendVersion& FBaseNodeController::GetInterfaceVersion() const
		{
			return FMetasoundFrontendVersion::GetInvalid();
		}

		FConstInputHandle FBaseNodeController::GetInputWithID(FGuid InVertexID) const
		{
			FInputControllerParams Params;

			if (FindInputControllerParamsWithID(InVertexID, Params))
			{
				// See early use of ConstCastSharedRef in this class for discussion.
				FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());
				return CreateInputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassInputPtr, ThisNode);
			}

			return IInputController::GetInvalidHandle();
		}

		FOutputHandle FBaseNodeController::GetOutputWithID(FGuid InVertexID)
		{
			FOutputControllerParams Params;

			if (FindOutputControllerParamsWithID(InVertexID, Params))
			{
				FNodeHandle ThisNode = this->AsShared();
				return CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
			}

			return IOutputController::GetInvalidHandle();
		}

		FConstOutputHandle FBaseNodeController::GetOutputWithID(FGuid InVertexID) const
		{
			FOutputControllerParams Params;

			if (FindOutputControllerParamsWithID(InVertexID, Params))
			{
				// See early use of ConstCastSharedRef in this class for discussion.
				FNodeHandle ThisNode = ConstCastSharedRef<INodeController>(this->AsShared());
				return CreateOutputController(Params.VertexID, Params.NodeVertexPtr, Params.ClassOutputPtr, ThisNode);
			}

			return IOutputController::GetInvalidHandle();
		}

		TArray<FBaseNodeController::FInputControllerParams> FBaseNodeController::GetInputControllerParams() const
		{
			TArray<FBaseNodeController::FInputControllerParams> Inputs;

			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				for (const FMetasoundFrontendVertex& NodeInputVertex : Node->Interface.Inputs)
				{
					FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetInputWithName(NodeInputVertex.Name);
					FConstClassInputAccessPtr ClassInputPtr = ClassPtr.GetInputWithName(NodeInputVertex.Name);

					Inputs.Add({NodeInputVertex.VertexID, NodeVertexPtr, ClassInputPtr});
				}
			}

			return Inputs;
		}

		TArray<FBaseNodeController::FOutputControllerParams> FBaseNodeController::GetOutputControllerParams() const
		{
			TArray<FBaseNodeController::FOutputControllerParams> Outputs;

			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				for (const FMetasoundFrontendVertex& NodeOutputVertex : Node->Interface.Outputs)
				{
					const FVertexName& VertexName = NodeOutputVertex.Name;

					FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetOutputWithName(VertexName);
					FConstClassOutputAccessPtr ClassOutputPtr = ClassPtr.GetOutputWithName(VertexName);

					Outputs.Add({NodeOutputVertex.VertexID, NodeVertexPtr, ClassOutputPtr});
				}
			}

			return Outputs;
		}

		bool FBaseNodeController::FindInputControllerParamsWithVertexName(const FVertexName& InName, FInputControllerParams& OutParams) const
		{
			FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetInputWithName(InName);

			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				FConstClassInputAccessPtr ClassInputPtr = ClassPtr.GetInputWithName(InName);

				OutParams = FInputControllerParams{Vertex->VertexID, NodeVertexPtr, ClassInputPtr};
				return true;
			}

			return false;
		}

		bool FBaseNodeController::FindOutputControllerParamsWithVertexName(const FVertexName& InName, FOutputControllerParams& OutParams) const
		{
			FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetOutputWithName(InName);

			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				FConstClassOutputAccessPtr ClassOutputPtr = ClassPtr.GetOutputWithName(InName);

				OutParams = FOutputControllerParams{Vertex->VertexID, NodeVertexPtr, ClassOutputPtr};
				return true;
			}

			return false;
		}

		bool FBaseNodeController::FindInputControllerParamsWithID(FGuid InVertexID, FInputControllerParams& OutParams) const
		{
			FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetInputWithVertexID(InVertexID);

			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				FConstClassInputAccessPtr ClassInputPtr = ClassPtr.GetInputWithName(Vertex->Name);

				OutParams = FInputControllerParams{InVertexID, NodeVertexPtr, ClassInputPtr};
				return true;
			}

			return false;
		}

		bool FBaseNodeController::FindOutputControllerParamsWithID(FGuid InVertexID, FOutputControllerParams& OutParams) const
		{
			FConstVertexAccessPtr NodeVertexPtr = NodePtr.GetOutputWithVertexID(InVertexID);

			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				FConstClassOutputAccessPtr ClassOutputPtr = ClassPtr.GetOutputWithName(Vertex->Name);

				OutParams = FOutputControllerParams{InVertexID, NodeVertexPtr, ClassOutputPtr};
				return true;
			}

			return false;
		}

		const FMetasoundFrontendClassInterface* FBaseNodeController::GetClassInterfaceForNode() const
		{
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
				{
					return &Class->GetInterfaceForNode(*Node);
				}
			}
			return nullptr;
		}

		FGraphHandle FBaseNodeController::AsGraph()
		{
			// TODO: consider adding support for external graph owned in another document.
			// Will require lookup support for external subgraphs..
			
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return GetOwningGraph()->GetOwningDocument()->GetSubgraphWithClassID(Class->ID);
			}

			return IGraphController::GetInvalidHandle();
		}

		FConstGraphHandle FBaseNodeController::AsGraph() const
		{
			// TODO: add support for graph owned in another asset.
			// Will require lookup support for external subgraphs.
			if (const FMetasoundFrontendClass* Class = ClassPtr.Get())
			{
				return GetOwningGraph()->GetOwningDocument()->GetSubgraphWithClassID(Class->ID);
			}

			return IGraphController::GetInvalidHandle();
		}

		FNodeHandle FBaseNodeController::ReplaceWithVersion(const FMetasoundFrontendVersionNumber& InNewVersion, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs)
		{
			const FMetasoundFrontendClassMetadata Metadata = GetClassMetadata();

			// Lookup new version in node registry
			FNodeRegistryKey NewVersionRegistryKey = FNodeRegistryKey(Metadata.GetType(), Metadata.GetClassName(), InNewVersion);
			FMetasoundFrontendRegistryContainer* Registry = FMetasoundFrontendRegistryContainer::Get();
			checkf(nullptr != Registry, TEXT("The metasound node registry should always be available if the metasound plugin is loaded"));

			FMetasoundFrontendClass NewMetasoundClass;
			const bool bFoundNewClass = Registry->FindFrontendClassFromRegistered(NewVersionRegistryKey, NewMetasoundClass);
			if (!bFoundNewClass)
			{
				FString ClassNameString = Metadata.GetClassName().ToString();
				FString NewVersionString = InNewVersion.ToString();
				UE_LOGF(LogMetaSound, Error, "Failed to change class version from %ls to %ls for class %ls. %ls %ls is not registered.", *Metadata.GetVersion().ToString(), *NewVersionString, *ClassNameString, *ClassNameString, *NewVersionString);

				return this->AsShared();
			}

			// Gathers all info from the node that could be applied to the replacement node. 
			// This is destructive in that some objects are moved off of the node.
			FNodeInstanceReplacementData InstanceData = CaptureNodeInstanceReplacementData();

			// Remove the actual node
			if (!ensureAlways(GetOwningGraph()->RemoveNode(*this)))
			{
				return this->AsShared();
			}

			// Make sure classes are up-to-date with registered versions of class.
			// Note that this may break other nodes in the graph that have stale
			// class API, but that's on the caller to fix-up or report invalid state.
			const FNodeRegistryKey RegistryKey = FNodeRegistryKey(NewMetasoundClass.Metadata);
			FDocumentHandle Document = GetOwningGraph()->GetOwningDocument();

			constexpr bool bRefreshFromRegistry = true;
			ensureAlways(Document->FindOrAddClass(RegistryKey, bRefreshFromRegistry).Get() != nullptr);

			FNodeHandle ReplacementNode = GetOwningGraph()->AddNode(NewMetasoundClass.Metadata, InstanceData.NodeID);
			if (!ensureAlways(ReplacementNode->IsValid()))
			{
				return this->AsShared();
			}

#if WITH_EDITOR
			InstanceData.Style.bMessageNodeUpdated = ReplacementNode->GetClassMetadata().GetVersion() > Metadata.GetVersion();
#endif // WITH_EDITOR

			ApplyNodeInstanceReplacementData(NewMetasoundClass, *ReplacementNode, MoveTemp(InstanceData), OutDisconnectedInputs, OutDisconnectedOutputs);

			return ReplacementNode;
		}

		FBaseNodeController::FNodeInstanceReplacementData FBaseNodeController::CaptureNodeInstanceReplacementData()
		{
			FNodeInstanceReplacementData ReplacementData;
			FMetasoundFrontendNode* OriginalNode = NodePtr.Get();
			if (!OriginalNode)
			{
				// This node handle is invalid. There is no nothing to replace.
				return ReplacementData;
			}

#if WITH_EDITOR
			ReplacementData.Style = MoveTemp(OriginalNode->Style);
#endif // WITH_EDITOR

			// Move any configuration and override data living on the node.
			ReplacementData.Configuration = MoveTemp(OriginalNode->Configuration);
			ReplacementData.ClassInterfaceOverride = MoveTemp(OriginalNode->ClassInterfaceOverride);;
			ReplacementData.NodeID = OriginalNode->GetID();
			
			// Cache input/output connections by name to try so they can be
			// hooked back up after swapping to the new class version.
			IterateInputs([Connections = &ReplacementData.InputConnections](FInputHandle InputHandle)
			{
				bool bLiteralSet = false;
				FMetasoundFrontendLiteral DefaultLiteral;
				if (const FMetasoundFrontendLiteral* Literal = InputHandle->GetLiteral())
				{
					DefaultLiteral = *Literal;
					bLiteralSet = true;
				}

				const FVertexNameAndType ConnectionKey(InputHandle->GetName(), InputHandle->GetDataType());
				Connections->Add(ConnectionKey, FInputConnectionInfo
				{
					InputHandle->GetConnectedOutput(),
					InputHandle->GetName(),
					InputHandle->GetDataType(),
					MoveTemp(DefaultLiteral),
					bLiteralSet
				});
			});


			IterateOutputs([Connections = &ReplacementData.OutputConnections](FOutputHandle OutputHandle)
			{
				const FVertexNameAndType ConnectionKey(OutputHandle->GetName(), OutputHandle->GetDataType());
				Connections->Add(ConnectionKey, FOutputConnectionInfo
				{
					OutputHandle->GetConnectedInputs(),
					OutputHandle->GetName(),
					OutputHandle->GetDataType()
				});
			});

			// nrvo.
			return ReplacementData;
		}

		void FBaseNodeController::ApplyNodeInstanceReplacementData(const FMetasoundFrontendClass& InReplacementClass, INodeController& InReplacementNode, FNodeInstanceReplacementData&& InInstanceData, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs)
		{
#if WITH_EDITOR
			InReplacementNode.SetNodeStyle(InInstanceData.Style);
#endif // WITH_EDITOR

			// The logic may appear a little backwards here because adding the node already instantiated the replacement
			// node configuration instances. The purpose is to revert the replacement if it should not have happened in 
			// the first place.
			if (!ShouldReplaceExistingNodeConfig(InReplacementNode.GetNodeConfiguration(), InInstanceData.Configuration))
			{
				// We should have kept the original node configuration. Set back to the original
				InReplacementNode.SetNodeConfiguration(MoveTemp(InInstanceData.Configuration));
			}
			

			InReplacementNode.IterateInputs([&InputConnections = InInstanceData.InputConnections](FInputHandle InputHandle)
			{
				const FVertexNameAndType ConnectionKey(InputHandle->GetName(), InputHandle->GetDataType());
				if (FInputConnectionInfo* ConnectionInfo = InputConnections.Find(ConnectionKey))
				{
					if (ConnectionInfo->bLiteralSet)
					{
						InputHandle->SetLiteral(ConnectionInfo->DefaultValue);
					}

					if (ConnectionInfo->ConnectedOutput->IsValid() && 
						InputHandle->CanConnectTo(*ConnectionInfo->ConnectedOutput).Connectable == FConnectability::EConnectable::Yes)
					{
						ensure(InputHandle->Connect(*ConnectionInfo->ConnectedOutput));
						
						// Remove connection to track missing connections between 
						// node versions.
						InputConnections.Remove(ConnectionKey);
					}
				}
			});

			// Track missing input connections
			if (nullptr != OutDisconnectedInputs)
			{
				for (const auto& ConnectionInfoKV : InInstanceData.InputConnections)
				{
					const FInputConnectionInfo& ConnectionInfo = ConnectionInfoKV.Value;
					if (ConnectionInfo.ConnectedOutput->IsValid())
					{
						OutDisconnectedInputs->Add(FVertexNameAndType{ConnectionInfo.Name, ConnectionInfo.DataType});
					}
				}
			}

			InReplacementNode.IterateOutputs([&OutputConnections = InInstanceData.OutputConnections](FOutputHandle OutputHandle)
			{
				const FVertexNameAndType ConnectionKey(OutputHandle->GetName(), OutputHandle->GetDataType());
				if (FOutputConnectionInfo* ConnectionInfo = OutputConnections.Find(ConnectionKey))
				{
					bool bConnectionSuccess = false;
					for (FInputHandle InputHandle : ConnectionInfo->ConnectedInputs)
					{
						if (InputHandle->IsValid() && 
							InputHandle->CanConnectTo(*OutputHandle).Connectable == FConnectability::EConnectable::Yes)
						{
							ensure(InputHandle->Connect(*OutputHandle));
							bConnectionSuccess = true;
						}
					}
					// Remove connection to track missing connections between 
					// node versions.
					if (bConnectionSuccess)
					{
						OutputConnections.Remove(ConnectionKey);
					}
				}
			});

			// Track missing output connections
			if (nullptr != OutDisconnectedOutputs)
			{
				for (const auto& ConnectionInfoKV : InInstanceData.OutputConnections)
				{
					const FOutputConnectionInfo& ConnectionInfo = ConnectionInfoKV.Value;
					const bool bAnyConnectedInputs = Algo::AnyOf(ConnectionInfo.ConnectedInputs, [](const FInputHandle& Input) { return Input->IsValid(); });
					if (bAnyConnectedInputs)
					{
						OutDisconnectedOutputs->Add(FVertexNameAndType{ConnectionInfo.VertexName, ConnectionInfo.DataType});
					}
				}
			}
		}

		void FBaseNodeController::SetNodeConfiguration(TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfig)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				// If neither the existing, nor new configs are valid, then there should not be any class interface override
				const bool bUpdateClassInterfaceOverride = Node->Configuration.IsValid() || InConfig.IsValid();

				Node->Configuration = MoveTemp(InConfig);

				if (bUpdateClassInterfaceOverride)
				{
					if (const FMetaSoundFrontendNodeConfiguration* NodeConfig = Node->Configuration.GetPtr())
					{
						if (const FMetasoundFrontendClass* Class = ClassPtr.Get(); ensure(Class))
						{
							// Update the class interface override if we have a node configration
							SetClassInterfaceOverride(NodeConfig->OverrideDefaultInterface(*Class));
						}
					}
					else
					{
						// Clear out the class interface override if there is no node configuration
						SetClassInterfaceOverride(TInstancedStruct<FMetasoundFrontendClassInterface>());
					}
				}
				else
				{
					// Class interface override should be invalid if there isn't 
					// a node config on the node
					check(!Node->ClassInterfaceOverride.IsValid());
				}
			}
		}

		TConstStructView<FMetaSoundFrontendNodeConfiguration> FBaseNodeController::GetNodeConfiguration() const
		{
			TConstStructView<FMetaSoundFrontendNodeConfiguration> ConfigOnNode;
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				ConfigOnNode = Node->Configuration;
			}
			return ConfigOnNode;
		}

		void FBaseNodeController::SetClassInterfaceOverride(TInstancedStruct<FMetasoundFrontendClassInterface> InClassInterfaceOverride)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				// If we are adding, removing or replacing the interface, then we need to
				// update the node interface.  If the override never existed, we can
				// skip the process of updating the node's interface. 
				const bool bUpdateNodeInterface = Node->ClassInterfaceOverride.IsValid() || InClassInterfaceOverride.IsValid();

				Node->ClassInterfaceOverride = MoveTemp(InClassInterfaceOverride);

				if (bUpdateNodeInterface)
				{
					if (const FMetasoundFrontendClass* Class = ClassPtr.Get(); ensure(Class))
					{
						auto DisconnectInput = [this](const FMetasoundFrontendVertex& InNodeInput)
						{
							this->GetInputWithVertexName(InNodeInput.Name)->Disconnect();
						};

						auto DisconnectOutput = [this](const FMetasoundFrontendVertex& InNodeInput)
						{
							this->GetOutputWithVertexName(InNodeInput.Name)->Disconnect();
						};

						Node->Interface.Update(Class->GetInterfaceForNode(*Node), DisconnectInput, DisconnectOutput);
					}
				}
			}
		}

		TConstStructView<FMetasoundFrontendClassInterface> FBaseNodeController::GetClassInterfaceOverride() const
		{
			TConstStructView<FMetasoundFrontendClassInterface> ClassInterfaceOverride;
			if (const FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				ClassInterfaceOverride = Node->ClassInterfaceOverride;
			}
			return ClassInterfaceOverride;
		}

		bool FBaseNodeController::ShouldReplaceExistingNodeConfig(TConstStructView<FMetaSoundFrontendNodeConfiguration> InRegisteredNodeConfig, TConstStructView<FMetaSoundFrontendNodeConfiguration> InExistingConfig) const
		{
			// The existing node configuration is not always replaced. Replacing
			// the node configuration struct loses any modified state on the existing
			// node config. In the scenario where the node configurations point to
			// the same derived class type, we do not replace them. 
			return InRegisteredNodeConfig.GetScriptStruct() != InExistingConfig.GetScriptStruct();
		}

		void FBaseNodeController::FindNodeConfigurationUpdates(const FMetasoundFrontendClass& InRegisteredClass, FNodeConfigurationUpdateData& OutNodeConfigurationUpdates, bool bInForceRegenerateClassInterfaceOverride) const
		{
			const INodeClassRegistry* Registry = INodeClassRegistry::Get();
			checkf(nullptr != Registry, TEXT("The metasound node registry should always be available if the metasound plugin is loaded"));

			OutNodeConfigurationUpdates.ExistingConfig = GetNodeConfiguration();
			OutNodeConfigurationUpdates.ExistingClassInterfaceOverride = GetClassInterfaceOverride();

			if (InRegisteredClass.Metadata.GetType() == EMetasoundFrontendClassType::Template)
			{
				if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(FNodeRegistryKey(InRegisteredClass.Metadata)))
				{
					OutNodeConfigurationUpdates.RegisteredConfig = Template->CreateFrontendTemplateNodeConfiguration();
				}
			}
			else
			{
				OutNodeConfigurationUpdates.RegisteredConfig = Registry->CreateFrontendNodeConfiguration(FNodeRegistryKey(InRegisteredClass.Metadata));
			}

			// Determine which node config will be used to create the class interface override;
			const bool bWillReplaceNodeConfig = ShouldReplaceExistingNodeConfig(OutNodeConfigurationUpdates.RegisteredConfig, OutNodeConfigurationUpdates.ExistingConfig);
			TConstStructView<FMetaSoundFrontendNodeConfiguration> EffectiveNodeConfig = bWillReplaceNodeConfig ? OutNodeConfigurationUpdates.RegisteredConfig : OutNodeConfigurationUpdates.ExistingConfig;

			// Update the 
			if (bWillReplaceNodeConfig || bInForceRegenerateClassInterfaceOverride)
			{
				// Set this flag to true even if the effective node configuration pointer 
				// is invalid. If the effective node configuration is null, this captures
				// the fact that a null node configuration should produce an empty 
				// ClassInterfaceOverride. 
				OutNodeConfigurationUpdates.bDidUpdateClassInterfaceOverride = true;
				if (const FMetaSoundFrontendNodeConfiguration* ConfigPtr = EffectiveNodeConfig.GetPtr())
				{
					OutNodeConfigurationUpdates.RegeneratedClassInterfaceOverride = ConfigPtr->OverrideDefaultInterface(InRegisteredClass);
				}
			}
		}

		const FMetasoundFrontendClassInterface& FBaseNodeController::GetApplicableRegistryInterface(const FMetasoundFrontendClass& InRegisteredClass, const FNodeConfigurationUpdateData& InNodeConfigurationUpdates) const
		{
			// Check for the latest class interface overrides
			if (InNodeConfigurationUpdates.bDidUpdateClassInterfaceOverride)
			{
				if (const FMetasoundFrontendClassInterface* Interface = InNodeConfigurationUpdates.RegeneratedClassInterfaceOverride.GetPtr())
				{
					return *Interface;
				}
			}
			else
			{
				// The class interface override wasn't updated. Use the original if it exists. 
				if (const FMetasoundFrontendClassInterface* Interface = InNodeConfigurationUpdates.ExistingClassInterfaceOverride.GetPtr())
				{
					return *Interface;
				}
			}

			// If the override is invalid, use the interface on the class.
			return InRegisteredClass.GetDefaultInterface();
		}

#if WITH_EDITORONLY_DATA
		void FBaseNodeController::DiffClassInterfaces(const FMetasoundFrontendClassInterface& InOriginalInterface, const FMetasoundFrontendClassInterface& InNewInterface, FClassInterfaceUpdates& OutClassInterfaceUpdates) const
		{
			Algo::Transform(InNewInterface.Inputs, OutClassInterfaceUpdates.AddedInputs, [](const FMetasoundFrontendClassInput& Input) { return &Input; });
			for (const FMetasoundFrontendClassInput& Input : InOriginalInterface.Inputs)
			{
				auto IsEquivalent = [&Input](const FMetasoundFrontendClassInput* RegistryInput)
				{
					return FMetasoundFrontendClassInput::IsFunctionalEquivalent(Input, *RegistryInput);
				};

				const int32 Index = OutClassInterfaceUpdates.AddedInputs.FindLastByPredicate(IsEquivalent);
				if (Index == INDEX_NONE)
				{
					OutClassInterfaceUpdates.RemovedInputs.Add(&Input);
				}
				else
				{
					OutClassInterfaceUpdates.AddedInputs.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}


			Algo::Transform(InNewInterface.Outputs, OutClassInterfaceUpdates.AddedOutputs, [](const FMetasoundFrontendClassOutput& Output) { return &Output; });
			for (const FMetasoundFrontendClassOutput& Output : InOriginalInterface.Outputs)
			{
				auto IsFunctionalEquivalent = [&Output](const FMetasoundFrontendClassOutput* Iter)
				{
					return FMetasoundFrontendClassVertex::IsFunctionalEquivalent(Output, *Iter);
				};

				const int32 Index = OutClassInterfaceUpdates.AddedOutputs.FindLastByPredicate(IsFunctionalEquivalent);
				if (Index == INDEX_NONE)
				{
					OutClassInterfaceUpdates.RemovedOutputs.Add(&Output);
				}
				else
				{
					OutClassInterfaceUpdates.AddedOutputs.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
		}

		bool FBaseNodeController::DiffAgainstRegistryInterface(FClassInterfaceUpdates& OutInterfaceUpdates, bool bInUseHighestMinorVersion, bool bInForceRegenerateClassInterfaceOverride) const
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(BaseNodeController::DiffAgainstRegistryInterface);

			OutInterfaceUpdates = FClassInterfaceUpdates();

			const FMetasoundFrontendClassMetadata& NodeClassMetadata = GetClassMetadata();
			const FMetasoundFrontendClassInterface& NodeClassInterface = GetClassInterface();
			Metasound::FNodeClassName NodeClassName = NodeClassMetadata.GetClassName().ToNodeClassName();

			FNodeClassRegistryKey RegistryKey;
			if (bInUseHighestMinorVersion)
			{
				FMetaSoundClassInfo ClassInfo;
				const bool bIsRegistered = ISearchEngine::Get().FindRegisteredClass(NodeClassMetadata.GetClassName().ToNodeClassName(), NodeClassMetadata.GetVersion().Major, ClassInfo);
				if (bIsRegistered)
				{
					RegistryKey = ClassInfo.ToRegistryKey();
				}
			}
			else
			{
				RegistryKey = FNodeClassRegistryKey(NodeClassMetadata);
			}

			const bool bDidFindClass = INodeClassRegistry::GetChecked().FindFrontendClassFromRegistered(RegistryKey, OutInterfaceUpdates.RegistryClass);
			if (!bDidFindClass)
			{
				// The class couldn't be found. Mark all relevant fields as removed. 
				Algo::Transform(NodeClassInterface.Inputs, OutInterfaceUpdates.RemovedInputs, [&](const FMetasoundFrontendClassInput& Input) { return &Input; });
				Algo::Transform(NodeClassInterface.Outputs, OutInterfaceUpdates.RemovedOutputs, [&](const FMetasoundFrontendClassOutput& Output) { return &Output; });
				OutInterfaceUpdates.RemovedConfiguration = GetNodeConfiguration();
				OutInterfaceUpdates.RemovedClassInterfaceOverride = GetClassInterfaceOverride();

				return false;
			}

			// Get updates of the node's configuration
			FNodeConfigurationUpdateData NodeConfigUpdates;
			FindNodeConfigurationUpdates(OutInterfaceUpdates.RegistryClass, NodeConfigUpdates, bInForceRegenerateClassInterfaceOverride);

			// Based upon all the node configuration updates, determine which class interface is the correct up-to-date interface
			const FMetasoundFrontendClassInterface& ApplicableRegistryInterface = GetApplicableRegistryInterface(OutInterfaceUpdates.RegistryClass, NodeConfigUpdates);

			// Diff the current interface with the most recent interface
			DiffClassInterfaces(NodeClassInterface, ApplicableRegistryInterface, OutInterfaceUpdates);
			const bool bFoundDifferenceInInterface = OutInterfaceUpdates.ContainsRemovedMembers() || OutInterfaceUpdates.ContainsAddedMembers();

			// Add node config and class interface override info if necessary. 
			if (ShouldReplaceExistingNodeConfig(NodeConfigUpdates.RegisteredConfig, NodeConfigUpdates.ExistingConfig))
			{
				OutInterfaceUpdates.AddedConfiguration = MoveTemp(NodeConfigUpdates.RegisteredConfig);
				OutInterfaceUpdates.RemovedConfiguration = NodeConfigUpdates.ExistingConfig;
			}

			if (NodeConfigUpdates.bDidUpdateClassInterfaceOverride)
			{
				// The class interface override is updated if there was any difference found
				// in the interface, or the override was added/removed. 
				const bool bReplaceClassInterfaceOverride = bFoundDifferenceInInterface || (NodeConfigUpdates.ExistingClassInterfaceOverride.IsValid() != NodeConfigUpdates.RegeneratedClassInterfaceOverride.IsValid());
				if (bReplaceClassInterfaceOverride)
				{
					OutInterfaceUpdates.AddedClassInterfaceOverride = MoveTemp(NodeConfigUpdates.RegeneratedClassInterfaceOverride);
					OutInterfaceUpdates.RemovedClassInterfaceOverride = NodeConfigUpdates.ExistingClassInterfaceOverride;
				}
			}

			return true;
		}

		bool FBaseNodeController::CanAutoUpdate(FClassInterfaceUpdates& OutInterfaceUpdates) const
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(BaseNodeController::CanAutoUpdate);

#if WITH_EDITORONLY_DATA
			OutInterfaceUpdates = { };

			const FMetasoundFrontendClassMetadata& NodeClassMetadata = GetClassMetadata();
			const IMetaSoundAssetManager& AssetManager = IMetaSoundAssetManager::GetChecked();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FMetasoundFrontendClass RegistryClass;
			if (!ISearchEngine::Get().FindClassWithHighestMinorVersion( NodeClassMetadata.GetClassName().ToNodeClassName(), NodeClassMetadata.GetVersion().Major, RegistryClass))
			{
				return false;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// 1. Document's class version is somehow higher than registries, so can't update.
			if (RegistryClass.Metadata.GetVersion() < NodeClassMetadata.GetVersion())
			{
				return false;
			}

			// 2. Document's class version is equal, so have to dif and check change IDs
			// to ensure a change wasn't created that didn't contain a version change.
			if (RegistryClass.Metadata.GetVersion() == NodeClassMetadata.GetVersion())
			{
				if (FVersioningManager::ChangeIDComparisonEnabledInAutoUpdate())
				{
					// WIP: Merging these paths.  Should no longer require using different
					// logic to determine changes in native vs asset class definitions.
					// Its less performant to ignore ChangeIDs, but auto-update is no longer
					// called at runtime and the editor now avoids running on mass numbers of
					// assets at once due to the utilization of AssetTag data for querying class info.
					const bool bIsAssetClass = AssetManager.IsAssetClass(RegistryClass.Metadata);
					if (bIsAssetClass)
					{
						if (RegistryClass.Metadata.GetChangeID() == NodeClassMetadata.GetChangeID())
						{
							const FGuid& NodeClassInterfaceChangeID = GetClassInterface().GetChangeID();
							if (RegistryClass.GetDefaultInterface().GetChangeID() == NodeClassInterfaceChangeID)
							{
								return false;
							}
						}
					}
					else
					{
						if (!MetaSoundAutoUpdateNativeClassesOfEqualVersionCVar)
						{
							return false;
						}
					}
				}

				constexpr bool bUseHighestMinorVersion = true;
				constexpr bool bForceRegenerateClassInterfaceOverride = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				DiffAgainstRegistryInterface(OutInterfaceUpdates, bUseHighestMinorVersion, bForceRegenerateClassInterfaceOverride);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				return OutInterfaceUpdates.ContainsChanges();
			}

			// 3. Document's class version is out-of-date, so dif and always return true that can auto-update
			// (Unlike the case where the version is equal, the version must be updated even if the interface
			// contains no changes).
			constexpr bool bUseHighestMinorVersion = true;
			constexpr bool bForceRegenerateClassInterfaceOverride = true;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			DiffAgainstRegistryInterface(OutInterfaceUpdates, bUseHighestMinorVersion, bForceRegenerateClassInterfaceOverride);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			return true;

#else // !WITH_EDITORONLY_DATA
			return false; // Auto-Update is now never permitted in builds without non-editoronly data
#endif // !WITH_EDITORONLY_DATA

		}
#endif // WITH_EDITORONLY_DATA

		FDocumentAccess FBaseNodeController::ShareAccess()
		{
			FDocumentAccess Access;

			Access.Node = NodePtr;
			Access.ConstNode = NodePtr;
			Access.ConstClass = ClassPtr;

			return Access;
		}

		FConstDocumentAccess FBaseNodeController::ShareAccess() const
		{
			FConstDocumentAccess Access;

			Access.ConstNode = NodePtr;
			Access.ConstClass = ClassPtr;

			return Access;
		}


		// 
		// FNodeController
		//
		FNodeController::FNodeController(EPrivateToken InToken, const FNodeController::FInitParams& InParams)
		: FBaseNodeController({InParams.NodePtr, InParams.ClassPtr, InParams.OwningGraph})
		, GraphPtr(InParams.GraphPtr)
		{
		}

		FNodeHandle FNodeController::CreateNodeHandle(const FNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						return MakeShared<FNodeController>(EPrivateToken::Token, InParams);
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FNodeController::CreateConstNodeHandle(const FNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						return MakeShared<const FNodeController>(EPrivateToken::Token, InParams);
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		bool FNodeController::IsValid() const
		{
			return FBaseNodeController::IsValid() && (nullptr != GraphPtr.Get());
		}

		FInputHandle FNodeController::CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FBaseInputController>(FBaseInputController::FInitParams{InVertexID, InNodeVertexPtr, InClassInputPtr, GraphPtr, InOwningNode});
		}

		FOutputHandle FNodeController::CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FBaseOutputController>(FBaseOutputController::FInitParams{InVertexID, InNodeVertexPtr, InClassOutputPtr, GraphPtr, InOwningNode});
		}

		FDocumentAccess FNodeController::ShareAccess() 
		{
			FDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		FConstDocumentAccess FNodeController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.ConstGraph = GraphPtr;

			return Access;
		}


		//
		// FOutputNodeController
		//
		FOutputNodeController::FOutputNodeController(FOutputNodeController::EPrivateToken InToken, const FOutputNodeController::FInitParams& InParams)
		: FBaseNodeController({InParams.NodePtr, InParams.ClassPtr, InParams.OwningGraph})
		, GraphPtr(InParams.GraphPtr)
		, OwningGraphClassOutputPtr(InParams.OwningGraphClassOutputPtr)
		{
		}

		FNodeHandle FOutputNodeController::CreateOutputNodeHandle(const FOutputNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					if (EMetasoundFrontendClassType::Output == Class->Metadata.GetType())
					{
						if (Class->ID == Node->ClassID)
						{
							return MakeShared<FOutputNodeController>(EPrivateToken::Token, InParams);
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Class of incorrect class type [ClassID:%ls] while creating output node.. Must be EMetasoundFrontendClassType::Output.", *Class->ID.ToString());
					}
				}
			}

			return INodeController::GetInvalidHandle();
		}

#if WITH_EDITOR
		const FText& FOutputNodeController::GetDescription() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata.GetDescription();
			}

			return Invalid::GetInvalidText();
		}

		FText FOutputNodeController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		void FOutputNodeController::SetDescription(const FText& InDescription)
		{
			// TODO: can we remove the const cast by constructing output nodes with a non-const access to class outputs?
			if (FMetasoundFrontendClassOutput* ClassOutput = ConstCastAccessPtr<FClassOutputAccessPtr>(OwningGraphClassOutputPtr).Get())
			{
				ClassOutput->Metadata.SetDescription(InDescription);
				OwningGraph->UpdateInterfaceChangeID();
			}
		}

		void FOutputNodeController::SetDisplayName(const FText& InDisplayName)
		{
			// TODO: can we remove the const cast by constructing output nodes with a non-const access to class outputs?
			if (FMetasoundFrontendClassOutput* ClassOutput = ConstCastAccessPtr<FClassOutputAccessPtr>(OwningGraphClassOutputPtr).Get())
			{
				ClassOutput->Metadata.SetDisplayName(InDisplayName);
				OwningGraph->UpdateInterfaceChangeID();
			}
		}
#endif // WITH_EDITOR

		void FOutputNodeController::SetNodeName(const FVertexName& InName)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				Node->Name = InName;

				for (FMetasoundFrontendVertex& Vertex : Node->Interface.Inputs)
				{
					Vertex.Name = InName;
				}

				for (FMetasoundFrontendVertex& Vertex : Node->Interface.Outputs)
				{
					Vertex.Name = InName;
				}
			}

			// TODO: can we remove the const cast by constructing output nodes with a non-const access to class outputs?
			if (FMetasoundFrontendClassOutput* ClassOutput = ConstCastAccessPtr<FClassOutputAccessPtr>(OwningGraphClassOutputPtr).Get())
			{
				ClassOutput->Name = InName;
				OwningGraph->UpdateInterfaceChangeID();
			}
		}

		FConstNodeHandle FOutputNodeController::CreateConstOutputNodeHandle(const FOutputNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					if (EMetasoundFrontendClassType::Output == Class->Metadata.GetType())
					{
						if (Class->ID == Node->ClassID)
						{
							return MakeShared<const FOutputNodeController>(EPrivateToken::Token, InParams);
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Class of incorrect class type [ClassID:%ls] while creating output node.. Must be EMetasoundFrontendClassType::Output.", *Class->ID.ToString());
					}
				}
			}

			return INodeController::GetInvalidHandle();
		}

#if WITH_EDITOR
		const FText& FOutputNodeController::GetDisplayTitle() const
		{
			static FText OutputDisplayTitle = LOCTEXT("OutputNode_Title", "Output");
			return OutputDisplayTitle;
		}
#endif // WITH_EDITOR

		const FMetasoundFrontendVersion& FOutputNodeController::GetInterfaceVersion() const
		{
			FConstDocumentHandle OwningDocument = OwningGraph->GetOwningDocument();
			FConstGraphHandle RootGraph = OwningDocument->GetRootGraph();

			// Test if this node exists on the document's root graph.
			const bool bIsNodeOnRootGraph = OwningGraph->IsValid() && (RootGraph->GetClassID() == OwningGraph->GetClassID());
			if (bIsNodeOnRootGraph)
			{
				if (const FMetasoundFrontendNode* Node = NodePtr.Get())
				{
					if (ensure(Node->Interface.Outputs.Num() == 1))
					{
						const FMetasoundFrontendVertex& Output = Node->Interface.Outputs.Last();
						for (const FMetasoundFrontendVersion& InterfaceVersion : OwningDocument->GetInterfaceVersions())
						{
							if (IInterfaceRegistry::Get().ContainsOutput(InterfaceVersion, Output))
							{
								return InterfaceVersion;
							}
						}
					}
				}
			}

			return FMetasoundFrontendVersion::GetInvalid();
		}

		bool FOutputNodeController::IsValid() const
		{
			return OwningGraph->IsValid() && (nullptr != OwningGraphClassOutputPtr.Get()) && (nullptr != GraphPtr.Get());
		}

		FInputHandle FOutputNodeController::CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FOutputNodeInputController>(FOutputNodeInputController::FInitParams{InVertexID, InNodeVertexPtr, InClassInputPtr, OwningGraphClassOutputPtr, GraphPtr, InOwningNode});
		}

		FOutputHandle FOutputNodeController::CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FOutputNodeOutputController>(FOutputNodeOutputController::FInitParams{InVertexID, InNodeVertexPtr, InClassOutputPtr, OwningGraphClassOutputPtr, GraphPtr, InOwningNode});
		}

		FDocumentAccess FOutputNodeController::ShareAccess()
		{
			FDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;
			Access.ConstClassOutput = OwningGraphClassOutputPtr;

			return Access;
		}

		FConstDocumentAccess FOutputNodeController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.ConstGraph = GraphPtr;
			Access.ConstClassOutput = OwningGraphClassOutputPtr;

			return Access;
		}

		FInputNodeController::FInputNodeController(EPrivateToken InToken, const FInputNodeController::FInitParams& InParams)
		: FBaseNodeController({InParams.NodePtr, InParams.ClassPtr, InParams.OwningGraph})
		, OwningGraphClassInputPtr(InParams.OwningGraphClassInputPtr)
		, GraphPtr(InParams.GraphPtr)
		{
		}

		FNodeHandle FInputNodeController::CreateInputNodeHandle(const FInputNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					if (EMetasoundFrontendClassType::Input == Class->Metadata.GetType())
					{
						if (Class->ID == Node->ClassID)
						{
							return MakeShared<FInputNodeController>(EPrivateToken::Token, InParams);
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Class of incorrect class type [ClassID:%ls] while creating input node. Must be EMetasoundFrontendClassType::Input.", *Class->ID.ToString());
					}
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FInputNodeController::CreateConstInputNodeHandle(const FInputNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					if (EMetasoundFrontendClassType::Input == Class->Metadata.GetType())
					{
						if (Class->ID == Node->ClassID)
						{
							return MakeShared<const FInputNodeController>(EPrivateToken::Token, InParams);
						}
						else
						{
							UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Class of incorrect class type [ClassID:%ls] while creating input node. Must be EMetasoundFrontendClassType::Input.", *Class->ID.ToString());
					}
				}
			}

			return INodeController::GetInvalidHandle();
		}

		bool FInputNodeController::IsValid() const
		{
			return OwningGraph->IsValid() && (nullptr != OwningGraphClassInputPtr.Get()) && (nullptr != GraphPtr.Get());
		}

		FInputHandle FInputNodeController::CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FInputNodeInputController>(FInputNodeInputController::FInitParams{InVertexID, InNodeVertexPtr, InClassInputPtr, OwningGraphClassInputPtr, GraphPtr, InOwningNode});
		}

		FOutputHandle FInputNodeController::CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FInputNodeOutputController>(FInputNodeOutputController::FInitParams{InVertexID, InNodeVertexPtr, InClassOutputPtr, OwningGraphClassInputPtr, GraphPtr, InOwningNode});
		}

#if WITH_EDITOR
		const FText& FInputNodeController::GetDescription() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				return OwningInput->Metadata.GetDescription();
			}

			return Invalid::GetInvalidText();
		}

		FText FInputNodeController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				return OwningInput->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		const FText& FInputNodeController::GetDisplayTitle() const
		{
			static FText InputDisplayTitle = LOCTEXT("InputNode_Title", "Input");
			return InputDisplayTitle;
		}
#endif // WITH_EDITOR

		const FMetasoundFrontendVersion& FInputNodeController::GetInterfaceVersion() const
		{
			FConstDocumentHandle OwningDocument = OwningGraph->GetOwningDocument();
			FConstGraphHandle RootGraph = OwningDocument->GetRootGraph();

			// Test if this node exists on the document's root graph.
			const bool bIsNodeOnRootGraph = OwningGraph->IsValid() && (RootGraph->GetClassID() == OwningGraph->GetClassID());
			if (bIsNodeOnRootGraph)
			{
				if (const FMetasoundFrontendNode* Node = NodePtr.Get())
				{
					if (ensure(Node->Interface.Inputs.Num() == 1))
					{
						const TSet<FMetasoundFrontendVersion>& InterfaceVersions = OwningDocument->GetInterfaceVersions();
						const FMetasoundFrontendVertex& Input = Node->Interface.Inputs.Last();
						for (const FMetasoundFrontendVersion& InterfaceVersion : InterfaceVersions)
						{
							// If the node is on the root graph, test if it is in the interfaces required inputs.
							if (IInterfaceRegistry::Get().ContainsInput(InterfaceVersion, Input))
							{
								return InterfaceVersion;
							}
						}
					}
				}
			}

			return FMetasoundFrontendVersion::GetInvalid();
		}

#if WITH_EDITOR
		void FInputNodeController::SetDescription(const FText& InDescription)
		{
			// TODO: can we remove these const casts by constructing FINputNodeController with non-const access to the class input?
			if (FMetasoundFrontendClassInput* ClassInput = ConstCastAccessPtr<FClassInputAccessPtr>(OwningGraphClassInputPtr).Get())
			{
				ClassInput->Metadata.SetDescription(InDescription);
				OwningGraph->UpdateInterfaceChangeID();
			}
		}
#endif // WITH_EDITOR

		void FInputNodeController::SetNodeName(const FVertexName& InName)
		{
			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				Node->Name = InName;

				for (FMetasoundFrontendVertex& Vertex : Node->Interface.Inputs)
				{
					Vertex.Name = InName;
				}

				for (FMetasoundFrontendVertex& Vertex : Node->Interface.Outputs)
				{
					Vertex.Name = InName;
				}
			}

			if (FMetasoundFrontendClassInput* ClassInput = ConstCastAccessPtr<FClassInputAccessPtr>(OwningGraphClassInputPtr).Get())
			{
				ClassInput->Name = InName;
			}
		}

#if WITH_EDITOR
		void FInputNodeController::SetDisplayName(const FText& InDisplayName)
		{
			// TODO: can we remove these const casts by constructing FINputNodeController with non-const access to the class input?
			if (FMetasoundFrontendClassInput* ClassInput = ConstCastAccessPtr<FClassInputAccessPtr>(OwningGraphClassInputPtr).Get())
			{
				ClassInput->Metadata.SetDisplayName(InDisplayName);
				OwningGraph->UpdateInterfaceChangeID();
			}
		}
#endif // WITH_EDITOR

		FDocumentAccess FInputNodeController::ShareAccess()
		{
			FDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;
			Access.ConstClassInput = OwningGraphClassInputPtr;

			return Access;
		}

		FConstDocumentAccess FInputNodeController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.ConstGraph = GraphPtr;
			Access.ConstClassInput = OwningGraphClassInputPtr;

			return Access;
		}

		// 
		// FVariableNodeController
		//
		FVariableNodeController::FVariableNodeController(EPrivateToken InToken, const FVariableNodeController::FInitParams& InParams)
		: FNodeController(FNodeController::EPrivateToken::Token, InParams)
		{
		}

		FNodeHandle FVariableNodeController::CreateNodeHandle(const FVariableNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();

						// Class type must be one of the associated variable class types.
						if (ensure(IsSupportedClassType(ClassType)))
						{
							return MakeShared<FVariableNodeController>(EPrivateToken::Token, InParams);
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FVariableNodeController::CreateConstNodeHandle(const FVariableNodeController::FInitParams& InParams)
		{
			if (const FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();

						// Class type must be one of the associated variable class types.
						if (ensure(IsSupportedClassType(ClassType)))
						{
							return MakeShared<const FVariableNodeController>(EPrivateToken::Token, InParams);
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		FInputHandle FVariableNodeController::CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const
		{
			if (const FMetasoundFrontendClassInput* ClassInput = InClassInputPtr.Get())
			{
				if (IsVariableDataType(ClassInput->TypeName))
				{
					FGraphAccessPtr SuperGraphPtr = ConstCastAccessPtr<FGraphAccessPtr>(Super::ShareAccess().ConstGraph);

					return MakeShared<FVariableInputController>(FVariableInputController::FInitParams{InVertexID, InNodeVertexPtr, InClassInputPtr, SuperGraphPtr, InOwningNode});
				}
			}
			return Super::CreateInputController(InVertexID, InNodeVertexPtr, InClassInputPtr, InOwningNode);
		}

		FOutputHandle FVariableNodeController::CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const
		{
			if (const FMetasoundFrontendClassOutput* ClassOutput = InClassOutputPtr.Get())
			{
				if (IsVariableDataType(ClassOutput->TypeName))
				{
					FGraphAccessPtr SuperGraphPtr = ConstCastAccessPtr<FGraphAccessPtr>(Super::ShareAccess().ConstGraph);
					return MakeShared<FVariableOutputController>(FVariableOutputController::FInitParams{InVertexID, InNodeVertexPtr, InClassOutputPtr, SuperGraphPtr, InOwningNode});
				}
			}
			return Super::CreateOutputController(InVertexID, InNodeVertexPtr, InClassOutputPtr, InOwningNode);
		}

		bool FVariableNodeController::IsSupportedClassType(EMetasoundFrontendClassType InClassType)
		{
			const bool bIsVariableNode = (InClassType == EMetasoundFrontendClassType::Variable)
				|| (InClassType == EMetasoundFrontendClassType::VariableAccessor)
				|| (InClassType == EMetasoundFrontendClassType::VariableDeferredAccessor)
				|| (InClassType == EMetasoundFrontendClassType::VariableMutator);
			return bIsVariableNode;
		}

		bool FVariableNodeController::IsVariableDataType(const FName& InTypeName)
		{
			FDataTypeRegistryInfo DataTypeInfo;
			if (ensure(IDataTypeRegistry::Get().GetDataTypeInfo(InTypeName, DataTypeInfo)))
			{
				return DataTypeInfo.bIsVariable;
			}
			return false;
		}
	}
}
#undef LOCTEXT_NAMESPACE
