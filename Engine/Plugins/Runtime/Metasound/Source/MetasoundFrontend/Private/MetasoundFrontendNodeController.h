// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "Misc/Guid.h"

namespace Metasound
{
	namespace Frontend
	{
		/** FBaseNodeController provides common functionality for multiple derived
		 * node controllers.
		 */
		class FBaseNodeController : public INodeController
		{
		public:

			struct FInitParams
			{
				FNodeAccessPtr NodePtr;
				FConstClassAccessPtr ClassPtr;
				FGraphHandle OwningGraph;
			};

			/** Construct a base node controller. */
			FBaseNodeController(const FInitParams& InParams);

			virtual bool IsValid() const override;

			// Owning graph info
			virtual FGuid GetOwningGraphClassID() const override;
			virtual FGraphHandle GetOwningGraph() override;
			virtual FConstGraphHandle GetOwningGraph() const override;

			// Info about this node.
			virtual FGuid GetID() const override;
			virtual FGuid GetClassID() const override;

			virtual bool ClearInputLiteral(FGuid InVertexID) override;
			virtual const FMetasoundFrontendLiteral* GetInputLiteral(const FGuid& InVertexID) const override;
			virtual void SetInputLiteral(const FMetasoundFrontendVertexLiteral& InVertexLiteral) override;

			virtual const FMetasoundFrontendClassInterface& GetClassInterface() const override;
			virtual const FMetasoundFrontendClassMetadata& GetClassMetadata() const override;

			virtual const FMetasoundFrontendNodeInterface& GetNodeInterface() const override;

#if WITH_EDITOR
			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const override;
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const override;
			virtual const FMetasoundFrontendClassStyle& GetClassStyle() const override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
			virtual const FMetasoundFrontendNodeStyle& GetNodeStyle() const override;
			virtual void SetNodeStyle(const FMetasoundFrontendNodeStyle& InStyle) override;
#endif // WITH_EDITORONLY_DATA

			virtual void SetNodeConfiguration(TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfig) override;
			virtual TConstStructView<FMetaSoundFrontendNodeConfiguration> GetNodeConfiguration() const override;

#if WITH_EDITORONLY_DATA
			UE_DEPRECATED(5.6, "Moved to FMetaSoundFrontendDocumentBuilder")
			virtual bool DiffAgainstRegistryInterface(FClassInterfaceUpdates& OutInterfaceUpdates, bool bInUseHighestMinorVersion, bool bInForceRegenerateClassInterfaceOverride) const override;

			UE_DEPRECATED(5.7, "Moved to FMetaSoundFrontendDocumentBuilder")
			virtual bool CanAutoUpdate(FClassInterfaceUpdates& OutInterfaceUpdates) const override;
#endif // WITH_EDITORONLY_DATA

			virtual FNodeHandle ReplaceWithVersion(const FMetasoundFrontendVersionNumber& InNewVersion, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs) override;

			virtual const FVertexName& GetNodeName() const override;

			// This only exists to allow for transform fix-ups to easily cleanup input/output node names.
			virtual void SetNodeName(const FVertexName& InName) override { checkNoEntry(); }

#if WITH_EDITOR
			/** Description of the given node. */
			virtual const FText& GetDescription() const override;

			/** Returns the readable display name of the given node (Used only within MetaSound
			  * Editor context, and not guaranteed to be a unique identifier). */
			virtual FText GetDisplayName() const override;

			/** Sets the description of the node. */
			virtual void SetDescription(const FText& InDescription) override { }

			/** Sets the display name of the node. */
			virtual void SetDisplayName(const FText& InDisplayName) override { };

			/** Returns the title of the given node (what to label in visual node). */
			virtual const FText& GetDisplayTitle() const override;
#endif // WITH_EDITOR

			virtual bool CanAddInput(const FVertexName& InVertexName) const override;
			virtual FInputHandle AddInput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) override;
			virtual bool RemoveInput(FGuid InVertexID) override;

			virtual bool CanAddOutput(const FVertexName& InVertexName) const override;
			virtual FInputHandle AddOutput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) override;
			virtual bool RemoveOutput(FGuid InVertexID) override;

			/** Returns all node inputs. */
			virtual TArray<FInputHandle> GetInputs() override;

			/** Returns all node inputs. */
			virtual TArray<FConstInputHandle> GetConstInputs() const override;

			virtual void IterateConstInputs(TUniqueFunction<void(FConstInputHandle)> InFunction) const override;
			virtual void IterateConstOutputs(TUniqueFunction<void(FConstOutputHandle)> InFunction) const override;

			virtual void IterateInputs(TUniqueFunction<void(FInputHandle)> InFunction) override;
			virtual void IterateOutputs(TUniqueFunction<void(FOutputHandle)> InFunction) override;

			virtual int32 GetNumInputs() const override;
			virtual int32 GetNumOutputs() const override;

			virtual FInputHandle GetInputWithVertexName(const FVertexName& InName) override;
			virtual FConstInputHandle GetConstInputWithVertexName(const FVertexName& InName) const override;

			/** Returns all node outputs. */
			virtual TArray<FOutputHandle> GetOutputs() override;

			/** Returns all node outputs. */
			virtual TArray<FConstOutputHandle> GetConstOutputs() const override;

			virtual FOutputHandle GetOutputWithVertexName(const FVertexName& InName) override;
			virtual FConstOutputHandle GetConstOutputWithVertexName(const FVertexName& InName) const override;

			virtual bool IsInterfaceMember() const override;
			virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const override;

			/** Returns an input with the given id. 
			 *
			 * If the input does not exist, an invalid handle is returned. 
			 */
			virtual FInputHandle GetInputWithID(FGuid InVertexID) override;

			/** Returns an input with the given name. 
			 *
			 * If the input does not exist, an invalid handle is returned. 
			 */
			virtual FConstInputHandle GetInputWithID(FGuid InVertexID) const override;

			/** Returns an output with the given name. 
			 *
			 * If the output does not exist, an invalid handle is returned. 
			 */
			virtual FOutputHandle GetOutputWithID(FGuid InVertexID) override;

			/** Returns an output with the given name. 
			 *
			 * If the output does not exist, an invalid handle is returned. 
			 */
			virtual FConstOutputHandle GetOutputWithID(FGuid InVertexID) const override;

			virtual FGraphHandle AsGraph() override;
			virtual FConstGraphHandle AsGraph() const override;

		protected:

			const FMetasoundFrontendClassInterface* GetClassInterfaceForNode() const;

			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;


			FNodeAccessPtr NodePtr;
			FConstClassAccessPtr ClassPtr;
			FGraphHandle OwningGraph;

			struct FInputControllerParams
			{
				FGuid VertexID;
				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassInputAccessPtr ClassInputPtr;
			};

			struct FOutputControllerParams
			{
				FGuid VertexID;
				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassOutputAccessPtr ClassOutputPtr;
			};

			virtual TArray<FInputControllerParams> GetInputControllerParams() const;
			virtual TArray<FOutputControllerParams> GetOutputControllerParams() const;

			virtual bool FindInputControllerParamsWithVertexName(const FVertexName& InName, FInputControllerParams& OutParams) const;
			virtual bool FindOutputControllerParamsWithVertexName(const FVertexName& InName, FOutputControllerParams& OutParams) const;

			virtual bool FindInputControllerParamsWithID(FGuid InVertexID, FInputControllerParams& OutParams) const;
			virtual bool FindOutputControllerParamsWithID(FGuid InVertexID, FOutputControllerParams& OutParams) const;

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const = 0;
			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const = 0;

		private:
			void SetClassInterfaceOverride(TInstancedStruct<FMetasoundFrontendClassInterface> InClassInterfaceOverride);
			TConstStructView<FMetasoundFrontendClassInterface> GetClassInterfaceOverride() const;

			struct FNodeConfigurationUpdateData
			{
				TConstStructView<FMetaSoundFrontendNodeConfiguration> ExistingConfig;
				TConstStructView<FMetasoundFrontendClassInterface> ExistingClassInterfaceOverride;
				TInstancedStruct<FMetaSoundFrontendNodeConfiguration> RegisteredConfig;
				TInstancedStruct<FMetasoundFrontendClassInterface> RegeneratedClassInterfaceOverride;
				bool bDidUpdateClassInterfaceOverride = false;
			};


			bool FindRegisteredClass(const FMetasoundFrontendClassMetadata& InMetadata, bool bInUseHighestMinorVersion, FMetasoundFrontendClass& OutInterface) const;

			bool ShouldReplaceExistingNodeConfig(TConstStructView<FMetaSoundFrontendNodeConfiguration> InRegisteredNodeConfig, TConstStructView<FMetaSoundFrontendNodeConfiguration> InExistingConfig) const;
			const FMetasoundFrontendClassInterface& GetApplicableRegistryInterface(const FMetasoundFrontendClass& InRegisteredClass, const FNodeConfigurationUpdateData& InNodeConfigUpdates) const;
			void FindNodeConfigurationUpdates(const FMetasoundFrontendClass& InRegisteredClass, FNodeConfigurationUpdateData& OutNodeConfigUpdates, bool bInForceRegenerateClassInterfaceOverride) const;

#if WITH_EDITORONLY_DATA
			void DiffClassInterfaces(const FMetasoundFrontendClassInterface& InOriginalInterface, const FMetasoundFrontendClassInterface& InNewInterface, FClassInterfaceUpdates& OutClassInterfaceUpdates) const;
#endif // WITH_EDITORONLY_DATA

			struct FInputConnectionInfo
			{
				FOutputHandle ConnectedOutput;
				FVertexName Name;
				FName DataType;
				FMetasoundFrontendLiteral DefaultValue;
				bool bLiteralSet = false;
			};

			struct FOutputConnectionInfo
			{
				TArray<FInputHandle> ConnectedInputs;
				FVertexName VertexName;
				FName DataType;
			};

			struct FNodeInstanceReplacementData
			{

#if WITH_EDITOR
				FMetasoundFrontendNodeStyle Style;
#endif // WITH_EDITOR

				TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration;
				TInstancedStruct<FMetasoundFrontendClassInterface> ClassInterfaceOverride;
				TMap<FVertexNameAndType, FInputConnectionInfo> InputConnections;
				TMap<FVertexNameAndType, FOutputConnectionInfo> OutputConnections;
				FGuid NodeID;
			};

			FNodeInstanceReplacementData CaptureNodeInstanceReplacementData();
			void ApplyNodeInstanceReplacementData(const FMetasoundFrontendClass& InReplacementClass, INodeController& InReplacementNode, FNodeInstanceReplacementData&& InInstanceData, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs);

		};

		/** Represents an external node (defined in either code or by an asset's root graph). */
		class FNodeController : public FBaseNodeController
		{
		protected:

			// Private token only allows members, friends or derived classes to call constructor.
			enum EPrivateToken { Token };

		public:
			struct FInitParams
			{
				FNodeAccessPtr NodePtr;
				FConstClassAccessPtr ClassPtr;
				FGraphAccessPtr GraphPtr;
				FGraphHandle OwningGraph;
			};

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FNodeController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a node handle for an external node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FNodeHandle CreateNodeHandle(const FInitParams& InParams);

			/** Create a node handle for an external node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FConstNodeHandle CreateConstNodeHandle(const FInitParams& InParams);

			virtual ~FNodeController() = default;

			virtual bool IsValid() const override;

		protected:
			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const override;
			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const override;

		private:

			FGraphAccessPtr GraphPtr;
		};

		/** FOutputNodeController represents an output node. */
		class FOutputNodeController: public FBaseNodeController
		{

			// Private token only allows members or friends to call constructor.
			enum EPrivateToken { Token };

		public:
			struct FInitParams
			{
				FNodeAccessPtr NodePtr;
				FConstClassAccessPtr ClassPtr;
				FConstClassOutputAccessPtr OwningGraphClassOutputPtr; 
				FGraphAccessPtr GraphPtr;
				FGraphHandle OwningGraph;
			};

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FOutputNodeController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a node handle for an output node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FNodeHandle CreateOutputNodeHandle(const FInitParams& InParams);

			/** Create a node handle for an output node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FConstNodeHandle CreateConstOutputNodeHandle(const FInitParams& InParams);


			virtual ~FOutputNodeController() = default;

			virtual bool IsValid() const override;

#if WITH_EDITOR
			virtual const FText& GetDescription() const override;
			virtual FText GetDisplayName() const override;
			virtual const FText& GetDisplayTitle() const override;
			virtual void SetDescription(const FText& InDescription) override;
			virtual void SetDisplayName(const FText& InText) override;
#endif // WITH_EDITOR

			virtual void SetNodeName(const FVertexName& InName) override;
			virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const override;

		protected:

			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const override;

			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const override;

		private:

			FGraphAccessPtr GraphPtr;
			FConstClassOutputAccessPtr OwningGraphClassOutputPtr; 
		};

		/** FInputNodeController represents an input node. */
		class FInputNodeController: public FBaseNodeController
		{

			// Private token only allows members or friends to call constructor.
			enum EPrivateToken { Token };

		public:
			struct FInitParams
			{
				FNodeAccessPtr NodePtr;
				FConstClassAccessPtr ClassPtr;
				FConstClassInputAccessPtr OwningGraphClassInputPtr; 
				FGraphAccessPtr GraphPtr;
				FGraphHandle OwningGraph;
			};

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FInputNodeController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a node handle for an input node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FNodeHandle CreateInputNodeHandle(const FInitParams& InParams);

			/** Create a node handle for an input node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FConstNodeHandle CreateConstInputNodeHandle(const FInitParams& InParams);

			virtual ~FInputNodeController() = default;

#if WITH_EDITOR
			virtual const FText& GetDescription() const override;
			virtual FText GetDisplayName() const override;
			virtual const FText& GetDisplayTitle() const override;
			virtual void SetDescription(const FText& InDescription) override;
			virtual void SetDisplayName(const FText& InText) override;
#endif // WITH_EDITOR

			virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const override;
			virtual bool IsValid() const override;
			virtual void SetNodeName(const FVertexName& InName) override;

			// No-ops as inputs do not handle literals the same way as other nodes
			virtual bool ClearInputLiteral(FGuid InVertexID) override { return false; }
			virtual const FMetasoundFrontendLiteral* GetInputLiteral(const FGuid& InVertexID) const override { return nullptr; }
			virtual void SetInputLiteral(const FMetasoundFrontendVertexLiteral& InVertexLiteral) override { }

		protected:

			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const override;

			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const override;

		private:

			FConstClassInputAccessPtr OwningGraphClassInputPtr;
			FGraphAccessPtr GraphPtr;
		};

		/** Represents an variable node */
		class FVariableNodeController : public FNodeController
		{
			using Super = FNodeController;

			// Private token only allows members or friends to call constructor.
			enum EPrivateToken { Token };

		public:
			using FInitParams = FNodeController::FInitParams;

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FVariableNodeController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a node handle for a variable node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FNodeHandle CreateNodeHandle(const FInitParams& InParams);

			/** Create a node handle for a variable node.
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FConstNodeHandle CreateConstNodeHandle(const FInitParams& InParams);

			virtual ~FVariableNodeController() = default;

		protected:

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const override;
			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const override;

		private:

			static bool IsSupportedClassType(EMetasoundFrontendClassType InClassType);
			static bool IsVariableDataType(const FName& InTypeName);
		};
	}
}

