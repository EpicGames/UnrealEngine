// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodeMigration.h"

namespace Metasound::Frontend
{

#if WITH_EDITORONLY_DATA
	bool operator==(const FNodeMigrationInfo& InLHS, const FNodeMigrationInfo& InRHS)
	{
		return (InLHS.ClassName == InRHS.ClassName) &&
			(InLHS.UEVersion == InRHS.UEVersion) &&
			(InLHS.MajorVersion == InRHS.MajorVersion) &&
			(InLHS.MinorVersion == InRHS.MinorVersion ) &&
			(InLHS.FromPlugin == InRHS.FromPlugin) &&
			(InLHS.FromModule == InRHS.FromModule) &&
			(InLHS.ToPlugin == InRHS.ToPlugin) &&
			(InLHS.ToModule == InRHS.ToModule);
	}

	FString FNodeMigrationInfo::ToString() const
	{
		TStringBuilder<256> StringBuilder;
		return StringBuilder.Append(TEXT("Migration: "))
			.Append(ClassName.ToString())
			.AppendChar(' ')
			.Append(FString::FromInt(MajorVersion))
			.AppendChar('.')
			.Append(FString::FromInt(MinorVersion))
			.Append(TEXT(" From "))
			.Append(FromPlugin.ToString())
			.AppendChar('/')
			.Append(FromModule.ToString())
			.Append(TEXT(" To "))
			.Append(ToPlugin.ToString())
			.AppendChar('/')
			.Append(ToModule.ToString())
			.Append(TEXT(" in UE"))
			.Append(UEVersion.ToString())
			.ToString();
	}
#endif // if WITH_EDITORONLY_DATA

} // namespace Metasound::Frontend
