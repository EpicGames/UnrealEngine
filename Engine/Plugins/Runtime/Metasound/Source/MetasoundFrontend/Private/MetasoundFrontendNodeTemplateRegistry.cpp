// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodeTemplateRegistry.h"

#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeClassRegistryPrivate.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendNodeTemplateRegistry)

namespace Metasound::Frontend
{
#if WITH_EDITOR
	FText INodeTemplate::ResolveMemberDisplayName(FName FullName, FText DisplayName, bool bIncludeNamespace)
	{
		FName Namespace;
		FName ShortParamName;
		Audio::FParameterPath::SplitName(FullName, Namespace, ShortParamName);
		if (DisplayName.IsEmpty())
		{
			DisplayName = FText::FromName(ShortParamName);
		}

		if (bIncludeNamespace && !Namespace.IsNone())
		{
			return FText::Format(NSLOCTEXT("MetasoundFrontend", "DisplayNameWithNamespaceFormat", "{0} ({1})"), DisplayName, FText::FromName(Namespace));
		}

		return DisplayName;
	}
#endif // WITH_EDITOR

	class FNodeTemplateRegistry : public INodeTemplateRegistry
	{
	public:
		FNodeTemplateRegistry() = default;
		virtual ~FNodeTemplateRegistry() = default;

		virtual const INodeTemplate* FindTemplate(const FNodeClassRegistryKey& InKey) const override;
		virtual const INodeTemplate* FindTemplate(const FMetasoundFrontendClassName& InClassName) const override;

		void Register(TUniquePtr<INodeTemplate>&& InEntry);
		void Unregister(const FNodeClassRegistryKey& InKey);

	private:
		TMap<FNodeClassRegistryKey, TUniquePtr<const INodeTemplate>> Templates;
		TMultiMap<FMetasoundFrontendClassName, const INodeTemplate*> TemplateByClassName;
	};

	void FNodeTemplateRegistry::Register(TUniquePtr<INodeTemplate>&& InTemplate)
	{
		if (ensure(InTemplate.IsValid()))
		{
			const FNodeClassRegistryKey Key = FNodeClassRegistryKey(InTemplate->GetFrontendClass().Metadata);
			if (ensure(Key.IsValid()))
			{
				TUniquePtr<const INodeTemplate>& Entry = Templates.Add(Key, MoveTemp(InTemplate));
				TemplateByClassName.Add(Entry->GetFrontendClass().Metadata.GetClassName(), Entry.Get());
			}
		}
	}

	void FNodeTemplateRegistry::Unregister(const FNodeClassRegistryKey& InKey)
	{
		TUniquePtr<const INodeTemplate> Removed;
		if (ensure(Templates.RemoveAndCopyValue(InKey, Removed)))
		{
			ensure(TemplateByClassName.Remove(Removed->GetFrontendClass().Metadata.GetClassName()));
		}
	}

	const INodeTemplate* FNodeTemplateRegistry::FindTemplate(const FNodeClassRegistryKey& InKey) const
	{
		if (const TUniquePtr<const INodeTemplate>* Template = Templates.Find(InKey))
		{
			return Template->Get();
		}

		return nullptr;
	}

	const INodeTemplate* FNodeTemplateRegistry::FindTemplate(const FMetasoundFrontendClassName& InClassName) const
	{
		if (TemplateByClassName.Contains(InClassName))
		{
			TArray<const INodeTemplate*> FoundTemplates;
			constexpr bool bMaintainOrder = false;
			TemplateByClassName.MultiFind(InClassName, FoundTemplates, bMaintainOrder);
			Algo::Sort(FoundTemplates, [](const INodeTemplate* A, const INodeTemplate* B) { return A->GetVersionNumber() < B->GetVersionNumber(); });

			return FoundTemplates.Last();
		}

		return nullptr;
	}

	INodeTemplateRegistry& INodeTemplateRegistry::Get()
	{
		static FNodeTemplateRegistry Registry;
		return Registry;
	}

	const TArray<FMetasoundFrontendClassInputDefault>* FNodeTemplateBase::FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const
	{
		if (const FMetasoundFrontendVertex* Vertex = InBuilder.FindNodeInput(InNodeID, VertexName, &InPageID))
		{
			const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID, &InPageID);
			check(Node);
			if (const FMetasoundFrontendClass* Class = InBuilder.FindDependency(Node->ClassID))
			{
				const FMetasoundFrontendClassInterface& ClassInterface = Class->GetInterfaceForNode(*Node);

				const EMetasoundFrontendClassType ClassType = Class->Metadata.GetType();
				auto MatchesName = [&Vertex](const FMetasoundFrontendClassInput& Input) { return Input.Name == Vertex->Name; };
				if (const FMetasoundFrontendClassInput* Input = ClassInterface.Inputs.FindByPredicate(MatchesName))
				{
					return &Input->GetDefaults();
				}
			}
		}

		return nullptr;
	}

	TInstancedStruct<FMetaSoundFrontendTemplateNodeConfiguration> FNodeTemplateBase::CreateFrontendTemplateNodeConfiguration() const
	{
		return TInstancedStruct<FMetaSoundFrontendTemplateNodeConfiguration>();
	}

#if WITH_EDITOR
	FText FNodeTemplateBase::GetNodeDisplayName(const IMetaSoundDocumentInterface& Interface, const FGuid& InPageID, const FGuid& InNodeID) const
	{
		return { };
	}

	FText FNodeTemplateBase::GetInputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName InputName) const
	{
		return FText::FromName(InputName);
	}

	FText FNodeTemplateBase::GetOutputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName OutputName) const
	{
		return FText::FromName(OutputName);
	}

	bool FNodeTemplateBase::HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage) const
	{
		return true;
	}

	bool FNodeTemplateBase::OnEdgeAdded(const FTemplateNodeEdgeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		return false;
	}

	bool FNodeTemplateBase::OnNodeAdded(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		return false;
	}

	bool FNodeTemplateBase::OnNodeConfigurationUpdated(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		return false;
	}

	bool FNodeTemplateBase::OnRemoveSwappingEdge(const FTemplateNodeEdgeRemoveTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		return false;
	}

	bool FNodeTemplateBase::OnRemoveSwappingNode(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		return false;
	}

#endif // WITH_EDITOR

	void RegisterNodeTemplate(TUniquePtr<INodeTemplate>&& InTemplate)
	{
		class FTemplateRegistryEntry : public INodeTemplateRegistryEntry
		{
			const FMetasoundFrontendClass FrontendClass;

		public:
			FTemplateRegistryEntry(const INodeTemplate& InNodeTemplate)
				: FrontendClass(InNodeTemplate.GetFrontendClass())
			{
			}

			virtual ~FTemplateRegistryEntry() = default;

			/** Return a FMetasoundFrontendClass which describes the node. */
			virtual const FMetasoundFrontendClass& GetFrontendClass() const override
			{
				return FrontendClass;
			}
		};

		TUniquePtr<INodeTemplateRegistryEntry> RegEntry = TUniquePtr<INodeTemplateRegistryEntry>(new FTemplateRegistryEntry(*InTemplate.Get()));
		FNodeClassRegistry::Get().RegisterNodeTemplate(MoveTemp(RegEntry));

		static_cast<FNodeTemplateRegistry&>(INodeTemplateRegistry::Get()).Register(MoveTemp(InTemplate));
	}

	void UnregisterNodeTemplate(const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InTemplateVersion)
	{
		const FNodeClassRegistryKey Key = FNodeClassRegistryKey(EMetasoundFrontendClassType::Template, InClassName, InTemplateVersion);
		if (ensure(Key.IsValid()))
		{
			FNodeClassRegistry::Get().UnregisterNodeTemplate(Key);
			static_cast<FNodeTemplateRegistry&>(INodeTemplateRegistry::Get()).Unregister(Key);
		}
	}
} // namespace Metasound::Frontend
