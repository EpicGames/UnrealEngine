// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodeUpdateTransform.h"
#include "Algo/Find.h"
#include "MetasoundAssetManager.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendTransform.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
#if WITH_EDITORONLY_DATA
	FBaseNodeUpdateTransform::FBaseNodeUpdateTransform(const FNodeClassRegistryKey& InNewNodeClassKey)
	: NewNodeClassKey(InNewNodeClassKey)
	{
	}

	FNodeUpdateTransformData FBaseNodeUpdateTransform::CreateNodeUpdateTransformData(const FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid& InNodeID, const FGuid* InPageID) const
	{
		FNodeUpdateTransformData NodeUpdateTransformData;
		NodeUpdateTransformData.NodeReplacementData = InOutBuilder.CaptureNodeInstanceReplacementData(InNodeID, InPageID);
		return NodeUpdateTransformData;
	}
	
	FNodeClassRegistryKey FBaseNodeUpdateTransform::GetNewNodeClassKey() const
	{
		return NewNodeClassKey;
	}
	
	void FBaseNodeUpdateTransform::Update(FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid InNodeID, const FGuid* InPageID) const
	{
		using namespace NodeReplacement;

		const FMetasoundFrontendNode* OldNode = InOutBuilder.FindNode(InNodeID, InPageID);
		check(OldNode);
		const FMetasoundFrontendClass* OldClass = InOutBuilder.FindDependency(OldNode->ClassID);
		check(OldClass);
		
		const FNodeClassRegistryKey OldNodeClassKey = FNodeClassRegistryKey(OldClass->Metadata);
		FNodeUpdateTransformData NodeUpdateTransformData = CreateNodeUpdateTransformData(InOutBuilder, InNodeID, InPageID);

		InOutBuilder.RemoveNode(InNodeID, InPageID);
	
		const FMetasoundFrontendNode* NewNode = InOutBuilder.AddNodeByClassName(NewNodeClassKey.ClassName, NewNodeClassKey.Version.Major, NewNodeClassKey.Version.Minor, InNodeID, InPageID);
		check(NewNode);

		const bool bNodeVersionUpdated = NewNodeClassKey.ClassName == OldNodeClassKey.ClassName &&
		NewNodeClassKey.Version > OldNodeClassKey.Version;
		NodeUpdateTransformData.NodeReplacementData.Style.bMessageNodeUpdated = bNodeVersionUpdated;

		TArray<FVertexNameAndType> DisconnectedInputs;
		TArray<FVertexNameAndType> DisconnectedOutputs;
		InOutBuilder.ApplyNodeInstanceReplacementData(MoveTemp(NodeUpdateTransformData.NodeReplacementData), InNodeID, InPageID, &DisconnectedInputs, &DisconnectedOutputs);

		// Log disconnected inputs/outputs
		if ((DisconnectedInputs.Num() > 0) || (DisconnectedOutputs.Num() > 0))
		{
			UObject& MetaSound = InOutBuilder.CastDocumentObjectChecked<UObject>();
			const FMetasoundAssetBase* MetaSoundAsset = IMetaSoundAssetManager::GetChecked().GetAsAsset(MetaSound);
			
			const FString DebugAssetPath = MetaSoundAsset->GetOwningAssetName();

			for (const FVertexNameAndType& InputPin : DisconnectedInputs)
			{
				METASOUND_VERSIONING_LOG(Display, TEXT("Auto-Updating '%s' node class '%s' to '%s': Previously connected input '%s' with data type '%s' no longer exists or was otherwise disconnected when replacing the node."),
					*DebugAssetPath, *OldNodeClassKey.ToString(), *NewNodeClassKey.ToString(), *InputPin.Get<0>().ToString(), *InputPin.Get<1>().ToString());
			}

			for (const FVertexNameAndType& OutputPin : DisconnectedOutputs)
			{
				METASOUND_VERSIONING_LOG(Display, TEXT("Auto-Updating '%s' node class '%s' to '%s': Previously connected output '%s' with data type '%s' no longer exists or was otherwise disconnected when replacing the node."),
					*DebugAssetPath,  *OldNodeClassKey.ToString(), *NewNodeClassKey.ToString(), *OutputPin.Get<0>().ToString(), *OutputPin.Get<1>().ToString());
			}
		}
	}

	bool FBaseNodeUpdateTransform::ShouldAutoApply() const
	{
		return true;
	}

	FMaintainDefaultsNodeUpdateTransform::FMaintainDefaultsNodeUpdateTransform(const FNodeClassRegistryKey& InTargetNodeClassKey)
		: FBaseNodeUpdateTransform(InTargetNodeClassKey)
	{
		
	}
	
	FNodeUpdateTransformData FMaintainDefaultsNodeUpdateTransform::CreateNodeUpdateTransformData(const FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid& InNodeID, const FGuid* InPageID) const
	{
		using namespace NodeReplacement;
		
		FNodeUpdateTransformData NodeUpdateTransformData = FBaseNodeUpdateTransform::CreateNodeUpdateTransformData(InOutBuilder, InNodeID, InPageID);

		// Additional logic to cache off defaults that should be maintained
		const FMetasoundFrontendNode* Node = InOutBuilder.FindNode(InNodeID, InPageID);
		check(Node);
		
		TArray<const FMetasoundFrontendVertex*> NodeInputs = InOutBuilder.FindNodeInputs(InNodeID, FName(), InPageID); 
		for (const FMetasoundFrontendVertex* NodeInput : NodeInputs)
		{
			check(NodeInput);
			const FMetasoundFrontendVertexLiteral* VertexLiteral = InOutBuilder.FindNodeInputDefault(InNodeID, NodeInput->VertexID, InPageID);
			// Default matching node class default at time of creation will be null
			if (!VertexLiteral)
			{
				// Get class default to set value back to in ApplyNodeUpdateTransformData
				if (const TArray<FMetasoundFrontendClassInputDefault>* NodeClassDefaults = InOutBuilder.FindNodeClassInputDefaults(InNodeID, NodeInput->Name, InPageID))
				{
					// Paged defaults not currently supported on external node inputs, so just use the default page one 
					const FMetasoundFrontendClassInputDefault& NodeClassDefault = NodeClassDefaults->Last();
					const FVertexNameAndType InputKey = { NodeInput->Name, NodeInput->TypeName };
					FInputConnectionInfo* InputData = NodeUpdateTransformData.NodeReplacementData.InputConnections.Find(InputKey);
					check(InputData);
					InputData->DefaultValue = NodeClassDefault.Literal;
					InputData->bLiteralSet = true;
				}
			}
		}
		return NodeUpdateTransformData;
	}

	bool FMaintainDefaultsNodeUpdateTransform::ShouldAutoApply() const
	{
		return true;
	}
#endif // if WITH_EDITORONLY_DATA
} // namespace Metasound::Frontend
#undef UE_API
