// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundNodeCategories"

namespace Metasound
{
	namespace NodeCategories
	{
		// Base category for all conversion functions.
		const FText Conversions  = LOCTEXT("Metasound_ConversionsCategory", "Conversions");

		// Category for enum conversions. A subcategory of conversion functions to prevent cluttering up the action menu. 
		const FText EnumConversions = LOCTEXT("Metasound_EnumConversionsCategory", "EnumConversions");

		// Base category for all standard MetaSound functions.
		const FText Functions = LOCTEXT("Metasound_FunctionsCategory", "Functions");

		// Base category reserved for all asset graph references.
		const FText Graphs = LOCTEXT("Metasound_GraphsCategory", "Graphs");

		// Base category for all inputs
		const FText Inputs = LOCTEXT("Metasound_InputsCategory", "Inputs");

		// Base category for all outputs
		const FText Outputs = LOCTEXT("Metasound_OutputsCategory", "Outputs");

		// Base category for all variables 
		const FText Variables = LOCTEXT("Metasound_VariablesCategory", "Variables");

		// Base category for all favorites 
		const FText Favorites = LOCTEXT("Metasound_FavoritesCategory", "Favorites");

		// Base category for all recent actions 
		const FText Recents = LOCTEXT("Metasound_RecentsCategory", "Recently Used");
	}
}

#undef LOCTEXT_NAMESPACE
