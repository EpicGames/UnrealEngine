// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendOutputController.h"
#include "MetasoundFrontendInvalidController.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontendOutputController"

namespace Metasound
{
	namespace Frontend
	{
		//
		// FBaseOutputController
		//
		FBaseOutputController::FBaseOutputController(const FBaseOutputController::FInitParams& InParams)
		: ID(InParams.ID)
		, NodeVertexPtr(InParams.NodeVertexPtr)
		, ClassOutputPtr(InParams.ClassOutputPtr)
		, GraphPtr(InParams.GraphPtr)
		, OwningNode(InParams.OwningNode)
		{
		}

		bool FBaseOutputController::IsValid() const
		{
			return OwningNode->IsValid() && (nullptr != NodeVertexPtr.Get()) && (nullptr != GraphPtr.Get());
		}

		FGuid FBaseOutputController::GetID() const
		{
			return ID;
		}

		const FName& FBaseOutputController::GetDataType() const
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				return Vertex->TypeName;
			}
			return Invalid::GetInvalidName();
		}

		const FVertexName& FBaseOutputController::GetName() const
		{
			if (const FMetasoundFrontendVertex* Vertex = NodeVertexPtr.Get())
			{
				return Vertex->Name;
			}
			return Invalid::GetInvalidName();
		}

		EMetasoundFrontendVertexAccessType FBaseOutputController::GetVertexAccessType() const
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FConstOutputHandle ReroutedOutput = Frontend::FindReroutedOutput(AsShared());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (ReroutedOutput->IsValid())
			{
				if (ReroutedOutput->GetOwningNodeID() != GetOwningNodeID() || ReroutedOutput->GetID() != GetID())
				{
					return ReroutedOutput->GetVertexAccessType();
				}
			}

			if (const FMetasoundFrontendClassVertex* ClassOutput = ClassOutputPtr.Get())
			{
				return ClassOutput->AccessType;
			}

			return EMetasoundFrontendVertexAccessType::Unset;
		}

		FGuid FBaseOutputController::GetOwningNodeID() const
		{
			return OwningNode->GetID();
		}

		FNodeHandle FBaseOutputController::GetOwningNode()
		{
			return OwningNode;
		}

		FConstNodeHandle FBaseOutputController::GetOwningNode() const
		{
			return OwningNode;
		}

		bool FBaseOutputController::IsConnected() const 
		{
			return (FindEdges().Num() > 0);
		}

		TArray<FInputHandle> FBaseOutputController::GetConnectedInputs() 
		{
			TArray<FInputHandle> Inputs;

			// Create output handle from output node.
			FGraphHandle Graph = OwningNode->GetOwningGraph();

			for (const FMetasoundFrontendEdge& Edge : FindEdges())
			{
				FNodeHandle InputNode = Graph->GetNodeWithID(Edge.ToNodeID);

				FInputHandle Input = InputNode->GetInputWithID(Edge.ToVertexID);
				if (Input->IsValid())
				{
					Inputs.Add(Input);
				}
			}

			return Inputs;
		}

		TArray<FConstInputHandle> FBaseOutputController::GetConstConnectedInputs() const 
		{
			TArray<FConstInputHandle> Inputs;

			// Create output handle from output node.
			FConstGraphHandle Graph = OwningNode->GetOwningGraph();

			for (const FMetasoundFrontendEdge& Edge : FindEdges())
			{
				FConstNodeHandle InputNode = Graph->GetNodeWithID(Edge.ToNodeID);

				FConstInputHandle Input = InputNode->GetInputWithID(Edge.ToVertexID);
				if (Input->IsValid())
				{
					Inputs.Add(Input);
				}
			}

			return Inputs;
		}

		bool FBaseOutputController::Disconnect() 
		{
			bool bSuccess = true;
			for (FInputHandle Input : GetConnectedInputs())
			{
				if (Input->IsValid())
				{
					bSuccess &= Disconnect(*Input);
				}
			}
			return bSuccess;
		}

		bool FBaseOutputController::IsConnectionUserModifiable() const 
		{
			FConstNodeHandle Owner = GetOwningNode();
			if (Owner->GetClassMetadata().GetType() == EMetasoundFrontendClassType::Template)
			{
				const FNodeRegistryKey Key(GetOwningNode()->GetClassMetadata());
				const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(Key);
				if (ensure(Template))
				{
					return Template->IsOutputConnectionUserModifiable();
				}
			}

			return true;
		}

		FConnectability FBaseOutputController::CanConnectTo(const IInputController& InController) const
		{
			return InController.CanConnectTo(*this);
		}

		bool FBaseOutputController::Connect(IInputController& InController)
		{
			return InController.Connect(*this);
		}

		bool FBaseOutputController::ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName)
		{
			return InController.ConnectWithConverterNode(*this, InNodeClassName);
		}

		bool FBaseOutputController::Disconnect(IInputController& InController)
		{
			return InController.Disconnect(*this);
		}

		TArray<FMetasoundFrontendEdge> FBaseOutputController::FindEdges() const
		{
			if (const FMetasoundFrontendGraph* Graph = GraphPtr.Get())
			{
				const FGuid NodeID = GetOwningNodeID();
				FGuid VertexID = GetID();

				auto EdgeHasMatchingSource = [&](const FMetasoundFrontendEdge& Edge)
				{
					return (Edge.FromNodeID == NodeID) && (Edge.FromVertexID == VertexID);
				};

				return Graph->Edges.FilterByPredicate(EdgeHasMatchingSource);
			}

			return TArray<FMetasoundFrontendEdge>();
		}

#if WITH_EDITOR
		FText FBaseOutputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassOutput* Output = ClassOutputPtr.Get())
			{
				return Output->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		const FText& FBaseOutputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassOutput* Output = ClassOutputPtr.Get())
			{
				return Output->Metadata.GetDescription();
			}

			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FBaseOutputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassOutput* Output = ClassOutputPtr.Get())
			{
				return Output->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		FDocumentAccess FBaseOutputController::ShareAccess()
		{
			FDocumentAccess Access;

			Access.ConstVertex = NodeVertexPtr;
			Access.ConstClassOutput = ClassOutputPtr;
			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		FConstDocumentAccess FBaseOutputController::ShareAccess() const
		{
			FConstDocumentAccess Access;

			Access.ConstVertex = NodeVertexPtr;
			Access.ConstClassOutput = ClassOutputPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		//
		// FInputNodeOutputController
		// 
		FInputNodeOutputController::FInputNodeOutputController(const FInputNodeOutputController::FInitParams& InParams)
		: FBaseOutputController({InParams.ID, InParams.NodeVertexPtr, InParams.ClassOutputPtr, InParams.GraphPtr, InParams.OwningNode})
		, OwningGraphClassInputPtr(InParams.OwningGraphClassInputPtr)
		{
		}

		bool FInputNodeOutputController::IsValid() const
		{
			return FBaseOutputController::IsValid() && (nullptr != OwningGraphClassInputPtr.Get());
		}

#if WITH_EDITOR
		FText FInputNodeOutputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassInput* OwningInput = OwningGraphClassInputPtr.Get())
			{
				if (const FMetasoundFrontendClassOutput* ClassOutput = ClassOutputPtr.Get())
				{
					// If there is a valid ClassOutput, combine the names.
					CachedDisplayName = FText::Format(LOCTEXT("InputNodeOutputControllerFormat", "{1} {0}"), OwningInput->Metadata.GetDisplayName(), ClassOutput->Metadata.GetDisplayName());
				}
				else
				{
					// If there is no valid ClassOutput, use the owning value display name.
					CachedDisplayName = OwningInput->Metadata.GetDisplayName();
				}
			}

			return CachedDisplayName;
		}

		const FText& FInputNodeOutputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassInput* Input = OwningGraphClassInputPtr.Get())
			{
				return Input->Metadata.GetDescription();
			}

			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FInputNodeOutputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassInput* Input = OwningGraphClassInputPtr.Get())
			{
				return Input->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		void FInputNodeOutputController::SetName(const FVertexName& InName)
		{
			if (FMetasoundFrontendVertex* Vertex = ConstCastAccessPtr<FVertexAccessPtr>(NodeVertexPtr).Get())
			{
				Vertex->Name = InName;
			}
		}
		
		EMetasoundFrontendVertexAccessType FInputNodeOutputController::GetVertexAccessType() const
		{
			return OwningGraphClassInputPtr.Get()->AccessType;
		}

		FDocumentAccess FInputNodeOutputController::ShareAccess() 
		{
			FDocumentAccess Access = FBaseOutputController::ShareAccess();

			Access.ConstClassInput = OwningGraphClassInputPtr;

			return Access;
		}

		FConstDocumentAccess FInputNodeOutputController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseOutputController::ShareAccess();

			Access.ConstClassInput = OwningGraphClassInputPtr;

			return Access;
		}

		//
		// FOutputNodeOutputController
		//
		FOutputNodeOutputController::FOutputNodeOutputController(const FOutputNodeOutputController::FInitParams& InParams)
		: FBaseOutputController({InParams.ID, InParams.NodeVertexPtr, InParams.ClassOutputPtr, InParams.GraphPtr, InParams.OwningNode})
		, OwningGraphClassOutputPtr(InParams.OwningGraphClassOutputPtr)
		{
		}

		bool FOutputNodeOutputController::IsValid() const
		{
			return FBaseOutputController::IsValid() && (nullptr != OwningGraphClassOutputPtr.Get());
		}

#if WITH_EDITOR
		FText FOutputNodeOutputController::GetDisplayName() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata.GetDisplayName();
			}

			return Invalid::GetInvalidText();
		}

		const FText& FOutputNodeOutputController::GetTooltip() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata.GetDescription();
			}

			return Invalid::GetInvalidText();
		}

		const FMetasoundFrontendVertexMetadata& FOutputNodeOutputController::GetMetadata() const
		{
			if (const FMetasoundFrontendClassOutput* OwningOutput = OwningGraphClassOutputPtr.Get())
			{
				return OwningOutput->Metadata;
			}

			return Invalid::GetInvalidVertexMetadata();
		}
#endif // WITH_EDITOR

		void FOutputNodeOutputController::SetName(const FVertexName& InName)
		{
			if (FMetasoundFrontendVertex* Vertex = ConstCastAccessPtr<FVertexAccessPtr>(NodeVertexPtr).Get())
			{
				Vertex->Name = InName;
			}
		}

		EMetasoundFrontendVertexAccessType FOutputNodeOutputController::GetVertexAccessType() const
		{
			return OwningGraphClassOutputPtr.Get()->AccessType;
		}

		bool FOutputNodeOutputController::IsConnectionUserModifiable() const 
		{
			return false;
		}

		FConnectability FOutputNodeOutputController::CanConnectTo(const IInputController& InController) const 
		{
			// Cannot connect to a graph's output.
			static const FConnectability Connectability = {FConnectability::EConnectable::No};

			return Connectability;
		}

		bool FOutputNodeOutputController::Connect(IInputController& InController) 
		{
			return false;
		}

		bool FOutputNodeOutputController::ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName)
		{
			return false;
		}

		FVariableOutputController::FVariableOutputController(const FInitParams& InParams)
		: FBaseOutputController(InParams)
		{
		}

		bool FVariableOutputController::IsConnectionUserModifiable() const
		{
			// Variable connections are managed by the graph and cannot be modified
			// by the user.
			return false;
		}
	}
}

#undef LOCTEXT_NAMESPACE
