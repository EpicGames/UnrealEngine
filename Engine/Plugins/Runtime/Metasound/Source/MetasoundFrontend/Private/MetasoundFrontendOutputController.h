// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundVertex.h"
#include "UObject/Object.h"

namespace Metasound
{
	namespace Frontend
	{
		/** FBaseOutputController provides common functionality for multiple derived
		 * output controllers.
		 */
		class FBaseOutputController : public IOutputController
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:

			struct FInitParams
			{
				FGuid ID;

				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassOutputAccessPtr ClassOutputPtr;
				FGraphAccessPtr GraphPtr; 

				/* Node handle which owns this output. */
				FNodeHandle OwningNode;
			};

			/** Construct the output controller base.  */
			FBaseOutputController(const FInitParams& InParams);

			virtual ~FBaseOutputController() = default;

			virtual bool IsValid() const override;

			virtual FGuid GetID() const override;
			virtual const FName& GetDataType() const override;
			virtual const FVertexName& GetName() const override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Output metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			// Return info on containing node.
			virtual FGuid GetOwningNodeID() const override;
			virtual FNodeHandle GetOwningNode() override;
			virtual FConstNodeHandle GetOwningNode() const override;

			virtual void SetName(const FVertexName& InName) override { }

			virtual bool IsConnected() const override;
			virtual TArray<FInputHandle> GetConnectedInputs() override;
			virtual TArray<FConstInputHandle> GetConstConnectedInputs() const override;
			virtual bool Disconnect() override;

			// Connection logic.
			virtual bool IsConnectionUserModifiable() const override;
			virtual FConnectability CanConnectTo(const IInputController& InController) const override;
			virtual bool Connect(IInputController& InController) override;
			virtual bool ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName) override;
			virtual bool Disconnect(IInputController& InController) override;

		protected:
			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			FGuid ID;
			FConstVertexAccessPtr NodeVertexPtr;	
			FConstClassOutputAccessPtr ClassOutputPtr;
			FGraphAccessPtr GraphPtr; 
			FNodeHandle OwningNode;

		private:

			TArray<FMetasoundFrontendEdge> FindEdges() const;
		};


		/** FInputNodeOutputController represents the output vertex of an input 
		 * node. 
		 *
		 * FInputNodeOutputController is largely to represent inputs coming into 
		 * graph. 
		 */
		class FInputNodeOutputController : public FBaseOutputController
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:
			struct FInitParams
			{
				FGuid ID;

				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassOutputAccessPtr ClassOutputPtr;
				FConstClassInputAccessPtr OwningGraphClassInputPtr;
				FGraphAccessPtr GraphPtr; 

				/* Node handle which owns this output. */
				FNodeHandle OwningNode;
			};

			/** Constructs the output controller. */
			FInputNodeOutputController(const FInitParams& InParams);

			virtual ~FInputNodeOutputController() = default;

			virtual bool IsValid() const override;

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Input metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			virtual void SetName(const FVertexName& InName) override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

		protected:
			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

		private:
			mutable FText CachedDisplayName;

			FConstClassInputAccessPtr OwningGraphClassInputPtr;
		};

		/** FOutputNodeOutputController represents the output vertex of an input 
		 * node. 
		 *
		 * FOutputNodeOutputController is largely to represent inputs coming into 
		 * graph. 
		 */
		class FOutputNodeOutputController : public FBaseOutputController
		{
			using FRegistry = FMetasoundFrontendRegistryContainer;
		public:
			struct FInitParams
			{
				FGuid ID;

				FConstVertexAccessPtr NodeVertexPtr;
				FConstClassOutputAccessPtr ClassOutputPtr;
				FConstClassOutputAccessPtr OwningGraphClassOutputPtr;
				FGraphAccessPtr GraphPtr; 

				/* Node handle which owns this output. */
				FNodeHandle OwningNode;
			};

			/** Constructs the output controller. */
			FOutputNodeOutputController(const FInitParams& InParams);

			virtual ~FOutputNodeOutputController() = default;

			virtual bool IsValid() const override;

#if WITH_EDITOR
			virtual FText GetDisplayName() const override;
			virtual const FText& GetTooltip() const override;

			// Output metadata
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const override;
#endif // WITH_EDITOR

			virtual void SetName(const FVertexName& InName) override;
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const override;

			virtual bool IsConnectionUserModifiable() const override;
			virtual FConnectability CanConnectTo(const IInputController& InController) const override;
			virtual bool Connect(IInputController& InController) override;
			virtual bool ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName) override;

		private:
			FConstClassOutputAccessPtr OwningGraphClassOutputPtr;
		};

		/** Output controller for variable data type. */
		class FVariableOutputController : public FBaseOutputController
		{
		public:
			using FInitParams = FBaseOutputController::FInitParams;

			FVariableOutputController(const FInitParams& InParams);
			virtual ~FVariableOutputController() = default;

			/** Variable data type connections are not modifiable by users */
			virtual bool IsConnectionUserModifiable() const override;
		};

	} // namespace frontend
} // namespace metasound

