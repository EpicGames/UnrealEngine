// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundFrontendPages.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Logging/StructuredLog.h"
#include "Misc/Guid.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundTrace.h"

namespace Metasound::Frontend
{
	TArrayView<const FGuid> GetPageOrderForTargetPage(const FGuid& InTargetPage, TArrayView<const FGuid> InPageOrder)
	{
		int32 Pos = InPageOrder.Find(InTargetPage);
		if (Pos == INDEX_NONE)
		{
			return {};
		}
		else
		{
			// Discards and pages in the page order which occur before the target page. 
			return InPageOrder.RightChop(Pos);
		}
	}

#if WITH_EDITORONLY_DATA
	bool StripUnusedGraphPages(FMetaSoundFrontendDocumentBuilder& InBuilder, TArrayView<const FGuid> InTargetPages, TArrayView<const FGuid> InPageOrder)
	{
		bool bModified = false;
		const FMetasoundFrontendDocument& Document = InBuilder.GetConstDocumentChecked();

		// Find which graphs to remove. 
		TArray<FGuid> PageIDsToRemove;
		Document.RootGraph.IterateGraphPages([&PageIDsToRemove](const FMetasoundFrontendGraph& InGraph) { PageIDsToRemove.Add(InGraph.PageID);});
		
		for (const FGuid& TargetPage : InTargetPages)
		{
			TArrayView<const FGuid> PageOrderForTarget = GetPageOrderForTargetPage(TargetPage, InPageOrder);
			if (const FMetasoundFrontendGraph* ResolvedGraph = FindPreferredPage(Document.RootGraph.GetConstGraphPages(), PageOrderForTarget))
			{
				// Don't remove the specific page ID if it is the one which will
				// be chosen at runtime.
				PageIDsToRemove.Remove(ResolvedGraph->PageID);
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Failed to find graph to cook for target page %ls on MetaSound %ls", *TargetPage.ToString(), *InBuilder.GetDebugName());
			}
		}

		const int32 NumInitGraphs = Document.RootGraph.GetConstGraphPages().Num();

		// Remove unused graphs
		for (const FGuid& PageIDToRemove : PageIDsToRemove)
		{
			bool bDidRemove = InBuilder.RemoveGraphPage(PageIDToRemove);
			if (bDidRemove)
			{
				UE_LOGF(LogMetaSound, Display, "%ls: Removed Graph w/PageID '%ls'",
					*InBuilder.GetDebugName(),
					*PageIDToRemove.ToString());
				bModified = true;
			}
		}

		// Log results and perform sanity checking
		const int32 NumRemainingGraphs = Document.RootGraph.GetConstGraphPages().Num();

		UE_CLOGFMT(NumRemainingGraphs <= 0, LogMetaSound, Warning,
			"Document in MetaSound asset '{AssetPath}' had all default values "
				"cooked away leaving it in an invalid state. "
				"Graph must always have at least one implementation.",
			InBuilder.GetDebugName());

		if (NumInitGraphs > NumRemainingGraphs)
		{
			UE_LOGF(LogMetaSound, Display, "Cook removed %i graph page(s) from '%ls'", NumInitGraphs - NumRemainingGraphs, *InBuilder.GetDebugName());
		}

		return bModified;
	}

	bool StripUnusedClassInputPages(FMetaSoundFrontendDocumentBuilder& InBuilder, TArrayView<const FGuid> InTargetPages, TArrayView<const FGuid> InPageOrder)
	{
		bool bModified = false;
		const FMetasoundFrontendDocument& Document = InBuilder.GetConstDocumentChecked();
		for (const FMetasoundFrontendClassInput& GraphInput : Document.RootGraph.GetDefaultInterface().Inputs)
		{
			// Begin with the assumption that all pages will be removed. Then only keep the ones which will actaully
			// be utilized in a cooked build. 
			TArray<FGuid> PageIDsToRemove;
			Algo::Transform(GraphInput.GetDefaults(), PageIDsToRemove, [](const FMetasoundFrontendClassInputDefault& InDefault) { return InDefault.PageID; });

			
			for (const FGuid& TargetPage : InTargetPages)
			{
				TArrayView<const FGuid> PageOrderForTarget = GetPageOrderForTargetPage(TargetPage, InPageOrder);
				if (const FMetasoundFrontendClassInputDefault* ResolvedInput = FindPreferredPage(GraphInput.GetDefaults(), PageOrderForTarget))
				{
					// Don't remove the specific page ID if it is the one which will
					// be chosen at runtime.
					PageIDsToRemove.Remove(ResolvedInput->PageID);
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Failed to find class input default to cook for target page %ls on MetaSound %ls class input %ls", *TargetPage.ToString(), *InBuilder.GetDebugName(), *GraphInput.Name.ToString());
				}
			}

			const int32 NumInitDefaults = GraphInput.GetDefaults().Num();

			// Remove unused inputs
			for (const FGuid& PageIDToRemove : PageIDsToRemove)
			{
				const bool bDidRemove = InBuilder.StripGraphInputDefault(GraphInput.Name, PageIDToRemove);
				if (bDidRemove)
				{
					UE_LOGF(LogMetaSound, Display, "%ls: Removed Graph Input %ls w/PageID '%ls'",
						*InBuilder.GetDebugName(),
						*GraphInput.Name.ToString(),
						*PageIDToRemove.ToString());
					bModified = true;
				}
			}

			// Log results and perform sanity checking
			const int32 NumRemainingDefaults = GraphInput.GetDefaults().Num();

			UE_CLOGFMT(NumRemainingDefaults <= 0, LogMetaSound, Warning,
				"Document in MetaSound asset '{AssetPath}' had all default input values "
					"cooked away for input {InputName} leaving it in an invalid state. "
					"Graph must always have at least one implementation.",
				InBuilder.GetDebugName(), 
				GraphInput.Name);

			if (NumInitDefaults > NumRemainingDefaults)
			{
				UE_LOGF(LogMetaSound, Display, "Cook removed %i input default page(s) from input %ls on '%ls'", NumInitDefaults - NumRemainingDefaults, *GraphInput.Name.ToString(), *InBuilder.GetDebugName());
			}

		}

		return bModified;
	}

	bool StripUnusedPages(FMetaSoundFrontendDocumentBuilder& InBuilder, TArrayView<const FGuid> InTargetPages, TArrayView<const FGuid> InPageOrder)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::StripUnusedPages);

		checkf(InBuilder.GetConstDocumentChecked().Metadata.Version.Number == FVersioningManager::GetMaxDocumentVersion(),
			TEXT("Document must be versioned prior to stripping pages. Expected %s, found %s on asset %s"), 
			*FVersioningManager::GetMaxDocumentVersion().ToString(),
			*InBuilder.GetConstDocumentChecked().Metadata.Version.Number.ToString(), 
			*InBuilder.GetDebugName());

		if (InTargetPages.Num() < 1)
		{
			UE_LOGF(LogMetaSound, Error, "Must have at least one target page to strip unused pages in asset %ls", *InBuilder.GetDebugName());
			return false; // Not modified
		}

		bool bModified = StripUnusedGraphPages(InBuilder, InTargetPages, InPageOrder);
		bModified |= StripUnusedClassInputPages(InBuilder, InTargetPages, InPageOrder);

		return bModified;
	}
#endif // WITH_EDITORONLY_DATA
}
