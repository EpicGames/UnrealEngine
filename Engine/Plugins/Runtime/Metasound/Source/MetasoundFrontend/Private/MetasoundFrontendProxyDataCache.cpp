// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendProxyDataCache.h"


#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "IAudioProxyInitializer.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundTrace.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace MetasoundFrontendProxyDataCachePrivate
		{
			template<typename DocElementType>
			void CreateAndCacheProxies(FProxyDataCache& InCache, const DocElementType& InDocElement, TArrayView<const FGuid> InPageOrder)
			{
				checkf(IsInGameThread() || IsInAudioThread(), TEXT("Proxies should only be created in the game thread or audio thread"));
				IDataTypeRegistry& DataRegistry = IDataTypeRegistry::Get();
				TArray<UObject*> UObjectArray;

				auto CreateProxies = [&InCache, &DataRegistry, &UObjectArray](const FName& InDataType, const FMetasoundFrontendLiteral& InLiteral)
				{
					EMetasoundFrontendLiteralType LiteralType = InLiteral.GetType();
					if (LiteralType == EMetasoundFrontendLiteralType::UObject)
					{
						UObject* Object = nullptr;
						InLiteral.TryGet(Object);
						if (Object)
						{
							if (!InCache.Contains(Object))
							{
								InCache.CacheProxy(Object, DataRegistry.CreateProxyFromUObject(InDataType, Object));
							}
						}
					}
					else if (LiteralType == EMetasoundFrontendLiteralType::UObjectArray)
					{
						FName ElementDataTypeName = CreateElementTypeNameFromArrayTypeName(InDataType);
						UObjectArray.Reset();
						InLiteral.TryGet(UObjectArray);
						for (UObject* Object : UObjectArray)
						{
							if (Object)
							{
								if (!InCache.Contains(Object))
								{
									InCache.CacheProxy(Object, DataRegistry.CreateProxyFromUObject(ElementDataTypeName, Object));
								}
							}
						}
					}
				};
				ForEachLiteral(InDocElement, CreateProxies, InPageOrder);
			}
		}

		void FProxyDataCache::CreateAndCacheProxies(const FMetasoundFrontendDocument& InDocument, TArrayView<const FGuid> InPageOrder)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FProxyDataCache::CreateAndCacheProxies_Document)
			MetasoundFrontendProxyDataCachePrivate::CreateAndCacheProxies(*this, InDocument, InPageOrder);
		}

		void FProxyDataCache::CreateAndCacheProxies(const FMetasoundFrontendGraphClass& InGraphClass, TArrayView<const FGuid> InPageOrder)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FProxyDataCache::CreateAndCacheProxies_GraphClass)
			MetasoundFrontendProxyDataCachePrivate::CreateAndCacheProxies(*this, InGraphClass, InPageOrder);
		}

		void FProxyDataCache::CreateAndCacheProxies(const FMetasoundFrontendClass& InClass, TArrayView<const FGuid> InPageOrder)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FProxyDataCache::CreateAndCacheProxies_Class)
			MetasoundFrontendProxyDataCachePrivate::CreateAndCacheProxies(*this, InClass, InPageOrder);
		}

		bool FProxyDataCache::Contains(const UObject* InUObject) const
		{
			return ProxyCache.Contains(InUObject);
		}

		void FProxyDataCache::CacheProxy(const UObject* InUObject, TSharedPtr<Audio::IProxyData> InProxy)
		{
			ProxyCache.Add(InUObject, MoveTemp(InProxy));
		}

		const TSharedPtr<Audio::IProxyData>* FProxyDataCache::FindProxy(const UObject* InObject) const
		{
			return ProxyCache.Find(InObject);
		}
	}
}

