// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"


// Forward declarations
class UObject;
struct FMetasoundFrontendClass;
struct FMetasoundFrontendGraphClass;
struct FMetasoundFrontendDocument;

namespace Audio
{
	class IProxyData;
}

namespace Metasound
{
	namespace Frontend
	{
		/** FProxyDataCache creates and holds onto Audio::IProxyData produced from UObjects.
		 *
		 * A proxy data cache is useful in scenarios where proxies need to be retrieved
		 * from a thread where calling UObject functions is not safe. Additionally,
		 * the proxy data cache only creates a single proxy per a UObject when creating
		 * proxies from a FMetasoundFrontendDocument or any of it's subobjects. 
		 */
		class FProxyDataCache
		{
		public:
			/** Create and cache all proxies found in a document. */
			void CreateAndCacheProxies(const FMetasoundFrontendDocument& InDocument, TArrayView<const FGuid> InPageOrder);

			/** Create and cache all proxies found in a graph class. */
			void CreateAndCacheProxies(const FMetasoundFrontendGraphClass& InGraphClass, TArrayView<const FGuid> InPageOrder);

			/** Create and cache all proxies found in a frontend class class. */
			void CreateAndCacheProxies(const FMetasoundFrontendClass& InClass, TArrayView<const FGuid> InPageOrder);

			/** Cache a specific object associated with a UObject pointer.
			 *
			 * No methods or data are accessed from the UObject pointer passed 
			 * into this method. This method is safe even if the underlying UObject 
			 * is invalid or garbage collected before or during this call.
			 */
			void CacheProxy(const UObject* InUObject, TSharedPtr<Audio::IProxyData> InProxy);

			/** Returns true if there is a proxy stored which is associated with
			 * the UObject address. 
			 *
			 * No methods or data are accessed from the UObject pointer passed 
			 * into this method. This method is safe even if the underlying UObject 
			 * is invalid or garbage collected before or during this call.
			 */
			bool Contains(const UObject* InUObject) const;


			/** Returns a pointer to an existing proxy if one has been cached. 
			 * Returns nullptr if none exist.
			 *
			 * No methods or data are accessed from the UObject pointer passed 
			 * into this method. This method is safe even if the underlying UObject 
			 * is invalid or garbage collected before or during this call.
			 */
			const TSharedPtr<Audio::IProxyData>* FindProxy(const UObject* InObject) const;

		private:
			using FUObjectMemoryAddress = const void*;

			TMap<FUObjectMemoryAddress, TSharedPtr<Audio::IProxyData>> ProxyCache;
		};
	}
}
