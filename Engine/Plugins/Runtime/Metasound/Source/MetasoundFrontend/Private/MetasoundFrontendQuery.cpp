// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendQuery.h"

#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "AssetRegistry/AssetData.h"
#include "JsonObjectConverter.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetTagCollections.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundTrace.h"
#include "Misc/Guid.h"
#include "Templates/TypeHash.h"
#include "Traits/IsContiguousContainer.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundFrontendQuery)


namespace Metasound::Frontend
{
	namespace ClassTagsPrivate
	{
		static const FLazyName AccessFlags = "AccessFlags";

#if WITH_EDITORONLY_DATA
		static const FLazyName AssetFlags_Deprecated = "AssetFlags";
#endif // WITH_EDITORONLY_DATA

		static const FLazyName ClassVersion = "ClassVersion";

#if WITH_EDITORONLY_DATA
		static const FLazyName ClassCollections = "ClassCollections";
		static const FLazyName SearchInfo = "SearchInfo";
		static const FLazyName Inputs = "Inputs";
		static const FLazyName Outputs = "Outputs";
#endif // WITH_EDITORONLY_DATA

		template <typename TStructType>
		bool DeserializeTagFromJson(const FAssetData& InAssetData, FName TagName, TStructType& OutStruct)
		{
			FString TagString;
			const bool bTagDataFound = InAssetData.GetTagValue(TagName, TagString);
			if (bTagDataFound)
			{
				return FJsonObjectConverter::JsonObjectStringToUStruct(TagString, &OutStruct);
			}
			return false;
		};
	}
} // namespace Metasound::Frontend


FMetaSoundClassSearchInfo::FMetaSoundClassSearchInfo(const FMetasoundFrontendClassMetadata& InClassMetadata)
#if WITH_EDITOR
	: ClassDisplayName(InClassMetadata.GetDisplayName())
	, ClassDescription(InClassMetadata.GetDescription())
	, Hierarchy(InClassMetadata.GetCategoryHierarchy())
	, Keywords(InClassMetadata.GetKeywords())
#endif // WITH_EDITOR
{
}

FMetaSoundClassInterfaceInfo::FMetaSoundClassInterfaceInfo(const IMetaSoundDocumentInterface& InDocInterface)
{
#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;

	if (!IsRunningCookCommandlet())
	{
		const FMetasoundFrontendDocument& Document = InDocInterface.GetConstDocument();
		SearchInfo = FMetaSoundClassSearchInfo(Document.RootGraph.Metadata);

		Algo::Copy(Document.Interfaces, InheritedInterfaces);

		{
			auto CopyVertexInfo = [](const FMetasoundFrontendClassVertex& Vertex)
			{
				return FMetaSoundClassVertexInfo
				{
					.Name = Vertex.Name,
					.TypeName = Vertex.TypeName,
					.AccessType = Vertex.AccessType
				};
			};

			const FMetasoundFrontendClassInterface& ClassInterface = Document.RootGraph.GetDefaultInterface();
			Algo::Transform(ClassInterface.Inputs, Inputs, CopyVertexInfo);
			Algo::Transform(ClassInterface.Outputs, Outputs, CopyVertexInfo);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

FMetaSoundClassInterfaceInfo::FMetaSoundClassInterfaceInfo(const FMetasoundFrontendClass& InClass)
{
#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;

	if (!IsRunningCookCommandlet())
	{
		SearchInfo = FMetaSoundClassSearchInfo(InClass.Metadata);

		{
			auto CopyVertexInfo = [](const FMetasoundFrontendClassVertex& Vertex)
			{
				return FMetaSoundClassVertexInfo
				{
					.Name = Vertex.Name,
					.TypeName = Vertex.TypeName,
					.AccessType = Vertex.AccessType
				};
			};

			Algo::Transform(InClass.GetDefaultInterface().Inputs, Inputs, CopyVertexInfo);
			Algo::Transform(InClass.GetDefaultInterface().Outputs, Outputs, CopyVertexInfo);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

FMetaSoundClassInterfaceInfo::FMetaSoundClassInterfaceInfo(const FAssetData& InAssetData, bool& bOutIsValid)
{
	bOutIsValid = true;

#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;

	if (!IsRunningCookCommandlet())
	{
		{
			FMetaSoundAssetTagClassCollections TagCollections;
			const bool bTagFound = ClassTagsPrivate::DeserializeTagFromJson(InAssetData, ClassTagsPrivate::ClassCollections, TagCollections);
			if (bTagFound)
			{
				DefinedInterfaces = MoveTemp(TagCollections.DefinedInterfaces);
				InheritedInterfaces = MoveTemp(TagCollections.InheritedInterfaces);
			}
			else
			{
				bOutIsValid = false;
			}
		}

		bOutIsValid &= ClassTagsPrivate::DeserializeTagFromJson(InAssetData, ClassTagsPrivate::SearchInfo, SearchInfo);

		FMetaSoundClassVertexCollectionInfo VertexCollection;
		bOutIsValid &= ClassTagsPrivate::DeserializeTagFromJson(InAssetData, ClassTagsPrivate::Inputs, VertexCollection);
		Inputs = MoveTemp(VertexCollection.ClassVertexInfo);
		VertexCollection.ClassVertexInfo.Reset();
		bOutIsValid &= ClassTagsPrivate::DeserializeTagFromJson(InAssetData, ClassTagsPrivate::Outputs, VertexCollection);
		Outputs = MoveTemp(VertexCollection.ClassVertexInfo);
	}
#endif // WITH_EDITORONLY_DATA
}

void FMetaSoundClassInterfaceInfo::ExportToContext(FAssetRegistryTagsContext& OutContext) const
{
#if WITH_EDITORONLY_DATA
	using namespace Metasound::Frontend;
	using FAssetRegistryTag = UObject::FAssetRegistryTag;

	if (!OutContext.IsCooking())
	{
		auto SerializeTagToJson = []<typename TStructType>(const TStructType& InStruct)
		{
			FString JsonString;
			FJsonObjectConverter::UStructToJsonObjectString(InStruct, JsonString);
			return JsonString;
		};

		const FMetaSoundAssetTagClassCollections TagCollections { .DefinedInterfaces = DefinedInterfaces, .InheritedInterfaces = InheritedInterfaces };
		OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::ClassCollections, SerializeTagToJson(TagCollections), FAssetRegistryTag::TT_Hidden));
		OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::SearchInfo, SerializeTagToJson(SearchInfo), FAssetRegistryTag::TT_Hidden));

		FMetaSoundClassVertexCollectionInfo VertexCollection { Inputs };
		OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::Inputs, SerializeTagToJson(VertexCollection), FAssetRegistryTag::TT_Hidden));

		VertexCollection.ClassVertexInfo = Outputs;
		OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::Outputs, SerializeTagToJson(VertexCollection), FAssetRegistryTag::TT_Hidden));
	}
#endif // WITH_EDITORONLY_DATA
}


namespace Metasound
{
	namespace Frontend
	{
		FMetaSoundClassInfo::FMetaSoundClassInfo(EMetasoundFrontendClassType InClassType)
			: ClassType(InClassType)
			, bIsValid(0)
		{
		}

		FMetaSoundClassInfo::FMetaSoundClassInfo(const FAssetData& InAssetData)
			: ClassType(EMetasoundFrontendClassType::External)
			, bIsValid(0)
		{
			bool bSuccess = true;

#if WITH_EDITORONLY_DATA
			InterfaceInfo = FMetaSoundClassInterfaceInfo(InAssetData, bSuccess);
#endif // WITH_EDITORONLY_DATA

			{
				int32 AccessFlagsValue = 0;
				bool bFoundFlags = InAssetData.GetTagValue(ClassTagsPrivate::AccessFlags, AccessFlagsValue);

#if WITH_EDITORONLY_DATA
				if (!bFoundFlags)
				{
					bFoundFlags = InAssetData.GetTagValue(ClassTagsPrivate::AssetFlags_Deprecated, AccessFlagsValue);
				}
#endif // WITH_EDITORONLY_DATA

				if (bFoundFlags)
				{
					AccessFlags = static_cast<EMetasoundFrontendClassAccessFlags>((uint16)(AccessFlagsValue));
				}
				else
				{
					// When the referenceable flag was added, the prior document version assumed everything was referenceable.
					// Therefore, we can reliably set this flag as that is the default that the document version process inevitably
					// will upgrade the flag to.  Setting here enables caller to at least know it should be shown to end users or
					// is available by default for referencing if out-of-date.
					AccessFlags = EMetasoundFrontendClassAccessFlags::Referenceable;
				}
				bSuccess &= bFoundFlags;
			}

			bSuccess &= TryGetClassVersion(InAssetData, Version);
			bIsValid = (uint8)(bSuccess);
		}

		FMetaSoundClassInfo::FMetaSoundClassInfo(const FMetasoundFrontendClass& InClass)
			: ClassName(InClass.Metadata.GetClassName())
			, ClassType(InClass.Metadata.GetType())
			, Version(InClass.Metadata.GetVersion())
			, bIsValid(InClass.Metadata.GetType() != EMetasoundFrontendClassType::Invalid)
		{
#if WITH_EDITORONLY_DATA
			InterfaceInfo = FMetaSoundClassInterfaceInfo(InClass);
#endif // WITH_EDITORONLY_DATA

			AccessFlags = InClass.Metadata.GetAccessFlags();
		}

		void FMetaSoundClassInfo::ExportToContext(FAssetRegistryTagsContext& OutContext) const
		{
			using FAssetRegistryTag = UObject::FAssetRegistryTag;

			OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::ClassVersion, Version.ToString(), FAssetRegistryTag::TT_Alphabetical));
			OutContext.AddTag(FAssetRegistryTag(ClassTagsPrivate::AccessFlags, FString::Printf(TEXT("%i"), (int32)(AccessFlags)), FAssetRegistryTag::TT_Hidden));

#if WITH_EDITORONLY_DATA
			if (!OutContext.IsCooking())
			{
				InterfaceInfo.ExportToContext(OutContext);
			}
#endif // WITH_EDITORONLY_DATA
		}

		void FMetaSoundClassInfo::InitFromDocument(const IMetaSoundDocumentInterface& InDocInterface)
		{
			const FMetasoundFrontendDocument& Document = InDocInterface.GetConstDocument();
			const FMetasoundFrontendClassMetadata& ClassMetadata = Document.RootGraph.Metadata;

			AccessFlags = Document.RootGraph.Metadata.GetAccessFlags();
			ClassName = Document.RootGraph.Metadata.GetClassName();
			ClassType = EMetasoundFrontendClassType::External;
			Version = ClassMetadata.GetVersion();

#if WITH_EDITORONLY_DATA
			if (!IsRunningCookCommandlet())
			{
				InterfaceInfo = FMetaSoundClassInterfaceInfo(InDocInterface);
			}
#endif // WITH_EDITORONLY_DATA

			bIsValid = 1;
		}

		bool FMetaSoundClassInfo::TryGetClassVersion(const FAssetData& InAssetData, FMetasoundFrontendVersionNumber& OutVersion)
		{
			if (InAssetData.IsAssetLoaded())
			{
				if (const UObject* Object = InAssetData.GetAsset())
				{
					TScriptInterface<const IMetaSoundDocumentInterface> DocInterface(Object);
					if (DocInterface.GetObject())
					{
						OutVersion = DocInterface->GetConstDocument().Metadata.Version.Number;
						return true;
					}
				}
			}

			bool bSuccess = true;
			{
				FString ClassVersionStr;
				bSuccess &= InAssetData.GetTagValue(ClassTagsPrivate::ClassVersion, ClassVersionStr);
				bSuccess &= FMetasoundFrontendVersionNumber::Parse(ClassVersionStr, OutVersion);
			}

			return bSuccess;
		}

#if WITH_EDITORONLY_DATA
		bool FMetaSoundClassInfo::InheritsInterface(FName InterfaceName, const  FMetasoundFrontendVersionNumber& VersionNumber) const
		{
			if (bIsValid==false)
			{
				return false;
			}

			if (!VersionNumber.IsValid())
			{
				// match name only...
				return InterfaceInfo.InheritedInterfaces.ContainsByPredicate([InterfaceName](const FMetasoundFrontendVersion& VersionInfo) { return VersionInfo.Name == InterfaceName; });
			}
			
			// require exact version match...
			const FMetasoundFrontendVersion OfInterest = {InterfaceName, VersionNumber };
			return InterfaceInfo.InheritedInterfaces.Contains(OfInterest);
		}
#endif
	} // namespace Frontend

	namespace FrontendQueryPrivate
	{
		void CompactSelection(FFrontendQuerySelection& InSelection)
		{
			InSelection.Shrink();
		}

		// Not symmetric. Elements in LHS that are not in RHS. Modifies partition by resorting.
		FFrontendQueryPartition Difference(FFrontendQueryPartition& InLHS, FFrontendQueryPartition& InRHS)
		{
			static_assert(TIsContiguousContainer<FFrontendQueryPartition>::Value, "Partitions must be a contiguous container for Difference algorithm to no access invalid memory");

			if ((InLHS.Num() == 0) || (InRHS.Num() == 0))
			{
				return InLHS;
			}

			auto IsIDLessThan = [](const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS) { return InLHS.ID < InRHS.ID; };
			auto IsIDEqual = [](const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS) { return InLHS.ID == InRHS.ID; };

			InLHS.Sort(IsIDLessThan);
			InRHS.Sort(IsIDLessThan);

			FFrontendQueryPartition Result;
			const FFrontendQueryEntry* LHSPtr = InLHS.GetData();
			const FFrontendQueryEntry* LHSPtrEnd = LHSPtr + InLHS.Num();
			const FFrontendQueryEntry* RHSPtr = InRHS.GetData();
			const FFrontendQueryEntry* RHSPtrEnd = RHSPtr + InRHS.Num();

			while (LHSPtr != LHSPtrEnd)
			{
				if ((RHSPtr == RHSPtrEnd) || IsIDLessThan(*LHSPtr, *RHSPtr))
				{
					Result.Add(*LHSPtr);
					LHSPtr++;
				}
				else if (IsIDLessThan(*RHSPtr, *LHSPtr))
				{
					RHSPtr++;
				}
				else
				{
					// Values are equal
					LHSPtr++;
					RHSPtr++;
				}
			}
			
			return Result;
		}

		// Wrapper for step defined by function
		struct FStreamFunctionFrontendQueryStep: IFrontendQueryStreamStep
		{
			using FStreamFunction = FFrontendQueryStep::FStreamFunction;

			FStreamFunctionFrontendQueryStep(FStreamFunction&& InFunc)
			:	Func(MoveTemp(InFunc))
			{
			}

			void Stream(TArray<FFrontendQueryValue>& OutValues) override
			{
				Func(OutValues);
			}

			private:

			FStreamFunction Func;
		};

		// Wrapper for step defined by function
		struct FTransformFunctionFrontendQueryStep: IFrontendQueryTransformStep
		{
			using FTransformFunction = FFrontendQueryStep::FTransformFunction;

			FTransformFunctionFrontendQueryStep(FTransformFunction InFunc)
			:	Func(InFunc)
			{
			}

			void Transform(FFrontendQueryEntry::FValue& InValue) const override
			{
				Func(InValue);
			}

			FTransformFunction Func;
		};

		// Wrapper for step defined by function
		struct FMapFunctionFrontendQueryStep: IFrontendQueryMapStep
		{
			using FMapFunction = FFrontendQueryStep::FMapFunction;

			FMapFunctionFrontendQueryStep(FMapFunction InFunc)
			:	Func(InFunc)
			{
			}

			FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override
			{
				return Func(InEntry);
			}

			FMapFunction Func;
		};

		// Wrapper for step defined by function
		struct FMultiMapFunctionFrontendQueryStep : IFrontendQueryMultiMapStep
		{
			using FMultiMapFunction = FFrontendQueryStep::FMultiMapFunction;

			FMultiMapFunctionFrontendQueryStep(FMultiMapFunction InFunc)
				: Func(InFunc)
			{
			}

			TArray<FFrontendQueryKey> Map(const FFrontendQueryEntry& InEntry) const override
			{
				return Func(InEntry);
			}

			FMultiMapFunction Func;
		};

		// Wrapper for step defined by function
		struct FReduceFunctionFrontendQueryStep: IFrontendQueryReduceStep
		{
			using FReduceFunction = FFrontendQueryStep::FReduceFunction;

			FReduceFunctionFrontendQueryStep(FReduceFunction InFunc)
			:	Func(InFunc)
			{
			}
			void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override
			{
				return Func(InKey, InOutEntries);
			}

			FReduceFunction Func;
		};

		// Wrapper for step defined by function
		struct FFilterFunctionFrontendQueryStep: IFrontendQueryFilterStep
		{
			using FFilterFunction = FFrontendQueryStep::FFilterFunction;

			FFilterFunctionFrontendQueryStep(FFilterFunction InFunc)
			:	Func(InFunc)
			{
			}

			bool Filter(const FFrontendQueryEntry& InEntry) const override
			{
				return Func(InEntry);
			}

			FFilterFunction Func;
		};

		// Wrapper for step defined by function
		struct FScoreFunctionFrontendQueryStep: IFrontendQueryScoreStep
		{
			using FScoreFunction = FFrontendQueryStep::FScoreFunction;

			FScoreFunctionFrontendQueryStep(FScoreFunction InFunc)
			:	Func(InFunc)
			{
			}

			float Score(const FFrontendQueryEntry& InEntry) const override
			{
				return Func(InEntry);
			}

			FScoreFunction Func;
		};

		// Wrapper for step defined by function
		struct FSortFunctionFrontendQueryStep: IFrontendQuerySortStep
		{
			using FSortFunction = FFrontendQueryStep::FSortFunction;

			FSortFunctionFrontendQueryStep(FSortFunction InFunc)
			:	Func(InFunc)
			{
			}

			bool Sort(const FFrontendQueryEntry& InEntryLHS, const FFrontendQueryEntry& InEntryRHS) const override
			{
				return Func(InEntryLHS, InEntryRHS);
			}

			FSortFunction Func;
		};

		// Wrapper for step defined by function
		struct FLimitFunctionFrontendQueryStep: IFrontendQueryLimitStep
		{
			using FLimitFunction = FFrontendQueryStep::FLimitFunction;

			FLimitFunctionFrontendQueryStep(FLimitFunction InFunc)
			:	Func(InFunc)
			{
			}

			int32 Limit() const override
			{
				return Func();
			}

			FLimitFunction Func;
		};

		// Base step executer for all step types.
		template<typename StepType>
		struct TStepExecuterBase : public FFrontendQueryStep::IStepExecuter
		{
			using FIncremental = FFrontendQueryStep::FIncremental;

			TStepExecuterBase(TUniquePtr<StepType>&& InStep)
			:	Step(MoveTemp(InStep))
			{
			}

			virtual void Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::BaseQueryStep::Merge);
				MergePartitionCompositionIndependent(InOutIncremental, InOutSelection);
			}

			virtual bool IsDependentOnPartitionComposition() const override { return false; }
			virtual bool CanProcessRemovals() const override { return true; }
			virtual bool CanProduceEntries() const override { return false; }

		protected:

			void MergePartitionCompositionIndependent(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const
			{
				checkf(!IsDependentOnPartitionComposition(), TEXT("Incorrect merge function called on step which is dependent upon merge composition"));
				Remove(InOutIncremental.ActiveRemovalKeys, InOutIncremental.ActiveRemovalSelection, InOutSelection);
				Append(InOutIncremental.ActiveKeys, InOutIncremental.ActiveSelection, InOutSelection);
			}

			// Appends incremental results to the output results.
			void Append(const TSet<FFrontendQueryKey>& InActiveKeys, const FFrontendQuerySelection& InIncrementalSelection, FFrontendQuerySelection& InOutSelection) const
			{
				for (const FFrontendQueryKey& Key : InActiveKeys)
				{
					if (const FFrontendQueryPartition* Partition = InIncrementalSelection.Find(Key))
					{
						if (Partition->Num() > 0)
						{
							InOutSelection.FindOrAdd(Key).Append(*Partition);
						}
					}
				}
			}

			void Remove(const TSet<FFrontendQueryKey>& InRemovalActiveKeys, const FFrontendQuerySelection& InRemovalSelection, FFrontendQuerySelection& InOutSelection) const
			{
				for (const FFrontendQueryKey& Key : InRemovalActiveKeys)
				{
					if (FFrontendQueryPartition* Entries = InOutSelection.Find(Key))
					{
						if (const FFrontendQueryPartition* EntriesToRemove = InRemovalSelection.Find(Key))
						{
							for (const FFrontendQueryEntry& EntryToRemove : *EntriesToRemove)
							{
								Entries->Remove(EntryToRemove);
							}
						}

						if (Entries->Num() == 0)
						{
							InOutSelection.Remove(Key);
						}
					}
				}
			}

			TUniquePtr<StepType> Step;
		};

		struct FStreamStepExecuter : TStepExecuterBase<IFrontendQueryStreamStep>
		{
			using TStepExecuterBase<IFrontendQueryStreamStep>::TStepExecuterBase;

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::StreamQueryStep::Execute);
				InOutUpdatedKeys.Reset();

				if (Step.IsValid())
				{
					// Retrieve new values from step
					TArray<FFrontendQueryValue> NewValues;
					Step->Stream(NewValues);

					const int32 Num = NewValues.Num();
					if (NewValues.Num() > 0)
					{
						FFrontendQueryKey NullKey;
						InOutUpdatedKeys.Add(NullKey);
						FFrontendQueryPartition& NullPartition = InOutResult.FindOrAdd(NullKey);

						for (FFrontendQueryValue& Value : NewValues)
						{
							FFrontendQueryEntry Entry;
							Entry.ID = FGuid::NewGuid();
							Entry.Value = MoveTemp(Value);
							NullPartition.Add(MoveTemp(Entry));
						}
					}
				}
			}

			virtual bool CanProcessRemovals() const override 
			{ 
				// Cannot process removals since streams only add information.
				return false; 
			}

			virtual bool CanProduceEntries() const override 
			{ 
				return true; 
			}
		};

		struct FTransformStepExecuter : TStepExecuterBase<IFrontendQueryTransformStep>
		{
			using TStepExecuterBase<IFrontendQueryTransformStep>::TStepExecuterBase;

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::TransformQueryStep::Execute);
				if (Step.IsValid())
				{
					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
						{
							if (Partition->Num() > 0)
							{
								for (FFrontendQueryEntry& Entry : *Partition)
								{
									Step->Transform(Entry.Value);
								}
							}
						}
					}
				}
			}
		};

		template <typename MapQueryStep>
		struct TMapStepExecuterBase : public TStepExecuterBase<MapQueryStep>
		{
			using TStepExecuterBase<MapQueryStep>::TStepExecuterBase;
			using TStepExecuterBase<MapQueryStep>::Step;

			virtual void ExecuteStep(FFrontendQueryEntry&& Entry, FFrontendQuerySelection& OutResult) const = 0;

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::MapStepQuery::Execute);
				if (Step.IsValid())
				{
					if (InOutUpdatedKeys.Num() > 0)
					{
						FFrontendQuerySelection Result;

						for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
						{
							if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
							{
								// Map all entries associated with the key to a new key.
								if (Partition->Num() > 0)
								{
									for (FFrontendQueryEntry& Entry : *Partition)
									{
										ExecuteStep(MoveTemp(Entry), Result);
									}
								}

								// Remove entries since they have been mapped to a new key
								Partition->Reset();
							}
						}

						// Get the updated set of keys in the output.
						InOutUpdatedKeys.Reset();
						for (const auto& Pair : Result)
						{
							InOutUpdatedKeys.Add(Pair.Key);
						}

						// Append the new values to the final result.
						TStepExecuterBase<MapQueryStep>::Append(InOutUpdatedKeys, Result, InOutResult);
					}
				}
			}
		};

		struct FMapStepExecuter : TMapStepExecuterBase<IFrontendQueryMapStep>
		{
			using TMapStepExecuterBase<IFrontendQueryMapStep>::TMapStepExecuterBase;

			virtual void ExecuteStep(FFrontendQueryEntry&& Entry, FFrontendQuerySelection& OutResult) const override
			{
				FFrontendQueryKey NewKey = Step->Map(Entry);
				OutResult.FindOrAdd(MoveTemp(NewKey)).Add(MoveTemp(Entry));
			}
		};

		struct FMultiMapStepExecuter : TMapStepExecuterBase<IFrontendQueryMultiMapStep>
		{
			using TMapStepExecuterBase<IFrontendQueryMultiMapStep>::TMapStepExecuterBase;

			virtual void ExecuteStep(FFrontendQueryEntry&& Entry, FFrontendQuerySelection& OutResult) const override
			{
				TArray<FFrontendQueryKey> NewKeys = Step->Map(Entry);
				for (FFrontendQueryKey& NewKey : NewKeys)
				{
					OutResult.FindOrAdd(MoveTemp(NewKey)).Add(MoveTemp(Entry));
				}
			}
		};

		struct FReduceStepExecuter : TStepExecuterBase<IFrontendQueryReduceStep>
		{
			using FKey = FFrontendQueryKey;

		public:

			using TStepExecuterBase<IFrontendQueryReduceStep>::TStepExecuterBase;

			virtual void Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::ReduceQueryStep::Merge);
				Remove(InOutIncremental.ActiveRemovalKeys, InOutIncremental.ActiveRemovalSelection, InOutSelection);
				
				if (Step.IsValid())
				{
					for (const FFrontendQueryKey& Key : InOutIncremental.ActiveKeys)
					{
						if (const FFrontendQueryPartition* NewPartition = InOutIncremental.ActiveSelection.Find(Key))
						{
							if (NewPartition->Num() > 0)
							{
								FFrontendQueryPartition& Partition = InOutSelection.FindOrAdd(Key);
								FFrontendQueryPartition OriginalPartition = Partition;
								Partition.Append(*NewPartition);

								Step->Reduce(Key, Partition);

								FFrontendQueryPartition Removed = FrontendQueryPrivate::Difference(OriginalPartition, Partition);

								if (Removed.Num() > 0)
								{
									InOutIncremental.ActiveRemovalSelection.FindOrAdd(Key).Append(Removed);
									InOutIncremental.ActiveRemovalKeys.Add(Key);
								}
							}
						}
					}
				}
			}

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::ReduceQueryStep::Execute);
				if (Step.IsValid())
				{
					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
						{
							if (Partition->Num() > 0)
							{
								Step->Reduce(Key, *Partition);
							}
						}
					}
				}
			}


			virtual bool IsDependentOnPartitionComposition() const override
			{
				// Results may change depending on what is in the partition.
				return true;
			}
		};

		struct FFilterStepExecuter : TStepExecuterBase<IFrontendQueryFilterStep>
		{
			using TStepExecuterBase<IFrontendQueryFilterStep>::TStepExecuterBase;

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FilterQueryStep::Execute);
				if (Step.IsValid())
				{
					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
						{
							for (FFrontendQueryPartition::TIterator Iter = Partition->CreateIterator(); Iter; ++Iter)
							{
								const bool bKeepEntry = Step->Filter(*Iter);
								if (!bKeepEntry)
								{
									Iter.RemoveCurrent();
								}
							}
						}
					}
				}
			}
		};

		struct FScoreStepExecuter : TStepExecuterBase<IFrontendQueryScoreStep>
		{
			using TStepExecuterBase<IFrontendQueryScoreStep>::TStepExecuterBase;

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::ScoreQueryStep::Execute);
				if (Step.IsValid())
				{
					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
						{
							for (FFrontendQueryEntry& Entry : *Partition)
							{
								Entry.Score = Step->Score(Entry);
							}
						}
					}
				}
			}
		};
		
		struct FSortStepExecuter : TStepExecuterBase<IFrontendQuerySortStep>
		{
			using TStepExecuterBase<IFrontendQuerySortStep>::TStepExecuterBase;

			virtual void Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::SortQueryStep::Merge);
				Remove(InOutIncremental.ActiveRemovalKeys, InOutIncremental.ActiveRemovalSelection, InOutSelection);

				if (Step.IsValid())
				{
					auto SortFunc =  [&](const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS)
					{
						return Step->Sort(InLHS, InRHS);
					};

					for (const FFrontendQueryKey& Key : InOutIncremental.ActiveKeys)
					{
						if (const FFrontendQueryPartition* NewPartition = InOutIncremental.ActiveSelection.Find(Key))
						{
							if (FFrontendQueryPartition* OrigPartition = InOutSelection.Find(Key))
							{
								// Need to re-sort if merging two arrays
								OrigPartition->Append(*NewPartition);
								OrigPartition->Sort(SortFunc);
							}
							else
							{
								// New key entries are already sorted. 
								InOutSelection.Add(Key, *NewPartition);
							}
						}
					}
				}
			}

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::SortQueryStep::Execute);
				if (Step.IsValid())
				{
					auto SortFunc = [&](const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS)
					{
						return Step->Sort(InLHS, InRHS);
					};

					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* OrigPartition = InOutResult.Find(Key))
						{
							OrigPartition->Sort(SortFunc);
						}
					}
				}
			}

			virtual bool IsDependentOnPartitionComposition() const 
			{
				// Results may change depending on what is in the partition.
				return true;
			}
		};

		struct FLimitStepExecuter : TStepExecuterBase<IFrontendQueryLimitStep>
		{
			using TStepExecuterBase<IFrontendQueryLimitStep>::TStepExecuterBase;

			virtual void Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::LimitQueryStep::Merge);
				Remove(InOutIncremental.ActiveRemovalKeys, InOutIncremental.ActiveRemovalSelection, InOutSelection);

				if (Step.IsValid())
				{
					TSet<FFrontendQueryKey> UpdatedKeys;

					const int32 Limit = Step->Limit();

					for (const FFrontendQueryKey& Key : InOutIncremental.ActiveKeys)
					{
						if (const FFrontendQueryPartition* NewPartition = InOutIncremental.ActiveSelection.Find(Key))
						{
							FFrontendQueryPartition& ExistingPartition = InOutSelection.FindOrAdd(Key);

							const int32 NumToAdd = FMath::Min(Limit - ExistingPartition.Num(), NewPartition->Num());
							if (NumToAdd > 0)
							{
								UpdatedKeys.Add(Key);
								for (FFrontendQueryPartition::TConstIterator Iter = NewPartition->CreateConstIterator(); Iter; ++Iter)
								{
									ExistingPartition.Add(*Iter);
								}
							}
						}
					}
				}
			}

			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const override
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::LimitQueryStep::Execute);
				if (Step.IsValid())
				{
					const int32 Limit = Step->Limit();

					for (const FFrontendQueryKey& Key : InOutUpdatedKeys)
					{
						if (FFrontendQueryPartition* Partition = InOutResult.Find(Key))
						{
							const int32 Num = Partition->Num();
							if (Num > Limit)
							{
								// Create a new partition and fill with desired number
								// of objects. 
								FFrontendQueryPartition NewPartition;
								FFrontendQueryPartition::TConstIterator Iter = Partition->CreateConstIterator();
								for (int32 i = 0; (i < Limit) && Iter; i++)
								{
									NewPartition.Add(*Iter);
									++Iter;
								}

								// Replace the old partition with a new one.
								InOutResult[Key] = NewPartition;
							}
						}
					}
				}
			}

			virtual bool IsDependentOnPartitionComposition() const 
			{
				// Merge is required because values under limit may be different
				// after merge.
				return true;
			}
		};
	}

	FFrontendQueryKey::FFrontendQueryKey()
	: Key(TInPlaceType<FFrontendQueryKey::FNull>())
	, Hash(INDEX_NONE)
	{}

	FFrontendQueryKey::FFrontendQueryKey(int32 InKey)
	: Key(TInPlaceType<int32>(), InKey)
	, Hash(::GetTypeHash(InKey))
	{
	}

	FFrontendQueryKey::FFrontendQueryKey(const FString& InKey)
	: Key(TInPlaceType<FString>(), InKey)
	, Hash(GetTypeHash(InKey))
	{
	}

	FFrontendQueryKey::FFrontendQueryKey(const FName& InKey)
	: Key(TInPlaceType<FName>(), InKey)
	, Hash(GetTypeHash(InKey))
	{
	}

	bool FFrontendQueryKey::IsNull() const
	{
		return Key.GetIndex() != FKeyType::IndexOfType<FFrontendQueryKey::FNull>();
	}

	bool operator==(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS)
	{
		if (InLHS.Hash == InRHS.Hash)
		{
			if (InLHS.Key.GetIndex() == InRHS.Key.GetIndex())
			{
				switch(InLHS.Key.GetIndex())
				{
					case FFrontendQueryKey::FKeyType::IndexOfType<FFrontendQueryKey::FNull>():
						return true;

					case FFrontendQueryKey::FKeyType::IndexOfType<int32>():
						return InLHS.Key.Get<int32>() == InRHS.Key.Get<int32>();

					case FFrontendQueryKey::FKeyType::IndexOfType<FString>():
						return InLHS.Key.Get<FString>() == InRHS.Key.Get<FString>();

					case FFrontendQueryKey::FKeyType::IndexOfType<FName>():
						return InLHS.Key.Get<FName>() == InRHS.Key.Get<FName>();

					default:
						// Unhandled case type.
						checkNoEntry();
				}
			}
		}

		return false;
	}

	bool operator!=(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS)
	{
		return !(InLHS == InRHS);
	}

	bool operator<(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS)
	{
		if (InLHS.Hash != InRHS.Hash)
		{
			return InLHS.Hash < InRHS.Hash;
		}

		if (InLHS.Key.GetIndex() != InRHS.Key.GetIndex())
		{
			return InLHS.Key.GetIndex() < InRHS.Key.GetIndex();
		}

		switch(InLHS.Key.GetIndex())
		{
			case FFrontendQueryKey::FKeyType::IndexOfType<FFrontendQueryKey::FNull>():
				return false;

			case FFrontendQueryKey::FKeyType::IndexOfType<int32>():
				return InLHS.Key.Get<int32>() < InRHS.Key.Get<int32>();

			case FFrontendQueryKey::FKeyType::IndexOfType<FString>():
				return InLHS.Key.Get<FString>() < InRHS.Key.Get<FString>();

			case FFrontendQueryKey::FKeyType::IndexOfType<FName>():
				return InLHS.Key.Get<FName>().FastLess(InRHS.Key.Get<FName>());

			default:
				// Unhandled case type.
				checkNoEntry();
		}

		return false;
	}

	uint32 GetTypeHash(const FFrontendQueryKey& InKey)
	{
		return InKey.Hash;
	}

	uint32 GetTypeHash(const FFrontendQueryEntry& InEntry)
	{
		return GetTypeHash(InEntry.ID);
	}

	bool operator==(const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS)
	{
		return InLHS.ID == InRHS.ID;
	}

	FFrontendQueryStep::FFrontendQueryStep(FStreamFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FStreamStepExecuter>(MakeUnique<FrontendQueryPrivate::FStreamFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FTransformFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FTransformStepExecuter>(MakeUnique<FrontendQueryPrivate::FTransformFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FMapFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FMapStepExecuter>(MakeUnique<FrontendQueryPrivate::FMapFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FMultiMapFunction&& InFunc)
		: StepExecuter(MakeUnique<FrontendQueryPrivate::FMultiMapStepExecuter>(MakeUnique<FrontendQueryPrivate::FMultiMapFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FReduceFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FReduceStepExecuter>(MakeUnique<FrontendQueryPrivate::FReduceFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FFilterFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FFilterStepExecuter>(MakeUnique<FrontendQueryPrivate::FFilterFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}
	
	FFrontendQueryStep::FFrontendQueryStep(FScoreFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FScoreStepExecuter>(MakeUnique<FrontendQueryPrivate::FScoreFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FSortFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FSortStepExecuter>(MakeUnique<FrontendQueryPrivate::FSortFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(FLimitFunction&& InFunc)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FLimitStepExecuter>(MakeUnique<FrontendQueryPrivate::FLimitFunctionFrontendQueryStep>(MoveTemp(InFunc))))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryStreamStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FStreamStepExecuter>(MoveTemp(InStep)))
	{
	}
	
	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryTransformStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FTransformStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryMapStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FMapStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryMultiMapStep>&& InStep)
		: StepExecuter(MakeUnique<FrontendQueryPrivate::FMultiMapStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryReduceStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FReduceStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryFilterStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FFilterStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryScoreStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FScoreStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQuerySortStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FSortStepExecuter>(MoveTemp(InStep)))
	{
	}

	FFrontendQueryStep::FFrontendQueryStep(TUniquePtr<IFrontendQueryLimitStep>&& InStep)
	:	StepExecuter(MakeUnique<FrontendQueryPrivate::FLimitStepExecuter>(MoveTemp(InStep)))
	{
	}

	void FFrontendQueryStep::Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const
	{
		if (StepExecuter.IsValid())
		{
			StepExecuter->Execute(InOutUpdatedKeys, InOutResult);
		}
	}

	void FFrontendQueryStep::Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const
	{
		if (StepExecuter.IsValid())
		{
			StepExecuter->Merge(InOutIncremental, InOutSelection);
		}
	}

	bool FFrontendQueryStep::IsDependentOnPartitionComposition() const
	{
		if (StepExecuter.IsValid())
		{
			return StepExecuter->IsDependentOnPartitionComposition();
		}

		return false;
	}

	bool FFrontendQueryStep::CanProcessRemovals() const
	{
		if (StepExecuter.IsValid())
		{
			return StepExecuter->CanProcessRemovals();
		}

		return true;
	} 

	bool FFrontendQueryStep::CanProduceEntries() const
	{
		if (StepExecuter.IsValid())
		{
			return StepExecuter->CanProduceEntries();
		}

		return true;
	} 

	FFrontendQuery::FFrontendQuery()
	: Result(MakeShared<FFrontendQuerySelection, ESPMode::ThreadSafe>())
	{
	}

	FFrontendQuery& FFrontendQuery::AddStreamLambdaStep(FStreamFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddTransformLambdaStep(FTransformFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddMapLambdaStep(FMapFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddReduceLambdaStep(FReduceFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddFilterLambdaStep(FFilterFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddScoreLambdaStep(FScoreFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddSortLambdaStep(FSortFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddLimitLambdaStep(FLimitFunction&& InFunc)
	{
		return AddFunctionStep(MoveTemp(InFunc));
	}

	FFrontendQuery& FFrontendQuery::AddStep(TUniquePtr<FFrontendQueryStep>&& InStep)
	{
		if (ensure(InStep.IsValid()))
		{
			FStepInfo StepInfo;
			StepInfo.Step = MoveTemp(InStep);
			StepInfo.bProcessRemovals = StepInfo.Step->CanProcessRemovals();

			// Determine if this step is the last step which can produce entries. 
			if (StepInfo.Step->CanProduceEntries())
			{
				FinalEntryProducingStepIndex = Steps.Num();
			}
			
			// If any prior steps have the ability to remove a previously existing 
			// entry, and this step requires an incremental merge, then we need to 
			// cache the input to this step. If an entry is deleted from the input 
			// to this step, all inputs associated with a given key will be reevaluated.
			//
			// To cache the input to a step, we can equivalently cache the output 
			// of prior step.
			const bool bCacheAncestorStepOutput = Algo::AnyOf(Steps, [](const FStepInfo& Info) { return Info.Step->IsDependentOnPartitionComposition(); });
			if (bCacheAncestorStepOutput && StepInfo.Step->IsDependentOnPartitionComposition())
			{
				Steps.Last().bMergeAndCacheOutput = true;
			}
			else if (StepInfo.Step->IsDependentOnPartitionComposition())
			{
				// If the step is dependent upon the partition composition, and
				// input is not a merged partition, then the incremental must be merged
				// with the prior result for the step so that downstream steps are
				// provided the correct input data.
				StepInfo.bMergeAndCacheOutput = true;
			}

			Steps.Add(MoveTemp(StepInfo));
		}

		return *this;
	}

	const FFrontendQuerySelection& FFrontendQuery::Update()
	{
		TSet<FFrontendQueryKey> UpdatedKeys;
		return Update(UpdatedKeys);
	}

	const FFrontendQuerySelection& FFrontendQuery::Update(TSet<FFrontendQueryKey>& OutUpdatedKeys)
	{
		UpdateInternal(OutUpdatedKeys);

		return *Result;
	}

	const FFrontendQuerySelection& FFrontendQuery::GetSelection() const
	{
		return *Result;
	}

	void FFrontendQuery::MergeInternal(FFrontendQueryStep& Step, FFrontendQuery::FIncremental& InOutIncremental, FFrontendQuerySelection& InOutMergedSelection)
	{
		Step.Merge(InOutIncremental, InOutMergedSelection);
	}

	void FFrontendQuery::UpdateInternal(TSet<FFrontendQueryKey>& OutUpdatedKeys)
	{
		using namespace FrontendQueryPrivate;

		FIncremental Incremental;

		const int32 LastStepIndex = Steps.Num() - 1;

		// Perform incremental update sequentially
		for (int32 StepIndex = 0; StepIndex < Steps.Num(); StepIndex++)
		{
			FStepInfo& StepInfo = Steps[StepIndex];

			StepInfo.Step->Execute(Incremental.ActiveKeys, Incremental.ActiveSelection);

			if (StepInfo.bProcessRemovals)
			{
				StepInfo.Step->Execute(Incremental.ActiveRemovalKeys, Incremental.ActiveRemovalSelection);
			}

			const bool bIsLastProducingStep = StepIndex == FinalEntryProducingStepIndex;
			if (bIsLastProducingStep)
			{
				const bool bIsIncrementalEmpty = (Incremental.ActiveKeys.Num() == 0) && (Incremental.ActiveRemovalKeys.Num() == 0);
				if (bIsIncrementalEmpty)
				{
					// Early out if following steps will not produce any new outputs.
					return;
				}
			}

			// Determine if incremental results need to be merged because it's 
			// the last step, or because the step requires merging.
			const bool bIsLastStep = StepIndex == LastStepIndex;
			if (bIsLastStep)
			{
				StepInfo.Step->Merge(Incremental, *Result);
				CompactSelection(*Result);
				OutUpdatedKeys = Incremental.ActiveKeys.Union(Incremental.ActiveRemovalKeys);
			}
			else if (StepInfo.bMergeAndCacheOutput)
			{
				StepInfo.Step->Merge(Incremental, StepInfo.OutputCache);
				
				// If entries were removed during a merge, the subsequent step 
				// needs to re-evaluate the entire partition. The entire partition 
				// is added to the active set for each removed key.
				AppendPartitions(Incremental.ActiveRemovalKeys, StepInfo.OutputCache, Incremental.ActiveKeys, Incremental.ActiveSelection);

				CompactSelection(StepInfo.OutputCache);
			}
		}
	}

	void FFrontendQuery::AppendPartitions(const TSet<FFrontendQueryKey>& InKeysToAppend, const FFrontendQuerySelection& InSelection, TSet<FFrontendQueryKey>& OutKeysModified, FFrontendQuerySelection& OutSelection) const
	{
		for (const FFrontendQueryKey& Key : InKeysToAppend)
		{
			if (const FFrontendQueryPartition* Partition = InSelection.Find(Key))
			{
				OutKeysModified.Add(Key);
				OutSelection.FindOrAdd(Key) = *Partition;
			}
		}
	}
}

