// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendQuerySteps.h"

#include "Algo/MaxElement.h"
#include "MetasoundAssetManager.h"
#include "MetasoundAssetManagerTransactionPrivate.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeClassRegistryPrivate.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundLog.h"

namespace Metasound
{
	class FNodeClassRegistrationEventsPimpl : public IFrontendQueryStreamStep
	{
	public:
		FNodeClassRegistrationEventsPimpl()
		{
			TransactionStream = Frontend::FNodeClassRegistry::Get().CreateTransactionStream();
		}

		virtual void Stream(TArray<FFrontendQueryValue>& OutValues) override
		{
			using namespace Frontend;

			auto AddEntry = [&OutValues](const FNodeClassRegistryTransaction& InTransaction)
			{
				OutValues.Emplace(TInPlaceType<FNodeClassRegistryTransaction>(), InTransaction);
			};

			if (TransactionStream.IsValid())
			{
				TransactionStream->Stream(AddEntry);
			}
		}

	private:
		TUniquePtr<Frontend::FNodeClassRegistryTransactionStream> TransactionStream;
	};

	class FAssetRegistrationEventsPimpl : public IFrontendQueryStreamStep
	{
	public:
		FAssetRegistrationEventsPimpl()
		{
			using namespace Frontend;

			FMetaSoundAssetManagerBase& Manager = static_cast<FMetaSoundAssetManagerBase&>(IMetaSoundAssetManager::GetChecked());
			FAssetManagerTransactor& Transactor = static_cast<FAssetManagerTransactor&>(Manager.GetTransactor());
			TransactionStream = Transactor.CreateTransactionStream();
		}

		virtual void Stream(TArray<FFrontendQueryValue>& OutValues) override
		{
			using namespace Frontend;

			auto AddEntry = [&OutValues](const FAssetManagerTransaction& InTransaction)
			{
				OutValues.Emplace(TInPlaceType<FAssetManagerTransaction>(), InTransaction);
			};

			if (TransactionStream.IsValid())
			{
				TransactionStream->Stream(AddEntry);
			}
		}

	private:
		TUniquePtr<Frontend::FAssetManagerTransactionStream> TransactionStream;
	};

	FNodeClassRegistrationEvents::FNodeClassRegistrationEvents()
	: Pimpl(MakePimpl<FNodeClassRegistrationEventsPimpl>())
	{
	}

	void FNodeClassRegistrationEvents::Stream(TArray<FFrontendQueryValue>& OutValues)
	{
		Pimpl->Stream(OutValues);
	}

	FAssetRegistrationEvents::FAssetRegistrationEvents()
		: Pimpl(MakePimpl<FAssetRegistrationEventsPimpl>())
	{
	}

	void FAssetRegistrationEvents::Stream(TArray<FFrontendQueryValue>& OutValues)
	{
		Pimpl->Stream(OutValues);
	}

	FFrontendQueryKey FMapAssetRegistrationEventsToNodeRegistryKeys::Map(const FFrontendQueryEntry& InEntry) const 
	{
		using namespace Frontend;

		FNodeRegistryKey RegistryKey;

		if (ensure(InEntry.Value.IsType<FAssetManagerTransaction>()))
		{
			RegistryKey = InEntry.Value.Get<FAssetManagerTransaction>().GetNodeRegistryKey();
		}

		return FFrontendQueryKey(RegistryKey.ToString());
	}

	FFrontendQueryKey FMapClassRegistrationEventsToNodeRegistryKeys::Map(const FFrontendQueryEntry& InEntry) const
	{
		using namespace Frontend;

		FNodeRegistryKey RegistryKey;

		if (ensure(InEntry.Value.IsType<FNodeClassRegistryTransaction>()))
		{
			RegistryKey = InEntry.Value.Get<FNodeClassRegistryTransaction>().GetNodeRegistryKey();
		}

		return FFrontendQueryKey(RegistryKey.ToString());
	}

	void FReduceAssetRegistrationEventsToCurrentStatus::Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const
	{
		using namespace Frontend;

		// Track number of times each type of entry has been seen
		struct FTransactionStats
		{
			int32 Num = 0;
			const FFrontendQueryEntry* MostRecent = nullptr;

			void Update(const FFrontendQueryEntry& InEntry)
			{
				Num++;
				if (nullptr == MostRecent)
				{
					MostRecent = &InEntry;
				}
				else
				{
					if (FReduceAssetRegistrationEventsToCurrentStatus::GetTransactionTimestamp(InEntry) > FReduceAssetRegistrationEventsToCurrentStatus::GetTransactionTimestamp(*MostRecent))
					{
						MostRecent = &InEntry;
					}
				}
			}
		};

		// Accumulate the counts of each transaction type
		FTransactionStats NodeRegistrationStats;
		FTransactionStats NodeUnregistrationStats;

		for (const FFrontendQueryEntry& Entry : InOutEntries)
		{
			const FAssetManagerTransaction& Transaction = Entry.Value.Get<FAssetManagerTransaction>();
			switch (Transaction.GetTransactionType())
			{
			case FAssetManagerTransaction::ETransactionType::AssetRegistration:
				NodeRegistrationStats.Update(Entry);
				break;

			case FAssetManagerTransaction::ETransactionType::AssetUnregistration:
				NodeUnregistrationStats.Update(Entry);
				break;

			default:
			{
				checkNoEntry();
			}
			}
		}

		// Use the final number of each transaction type to determine the final state
		FFrontendQueryPartition RemainingEntries;
		if (NodeRegistrationStats.Num > NodeUnregistrationStats.Num)
		{
			// Registration trumps migration, so we do not consider migration at all here.
			RemainingEntries.Add(*NodeRegistrationStats.MostRecent);
		}
		else
		{
			// Make sure to propagate unregistration actions since reduce steps may get called
			// on subpartitions. 
			if (NodeUnregistrationStats.Num > NodeRegistrationStats.Num)
			{
				RemainingEntries.Add(*NodeUnregistrationStats.MostRecent);
			}
		}

		InOutEntries = MoveTemp(RemainingEntries);
	}

	FReduceAssetRegistrationEventsToCurrentStatus::FTimeType FReduceAssetRegistrationEventsToCurrentStatus::GetTransactionTimestamp(const FFrontendQueryEntry& InEntry)
	{
		using namespace Frontend;

		if (ensure(InEntry.Value.IsType<FAssetManagerTransaction>()))
		{
			return InEntry.Value.Get<FAssetManagerTransaction>().GetTimestamp();
		}
		return 0;
	}

	bool FReduceAssetRegistrationEventsToCurrentStatus::IsValidTransactionOfType(Frontend::FAssetManagerTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry)
	{
		using namespace Frontend;

		if (nullptr != InEntry)
		{
			if (InEntry->Value.IsType<FAssetManagerTransaction>())
			{
				return InEntry->Value.Get<FAssetManagerTransaction>().GetTransactionType() == InType;
			}
		}
		return false;
	}

	void FReduceRegistrationEventsToCurrentStatus::Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const
	{
		using namespace Frontend;

		// Track number of times each type of entry has been seen
		struct FTransactionStats
		{
			int32 Num = 0;
			const FFrontendQueryEntry* MostRecent = nullptr;

			void Update(const FFrontendQueryEntry& InEntry)
			{
				Num++;
				if (nullptr == MostRecent)
				{
					MostRecent = &InEntry;
				}
				else
				{
					if (FReduceRegistrationEventsToCurrentStatus::GetTransactionTimestamp(InEntry) > FReduceRegistrationEventsToCurrentStatus::GetTransactionTimestamp(*MostRecent))
					{
						MostRecent = &InEntry;
					}
				}
			}
		};

		// Accumulate the counts of each transaction type
		FTransactionStats NodeRegistrationStats;
		FTransactionStats NodeUnregistrationStats;
		FTransactionStats MigrationRegistrationStats;
		FTransactionStats MigrationUnregistrationStats;

		for (const FFrontendQueryEntry& Entry : InOutEntries)
		{
			const FNodeClassRegistryTransaction& Transaction = Entry.Value.Get<FNodeClassRegistryTransaction>();
			switch (Transaction.GetTransactionType())
			{
				case FNodeClassRegistryTransaction::ETransactionType::NodeRegistration:
					NodeRegistrationStats.Update(Entry);
					break;

				case FNodeClassRegistryTransaction::ETransactionType::NodeUnregistration:
					NodeUnregistrationStats.Update(Entry);
					break;

				case FNodeClassRegistryTransaction::ETransactionType::NodeMigrationRegistration:
					MigrationRegistrationStats.Update(Entry);
					break;

				case FNodeClassRegistryTransaction::ETransactionType::NodeMigrationUnregistration:
					MigrationUnregistrationStats.Update(Entry);
					break;

				default:
					{
						checkNoEntry();
					}
			}
		}

		// Use the final number of each transaction type to determine the final state
		FFrontendQueryPartition RemainingEntries;
		if (NodeRegistrationStats.Num > NodeUnregistrationStats.Num)
		{
			// Registration trumps migration, so we do not consider migration at all here.
			RemainingEntries.Add(*NodeRegistrationStats.MostRecent);
		}
		else 
		{
			// Make sure to propagate unregistration actions since reduce steps may get called
			// on subpartitions. 
			if (NodeUnregistrationStats.Num > NodeRegistrationStats.Num)
			{
				RemainingEntries.Add(*NodeUnregistrationStats.MostRecent);
			}
			
			// If  not registered, check if migrated
			if (MigrationRegistrationStats.Num > MigrationUnregistrationStats.Num)
			{
				RemainingEntries.Add(*MigrationRegistrationStats.MostRecent);
			}
			else if (MigrationUnregistrationStats.Num > MigrationRegistrationStats.Num)
			{
				RemainingEntries.Add(*MigrationUnregistrationStats.MostRecent);
			}
		}

		InOutEntries = MoveTemp(RemainingEntries);
	}

	FReduceRegistrationEventsToCurrentStatus::FTimeType FReduceRegistrationEventsToCurrentStatus::GetTransactionTimestamp(const FFrontendQueryEntry& InEntry)
	{
		using namespace Frontend;

		if (ensure(InEntry.Value.IsType<FNodeClassRegistryTransaction>()))
		{
			return InEntry.Value.Get<FNodeClassRegistryTransaction>().GetTimestamp();
		}
		return 0;
	}

	bool FReduceRegistrationEventsToCurrentStatus::IsValidTransactionOfType(Frontend::FNodeClassRegistryTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry)
	{
		using namespace Frontend;

		if (nullptr != InEntry)
		{
			if (InEntry->Value.IsType<FNodeClassRegistryTransaction>())
			{
				return InEntry->Value.Get<FNodeClassRegistryTransaction>().GetTransactionType() == InType;
			}
		}
		return false;
	}

	void FTransformAssetRegistrationEventsToClassInfo::Transform(FFrontendQueryEntry::FValue& InValue) const
	{
		using namespace Frontend;

		FMetaSoundClassInfo ClassInfo(EMetasoundFrontendClassType::Invalid);

		if (ensure(InValue.IsType<FAssetManagerTransaction>()))
		{
			const FAssetManagerTransaction& Transaction = InValue.Get<FAssetManagerTransaction>();

			if (Transaction.GetTransactionType() == Frontend::FAssetManagerTransaction::ETransactionType::AssetRegistration)
			{
				// It's possible that the node is no longer registered (we're processing removals) 
				// but that's okay because the returned default FrontendClass will be processed out later
				TArray<FMetaSoundClassInfo>ClassInfos = IMetaSoundAssetManager::GetChecked()
					.FindRegisteredClassInfo(FMetaSoundAssetKey(Transaction.GetNodeRegistryKey()));
				if(!ClassInfos.IsEmpty())
				{
					ClassInfo = ClassInfos.Last();
				}
			}
		}
		InValue.Set<FMetaSoundClassInfo>(MoveTemp(ClassInfo));
	}

	void FTransformNodeClassRegistrationEventsToClassInfo::Transform(FFrontendQueryEntry::FValue& InValue) const
	{
		using namespace Frontend;

		FMetaSoundClassInfo ClassInfo(EMetasoundFrontendClassType::Invalid);

		if (ensure(InValue.IsType<FNodeClassRegistryTransaction>()))
		{
			const FNodeClassRegistryTransaction& Transaction = InValue.Get<FNodeClassRegistryTransaction>();

			if (Transaction.GetTransactionType() == Frontend::FNodeClassRegistryTransaction::ETransactionType::NodeRegistration)
			{
				// It's possible that the node is no longer registered (we're processing removals) 
				// but that's okay because the returned default FrontendClass will be processed out later
				FMetasoundFrontendClass Class;
				FMetasoundFrontendRegistryContainer::Get()->FindFrontendClassFromRegistered(Transaction.GetNodeRegistryKey(), Class);
				ClassInfo = FMetaSoundClassInfo(Class);
			}
		}
		InValue.Set<FMetaSoundClassInfo>(MoveTemp(ClassInfo));
	}

	FFilterClassesByInputVertexDataType::FFilterClassesByInputVertexDataType(const FName& InTypeName)
	:	InputVertexTypeName(InTypeName)
	{
	}

	bool FFilterClassesByInputVertexDataType::Filter(const FFrontendQueryEntry& InEntry) const
	{
#if WITH_EDITORONLY_DATA
		using namespace Frontend;
		check(InEntry.Value.IsType<FMetaSoundClassInfo>());

		return InEntry.Value.Get<FMetaSoundClassInfo>().InterfaceInfo.Inputs.ContainsByPredicate(
			[this](const FMetaSoundClassVertexInfo& VertexInfo)
			{
				return VertexInfo.TypeName == InputVertexTypeName;
			}
		);
#else
		return false;
#endif // WITH_EDITORONLY_DATA
	}

	FFilterClassesByOutputVertexDataType::FFilterClassesByOutputVertexDataType(const FName& InTypeName)
	:	OutputVertexTypeName(InTypeName)
	{
	}

	bool FFilterClassesByOutputVertexDataType::Filter(const FFrontendQueryEntry& InEntry) const
	{
#if WITH_EDITORONLY_DATA
		using namespace Frontend;
		check(InEntry.Value.IsType<FMetaSoundClassInfo>());

		return InEntry.Value.Get<FMetaSoundClassInfo>().InterfaceInfo.Outputs.ContainsByPredicate(
			[this](const FMetaSoundClassVertexInfo& VertexInfo)
			{
				return VertexInfo.TypeName == OutputVertexTypeName;
			}
		);
#else
		return false;
#endif // WITH_EDITORONLY_DATA
	}

	FFrontendQueryKey FMapClassesToClassName::Map(const FFrontendQueryEntry& InEntry) const 
	{
		using namespace Frontend;

		return FFrontendQueryKey(InEntry.Value.Get<FMetaSoundClassInfo>().ClassName.GetFullName());
	}

	FFilterClassesByClassID::FFilterClassesByClassID(const FGuid InClassID)
		: ClassID(InClassID)
	{
	}

	bool FFilterClassesByClassID::Filter(const FFrontendQueryEntry& InEntry) const
	{
		return false;
	}

	FFrontendQueryKey FMapToFullClassName::Map(const FFrontendQueryEntry& InEntry) const
	{
		using namespace Frontend;

		const FMetaSoundClassInfo& ClassInfo = InEntry.Value.Get<FMetaSoundClassInfo>();
		return FFrontendQueryKey(ClassInfo.ClassName.GetFullName());
	}

	void FReduceClassesToHighestVersion::Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const
	{
		using namespace Frontend;

		FFrontendQueryEntry* HighestVersionEntry = nullptr;
		FMetasoundFrontendVersionNumber HighestVersion;

		for (FFrontendQueryEntry& Entry : InOutEntries)
		{
			const FMetasoundFrontendVersionNumber& Version = Entry.Value.Get<FMetaSoundClassInfo>().Version;

			if (!HighestVersionEntry || HighestVersion < Version)
			{
				HighestVersionEntry = &Entry;
				HighestVersion = Version;
			}
		}

		if (HighestVersionEntry)
		{
			FFrontendQueryEntry Entry = *HighestVersionEntry;
			InOutEntries.Reset();
			InOutEntries.Add(Entry);
		}
	}

	bool FSortClassesByVersion::Sort(const FFrontendQueryEntry& InEntryLHS, const FFrontendQueryEntry& InEntryRHS) const
	{
		const FMetasoundFrontendVersionNumber& VersionLHS = InEntryLHS.Value.Get<Frontend::FMetaSoundClassInfo>().Version;
		const FMetasoundFrontendVersionNumber& VersionRHS = InEntryRHS.Value.Get<Frontend::FMetaSoundClassInfo>().Version;
		return VersionLHS > VersionRHS;
	}
}
