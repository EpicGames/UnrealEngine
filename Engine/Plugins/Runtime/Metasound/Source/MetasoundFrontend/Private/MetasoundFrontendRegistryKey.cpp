// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundAssetManager.h"

namespace Metasound::Frontend
{
	namespace NodeClassInfoPrivate
	{
		auto GetVertexTypeName = [](const FMetasoundFrontendVertex& Vertex) { return Vertex.TypeName; };
	}

	FNodeClassInfo::FNodeClassInfo()
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: AssetClassID()
		, AssetPath()
#if WITH_EDITORONLY_DATA
		, InputTypes()
		, OutputTypes()
		, bIsPreset(false)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo::FNodeClassInfo(const FMetasoundFrontendClassMetadata& InMetadata)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: ClassName(InMetadata.GetClassName())
		, Type(InMetadata.GetType())
		, AssetClassID(FGuid(InMetadata.GetClassName().Name.ToString()))
		, AssetPath()
		, Version(InMetadata.GetVersion())
#if WITH_EDITORONLY_DATA
		, InputTypes()
		, OutputTypes()
		, bIsPreset(false)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo::FNodeClassInfo(const FMetasoundFrontendGraphClass& InClass)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: ClassName(InClass.Metadata.GetClassName())
		, Type(EMetasoundFrontendClassType::External) // Overridden as it is considered the same as an external class in registries
		, AssetClassID(FGuid(ClassName.Name.ToString()))
		, AssetPath()
		, Version(InClass.Metadata.GetVersion())
#if WITH_EDITORONLY_DATA
		, InputTypes()
		, OutputTypes()
		, bIsPreset(false)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo::FNodeClassInfo(const FMetasoundFrontendGraphClass& InClass, const FTopLevelAssetPath& InAssetPath)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: ClassName(InClass.Metadata.GetClassName())
		, Type(EMetasoundFrontendClassType::External) // Overridden as it is considered the same as an external class in registries
		, AssetClassID(FGuid(ClassName.Name.ToString()))
		, AssetPath(InAssetPath)
		, Version(InClass.Metadata.GetVersion())
#if WITH_EDITORONLY_DATA
		, InputTypes()
		, OutputTypes()
		, bIsPreset(false)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo::FNodeClassInfo(const FNodeClassInfo& Other)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: ClassName(Other.ClassName)
		, Type(Other.Type)
		, AssetClassID(Other.AssetClassID)
		, AssetPath(Other.AssetPath)
		, Version(Other.Version)
#if WITH_EDITORONLY_DATA
		, InputTypes(Other.InputTypes)
		, OutputTypes(Other.OutputTypes)
		, bIsPreset(Other.bIsPreset)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo::FNodeClassInfo(FNodeClassInfo&& Other)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: ClassName(MoveTemp(Other.ClassName))
		, Type(Other.Type)
		, AssetClassID(MoveTemp(Other.AssetClassID))
		, AssetPath(MoveTemp(Other.AssetPath))
		, Version(MoveTemp(Other.Version))
#if WITH_EDITORONLY_DATA
		, InputTypes(MoveTemp(Other.InputTypes))
		, OutputTypes(MoveTemp(Other.OutputTypes))
		, bIsPreset(Other.bIsPreset)
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	FNodeClassInfo& FNodeClassInfo::operator=(const FNodeClassInfo& Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ClassName = Other.ClassName;
		Type = Other.Type;
		AssetClassID = Other.AssetClassID;
		AssetPath = Other.AssetPath;
		Version = Other.Version;

#if WITH_EDITORONLY_DATA
		InputTypes = Other.InputTypes;
		OutputTypes = Other.OutputTypes;
		bIsPreset = Other.bIsPreset;
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return *this;
	}

	FNodeClassInfo& FNodeClassInfo::operator=(FNodeClassInfo&& Other)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ClassName = MoveTemp(Other.ClassName);
		Type = Other.Type;
		AssetClassID = MoveTemp(Other.AssetClassID);
		AssetPath = MoveTemp(Other.AssetPath);
		Version = MoveTemp(Other.Version);
#if WITH_EDITORONLY_DATA
		InputTypes = MoveTemp(Other.InputTypes);
		OutputTypes = MoveTemp(Other.OutputTypes);
		bIsPreset = Other.bIsPreset;
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return *this;
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(EMetasoundFrontendClassType InType, const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, int32 InMinorVersion)
		: Type(InType)
		, ClassName(InClassName)
		, Version({ InMajorVersion, InMinorVersion })
	{
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(EMetasoundFrontendClassType InType, const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InVersion)
		: Type(InType)
		, ClassName(InClassName)
		, Version(InVersion)
	{
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(const FNodeClassMetadata& InNodeMetadata)
		: Type(EMetasoundFrontendClassType::External) // Overridden as it is considered the same as an external class in registries
		, ClassName(InNodeMetadata.ClassName)
		, Version({ InNodeMetadata.MajorVersion, InNodeMetadata.MinorVersion })
	{
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(const FMetasoundFrontendClassMetadata& InNodeMetadata)
		: Type(InNodeMetadata.GetType())
		, ClassName(InNodeMetadata.GetClassName())
		, Version(InNodeMetadata.GetVersion())
	{
		checkf(InNodeMetadata.GetType() != EMetasoundFrontendClassType::Graph, TEXT("Cannot create key from 'graph' type. Likely meant to use FNodeClassRegistryKey ctor that is provided FMetasoundFrontendGraphClass"));
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(const FMetasoundFrontendGraphClass& InGraphClass)
		: Type(EMetasoundFrontendClassType::External) // Type overridden as all graphs are considered the same as an external class in the registry
		, ClassName(InGraphClass.Metadata.GetClassName())
		, Version(InGraphClass.Metadata.GetVersion())
	{
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(const FNodeClassInfo& InClassInfo)
		: Type(InClassInfo.Type)
		, ClassName(InClassInfo.ClassName)
		, Version(InClassInfo.Version)
	{
		checkf(InClassInfo.Type != EMetasoundFrontendClassType::Graph, TEXT("Cannot create key from 'graph' type. Likely meant to use FNodeClassRegistryKey ctor that is provided FMetasoundFrontendGraphClass"));
	}

	FNodeClassRegistryKey::FNodeClassRegistryKey(const FMetaSoundAssetKey& AssetKey)
		: Type(EMetasoundFrontendClassType::External)
		, ClassName(AssetKey.ClassName)
		, Version(AssetKey.Version)
	{
	}

	const FNodeClassRegistryKey& FNodeClassRegistryKey::GetInvalid()
	{
		static const FNodeClassRegistryKey InvalidKey;
		return InvalidKey;
	}

	bool FNodeClassRegistryKey::IsValid() const
	{
		return Type != EMetasoundFrontendClassType::Invalid && ClassName.IsValid() && Version.IsValid();
	}

	void FNodeClassRegistryKey::Reset()
	{
		Type = EMetasoundFrontendClassType::Invalid;
		ClassName = { };
		Version = { };
	}

	FString FNodeClassRegistryKey::ToString() const
	{
		TStringBuilder<128> KeyStringBuilder;
		KeyStringBuilder.Append(LexToString(Type));
		KeyStringBuilder.AppendChar('_');
		KeyStringBuilder.Append(ClassName.ToString());
		KeyStringBuilder.AppendChar('_');
		KeyStringBuilder.Append(FString::FromInt(Version.Major));
		KeyStringBuilder.AppendChar('.');
		KeyStringBuilder.Append(FString::FromInt(Version.Minor));
		return KeyStringBuilder.ToString();
	}

	FString FNodeClassRegistryKey::ToString(const FString& InScopeHeader) const
	{
		checkf(InScopeHeader.Len() < 128, TEXT("Scope text is limited to 128 characters"));

		TStringBuilder<256> Builder; // 128 for key and 128 for scope text

		Builder.Append(InScopeHeader);
		Builder.Append(TEXT(" ["));
		Builder.Append(ToString());
		Builder.Append(TEXT(" ]"));
		return Builder.ToString();
	}

	bool FNodeClassRegistryKey::Parse(const FString& InKeyString, FNodeClassRegistryKey& OutKey)
	{
		TArray<FString> Tokens;
		InKeyString.ParseIntoArray(Tokens, TEXT("_"));
		if (Tokens.Num() == 3)
		{
			EMetasoundFrontendClassType Type;
			if (Metasound::Frontend::StringToClassType(Tokens[0], Type))
			{
				FMetasoundFrontendClassName ClassName;
				if (FMetasoundFrontendClassName::Parse(Tokens[1], ClassName))
				{
					FMetasoundFrontendVersionNumber Version;
					FString MajorVersionString;
					FString MinorVersionString;
					if (Tokens[2].Split(TEXT("."), &MajorVersionString, &MinorVersionString))
					{
						Version.Major = FCString::Atoi(*MajorVersionString);
						Version.Minor = FCString::Atoi(*MinorVersionString);

						OutKey = FNodeClassRegistryKey(Type, ClassName, Version.Major, Version.Minor);
						return true;
					}
				}
			}
		}

		return false;
	}

	FString FGraphClassRegistryKey::ToString() const
	{
		TStringBuilder<256> Builder;
		Builder.Append(NodeKey.ToString());
		Builder.Append(TEXT(", "));
		Builder.Append(AssetPath.GetPackageName().ToString());
		Builder.Append(TEXT("/"));
		Builder.Append(AssetPath.GetAssetName().ToString());
		Builder.Append(TEXT(", "));
		Builder.Append(::LexToString(ObjectID));

		return Builder.ToString();
	}

	FString FGraphClassRegistryKey::ToString(const FString& InScopeHeader) const
	{
		TStringBuilder<512> Builder;
		Builder.Append(InScopeHeader);
		Builder.Append(TEXT(" ["));
		Builder.Append(ToString());
		Builder.Append(TEXT(" ]"));
		return Builder.ToString();
	}

	bool FGraphClassRegistryKey::IsValid() const
	{
		return NodeKey.IsValid() && AssetPath.IsValid() && (ObjectID != (uint32)INDEX_NONE);
	}
} // namespace Metasound::Frontend
