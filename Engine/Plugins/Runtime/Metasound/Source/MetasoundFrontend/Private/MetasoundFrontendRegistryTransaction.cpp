// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundFrontendRegistryTransaction.h"

namespace Metasound
{
	namespace Frontend
	{
		int32 MetaSoundFrontendDiscardStreamedRegistryTransactionsCVar = 1;

		FAutoConsoleVariableRef CVarMetaSoundFrontendDiscardStreamedRegistryTransactions(
			TEXT("au.MetaSound.Frontend.DiscardStreamedRegistryTransactions"),
			MetaSoundFrontendDiscardStreamedRegistryTransactionsCVar,
			TEXT("If enabled, MetaSound registry transactions are discarded after they have been streamed.\n")
			TEXT("0: Disabled, !0: Enabled (default)"),
			ECVF_Default);
	}
}

