// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/ForEach.h"
#include "Algo/MinElement.h"
#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "MetasoundLog.h"
#include "Misc/ScopeLock.h"


namespace Metasound
{
	namespace Frontend
	{
		extern int32 MetaSoundFrontendDiscardStreamedRegistryTransactionsCVar;
		extern FAutoConsoleVariableRef CVarMetaSoundFrontendDiscardStreamedRegistryTransactions;

		using FTransactionReaderHandle = uint32;

		/** Maintains a limited history of TransactionTypes. Calls are threadsafe (excluding
		 * the constructor and destructor.)
		 */
		template<typename TransactionType>
		class TTransactionBuffer
		{

		public:
			TTransactionBuffer()
			: NumRemovedTransactions(0)
			, CurrentReaderHandle(0)
			{
			}

			/** Add a transaction to the history. Threadsafe. 
			 *
			 * @return The transaction ID associated with the action. */
			int32 AddTransaction(TransactionType&& InRegistryTransaction)
			{
				FScopeLock Lock(&RegistryTransactionMutex);
				{
					TransactionHistory.Add(MoveTemp(InRegistryTransaction));
					return TransactionHistory.Num() + NumRemovedTransactions;
				}
			}

			/** Create a new transaction reader for this buffer. */
			FTransactionReaderHandle CreateReader()
			{
				FScopeLock Lock(&RegistryTransactionMutex);
				{
					FTransactionReaderHandle NewReaderHandle = CurrentReaderHandle;
					CurrentReaderHandle++;
					ReaderPositions.Add(NewReaderHandle, NumRemovedTransactions);
					return NewReaderHandle;
				}
			}

			/** Create a duplicate reader which has the same read position as
			 * the provided reader handle.  */
			FTransactionReaderHandle DuplicateReader(FTransactionReaderHandle InReaderHandle)
			{
				FScopeLock Lock(&RegistryTransactionMutex);
				{
					int32 Position = NumRemovedTransactions;
					if (int32* ExistingReaderPosition = ReaderPositions.Find(InReaderHandle))
					{
						Position = *ExistingReaderPosition;
					}

					FTransactionReaderHandle NewReaderHandle = CurrentReaderHandle;
					CurrentReaderHandle++;
					ReaderPositions.Add(NewReaderHandle, Position);
					return NewReaderHandle;
				}
			}

			/** Destroy a reader associated with this buffer. */
			void DestroyReader(FTransactionReaderHandle ReaderHandle)
			{
				FScopeLock Lock(&RegistryTransactionMutex);
				{
					ReaderPositions.Remove(ReaderHandle);
					RemoveTransactionsStreamedToAllReaders();
				}
			}


			/** Invoke a function on all unread transactions.
			 *
			 * @param InReaderHandle - Handle of a valid reader for this buffer.
			 * @param InCallable - A callable of the form Callable(const TransactionType&)
			 */
			template<typename CallableType>
			void StreamTransactions(FTransactionReaderHandle InReaderHandle, CallableType InCallable)
			{
				FScopeLock Lock(&RegistryTransactionMutex);
				{
					if (ensure(ReaderPositions.Contains(InReaderHandle)))
					{
						int32 Start = ReaderPositions[InReaderHandle] - NumRemovedTransactions;
						if (Start < 0)
						{
							UE_LOGF(LogMetaSound, Warning, "Missed transactions in transaction stream. MetaSound registries may be incorrect");
							Start = 0;
						}

						const int32 OutNum = TransactionHistory.Num() - Start;
						if (OutNum > 0)
						{
							TArrayView<const TransactionType> TransactionView = MakeArrayView(&TransactionHistory[Start], OutNum);
							Algo::ForEach(TransactionView, InCallable);
							ReaderPositions[InReaderHandle] += OutNum;

							RemoveTransactionsStreamedToAllReaders();
						}
					}
				}
			}

		private:

			// Remove transactions that have been streamed by all valid readers.
			void RemoveTransactionsStreamedToAllReaders()
			{
				if (0 != MetaSoundFrontendDiscardStreamedRegistryTransactionsCVar)
				{
					const auto* MinElement = Algo::MinElementBy(ReaderPositions, [](const auto& Pair){return Pair.Value;});
					if (nullptr != MinElement)
					{
						int32 NumToRemove = (MinElement->Value) - NumRemovedTransactions;
						if (NumToRemove > 0)
						{
							TransactionHistory.RemoveAt(0, NumToRemove);
							NumRemovedTransactions += NumToRemove;
						}
					}
					else if (ReaderPositions.Num() == 0)
					{
						NumRemovedTransactions += TransactionHistory.Num();
						TransactionHistory.Reset();
					}
				}
			}

			FCriticalSection RegistryTransactionMutex;

			int32 NumRemovedTransactions = 0;
			FTransactionReaderHandle CurrentReaderHandle = 0;
			TSortedMap<FTransactionReaderHandle, int32> ReaderPositions;
			TArray<TransactionType> TransactionHistory;
		};


		/** A transaction stream which streams data from a transaction buffer. */
		template<typename TransactionType>
		class TTransactionStream
		{
			using TransactionBufferType = TTransactionBuffer<TransactionType>;
		public:
			TTransactionStream(TSharedRef<TransactionBufferType, ESPMode::ThreadSafe> InBuffer)
			: Buffer(InBuffer)
			{
				ReaderHandle = Buffer->CreateReader();
			}

			TTransactionStream(const TTransactionStream& InOther)
			: Buffer(InOther.Buffer)
			{
				ReaderHandle = Buffer->DuplicateReader(InOther.ReaderHandle);
			}

			TTransactionStream& operator=(const TTransactionStream& InOther)
			{
				Buffer.RemoveReader(ReaderHandle);
				Buffer = InOther.Buffer;
				ReaderHandle = Buffer.DuplicateReader(InOther.ReaderHandle);
				return *this;
			}

			TTransactionStream(TTransactionStream&& InOther)
			: Buffer(MoveTemp(InOther.Buffer))
			, ReaderHandle(InOther.ReaderHandle)
			{
				InOther.ReaderHandle = -1;
			}

			TTransactionStream& operator=(TTransactionStream&& InOther)
			{
				Buffer = MoveTemp(InOther.Buffer);
				ReaderHandle = InOther.ReaderHandle;
				InOther.ReaderHandle = -1;
				return *this;
			}

			~TTransactionStream()
			{
				Buffer->DestroyReader(ReaderHandle);
			}

			/** Invoke a function on all unstreamed transactions.
			 *
			 * @param InCallable - A callable of the form Callable(const TransactionType&)
			 */
			template <typename CallableType>
			void Stream(CallableType Callable)
			{
				Buffer->StreamTransactions(ReaderHandle, Callable);
			}

		private:
			TSharedRef<TransactionBufferType, ESPMode::ThreadSafe> Buffer;
			FTransactionReaderHandle ReaderHandle = -1;
		};
	}
}


