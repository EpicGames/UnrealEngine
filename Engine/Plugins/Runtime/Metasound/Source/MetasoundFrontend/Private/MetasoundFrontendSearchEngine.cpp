// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendSearchEngineCore.h"
#include "MetasoundFrontendSearchEngineEditorOnly.h"

#include "Algo/MaxElement.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundTrace.h"
#include "MetasoundAssetManager.h"

namespace Metasound::Frontend
{
	bool ISearchEngine::FindClassWithHighestMinorVersion(const FMetasoundFrontendClassName& InName, int32 InMajorVersion, FMetasoundFrontendClass& OutClass)
	{
		FMetaSoundClassInfo ClassInfo;
		const bool bFound = FindRegisteredClass(InName, InMajorVersion, ClassInfo);
		if (bFound)
		{
			const FNodeClassRegistryKey Key(ClassInfo.ClassType, ClassInfo.ClassName, ClassInfo.Version);
			return INodeClassRegistry::Get()->FindFrontendClassFromRegistered(Key, OutClass);
		}

		return false;
	}

#if WITH_EDITORONLY_DATA
	TArray<FMetasoundFrontendClass> ISearchEngine::FindAllClasses(bool bInIncludeAllVersions)
	{
		constexpr bool bIncludeUnloadedAssets = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArray<FMetaSoundClassInfo> AllClasses = FindAllClasses(bInIncludeAllVersions
			? ISearchEngine::EResultVersion::All
			: ISearchEngine::EResultVersion::Highest,
			bIncludeUnloadedAssets);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TArray<FMetasoundFrontendClass> ToReturn;
		for (const FMetaSoundClassInfo& ClassInfo : AllClasses)
		{
			FMetasoundFrontendClass Class;
			const FNodeClassRegistryKey Key(ClassInfo.ClassType, ClassInfo.ClassName, ClassInfo.Version);
			if (INodeClassRegistry::Get()->FindFrontendClassFromRegistered(Key, Class))
			{
				ToReturn.Add(MoveTemp(Class));
			}
		}

		return ToReturn;
	}

	bool ISearchEngine::FindClassWithHighestVersion(const FMetasoundFrontendClassName& InName, FMetasoundFrontendClass& OutClass)
	{
		FMetaSoundClassInfo ClassInfo;
		const bool bFound = FindClassWithHighestVersion(InName, ClassInfo);
		if (bFound)
		{
			const FNodeClassRegistryKey Key(ClassInfo.ClassType, ClassInfo.ClassName, ClassInfo.Version);
			return INodeClassRegistry::Get()->FindFrontendClassFromRegistered(Key, OutClass);
		}

		return false;
	}
#endif // WITH_EDITORONLY_DATA

	ISearchEngine& ISearchEngine::Get()
	{
#if WITH_EDITORONLY_DATA
		static FSearchEngineEditorOnly SearchEngine;
#else
		static FSearchEngineCore SearchEngine;
#endif
		return SearchEngine;
	}
} // namespace Metasound::Frontend
