// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendSearchEngineCore.h"

#include "Algo/MaxElement.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistryPrivate.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundTrace.h"

namespace Metasound::Frontend
{
	namespace SearchEngineQuerySteps
	{
		FFrontendQueryKey CreateKeyFromFullClassNameAndMajorVersion(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion)
		{
			return FFrontendQueryKey(FString::Format(TEXT("{0}_v{1}"), { InClassName.ToString(), InMajorVersion }));
		}

		FFrontendQueryKey CreateKeyFromUClassPathName(const FTopLevelAssetPath& InClassPath)
		{
			return FFrontendQueryKey(InClassPath.ToString());
		}

		FFrontendQueryKey FMapClassToFullClassNameAndMajorVersion::Map(const FFrontendQueryEntry& InEntry) const
		{
			if (ensure(InEntry.Value.IsType<FMetaSoundClassInfo>()))
			{
				const FMetaSoundClassInfo& ClassInfo = InEntry.Value.Get<FMetaSoundClassInfo>();
				return CreateKeyFromFullClassNameAndMajorVersion(ClassInfo.ClassName, ClassInfo.Version.Major);
			}
			return FFrontendQueryKey();
		}

		FInterfaceRegistryTransactionSource::FInterfaceRegistryTransactionSource()
		: TransactionStream(FInterfaceRegistry::Get().CreateTransactionStream())
		{
		}

		void FInterfaceRegistryTransactionSource::Stream(TArray<FFrontendQueryValue>& OutEntries)
		{
			auto AddValue = [&OutEntries](const FInterfaceRegistryTransaction& InTransaction)
			{
				OutEntries.Emplace(TInPlaceType<FInterfaceRegistryTransaction>(), InTransaction);
			};

			if (TransactionStream.IsValid())
			{
				TransactionStream->Stream(AddValue);
			}
		}

		FFrontendQueryKey FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions::Map(const FFrontendQueryEntry& InEntry) const
		{
			FMetasoundFrontendVersion Version;
			if (ensure(InEntry.Value.IsType<FInterfaceRegistryTransaction>()))
			{
				Version = InEntry.Value.Get<FInterfaceRegistryTransaction>().GetInterfaceVersion();
			}
			return FFrontendQueryKey { Version.ToString() };
		}

		FFrontendQueryKey FMapInterfaceRegistryTransactionToInterfaceName::Map(const FFrontendQueryEntry& InEntry) const
		{
			if (ensure(InEntry.Value.IsType<FInterfaceRegistryTransaction>()))
			{
				return FFrontendQueryKey { InEntry.Value.Get<FInterfaceRegistryTransaction>().GetInterfaceVersion().Name };
			}
			return FFrontendQueryKey();
		}

		void FReduceInterfaceRegistryTransactionsToCurrentStatus::Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const
		{
			// Get most recent transaction
			const FFrontendQueryEntry* FinalEntry = Algo::MaxElementBy(InOutEntries, GetTransactionTimestamp);

			// Check if most recent transaction is valid and not an unregister transaction.
			if (IsValidTransactionOfType(FInterfaceRegistryTransaction::ETransactionType::InterfaceRegistration, FinalEntry))
			{
				FFrontendQueryEntry Entry = *FinalEntry;
				InOutEntries.Reset();
				InOutEntries.Add(Entry);
			}
			else
			{
				InOutEntries.Reset();
			}
		}

		FInterfaceRegistryTransaction::FTimeType FReduceInterfaceRegistryTransactionsToCurrentStatus::GetTransactionTimestamp(const FFrontendQueryEntry& InEntry)
		{
			if (ensure(InEntry.Value.IsType<FInterfaceRegistryTransaction>()))
			{
				return InEntry.Value.Get<FInterfaceRegistryTransaction>().GetTimestamp();
			}
			return 0;
		}

		bool FReduceInterfaceRegistryTransactionsToCurrentStatus::IsValidTransactionOfType(FInterfaceRegistryTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry)
		{
			if (nullptr != InEntry)
			{
				if (InEntry->Value.IsType<FInterfaceRegistryTransaction>())
				{
					return InEntry->Value.Get<FInterfaceRegistryTransaction>().GetTransactionType() == InType;
				}
			}
			return false;
		}

		FFrontendQueryKey FMapVersionToNameAndMajorVersion::GetKey(const FString& InName, int32 InMajorVersion)
		{
			return FFrontendQueryKey{FString::Format(TEXT("{0}_{1}"), { InName, InMajorVersion })};
		}

		FFrontendQueryKey FMapVersionToNameAndMajorVersion::Map(const FFrontendQueryEntry& InEntry) const
		{
			FString Key;
			if (ensure(InEntry.Value.IsType<FMetasoundFrontendVersion>()))
			{
				const FMetasoundFrontendVersion& InVersion = InEntry.Value.Get<FMetasoundFrontendVersion>();

				return FMapVersionToNameAndMajorVersion::GetKey(InVersion.Name.ToString(), InVersion.Number.Major);
			}
			return FFrontendQueryKey{Key};
		}

		void FReduceToHighestVersion::Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const
		{
			FFrontendQueryEntry* HighestVersionEntry = nullptr;
			FMetasoundFrontendVersionNumber HighestVersion = FMetasoundFrontendVersionNumber::GetInvalid();

			for (FFrontendQueryEntry& Entry : InOutEntries)
			{
				if (ensure(Entry.Value.IsType<FMetasoundFrontendVersion>()))
				{
					const FMetasoundFrontendVersionNumber& VersionNumber = Entry.Value.Get<FMetasoundFrontendVersion>().Number;
					if (VersionNumber > HighestVersion)
					{
						HighestVersionEntry = &Entry;
						HighestVersion = VersionNumber;
					}
				}
			}

			if (HighestVersionEntry)
			{
				FFrontendQueryEntry Entry = *HighestVersionEntry;
				InOutEntries.Reset();
				InOutEntries.Add(Entry);
			}
			else
			{
				InOutEntries.Reset();
			}
		}

		TArray<FFrontendQueryKey> FMapUClassToDefaultInterface::Map(const FFrontendQueryEntry& InEntry) const
		{
			TArray<FFrontendQueryKey> Keys;
			if (const FMetasoundFrontendVersion* InterfaceVersion = InEntry.Value.TryGet<FMetasoundFrontendVersion>())
			{
				FMetasoundFrontendInterface Interface;
				const bool bFound = FInterfaceRegistry::Get().FindInterface(*InterfaceVersion, Interface);
				if (ensure(bFound))
				{
					auto IsDefaultInterface = [](const FMetasoundFrontendInterfaceUClassOptions& InOptions)
					{
						return InOptions.bIsDefault;
					};
					auto GetUClassQueryKey = [](const FMetasoundFrontendInterfaceUClassOptions& InOptions)
					{
						return CreateKeyFromUClassPathName(InOptions.ClassPath);
					};
					Algo::TransformIf(Interface.Metadata.UClassOptions, Keys, IsDefaultInterface, GetUClassQueryKey);
				}
			}

			return Keys;
		}

		bool FRemoveDeprecatedClasses::Filter(const FFrontendQueryEntry& InEntry) const
		{
			if (InEntry.Value.IsType<FMetaSoundClassInfo>())
			{
				const EMetasoundFrontendClassAccessFlags ThisAccessFlags = InEntry.Value.Get<FMetaSoundClassInfo>().AccessFlags;
				if (EnumHasAnyFlags(ThisAccessFlags, EMetasoundFrontendClassAccessFlags::Deprecated))
				{
					return false;
				}
			}
			return true;
		}

		class FMapNodeRegistrationEventsToClassAndMajorVersion : public IFrontendQueryMapStep
		{
		public:
			virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override
			{
				if (InEntry.Value.IsType<FNodeClassRegistryTransaction>())
				{
					const FNodeClassRegistryKey& RegistryKey = InEntry.Value.Get<FNodeClassRegistryTransaction>().GetNodeRegistryKey();
					return CreateKeyFromFullClassNameAndMajorVersion(RegistryKey.ClassName, RegistryKey.Version.Major);
				}
				else
				{
					return FFrontendQueryKey();
				}
			}
		};

		FFrontendQueryKey FMapVersionToName::Map(const FFrontendQueryEntry& InEntry) const
		{
			FName Key;
			if (ensure(InEntry.Value.IsType<FMetasoundFrontendVersion>()))
			{
				Key = InEntry.Value.Get<FMetasoundFrontendVersion>().Name;
			}
			return FFrontendQueryKey { Key };
		}

		void FTransformInterfaceRegistryTransactionToInterfaceVersion::Transform(FFrontendQueryEntry::FValue& InValue) const
		{
			FMetasoundFrontendVersion Version;
			if (ensure(InValue.IsType<FInterfaceRegistryTransaction>()))
			{
				Version = InValue.Get<FInterfaceRegistryTransaction>().GetInterfaceVersion();
			}
			InValue.Set<FMetasoundFrontendVersion>(Version);
		}

		TArray<FMetaSoundClassInfo> BuildArrayOfClassesFromPartition(const FFrontendQueryPartition& InPartition)
		{
			TArray<FMetaSoundClassInfo> Result;
			for (const FFrontendQueryEntry& Entry : InPartition)
			{
				check(Entry.Value.IsType<FMetaSoundClassInfo>());
				Result.Add(Entry.Value.Get<FMetaSoundClassInfo>());
			}
			return Result;
		}

		FMetaSoundClassInfo BuildSingleClassFromPartition(const FFrontendQueryPartition& InPartition)
		{
			FMetaSoundClassInfo MetaSoundClass(EMetasoundFrontendClassType::Invalid);
			if (ensureMsgf(InPartition.Num() > 0, TEXT("Cannot retrieve class from empty partition")))
			{
				MetaSoundClass = InPartition.CreateConstIterator()->Value.Get<FMetaSoundClassInfo>();
			}
			return MetaSoundClass;
		}

		FMetasoundFrontendVersion BuildSingleVersionFromPartition(const FFrontendQueryPartition& InPartition)
		{
			FMetasoundFrontendVersion Version;
			if (ensureMsgf(InPartition.Num() > 0, TEXT("Cannot retrieve class from empty partition")))
			{
				Version = InPartition.CreateConstIterator()->Value.Get<FMetasoundFrontendVersion>();
			}
			return Version;
		}

		TArray<FMetasoundFrontendVersion> BuildArrayOfVersionsFromPartition(const FFrontendQueryPartition& InPartition)
		{
			TArray<FMetasoundFrontendVersion> Result;
			for (const FFrontendQueryEntry& Entry : InPartition)
			{
				check(Entry.Value.IsType<FMetasoundFrontendVersion>());
				Result.Add(Entry.Value.Get<FMetasoundFrontendVersion>());
			}
			return Result;
		}
	} // namespace SearchEngineQuerySteps


	FFrontendQuery FFindClassWithHighestMinorVersionQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FMapNodeRegistrationEventsToClassAndMajorVersion>();
		return Query;
	}

	FNodeRegistryKey FFindClassWithHighestMinorVersionQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;

		FMetasoundFrontendVersionNumber HighestVersionNumber { -1, -1 };
		FNodeRegistryKey Key = FNodeRegistryKey::GetInvalid();
		for (const FFrontendQueryEntry& InEntry : InPartition)
		{
			if (InEntry.Value.IsType<FNodeClassRegistryTransaction>())
			{
				const FNodeClassRegistryTransaction& Transaction = InEntry.Value.Get<FNodeClassRegistryTransaction>();
				if (FNodeClassRegistryTransaction::ETransactionType::NodeRegistration == Transaction.GetTransactionType())
				{
					const FNodeClassRegistryKey& TransactionKey = Transaction.GetNodeRegistryKey();
					if (TransactionKey.Version > HighestVersionNumber)
					{
						HighestVersionNumber = TransactionKey.Version;
						Key = Transaction.GetNodeRegistryKey();
					}
				}
			}
		}

		return Key;
	}

	FFrontendQuery FFindAllRegisteredInterfacesWithNameQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FInterfaceRegistryTransactionSource>()
			.AddStep<FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions>()
			.AddStep<FReduceInterfaceRegistryTransactionsToCurrentStatus>()
			.AddStep<FMapInterfaceRegistryTransactionToInterfaceName>()
			.AddStep<FTransformInterfaceRegistryTransactionToInterfaceVersion>();
		return Query;
	}

	TArray<FMetasoundFrontendVersion> FFindAllRegisteredInterfacesWithNameQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfVersionsFromPartition(InPartition);
	}

	FFrontendQuery FFindHighestVersionOfInterfaceQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FInterfaceRegistryTransactionSource>()
			.AddStep<FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions>()
			.AddStep<FReduceInterfaceRegistryTransactionsToCurrentStatus>()
			.AddStep<FTransformInterfaceRegistryTransactionToInterfaceVersion>()
			.AddStep<FMapVersionToName>()
			.AddStep<FReduceToHighestVersion>();
		return Query;
	}

	FMetasoundFrontendVersion FFindHighestVersionOfInterfaceQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildSingleVersionFromPartition(InPartition);
	}

	FFrontendQuery FFindAllInterfacesQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FInterfaceRegistryTransactionSource>()
			.AddStep<FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions>()
			.AddStep<FReduceInterfaceRegistryTransactionsToCurrentStatus>()
			.AddStep<FTransformInterfaceRegistryTransactionToInterfaceVersion>()
			.AddStep<FMapVersionToNameAndMajorVersion>()
			.AddStep<FReduceToHighestVersion>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetasoundFrontendVersion> FFindAllInterfacesQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfVersionsFromPartition(InPartition);
	}

	FFrontendQuery FFindAllInterfacesIncludingAllVersionsQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FInterfaceRegistryTransactionSource>()
			.AddStep<FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions>()
			.AddStep<FReduceInterfaceRegistryTransactionsToCurrentStatus>()
			.AddStep<FTransformInterfaceRegistryTransactionToInterfaceVersion>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetasoundFrontendVersion> FFindAllInterfacesIncludingAllVersionsQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfVersionsFromPartition(InPartition);
	}

	FFrontendQuery FFindAllDefaultInterfaceVersionsForUClassQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FInterfaceRegistryTransactionSource>()
			.AddStep<FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions>()
			.AddStep<FReduceInterfaceRegistryTransactionsToCurrentStatus>()
			.AddStep<FTransformInterfaceRegistryTransactionToInterfaceVersion>()
			.AddStep<FMapUClassToDefaultInterface>();
		return Query;
	}

	TArray<FMetasoundFrontendVersion> FFindAllDefaultInterfaceVersionsForUClassQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfVersionsFromPartition(InPartition);
	}

	void FSearchEngineCore::Prime()
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::Prime);

		FindClassWithHighestMinorVersionQuery.Prime();
		FindAllRegisteredInterfacesWithNameQuery.Prime();
		FindHighestVersionOfInterfaceQueryPolicy.Prime();
		FindAllDefaultInterfaceVersionsForUClassQuery.Prime();
	}

	bool FSearchEngineCore::FindRegisteredClass(const FMetasoundFrontendClassName& InName, int32 InMajorVersion, FMetaSoundClassInfo& OutClass)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::FindRegisteredClass);
		using namespace SearchEngineQuerySteps;

		const FFrontendQueryKey QueryKey = CreateKeyFromFullClassNameAndMajorVersion(InName, InMajorVersion);
		FNodeRegistryKey NodeRegistryKey; 
		if (FindClassWithHighestMinorVersionQuery.UpdateAndFindResult(QueryKey, NodeRegistryKey))
		{
			if (FMetasoundFrontendRegistryContainer* NodeRegistry = FMetasoundFrontendRegistryContainer::Get())
			{
				FMetasoundFrontendClass Class;
				const bool bFoundClass = NodeRegistry->FindFrontendClassFromRegistered(NodeRegistryKey, Class);
				if (bFoundClass)
				{
					OutClass = FMetaSoundClassInfo(Class);
					return true;
				}
			}
		}
		return false;
	}

	TArray<FMetasoundFrontendVersion> FSearchEngineCore::FindUClassDefaultInterfaceVersions(const FTopLevelAssetPath& InUClassPath)
	{
		using namespace SearchEngineQuerySteps;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::FindUClassDefaultInterfaceVersions);

		const FFrontendQueryKey Key = CreateKeyFromUClassPathName(InUClassPath);
		return FindAllDefaultInterfaceVersionsForUClassQuery.UpdateAndFindResult(Key);
	}

	TArray<FMetasoundFrontendVersion> FSearchEngineCore::FindAllRegisteredInterfacesWithName(FName InInterfaceName)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::FindAllRegisteredInterfacesWithName);

		const FFrontendQueryKey Key{InInterfaceName};
		return FindAllRegisteredInterfacesWithNameQuery.UpdateAndFindResult(Key);
	}

	bool FSearchEngineCore::FindHighestInterfaceVersion(FName InInterfaceName, FMetasoundFrontendVersion& OutVersion)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::FindHighestInterfaceVersion);

		const FFrontendQueryKey Key { InInterfaceName };

		return FindHighestVersionOfInterfaceQueryPolicy.UpdateAndFindResult(Key, OutVersion);
	}

	bool FSearchEngineCore::FindInterfaceWithHighestVersion(FName InInterfaceName, FMetasoundFrontendInterface& OutInterface)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::FSearchEngineCore::FindInterfaceWithHighestVersion);

		FMetasoundFrontendVersion Version;
		const bool bVersionFound = FindHighestInterfaceVersion(InInterfaceName, Version);
		if (bVersionFound)
		{
			return IInterfaceRegistry::Get().FindInterface(Version, OutInterface);
		}

		return false;
	}
} // namespace Metasound::Frontend
