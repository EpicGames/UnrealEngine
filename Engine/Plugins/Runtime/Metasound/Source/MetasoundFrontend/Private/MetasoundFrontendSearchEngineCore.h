// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistryPrivate.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundTrace.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace SearchEngineQuerySteps
		{
			/* A collection of reusable steps to be used with the Metasound frontend query system. */

			class FMapClassToFullClassNameAndMajorVersion : public IFrontendQueryMapStep
			{
			public:
				virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FInterfaceRegistryTransactionSource : public IFrontendQueryStreamStep
			{
			public:
				FInterfaceRegistryTransactionSource();
				virtual void Stream(TArray<FFrontendQueryValue>& OutEntries) override;

			private:
				TUniquePtr<FInterfaceTransactionStream> TransactionStream;
			};

			class FMapInterfaceRegistryTransactionsToInterfaceRegistryVersions : public IFrontendQueryMapStep
			{
			public:
				virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FMapInterfaceRegistryTransactionToInterfaceName : public IFrontendQueryMapStep
			{
			public:
				virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FReduceInterfaceRegistryTransactionsToCurrentStatus : public IFrontendQueryReduceStep
			{
			public:
				virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override;
			private:
				static FInterfaceRegistryTransaction::FTimeType GetTransactionTimestamp(const FFrontendQueryEntry& InEntry);
				static bool IsValidTransactionOfType(FInterfaceRegistryTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry);
			};

			class FMapUClassToDefaultInterface : public IFrontendQueryMultiMapStep
			{
			public:
				virtual TArray<FFrontendQueryKey> Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FMapVersionToNameAndMajorVersion : public IFrontendQueryMapStep
			{
			public:
				static FFrontendQueryKey GetKey(const FString& InName, int32 InMajorVersion);
				virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FReduceToHighestVersion : public IFrontendQueryReduceStep
			{
			public:
				virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override;
			};

			class FRemoveDeprecatedClasses : public IFrontendQueryFilterStep
			{
			public:
				FRemoveDeprecatedClasses() = default;

				virtual bool Filter(const FFrontendQueryEntry& InEntry) const override;
			};

			class FMapVersionToName : public IFrontendQueryMapStep
			{
			public:
				virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
			};

			class FTransformInterfaceRegistryTransactionToInterfaceVersion : public IFrontendQueryTransformStep
			{
			public:
				virtual void Transform(FFrontendQueryEntry::FValue& InValue) const override;
			};

			// Convert an array of partitions into an array of metasound frontend classes.
			TArray<FMetaSoundClassInfo> BuildArrayOfClassesFromPartition(const FFrontendQueryPartition& InPartition);

			FMetaSoundClassInfo BuildSingleClassFromPartition(const FFrontendQueryPartition& InPartition);

			FMetasoundFrontendVersion BuildSingleVersionFromPartition(const FFrontendQueryPartition& InPartition);

			// Convert an array of partitions into an array of metasound frontend versions.
			TArray<FMetasoundFrontendVersion> BuildArrayOfVersionsFromPartition(const FFrontendQueryPartition& InPartition);
		}

		// Base interface for a search engine query. This allows the query to be
		// primed to improve execution performance in critical call-stacks.
		struct ISearchEngineQuery
		{
			// Prime the query to improve execution performance during subsequent 
			// queries. 
			virtual void Prime() = 0;
			virtual ~ISearchEngineQuery() = default;
		};

		// TSearchEngineQuery provides a base for all queries used in the search
		// engine. It requires a `QueryPolicy` as a template parameter. 
		//
		// A `QueryPolicy` is a struct containing a static function to create the 
		// query, a static function to build a result from the query selection, 
		// and query result type.
		//
		// Example:
		//
		// struct FQueryPolicyExample
		// {
		//		using ResultType = FMetaSoundClassInfo;
		//
		//		static FFrontendQuery CreateQuery()
		//		{
		//			using namespace SearchEngineQuerySteps;
		//			FFrontendQuery Query;
		//			Query.AddStep<FNodeClassRegistrationEvents>()
		//				.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
		//				.AddStep<FReduceRegistrationEventsToCurrentStatus>()
		//				.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>();
		//			return Query;
		//		}
		//
		//		static ResultType BuildResult(const FFrontendQueryPartition& InPartition)
		//		{
		//			FMetaSoundClassInfo MetaSoundClass;
		//			if (ensureMsgf(InPartition.Num() > 0, TEXT("Cannot retrieve class from empty partition")))
		//			{
		//				MetaSoundClass = InPartition.CreateConstIterator()->Value.Get<FMetaSoundClassInfo>();
		//			}
		//			return MetaSoundClass;
		//		}
		// };
		template<typename QueryPolicy, typename... ArgTypes>
		struct TSearchEngineQuery : public ISearchEngineQuery
		{
			using ResultType = typename QueryPolicy::ResultType;

			TSearchEngineQuery(ArgTypes&&... Args)
			: Query(QueryPolicy::CreateQuery(Forward<ArgTypes>(Args)...))
			{
			}

			virtual void Prime() override
			{
				METASOUND_LLM_SCOPE;
				FScopeLock Lock(&QueryCriticalSection);
				Update();
			}

			// Updates the query to the most recent value and returns the
			// result.
			ResultType UpdateAndFindResult(const FFrontendQueryKey& InKey)
			{
				METASOUND_LLM_SCOPE;
				FScopeLock Lock(&QueryCriticalSection);

				Update();
				if (const ResultType* Result = ResultCache.Find(InKey))
				{
					return *Result;
				}
				return ResultType();
			}

			// Updates the query to the most recent value and assigns the value 
			// to OutResult. Returns true on success, false on failure. 
			bool UpdateAndFindResult(const FFrontendQueryKey& InKey, ResultType& OutResult)
			{
				METASOUND_LLM_SCOPE;
				FScopeLock Lock(&QueryCriticalSection);

				Update();
				if (const ResultType* Result = ResultCache.Find(InKey))
				{
					OutResult = *Result;
					return true;
				}
				return false;
			}

		private:
			void Update()
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::SearchEngineQuery::Update);

				TSet<FFrontendQueryKey> UpdatedKeys;
				const FFrontendQuerySelection& Selection = Query.Update(UpdatedKeys);

				if (UpdatedKeys.Num() > 0)
				{
					METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(MetaSound::SearchEngineQuery::BuildResult);

					for (const FFrontendQueryKey& Key : UpdatedKeys)
					{
						const FFrontendQueryPartition* Partition = Query.GetSelection().Find(Key);
						if (nullptr != Partition)
						{
							ResultCache.Add(Key, QueryPolicy::BuildResult(*Partition));
						}
						else
						{
							ResultCache.Remove(Key);
						}
					}
				}
			}

			FCriticalSection QueryCriticalSection;
			FFrontendQuery Query;
			TMap<FFrontendQueryKey, ResultType> ResultCache;
		};


		// Policy for finding highest versioned metasound class by name and major version.
		struct FFindClassWithHighestMinorVersionQueryPolicy
		{
			using ResultType = FNodeRegistryKey;

			static FFrontendQuery CreateQuery();
			static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
		};

		// Policy for finding all registered interface versions (name & version number)
		// by name.
		struct FFindAllRegisteredInterfacesWithNameQueryPolicy
		{
			using ResultType = TArray<FMetasoundFrontendVersion>;

			static FFrontendQuery CreateQuery();
			static TArray<FMetasoundFrontendVersion> BuildResult(const FFrontendQueryPartition& InPartition);
		};

		// Policy for finding the highest version registered interfaces by name.
		struct FFindHighestVersionOfInterfaceQueryPolicy
		{
			using ResultType = FMetasoundFrontendVersion;

			static FFrontendQuery CreateQuery();
			static FMetasoundFrontendVersion BuildResult(const FFrontendQueryPartition& InPartition);
		};

		// Policy for finding all registered interfaces. 
		struct FFindAllInterfacesQueryPolicy
		{
			using ResultType = TArray<FMetasoundFrontendVersion>;

			static FFrontendQuery CreateQuery();
			static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
		};

		// Policy for finding all registered interfaces (including deprecated). 
		struct FFindAllInterfacesIncludingAllVersionsQueryPolicy
		{
			using ResultType = TArray<FMetasoundFrontendVersion>;

			static FFrontendQuery CreateQuery();
			static TArray<FMetasoundFrontendVersion> BuildResult(const FFrontendQueryPartition& InPartition);
		};

		// Policy for finding all default interface versions for a given UClass
		struct FFindAllDefaultInterfaceVersionsForUClassQueryPolicy
		{
			using ResultType = TArray<FMetasoundFrontendVersion>;

			static FFrontendQuery CreateQuery();
			static TArray<FMetasoundFrontendVersion> BuildResult(const FFrontendQueryPartition& InPartition);
		};

		/** Supports essential search engine functionality needed for runtime. */
		class FSearchEngineCore : public ISearchEngine
		{
			FSearchEngineCore(const FSearchEngineCore&) = delete;
			FSearchEngineCore(FSearchEngineCore&&) = delete;
			FSearchEngineCore& operator=(const FSearchEngineCore&) = delete;
			FSearchEngineCore& operator=(FSearchEngineCore&&) = delete;

		public:
			FSearchEngineCore() = default;
			virtual ~FSearchEngineCore() = default;

			virtual void Prime() override;

			virtual bool FindRegisteredClass(const FMetasoundFrontendClassName& InName, int32 InMajorVersion, FMetaSoundClassInfo& OutClass) override;

			virtual TArray<FMetasoundFrontendVersion> FindUClassDefaultInterfaceVersions(const FTopLevelAssetPath& InUClassPath) override;
			virtual TArray<FMetasoundFrontendVersion> FindAllRegisteredInterfacesWithName(FName InInterfaceName) override;

			virtual bool FindHighestInterfaceVersion(FName InInterfaceName, FMetasoundFrontendVersion& OutVersion) override;
			virtual bool FindInterfaceWithHighestVersion(FName InInterfaceName, FMetasoundFrontendInterface& OutInterface) override;

		private:
			TSearchEngineQuery<FFindClassWithHighestMinorVersionQueryPolicy> FindClassWithHighestMinorVersionQuery;
			TSearchEngineQuery<FFindAllRegisteredInterfacesWithNameQueryPolicy> FindAllRegisteredInterfacesWithNameQuery;
			TSearchEngineQuery<FFindAllDefaultInterfaceVersionsForUClassQueryPolicy> FindAllDefaultInterfaceVersionsForUClassQuery;
			TSearchEngineQuery<FFindHighestVersionOfInterfaceQueryPolicy> FindHighestVersionOfInterfaceQueryPolicy;
		};
	}
}
