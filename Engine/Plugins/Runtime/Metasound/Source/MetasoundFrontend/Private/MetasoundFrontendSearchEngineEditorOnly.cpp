// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundFrontendSearchEngineEditorOnly.h"

#include "Containers/Array.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendQuerySteps.h"
#include "MetasoundFrontendRegistryTransaction.h"
#include "MetasoundFrontendSearchEngineCore.h"
#include "MetasoundTrace.h"

#if WITH_EDITORONLY_DATA

namespace Metasound::Frontend
{
	FFrontendQuery FFindAllAssetClassInfoQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FAssetRegistrationEvents>()
			.AddStep<FMapAssetRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceAssetRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformAssetRegistrationEventsToClassInfo>()
			.AddStep<FRemoveDeprecatedClasses>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindAllAssetClassInfoQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindAllClassesQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FRemoveDeprecatedClasses>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindAllClassesQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindAllRegisteredClassesIncludingAllVersionsQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindAllRegisteredClassesIncludingAllVersionsQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindAllRegisteredAssetsQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FAssetRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FMapToNull>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindAllRegisteredAssetsQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindClassesWithNameSortedQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FMapClassesToClassName>()
			.AddStep<FSortClassesByVersion>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindClassesWithNameSortedQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindClassesWithNameUnsortedQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FMapClassesToClassName>();
		return Query;
	}

	TArray<FMetaSoundClassInfo> FFindClassesWithNameUnsortedQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildArrayOfClassesFromPartition(InPartition);
	}

	FFrontendQuery FFindClassWithHighestVersionQueryPolicy::CreateQuery()
	{
		using namespace SearchEngineQuerySteps;
		FFrontendQuery Query;
		Query.AddStep<FNodeClassRegistrationEvents>()
			.AddStep<FMapClassRegistrationEventsToNodeRegistryKeys>()
			.AddStep<FReduceRegistrationEventsToCurrentStatus>()
			.AddStep<FTransformNodeClassRegistrationEventsToClassInfo>()
			.AddStep<FRemoveDeprecatedClasses>()
			.AddStep<FMapToFullClassName>()
			.AddStep<FReduceClassesToHighestVersion>();
		return Query;
	}

	FMetaSoundClassInfo FFindClassWithHighestVersionQueryPolicy::BuildResult(const FFrontendQueryPartition& InPartition)
	{
		using namespace SearchEngineQuerySteps;
		return BuildSingleClassFromPartition(InPartition);
	}

	void FSearchEngineEditorOnly::Prime()
	{
		Super::Prime();
		{
			METASOUND_LLM_SCOPE;
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FSearchEngineEditorOnly::Prime);

			FindAllClassesQuery.Prime();
			FindAllClassesIncludingAllVersionsQuery.Prime();
			FindClassesWithNameUnsortedQuery.Prime();
			FindClassesWithNameSortedQuery.Prime();
			FindClassWithHighestVersionQuery.Prime();
			FindAllInterfacesIncludingAllVersionsQuery.Prime();
			FindAllInterfacesQuery.Prime();
		}
	}

	TArray<FMetasoundFrontendClass> FSearchEngineEditorOnly::FindClassesWithName(const FMetasoundFrontendClassName& InName, bool bInSortByVersion)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(metasound::FSearchEngineEditorOnly::FindClassesWithName);

		TArray<FMetaSoundClassInfo> ClassInfos = FindClassesWithName(InName, bInSortByVersion ? ESortByVersion::No : ESortByVersion::Yes);
		TArray<FMetasoundFrontendClass> Classes;
		Algo::Transform(ClassInfos, Classes, [](const FMetaSoundClassInfo& ClassInfo)
		{
			FMetasoundFrontendClass Class;
			INodeClassRegistry::Get()->FindFrontendClassFromRegistered(ClassInfo.ToRegistryKey(), Class);
			return Class;
		});

		return Classes;
	}

	TArray<FMetaSoundClassInfo> FSearchEngineEditorOnly::FindAllClasses(ISearchEngine::EResultVersion VersionOptions, bool bIncludeUnloadedAssets)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FSearchEngineEditorOnly::FindAllClasses);

		TArray<FMetaSoundClassInfo> ClassInfos;

		const FFrontendQueryKey NullKey;

		switch (VersionOptions)
		{
			case ISearchEngine::EResultVersion::All:
			{
				ClassInfos = FindAllClassesIncludingAllVersionsQuery.UpdateAndFindResult(NullKey);
			}
			break;

			case ISearchEngine::EResultVersion::Highest:
			{
				ClassInfos = FindAllClassesQuery.UpdateAndFindResult(NullKey);
			}
			break;

			default:
				checkNoEntry();
		}
		
		if (bIncludeUnloadedAssets)
		{
			TMap<FNodeClassRegistryKey, FMetaSoundClassInfo> KeysToInfo;
			Algo::Transform(ClassInfos, KeysToInfo, [](const FMetaSoundClassInfo& Info) { return TPair<FNodeClassRegistryKey, FMetaSoundClassInfo>(Info.ToRegistryKey(), Info); });

			TArray<FMetaSoundClassInfo> AssetInfos = FindAllAssetClassInfoQuery.UpdateAndFindResult(NullKey);			
			for (FMetaSoundClassInfo& Info : AssetInfos)
			{
				if (!KeysToInfo.Contains(Info.ToRegistryKey()))
				{
					ClassInfos.Add(MoveTemp(Info));
				}
			}
		}

		return ClassInfos;
	}

	TArray<FMetaSoundClassInfo> FSearchEngineEditorOnly::FindClassesWithName(const FMetasoundFrontendClassName& InName, ISearchEngine::ESortByVersion SortByVersion)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FSearchEngineEditorOnly::FindClassesWithName);

		const FFrontendQueryKey Key(InName.GetFullName());
		switch (SortByVersion)
		{
			case ISearchEngine::ESortByVersion::Yes:
				return FindClassesWithNameSortedQuery.UpdateAndFindResult(Key);
			case ISearchEngine::ESortByVersion::No:
				return FindClassesWithNameUnsortedQuery.UpdateAndFindResult(Key);
			default:
				checkNoEntry();
		}

		return { };
	}

	bool FSearchEngineEditorOnly::FindClassWithHighestVersion(const FMetasoundFrontendClassName& InName, FMetaSoundClassInfo& OutClassInfo)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FSearchEngineEditorOnly::FindClassWithHighestVersion);

		const FFrontendQueryKey Key{InName.GetFullName()};
		return FindClassWithHighestVersionQuery.UpdateAndFindResult(Key, OutClassInfo);
	}

	TArray<FMetasoundFrontendVersion> FSearchEngineEditorOnly::FindAllInterfaceVersions(bool bInIncludeAllVersions)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FSearchEngineEditorOnly::FindAllInterfaceVersions);

		const FFrontendQueryKey NullKey;
		if (bInIncludeAllVersions)
		{
			return FindAllInterfacesIncludingAllVersionsQuery.UpdateAndFindResult(NullKey);
		}
		else
		{
			return FindAllInterfacesQuery.UpdateAndFindResult(NullKey);
		}
	}
} // namespace Metasound::Frontend
#endif // WITH_EDITORONLY_DATA
