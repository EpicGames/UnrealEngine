// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundFrontendSearchEngineCore.h"

#if WITH_EDITORONLY_DATA
namespace Metasound::Frontend
{
	// Policy for finding all registered asset classes that are not deprecated and referenceable.
	struct FFindAllAssetClassInfoQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding all registered metasound classes that are not deprecated and referenceable.
	struct FFindAllClassesQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding all registered metasound classes, including deprecated classes. 
	struct FFindAllRegisteredClassesIncludingAllVersionsQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding all registered metasound assets. 
	struct FFindAllRegisteredAssetsQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding all registered metasound classes sorted by version 
	// and indexed by name.
	struct FFindClassesWithNameSortedQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding all registered metasound classes indexed by name.
	struct FFindClassesWithNameUnsortedQueryPolicy
	{
		using ResultType = TArray<FMetaSoundClassInfo>;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// Policy for finding highest versioned metasound class by name.
	struct FFindClassWithHighestVersionQueryPolicy
	{
		using ResultType = FMetaSoundClassInfo;

		static FFrontendQuery CreateQuery();
		static ResultType BuildResult(const FFrontendQueryPartition& InPartition);
	};

	// To minimize runtime memory costs during gampeplay, some search engine
	// queries are only exposed when the editor only data is available. This
	// class supports the editor only functionality of the search engine.
	class FSearchEngineEditorOnly : public FSearchEngineCore
	{
		FSearchEngineEditorOnly(const FSearchEngineEditorOnly&) = delete;
		FSearchEngineEditorOnly(FSearchEngineEditorOnly&&) = delete;
		FSearchEngineEditorOnly& operator=(const FSearchEngineEditorOnly&) = delete;
		FSearchEngineEditorOnly& operator=(FSearchEngineEditorOnly&&) = delete;

		using Super = FSearchEngineCore;

	public:
		FSearchEngineEditorOnly() = default;
		virtual ~FSearchEngineEditorOnly() = default;

		virtual void Prime() override;
		virtual TArray<FMetaSoundClassInfo> FindAllClasses(ISearchEngine::EResultVersion VersionOptions, bool bIncludeUnloadedAssets = false) override;
		virtual TArray<FMetaSoundClassInfo> FindClassesWithName(const FMetasoundFrontendClassName& InName, ESortByVersion SortByVersion = ESortByVersion::Yes) override;
		virtual TArray<FMetasoundFrontendClass> FindClassesWithName(const FMetasoundFrontendClassName& InName, bool bInSortByVersion) override;
		virtual bool FindClassWithHighestVersion(const FMetasoundFrontendClassName& InName, FMetaSoundClassInfo& OutClassInfo) override;
		virtual TArray<FMetasoundFrontendVersion> FindAllInterfaceVersions(bool bInIncludeAllVersions) override;

	private:
		TSearchEngineQuery<FFindAllAssetClassInfoQueryPolicy> FindAllAssetClassInfoQuery;
		TSearchEngineQuery<FFindAllClassesQueryPolicy> FindAllClassesQuery;
		TSearchEngineQuery<FFindAllRegisteredClassesIncludingAllVersionsQueryPolicy> FindAllClassesIncludingAllVersionsQuery;
		TSearchEngineQuery<FFindClassesWithNameUnsortedQueryPolicy> FindClassesWithNameUnsortedQuery;
		TSearchEngineQuery<FFindClassesWithNameSortedQueryPolicy> FindClassesWithNameSortedQuery;
		TSearchEngineQuery<FFindClassWithHighestVersionQueryPolicy> FindClassWithHighestVersionQuery;
		TSearchEngineQuery<FFindAllInterfacesQueryPolicy> FindAllInterfacesQuery;
		TSearchEngineQuery<FFindAllInterfacesIncludingAllVersionsQueryPolicy> FindAllInterfacesIncludingAllVersionsQuery;
	};
} // namespace Metasound::Frontend
#endif // WITH_EDITORONLY_DATA
