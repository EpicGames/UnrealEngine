// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendSubgraphNodeController.h"
#include "MetasoundFrontendInputController.h"
#include "MetasoundFrontendOutputController.h"

#include "Algo/Transform.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace FrontendSubgraphNodeControllerPrivate
		{
			template<typename SetAType, typename SetBType>
			void InplaceBidirectionalVertexSetDifference(TArray<const SetAType*>& SetA, TArray<const SetBType*>& SetB)
			{
				// TODO: comparison is done by `Name`. May update to a `ClassVertexID` in the future. 
				// Reverse iterate to allow removal of elements from SetA within loop.
				for (int32 SetAIndex = (SetA.Num() - 1); SetAIndex >= 0; SetAIndex--) 
				{
					const FVertexName& SetAName = SetA[SetAIndex]->Name;
					int32 SetBIndex = SetB.IndexOfByPredicate([&](const SetBType* InVertex) { return InVertex->Name == SetAName; });
					if (INDEX_NONE != SetBIndex)
					{
						SetA.RemoveAt(SetAIndex);
						SetB.RemoveAt(SetBIndex);
					}
				}
			}

			// Manipulates the NodeVertexArray to match the ClassVertexArray.
			template<typename NodeVertexType, typename ClassVertexType>
			bool ConformNodeVertexArrayToClassVertexArray(TArray<NodeVertexType>& NodeVertexArray, const TArray<ClassVertexType>& ClassVertexArray)
			{
				// TODO: May need to change this to support dynamic pins. 
				bool bDidAlterNodeVertexArray = false;

				TArray<const NodeVertexType*> NodeVerticesToRemove;
				Algo::Transform(NodeVertexArray, NodeVerticesToRemove, [](const NodeVertexType& Vertex) { return &Vertex; });

				TArray<const ClassVertexType*> ClassVerticesToAdd;
				Algo::Transform(ClassVertexArray, ClassVerticesToAdd, [](const ClassVertexType& Vertex) { return &Vertex; });
			
				FrontendSubgraphNodeControllerPrivate::InplaceBidirectionalVertexSetDifference(NodeVerticesToRemove, ClassVerticesToAdd);

				bDidAlterNodeVertexArray = (NodeVerticesToRemove.Num() > 0) || (ClassVerticesToAdd.Num() > 0);

				// Remove extra node vertices.
				for (const NodeVertexType* VertexToRemove : NodeVerticesToRemove)
				{
					NodeVertexArray.RemoveAll([&](const NodeVertexType& InVertex) { return InVertex.VertexID == VertexToRemove->VertexID; });
				}

				// Add missing node vertices. 
				for (const ClassVertexType* VertexToAdd : ClassVerticesToAdd)
				{
					NodeVertexType NewNodeVertex(*VertexToAdd);
					NewNodeVertex.VertexID = FGuid::NewGuid();
					NodeVertexArray.Add(NewNodeVertex);
				}

				return bDidAlterNodeVertexArray;
			}
		}

		// 
		// FSubgraphNodeController
		//
		FSubgraphNodeController::FSubgraphNodeController(EPrivateToken InToken, const FSubgraphNodeController::FInitParams& InParams)
		: FBaseNodeController({InParams.NodePtr, InParams.ClassPtr, InParams.OwningGraph})
		, GraphPtr(InParams.GraphPtr)
		{
		}

		FNodeHandle FSubgraphNodeController::CreateNodeHandle(const FSubgraphNodeController::FInitParams& InParams)
		{
			if (FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						return MakeShared<FSubgraphNodeController>(EPrivateToken::Token, InParams);
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FSubgraphNodeController::CreateConstNodeHandle(const FSubgraphNodeController::FInitParams& InParams)
		{
			if (FMetasoundFrontendNode* Node = InParams.NodePtr.Get())
			{
				if (const FMetasoundFrontendClass* Class = InParams.ClassPtr.Get())
				{
					// Cannot make a valid node handle if the node description and class description differ
					if (Node->ClassID == Class->ID)
					{
						return MakeShared<const FSubgraphNodeController>(EPrivateToken::Token, InParams);
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Frontend Node [NodeID:%ls, ClassID:%ls] is not of expected class class [ClassID:%ls]", *Node->GetID().ToString(), *Node->ClassID.ToString(), *Class->ID.ToString());
					}
				}
			}
			return INodeController::GetInvalidHandle();
		}

		bool FSubgraphNodeController::IsValid() const
		{
			return FBaseNodeController::IsValid() && (nullptr != GraphPtr.Get());
		}

		int32 FSubgraphNodeController::GetNumInputs() const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::GetNumInputs();
		}

		int32 FSubgraphNodeController::GetNumOutputs() const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::GetNumOutputs();
		}

		TArray<FBaseNodeController::FInputControllerParams> FSubgraphNodeController::GetInputControllerParams() const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::GetInputControllerParams();
		}

		TArray<FBaseNodeController::FOutputControllerParams> FSubgraphNodeController::GetOutputControllerParams() const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::GetOutputControllerParams();
		}

		bool FSubgraphNodeController::FindInputControllerParamsWithVertexName(const FVertexName& InName, FInputControllerParams& OutParams) const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::FindInputControllerParamsWithVertexName(InName, OutParams);
		}

		bool FSubgraphNodeController::FindOutputControllerParamsWithVertexName(const FVertexName& InName, FOutputControllerParams& OutParams) const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::FindOutputControllerParamsWithVertexName(InName, OutParams);
		}

		bool FSubgraphNodeController::FindInputControllerParamsWithID(FGuid InVertexID, FInputControllerParams& OutParams) const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::FindInputControllerParamsWithID(InVertexID, OutParams);
		}

		bool FSubgraphNodeController::FindOutputControllerParamsWithID(FGuid InVertexID, FOutputControllerParams& OutParams) const
		{
			// TODO: Trigger ConformNodeInterfaceToClassInterface() on an
			// event callback rather than on every call to this function.
			const_cast<FSubgraphNodeController*>(this)->ConformNodeInterfaceToClassInterface();

			return Super::FindOutputControllerParamsWithID(InVertexID, OutParams);
		}

		void FSubgraphNodeController::ConformNodeInterfaceToClassInterface() 
		{
			using namespace FrontendSubgraphNodeControllerPrivate;

			if (FMetasoundFrontendNode* Node = NodePtr.Get())
			{
				if (const FMetasoundFrontendClassInterface* ClassInterface = GetClassInterfaceForNode())
				{
					ConformNodeVertexArrayToClassVertexArray(Node->Interface.Inputs, ClassInterface->Inputs);
					ConformNodeVertexArrayToClassVertexArray(Node->Interface.Outputs, ClassInterface->Outputs);
					// TODO: Conform environment variables by aggregating all dependent environment variables.
				}
			}
		}

		FDocumentAccess FSubgraphNodeController::ShareAccess() 
		{
			FDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.Graph = GraphPtr;
			Access.ConstGraph = GraphPtr;

			return Access;
		}

		FConstDocumentAccess FSubgraphNodeController::ShareAccess() const
		{
			FConstDocumentAccess Access = FBaseNodeController::ShareAccess();

			Access.ConstGraph = GraphPtr;

			return Access;
		}

		FInputHandle FSubgraphNodeController::CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FBaseInputController>(FBaseInputController::FInitParams{InVertexID, InNodeVertexPtr, InClassInputPtr, GraphPtr, InOwningNode});
		}

		FOutputHandle FSubgraphNodeController::CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const
		{
			return MakeShared<FBaseOutputController>(FBaseOutputController::FInitParams{InVertexID, InNodeVertexPtr, InClassOutputPtr, GraphPtr, InOwningNode});
		}
	}
}
