// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendNodeController.h"

namespace Metasound
{
	namespace Frontend
	{
		/** Controller for nodes representing subgraphs. Handles synchronization
		 * of node interface to subgraph interface. 
		 */
		class FSubgraphNodeController : public FBaseNodeController
		{
			// Private token only allows members or friends to call constructor.
			enum EPrivateToken { Token };

			using Super = FBaseNodeController;

		public:
			struct FInitParams
			{
				FNodeAccessPtr NodePtr;
				FConstClassAccessPtr ClassPtr;
				FGraphAccessPtr GraphPtr;
				FGraphHandle OwningGraph;
			};

			// Constructor takes a private token so it can only be instantiated by
			// using the static creation functions. This protects against some
			// error conditions which would result in a zombie object. The creation
			// methods can detect the error conditions and return an invalid controller
			// on error
			FSubgraphNodeController(EPrivateToken InToken, const FInitParams& InParams);

			/** Create a node handle for a external or subgraph node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FNodeHandle CreateNodeHandle(const FInitParams& InParams);

			/** Create a node handle for a external or subgraph node. 
			 *
			 * @return A Node handle. On error, an invalid node handle is returned. 
			 */
			static FConstNodeHandle CreateConstNodeHandle(const FInitParams& InParams);

			virtual ~FSubgraphNodeController() = default;

			bool IsValid() const override;

			virtual int32 GetNumInputs() const override;

			virtual int32 GetNumOutputs() const override;

		protected:
			using FInputControllerParams = FBaseNodeController::FInputControllerParams;
			using FOutputControllerParams = FBaseNodeController::FOutputControllerParams;

			virtual FDocumentAccess ShareAccess() override;
			virtual FConstDocumentAccess ShareAccess() const override;

			virtual TArray<FInputControllerParams> GetInputControllerParams() const override;
			virtual TArray<FOutputControllerParams> GetOutputControllerParams() const override;

			virtual bool FindInputControllerParamsWithVertexName(const FVertexName& InName, FInputControllerParams& OutParams) const override;
			virtual bool FindOutputControllerParamsWithVertexName(const FVertexName& InName, FOutputControllerParams& OutParams) const override;

			virtual bool FindInputControllerParamsWithID(FGuid InVertexID, FInputControllerParams& OutParams) const override;
			virtual bool FindOutputControllerParamsWithID(FGuid InVertexID, FOutputControllerParams& OutParams) const override;

		private:
			void ConformNodeInterfaceToClassInterface();

			virtual FInputHandle CreateInputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassInputAccessPtr InClassInputPtr, FNodeHandle InOwningNode) const override;
			virtual FOutputHandle CreateOutputController(FGuid InVertexID, FConstVertexAccessPtr InNodeVertexPtr, FConstClassOutputAccessPtr InClassOutputPtr, FNodeHandle InOwningNode) const override;

			FGraphAccessPtr GraphPtr;
		};
	}
}
