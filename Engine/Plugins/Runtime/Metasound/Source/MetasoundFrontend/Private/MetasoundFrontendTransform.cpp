// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundFrontendTransform.h"

#include "Algo/Transform.h"
#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Interfaces/MetasoundFrontendInterface.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"
#include "MetasoundAccessPtr.h"
#include "MetasoundAssetBase.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendDocumentController.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundFrontendDocumentVersioning.h"
#include "MetasoundFrontendNodeUpdateTransform.h"
#include "MetasoundFrontendSearchEngine.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"
#include "Misc/App.h"

namespace Metasound
{
	namespace Frontend
	{
		namespace DocumentTransform
		{
		bool bVersioningLoggingEnabled = true;
			
#if WITH_EDITOR
			FGetNodeDisplayNameProjection NodeDisplayNameProjection = [] (const FNodeHandle&) { return FText(); };

			bool GetVersioningLoggingEnabled()
			{
				return bVersioningLoggingEnabled;
			}

			void SetVersioningLoggingEnabled(bool bEnabled)
			{
				bVersioningLoggingEnabled = bEnabled;
			}

			void RegisterNodeDisplayNameProjection(FGetNodeDisplayNameProjection&& InNameProjection)
			{
				NodeDisplayNameProjection = MoveTemp(InNameProjection);
			}

			FGetNodeDisplayNameProjectionRef GetNodeDisplayNameProjection()
			{
				return NodeDisplayNameProjection;
			}
#endif // WITH_EDITOR
		} // namespace DocumentTransform

		bool IDocumentTransform::Transform(FMetasoundFrontendDocument& InOutDocument) const
		{
			FDocumentAccessPtr DocAccessPtr = MakeAccessPtr<FDocumentAccessPtr>(InOutDocument.AccessPoint, InOutDocument);
			return Transform(FDocumentController::CreateDocumentHandle(DocAccessPtr));
		}

		bool INodeTransform::Transform(const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const
		{
			return false;
		}

		FModifyRootGraphInterfaces::FModifyRootGraphInterfaces(const TArray<FMetasoundFrontendInterface>& InInterfacesToRemove, const TArray<FMetasoundFrontendInterface>& InInterfacesToAdd)
			: InterfacesToRemove(InInterfacesToRemove)
			, InterfacesToAdd(InInterfacesToAdd)
		{
			Init();
		}

		FModifyRootGraphInterfaces::FModifyRootGraphInterfaces(const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToRemove, const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToAdd)
		{
			Algo::Transform(InInterfaceVersionsToRemove, InterfacesToRemove, [](const FMetasoundFrontendVersion& Version)
			{
				FMetasoundFrontendInterface Interface;
				const bool bFromInterfaceFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
				if (!ensureAlways(bFromInterfaceFound))
				{
					METASOUND_VERSIONING_LOG(Error, TEXT("Failed to find interface '%s' to remove"), *Version.ToString());
				}
				return Interface;
			});

			Algo::Transform(InInterfaceVersionsToAdd, InterfacesToAdd, [](const FMetasoundFrontendVersion& Version)
			{
				FMetasoundFrontendInterface Interface;
				const bool bToInterfaceFound = IInterfaceRegistry::Get().FindInterface(Version, Interface);
				if (!ensureAlways(bToInterfaceFound))
				{
					METASOUND_VERSIONING_LOG(Error, TEXT("Failed to find interface '%s' to add"), *Version.ToString());
				}
				return Interface;
			});

			Init();
		}

#if WITH_EDITOR
		void FModifyRootGraphInterfaces::SetDefaultNodeLocations(bool bInSetDefaultNodeLocations)
		{
			bSetDefaultNodeLocations = bInSetDefaultNodeLocations;
		}
#endif // WITH_EDITOR

		void FModifyRootGraphInterfaces::SetNamePairingFunction(const TFunction<bool(FName, FName)>& InNamePairingFunction)
		{
			// Reinit required to rebuild list of pairs
			Init(&InNamePairingFunction);
		}

		bool FModifyRootGraphInterfaces::AddMissingVertices(FGraphHandle GraphHandle) const
		{
			for (const FInputData& InputData : InputsToAdd)
			{
				const FMetasoundFrontendClassInput& InputToAdd = InputData.Input;
				GraphHandle->AddInputVertex(InputToAdd);
			}

			for (const FOutputData& OutputData : OutputsToAdd)
			{
				const FMetasoundFrontendClassOutput& OutputToAdd = OutputData.Output;
				GraphHandle->AddOutputVertex(OutputToAdd);
			}

			return !InputsToAdd.IsEmpty() || !OutputsToAdd.IsEmpty();
		}

		void FModifyRootGraphInterfaces::Init(const TFunction<bool(FName, FName)>* InNamePairingFunction)
		{
			InputsToRemove.Reset();
			InputsToAdd.Reset();
			OutputsToRemove.Reset();
			OutputsToAdd.Reset();
			PairedInputs.Reset();
			PairedOutputs.Reset();

			for (const FMetasoundFrontendInterface& FromInterface : InterfacesToRemove)
			{
				InputsToRemove.Append(FromInterface.Inputs);
				OutputsToRemove.Append(FromInterface.Outputs);
			}

			// This function combines all the inputs of all interfaces into one input list and ptrs to their originating interfaces.
			// The interface ptr will be used to query the interface for required validations on inputs. Interfaces define required inputs (and possibly other validation requirements).
			for (const FMetasoundFrontendInterface& ToInterface : InterfacesToAdd)
			{
				TArray<FInputData> NewInputDataArray;
				for (const FMetasoundFrontendClassInput& Input : ToInterface.Inputs)
				{
					FInputData NewData;
					NewData.Input = Input;
					NewData.InputInterface = &ToInterface;
					NewInputDataArray.Add(NewData);
				}

				InputsToAdd.Append(NewInputDataArray);

				TArray<FOutputData> NewOutputDataArray;
				for (const FMetasoundFrontendClassOutput& Output : ToInterface.Outputs)
				{
					FOutputData NewData;
					NewData.Output = Output;
					NewData.OutputInterface = &ToInterface;
					NewOutputDataArray.Add(NewData);
				}

				OutputsToAdd.Append(NewOutputDataArray);
			}

			// Iterate in reverse to allow removal from `InputsToAdd`
			for (int32 AddIndex = InputsToAdd.Num() - 1; AddIndex >= 0; AddIndex--)
			{
				const FMetasoundFrontendClassVertex& VertexToAdd = InputsToAdd[AddIndex].Input;

				const int32 RemoveIndex = InputsToRemove.IndexOfByPredicate([&](const FMetasoundFrontendClassVertex& VertexToRemove)
				{
					if (VertexToAdd.TypeName != VertexToRemove.TypeName)
					{
						return false;
					}

					if (InNamePairingFunction && *InNamePairingFunction)
					{
						return (*InNamePairingFunction)(VertexToAdd.Name, VertexToRemove.Name);
					}

					FName ParamA;
					FName ParamB;
					FName Namespace;
					VertexToAdd.SplitName(Namespace, ParamA);
					VertexToRemove.SplitName(Namespace, ParamB);

					return ParamA == ParamB;
				});

				if (INDEX_NONE != RemoveIndex)
				{
					PairedInputs.Add(FVertexPair{InputsToRemove[RemoveIndex], InputsToAdd[AddIndex].Input});
					InputsToRemove.RemoveAtSwap(RemoveIndex);
					InputsToAdd.RemoveAtSwap(AddIndex);
				}
			}

			// Iterate in reverse to allow removal from `OutputsToAdd`
			for (int32 AddIndex = OutputsToAdd.Num() - 1; AddIndex >= 0; AddIndex--)
			{
				const FMetasoundFrontendClassVertex& VertexToAdd = OutputsToAdd[AddIndex].Output;

				const int32 RemoveIndex = OutputsToRemove.IndexOfByPredicate([&](const FMetasoundFrontendClassVertex& VertexToRemove)
				{
					if (VertexToAdd.TypeName != VertexToRemove.TypeName)
					{
						return false;
					}

					if (InNamePairingFunction && *InNamePairingFunction)
					{
						return (*InNamePairingFunction)(VertexToAdd.Name, VertexToRemove.Name);
					}

					FName ParamA;
					FName ParamB;
					FName Namespace;
					VertexToAdd.SplitName(Namespace, ParamA);
					VertexToRemove.SplitName(Namespace, ParamB);

					return ParamA == ParamB;
				});

				if (INDEX_NONE != RemoveIndex)
				{
					PairedOutputs.Add(FVertexPair { OutputsToRemove[RemoveIndex], OutputsToAdd[AddIndex].Output });
					OutputsToRemove.RemoveAtSwap(RemoveIndex);
					OutputsToAdd.RemoveAtSwap(AddIndex);
				}
			}
		}

		bool FModifyRootGraphInterfaces::RemoveUnsupportedVertices(FGraphHandle GraphHandle) const
		{
			// Remove unsupported inputs
			for (const FMetasoundFrontendClassVertex& InputToRemove : InputsToRemove)
			{
				if (const FMetasoundFrontendClassInput* ClassInput = GraphHandle->FindClassInputWithName(InputToRemove.Name).Get())
				{
					if (FMetasoundFrontendClassInput::IsFunctionalEquivalent(*ClassInput, InputToRemove))
					{
						GraphHandle->RemoveInputVertex(InputToRemove.Name);
					}
				}
			}

			// Remove unsupported outputs
			for (const FMetasoundFrontendClassVertex& OutputToRemove : OutputsToRemove)
			{
				if (const FMetasoundFrontendClassOutput* ClassOutput = GraphHandle->FindClassOutputWithName(OutputToRemove.Name).Get())
				{
					if (FMetasoundFrontendClassOutput::IsFunctionalEquivalent(*ClassOutput, OutputToRemove))
					{
						GraphHandle->RemoveOutputVertex(OutputToRemove.Name);
					}
				}
			}

			return !InputsToRemove.IsEmpty() || !OutputsToRemove.IsEmpty();
		}

		bool FModifyRootGraphInterfaces::SwapPairedVertices(FGraphHandle GraphHandle) const
		{
			for (const FVertexPair& InputPair : PairedInputs)
			{
				const FMetasoundFrontendClassVertex& OriginalVertex = InputPair.Get<0>();
				FMetasoundFrontendClassInput NewVertex = InputPair.Get<1>();

				// Cache off node locations and connections to push to new node
				TMap<FGuid, FVector2D> Locations;
				TArray<FInputHandle> ConnectedInputs;
				if (const FMetasoundFrontendClassInput* ClassInput = GraphHandle->FindClassInputWithName(OriginalVertex.Name).Get())
				{
					if (FMetasoundFrontendVertex::IsFunctionalEquivalent(*ClassInput, OriginalVertex))
					{
						const FMetasoundFrontendLiteral& DefaultLiteral = ClassInput->FindConstDefaultChecked(Frontend::DefaultPageID);
						NewVertex.FindDefaultChecked(Frontend::DefaultPageID) = DefaultLiteral;
						NewVertex.NodeID = ClassInput->NodeID;
						FNodeHandle OriginalInputNode = GraphHandle->GetInputNodeWithName(OriginalVertex.Name);

#if WITH_EDITOR
						Locations = OriginalInputNode->GetNodeStyle().Display.Locations;
#endif // WITH_EDITOR

						FOutputHandle OriginalInputNodeOutput = OriginalInputNode->GetOutputWithVertexName(OriginalVertex.Name);
						ConnectedInputs = OriginalInputNodeOutput->GetConnectedInputs();
						GraphHandle->RemoveInputVertex(OriginalVertex.Name);
					}
				}

				FNodeHandle NewInputNode = GraphHandle->AddInputVertex(NewVertex);

#if WITH_EDITOR
				// Copy prior node locations
				if (!Locations.IsEmpty())
				{
					FMetasoundFrontendNodeStyle Style = NewInputNode->GetNodeStyle();
					Style.Display.Locations = Locations;
					NewInputNode->SetNodeStyle(Style);
				}
#endif // WITH_EDITOR

				// Copy prior node connections
				FOutputHandle OutputHandle = NewInputNode->GetOutputWithVertexName(NewVertex.Name);
				for (FInputHandle& ConnectedInput : ConnectedInputs)
				{
					OutputHandle->Connect(*ConnectedInput);
				}
			}

			// Swap paired outputs.
			for (const FVertexPair& OutputPair : PairedOutputs)
			{
				const FMetasoundFrontendClassVertex& OriginalVertex = OutputPair.Get<0>();
				FMetasoundFrontendClassVertex NewVertex = OutputPair.Get<1>();

#if WITH_EDITOR
				// Cache off node locations to push to new node
				// Default add output node to origin.
				TMap<FGuid, FVector2D> Locations;
				Locations.Add(FGuid::NewGuid(), FVector2D { 0.f, 0.f });
#endif // WITH_EDITOR

				FOutputHandle ConnectedOutput = IOutputController::GetInvalidHandle();
				if (const FMetasoundFrontendClassOutput* ClassOutput = GraphHandle->FindClassOutputWithName(OriginalVertex.Name).Get())
				{
					if (FMetasoundFrontendVertex::IsFunctionalEquivalent(*ClassOutput, OriginalVertex))
					{
						NewVertex.NodeID = ClassOutput->NodeID;

#if WITH_EDITOR
						// Interface members do not serialize text to avoid localization
						// mismatches between assets and interfaces defined in code.
						NewVertex.Metadata.SetSerializeText(false);
#endif // WITH_EDITOR

						FNodeHandle OriginalOutputNode = GraphHandle->GetOutputNodeWithName(OriginalVertex.Name);

#if WITH_EDITOR
						Locations = OriginalOutputNode->GetNodeStyle().Display.Locations;
#endif // WITH_EDITOR

						FInputHandle Input = OriginalOutputNode->GetInputWithVertexName(OriginalVertex.Name);
						ConnectedOutput = Input->GetConnectedOutput();
						GraphHandle->RemoveOutputVertex(OriginalVertex.Name);
					}
				}

				FNodeHandle NewOutputNode = GraphHandle->AddOutputVertex(NewVertex);

#if WITH_EDITOR
				if (Locations.Num() > 0)
				{
					FMetasoundFrontendNodeStyle Style = NewOutputNode->GetNodeStyle();
					Style.Display.Locations = Locations;
					NewOutputNode->SetNodeStyle(Style);
				}
#endif // WITH_EDITOR

				// Copy prior node connections
				FInputHandle InputHandle = NewOutputNode->GetInputWithVertexName(NewVertex.Name);
				ConnectedOutput->Connect(*InputHandle);
			}

			return !PairedInputs.IsEmpty() || !PairedOutputs.IsEmpty();
		}

		bool FModifyRootGraphInterfaces::Transform(FDocumentHandle InDocument) const
		{
			bool bDidEdit = false;

			FGraphHandle GraphHandle = InDocument->GetRootGraph();
			if (ensure(GraphHandle->IsValid()))
			{
				bDidEdit |= UpdateInterfacesInternal(InDocument);

				const bool bAddedVertices = AddMissingVertices(GraphHandle);
				bDidEdit |= bAddedVertices;

				bDidEdit |= SwapPairedVertices(GraphHandle);
				bDidEdit |= RemoveUnsupportedVertices(GraphHandle);

#if WITH_EDITORONLY_DATA
				if (bAddedVertices && bSetDefaultNodeLocations)
				{
					UpdateAddedVertexNodePositions(GraphHandle);
				}
#endif // WITH_EDITORONLY_DATA
			}

			return bDidEdit;
		}

		bool FModifyRootGraphInterfaces::Transform(FMetasoundFrontendDocument& InOutDocument) const
		{
			FDocumentAccessPtr DocAccessPtr = MakeAccessPtr<FDocumentAccessPtr>(InOutDocument.AccessPoint, InOutDocument);
			return Transform(FDocumentController::CreateDocumentHandle(DocAccessPtr));
		}

		bool FModifyRootGraphInterfaces::UpdateInterfacesInternal(FDocumentHandle DocumentHandle) const
		{
			for (const FMetasoundFrontendInterface& Interface : InterfacesToRemove)
			{
				DocumentHandle->RemoveInterfaceVersion(Interface.Metadata.Version);
			}

			for (const FMetasoundFrontendInterface& Interface : InterfacesToAdd)
			{
				DocumentHandle->AddInterfaceVersion(Interface.Metadata.Version);
			}

			return !InterfacesToRemove.IsEmpty() || !InterfacesToAdd.IsEmpty();
		}

#if WITH_EDITORONLY_DATA
		void FModifyRootGraphInterfaces::UpdateAddedVertexNodePositions(FGraphHandle GraphHandle) const
		{
			auto SortAndPlaceMemberNodes = [&GraphHandle](EMetasoundFrontendClassType ClassType, TSet<FName>& AddedNames, TFunctionRef<int32(const FVertexName&)> InGetSortOrder)
			{
				// Add graph member nodes by sort order
				TSortedMap<int32, FNodeHandle> SortOrderToName;
				GraphHandle->IterateNodes([&GraphHandle, &SortOrderToName, &InGetSortOrder](FNodeHandle NodeHandle)
				{
					const int32 Index = InGetSortOrder(NodeHandle->GetNodeName());
					SortOrderToName.Add(Index, NodeHandle);
				}, ClassType);

				// Prime the first location as an offset prior to an existing location (as provided by a swapped member)
				//  to avoid placing away from user's active area if possible.
				FVector2D NextLocation = { 0.0f, 0.0f };
				{
					int32 NumBeforeDefined = 1;
					for (const TPair<int32, FNodeHandle>& Pair : SortOrderToName)
					{
						const FConstNodeHandle& NodeHandle = Pair.Value;
						const FName NodeName = NodeHandle->GetNodeName();
						if (AddedNames.Contains(NodeName))
						{
							NumBeforeDefined++;
						}
						else
						{
							const TMap<FGuid, FVector2D>& Locations = NodeHandle->GetNodeStyle().Display.Locations;
							if (!Locations.IsEmpty())
							{
								auto It = Locations.CreateConstIterator();
								const TPair<FGuid, FVector2D>& Location = *It;
								NextLocation = Location.Value - (NumBeforeDefined * DisplayStyle::NodeLayout::DefaultOffsetY);
								break;
							}
						}
					}
				}

				// Iterate through sorted map in sequence, slotting in new locations after existing swapped nodes with predefined locations.
				for (TPair<int32, FNodeHandle>& Pair : SortOrderToName)
				{
					FNodeHandle& NodeHandle = Pair.Value;
					const FName NodeName = NodeHandle->GetNodeName();
					if (AddedNames.Contains(NodeName))
					{
						FMetasoundFrontendNodeStyle NewStyle = NodeHandle->GetNodeStyle();
						NewStyle.Display.Locations.Add(FGuid::NewGuid(), NextLocation);
						NextLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
						NodeHandle->SetNodeStyle(NewStyle);
					}
					else
					{
						for (const TPair<FGuid, FVector2D>& Location : NodeHandle->GetNodeStyle().Display.Locations)
						{
							NextLocation = Location.Value + DisplayStyle::NodeLayout::DefaultOffsetY;
						}
					}
				}
			};

			// Sort/Place Inputs
			{
				TSet<FName> AddedNames;
				Algo::Transform(InputsToAdd, AddedNames, [](const FInputData& InputData) { return InputData.Input.Name; });
				auto GetInputSortOrder = [&GraphHandle](const FVertexName& InVertexName) { return GraphHandle->GetSortOrderIndexForInput(InVertexName); };
				SortAndPlaceMemberNodes(EMetasoundFrontendClassType::Input, AddedNames, GetInputSortOrder);
			}

			// Sort/Place Outputs
			{
				TSet<FName> AddedNames;
				Algo::Transform(OutputsToAdd, AddedNames, [](const FOutputData& OutputData) { return OutputData.Output.Name; });
				auto GetOutputSortOrder = [&GraphHandle](const FVertexName& InVertexName) { return GraphHandle->GetSortOrderIndexForOutput(InVertexName); };
				SortAndPlaceMemberNodes(EMetasoundFrontendClassType::Output, AddedNames, GetOutputSortOrder);
			}
		}

		bool FAutoUpdateRootGraph::Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FAutoUpdateRootGraph::Transform);
			
			// If document config provided, run configure
			if (TConstStructView<FMetaSoundFrontendDocumentTemplate> DocTemplate = InOutBuilderToTransform.GetConstDocumentTemplate(); DocTemplate.IsValid())
			{
				InOutBuilderToTransform.ConfigureDocument();
				
				TScriptInterface<IMetaSoundDocumentInterface> DocInterface(&InOutBuilderToTransform.CastDocumentObjectChecked<UObject>());
				DocInterface->ConformObjectToDocument();
				InOutBuilderToTransform.SynchronizeDependencyMetadata();
			}

			// Update external node dependencies
			// This should be run post document configuration as well as a template's implementing code may
			// or may not be updated to generate newest dependencies.
			return UpdateExternalDependencies(InOutBuilderToTransform);
		}

		bool FAutoUpdateRootGraph::CanAutoUpdate(const FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InNodeID, const FGuid& InPageID)
		{
			// This logic should mirror logic in FAutoUpdateRootGraph::UpdateExternalDependencies
			const FMetasoundFrontendNode* Node = InOutBuilderToTransform.FindNode(InNodeID, &InPageID);
			if (!Node)
			{
				return false;
			}
			const FMetasoundFrontendClass* Class = InOutBuilderToTransform.FindDependency(Node->ClassID);
			if (!Class)
			{
				return false;
			}
			
			FMetaSoundClassInfo RegistryClassInfo;
			if (ISearchEngine::Get().FindRegisteredClass(Class->Metadata.GetClassName(), Class->Metadata.GetVersion().Major, RegistryClassInfo))
			{
				// Minor version is same as registry version, but still require update if there are pending interface changes
				if (RegistryClassInfo.Version == Class->Metadata.GetVersion())
				{
					if (HasInterfaceUpdate(InOutBuilderToTransform, InNodeID, InPageID))
					{
						return true;
					}
					// No interface update and already at same version as registry
					return HasAutoAppliedCustomUpdateTransform(Class->Metadata.GetClassName(), Class->Metadata.GetVersion().Major);
				}
				// Minor version is not updated to registry minor version
				return true;
			}
			// Check for autoapplied major version transform
			return HasAutoAppliedCustomUpdateTransform(Class->Metadata.GetClassName(), Class->Metadata.GetVersion().Major);
		}

		bool FAutoUpdateRootGraph::HasAutoAppliedCustomUpdateTransform(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion)
		{
			if (const TSharedPtr<const FBaseNodeUpdateTransform> MajorCustomUpdateTransform = INodeClassRegistry::Get()->FindCustomNodeUpdateTransform(InClassName, InMajorVersion))
			{
				return MajorCustomUpdateTransform && MajorCustomUpdateTransform->ShouldAutoApply();
			}
			return false;
		}
		
		bool FAutoUpdateRootGraph::HasInterfaceUpdate(const FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InNodeID, const FGuid& InPageID)
		{
			constexpr bool bUseHighestMinorVersion = true;
			constexpr bool bForceRegenerateClassInterfaceOverride = true;
			FClassInterfaceUpdates InterfaceUpdates;
		
			InOutBuilderToTransform.DiffAgainstRegistryInterface(InNodeID, InPageID, bUseHighestMinorVersion, InterfaceUpdates, bForceRegenerateClassInterfaceOverride);
			return InterfaceUpdates.ContainsChanges() || !InOutBuilderToTransform.NodeInterfaceMatchesClassInterface(InNodeID, InPageID);
		}

		// Update node to highest minor version for this major version. Even if the node is already at the highest minor version, there may be unversioned interface updates
		bool FAutoUpdateRootGraph::ProcessMinorVersionUpdate(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FNodeClassRegistryKey& InCurrentNodeKey, const FNodeClassRegistryKey& InHighestRegistryVersionKey, const FGuid& InNodeID, const FGuid& InPageID, FString* InOutAutoUpdateReasonString)
		{
			if (InHighestRegistryVersionKey.IsValid())
			{
				// The node is already at the highest minor version for this major version and cannot be updated further
				if (InCurrentNodeKey.Version.Minor == InHighestRegistryVersionKey.Version.Minor)
				{
					// Check for interface update
					if (HasInterfaceUpdate(InOutBuilderToTransform, InNodeID, InPageID))
					{
						TSharedRef<FBaseNodeUpdateTransform> Transform = MakeShared<FBaseNodeUpdateTransform>(InCurrentNodeKey);
						Transform->Update(InOutBuilderToTransform, InNodeID, &InPageID);
						if (InOutAutoUpdateReasonString)
						{
							*InOutAutoUpdateReasonString = TEXT("Interface change detected");
						}
						return true;
					}
					// Node is fully updated to the highest minor version and has no pending interface changes
					return false;
				}
				else
				{
					// Out of date, so use default update transform to highest registry minor version
					TSharedRef<FBaseNodeUpdateTransform> Transform = MakeShared<FBaseNodeUpdateTransform>(InHighestRegistryVersionKey);
					Transform->Update(InOutBuilderToTransform, InNodeID, &InPageID);
					if (InOutAutoUpdateReasonString)
					{
						*InOutAutoUpdateReasonString = TEXT("Default minor version update to ") + InHighestRegistryVersionKey.Version.ToString();
					}
					return true;
				}
			}
			// It is possible the major version has been deprecated/removed, which is valid if there is an autoapplied major transform.
			// But at this point there are no minor version updates that can be applied
			return false;
		}
		
		bool FAutoUpdateRootGraph::ProcessMajorVersionUpdate(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FNodeClassRegistryKey& InCurrentNodeRegistryKey, const FGuid& InNodeID, const FGuid& InPageID, FNodeClassRegistryKey& OutNewNodeRegistryKey)
		{
			const TSharedPtr<const FBaseNodeUpdateTransform> Transform = INodeClassRegistry::Get()->FindCustomNodeUpdateTransform(InCurrentNodeRegistryKey.ClassName, InCurrentNodeRegistryKey.Version.Major);
			if (Transform && Transform->ShouldAutoApply())
			{
				Transform->Update(InOutBuilderToTransform, InNodeID, &InPageID);
				OutNewNodeRegistryKey = Transform->GetNewNodeClassKey();
				return true;
			}
			// If there is not a custom major version update transform that can be autoapplied, there's no major update to be processed in autoupdate
			return false;
		}

		bool FAutoUpdateRootGraph::UpdateExternalDependencies(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FAutoUpdateRootGraph::UpdateExternalDependencies);
		
			bool bDidEdit = false;
			const FMetasoundFrontendGraphClass& RootGraph = InOutBuilderToTransform.GetConstDocumentChecked().RootGraph;
			RootGraph.IterateGraphPages([this, &InOutBuilderToTransform, &bDidEdit](const FMetasoundFrontendGraph& Graph)
			{
				const FGuid PageID = Graph.PageID;
					
				// Make a copy of info from node array because updates can modify the node array
				using FNodeInfo = TPair<FGuid/*NodeID*/, FNodeClassRegistryKey/*NodeRegistryKey*/>;	
				TArray<FNodeInfo> NodeInfos;
				InOutBuilderToTransform.IterateNodesByClassType([&](const FMetasoundFrontendClass& Class, const FMetasoundFrontendNode& Node)
				{
					NodeInfos.Emplace(FNodeInfo(Node.GetID(), FNodeClassRegistryKey(Class.Metadata)));
				}, EMetasoundFrontendClassType::External, &PageID);

				for (const FNodeInfo& NodeInfo : NodeInfos)
				{
					const FGuid& NodeID = NodeInfo.Key;
					FNodeClassRegistryKey CurrentNodeClassKey = NodeInfo.Value;
					
					// Process minor and major version updates until no more updates can be applied	
					bool bUpdated = true;
					while (bUpdated)
					{
						bUpdated = false;
						
						FMetaSoundClassInfo RegistryClassInfo;
						ISearchEngine::Get().FindRegisteredClass(CurrentNodeClassKey.ClassName, CurrentNodeClassKey.Version.Major, RegistryClassInfo);
						const FNodeClassRegistryKey RegistryClassKey = RegistryClassInfo.ToRegistryKey();
						
						FString AutoUpdateReasonLog;
						if (ProcessMinorVersionUpdate(InOutBuilderToTransform, CurrentNodeClassKey, RegistryClassKey, NodeID, PageID, &AutoUpdateReasonLog))
						{
							METASOUND_VERSIONING_LOG(Display, TEXT("Auto-Updating '%s' node class '%s': %s"), *DebugAssetPath, *CurrentNodeClassKey.ToString(), *AutoUpdateReasonLog);
							bUpdated = true;
							bDidEdit |= true;
							CurrentNodeClassKey = RegistryClassKey;
						}
						if (ProcessMajorVersionUpdate(InOutBuilderToTransform, CurrentNodeClassKey, NodeID, PageID, /*NewRegistryKey*/CurrentNodeClassKey))
						{
							METASOUND_VERSIONING_LOG(Display, TEXT("Auto-Updating '%s' node class '%s': Custom autoapplied major version update"), *DebugAssetPath, *CurrentNodeClassKey.ToString());
							bUpdated = true;
							bDidEdit |= true;
						}
					}
						
					// Check that final node class is valid (matches something in the registry)
					FMetaSoundClassInfo RegistryClassInfo;
					if (!ISearchEngine::Get().FindRegisteredClass(CurrentNodeClassKey.ClassName, CurrentNodeClassKey.Version.Major, RegistryClassInfo))
					{
						METASOUND_VERSIONING_LOG(Error, TEXT("Failure Auto-Updating '%s': Final node class '%s' does not exist in the registry"), *DebugAssetPath, *CurrentNodeClassKey.ToString());
					}
					bDidEdit |= bUpdated;
				}	
			});
			
			if (bDidEdit)
			{
				InOutBuilderToTransform.RemoveUnusedDependencies();
				InOutBuilderToTransform.SynchronizeDependencyMetadata();
			}
			return bDidEdit;
		}
		
		bool FAutoUpdateRootGraph::Transform(FDocumentHandle InDocument)
		{
			return false;
		}
#endif // WITH_EDITORONLY_DATA

		bool FRebuildPresetRootGraph::Transform(FDocumentHandle InDocument) const
		{
			return false;
		}
	
		FRebuildPresetRootGraph::FRebuildPresetRootGraph(const FMetasoundFrontendDocument& InReferencedDocument)
		{
		}

		bool FRebuildPresetRootGraph::Transform(FMetasoundFrontendDocument& InDocument) const
		{
			return false;
		}

		bool FRebuildPresetRootGraph::Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::Frontend::FRebuildPresetRootGraph::Transform);

			// This function is to be moved (see FMetaSoundFrontendPresetTemplate) and should no longer be directly called outside that context.
			const FMetaSoundFrontendPresetTemplate* PresetTemplate = InOutBuilderToTransform.GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>();
			if (!PresetTemplate || !ParentBuilder)
			{
				return false;
			}
	
			// Determine the inputs and outputs needed in the wrapping graph. Also
			// cache any exiting literals that have been set on the wrapping graph.
			TArray<FMetasoundFrontendClassInput> ClassInputs = GenerateRequiredClassInputs(InOutBuilderToTransform);
			TArray<FMetasoundFrontendClassOutput> ClassOutputs = GenerateRequiredClassOutputs(InOutBuilderToTransform);

#if WITH_EDITORONLY_DATA
			// Cache off member metadata so it be can be readded if necessary after the graph is cleared 
			FMemberIDToMetadataMap CachedMemberMetadata = CacheMemberMetadata(*PresetTemplate, InOutBuilderToTransform);
#endif // WITH_EDITORONLY_DATA

			FGuid PresetNodeID;
			InOutBuilderToTransform.IterateNodesByClassType([&](const FMetasoundFrontendClass&, const FMetasoundFrontendNode& Node)
			{
				PresetNodeID = Node.GetID();
			}, EMetasoundFrontendClassType::External);

			if (!PresetNodeID.IsValid())
			{
				// This ID was originally being set to FGuid::NewGuid. 
				// If you were reliant on that ID, please resave the asset so it is serialized with a valid ID
				const FMetasoundFrontendDocument& DocumentToTransform = InOutBuilderToTransform.GetConstDocumentChecked();
				PresetNodeID = DocumentToTransform.RootGraph.ID;
			}

			// Clear the root graph so it can be rebuilt.
			TSharedRef<FDocumentModifyDelegates> ModifyDelegates = MakeShared<FDocumentModifyDelegates>(InOutBuilderToTransform.GetDocumentDelegates());
			InOutBuilderToTransform.ClearDocument(ModifyDelegates);

#if WITH_EDITORONLY_DATA
			InOutBuilderToTransform.SetIsGraphEditable(false);
#endif // WITH_EDITORONLY_DATA

			// Ensure preset interfaces match those found in referenced graph.  Referenced graph is assumed to be
			// well-formed (i.e. all inputs/outputs/environment variables declared by interfaces are present, and
			// of proper name & data type). Pass in parent builder to copy member ids so added member ids can be consistent
			const FMetasoundFrontendDocument& ParentDocument = ParentBuilder->GetConstDocumentChecked();
			const TSet<FMetasoundFrontendVersion>& RefInterfaceVersions = ParentDocument.Interfaces;
			for (const FMetasoundFrontendVersion& Version : RefInterfaceVersions)
			{
				InOutBuilderToTransform.AddInterface(Version.Name, /*bAddUserModifiableInterfaceOnly = */false, ParentBuilder);
			}

			// Add referenced node
			const FMetasoundFrontendNode* ReferencedNode = InOutBuilderToTransform.AddGraphNode(ParentDocument.RootGraph, PresetNodeID);
			check(ReferencedNode);
			
#if WITH_EDITOR
			// Set node location, offset to be to the right of input nodes
			const FGuid EdNodeGuid = FGuid::NewGuid(); // EdNodes are now never serialized and are transient, so just assign here
			InOutBuilderToTransform.SetNodeLocation(PresetNodeID, DisplayStyle::NodeLayout::DefaultOffsetX, &EdNodeGuid);
#endif // WITH_EDITOR

			// Connect parent graph to referenced graph
			AddAndConnectInputs(*PresetTemplate, ClassInputs, InOutBuilderToTransform, PresetNodeID);
			AddAndConnectOutputs(ClassOutputs, InOutBuilderToTransform, PresetNodeID);

#if WITH_EDITORONLY_DATA
			AddMemberMetadata(CachedMemberMetadata, InOutBuilderToTransform);
#endif // WITH_EDITORONLY_DATA

			return true;
		}

#if WITH_EDITORONLY_DATA
		void FRebuildPresetRootGraph::AddMemberMetadata(const FMemberIDToMetadataMap& InCachedMemberMetadata, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const
		{
			// Add member metadata if a member with the corresponding node ID exists in the preset graph
			if (!InCachedMemberMetadata.IsEmpty())
			{
				for (const TPair<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>& MemberMetadataPair : InCachedMemberMetadata)
				{
					if (InOutBuilderToTransform.FindNode(MemberMetadataPair.Key))
					{
						InOutBuilderToTransform.SetMemberMetadata(*MemberMetadataPair.Value);
					}
				}
			}
		}
		
		FRebuildPresetRootGraph::FMemberIDToMetadataMap FRebuildPresetRootGraph::CacheMemberMetadata(const FMetaSoundFrontendPresetTemplate& PresetTemplate, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const
		{
			FMemberIDToMetadataMap CachedMemberMetadata;

			auto CreateNewLiteral = [&](UMetaSoundFrontendMemberMetadata* TemplateObject, FGuid MemberID) -> UMetaSoundFrontendMemberMetadata*
			{
				if (TemplateObject)
				{
					UMetaSoundFrontendMemberMetadata* NewMetadata = NewObject<UMetaSoundFrontendMemberMetadata>(&InOutBuilderToTransform.CastDocumentObjectChecked<UObject>(), TemplateObject->GetClass(), FName(), RF_Transactional, TemplateObject);
					check(NewMetadata);
					NewMetadata->MemberID = MemberID;
					return NewMetadata;
				}
				return nullptr;
			};
			
			const FMetasoundFrontendDocument& ParentDocument = ParentBuilder->GetConstDocumentChecked();
			const FMetasoundFrontendDocument& DocumentToTransform = InOutBuilderToTransform.GetConstDocumentChecked();
			for (const FMetasoundFrontendClassInput& ParentClassInput : ParentDocument.RootGraph.GetDefaultInterface().Inputs)
			{
				UMetaSoundFrontendMemberMetadata* MemberMetadata = nullptr;
				// Member id is id in InOutBuilder, which may not be the same as the node id of the class input of ParentBuilder
				FGuid MemberID;
				const FMetasoundFrontendClassInput* ExistingClassInput = InOutBuilderToTransform.FindGraphInput(ParentClassInput.Name);
				if (!ExistingClassInput)
				{
					// Try to search by id for the case of parent input rename
					const TArray<FMetasoundFrontendClassInput>& PresetInputs = DocumentToTransform.RootGraph.GetDefaultInterface().Inputs;
					ExistingClassInput = PresetInputs.FindByPredicate(
						[&](const FMetasoundFrontendClassInput& PresetInput)
					{
						return PresetInput.NodeID == ParentClassInput.NodeID;
					});
				}

				if (ExistingClassInput && ExistingClassInput->TypeName == ParentClassInput.TypeName)
				{
					MemberID = ExistingClassInput->NodeID;
					// If the input vertex already exists in the parent graph,
					// check if parent should be used or not from set of managed
					// input names.
					if (const FPresetVertexMetadata* VertexMetadata = PresetTemplate.FindConstVertexMetadata<FPresetVertexMetadata>(MemberID);
						VertexMetadata && !VertexMetadata->bOverrideInheritedDefault)
					{
						UMetaSoundFrontendMemberMetadata* ParentMetadata = ParentBuilder->FindMemberMetadata(ParentClassInput.NodeID);
						MemberMetadata = CreateNewLiteral(ParentMetadata, MemberID);
					}
					else
					{
						// Use existing defaults
						MemberMetadata = InOutBuilderToTransform.FindMemberMetadata(ExistingClassInput->NodeID);
					}
				}
				else
				{
					UMetaSoundFrontendMemberMetadata* ParentMetadata = ParentBuilder->FindMemberMetadata(ParentClassInput.NodeID);
					MemberID = ParentClassInput.NodeID;
					MemberMetadata = CreateNewLiteral(ParentMetadata, MemberID);
				}

				if (MemberMetadata)
				{
					CachedMemberMetadata.Emplace(MemberID, MemberMetadata);
				}
			}

			return CachedMemberMetadata;
		}
#endif // WITH_EDITORONLY_DATA

		void FRebuildPresetRootGraph::AddAndConnectInputs(const FMetaSoundFrontendPresetTemplate& PresetTemplate, const TArray<FMetasoundFrontendClassInput>& InClassInputs, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InReferencedNodeID) const
		{
			TArray<const FMetasoundFrontendVertex*> ReferencedNodeInputVertices;

#if WITH_EDITORONLY_DATA
			// Input template node IDs with associated connected vertex (from ReferencedNodeInputVertices)
			TMap<const FMetasoundFrontendVertex*, const FGuid> InputTemplateNodeIDs;
#endif // WITH_EDITORONLY_DATA

			for (const FMetasoundFrontendClassInput& ClassInput : InClassInputs)
			{
				const FName InputName = ClassInput.Name;
				// Input may have already been added if interface member, but defaults need to be overridden 
				const FMetasoundFrontendNode* InputNode = InOutBuilderToTransform.FindGraphInputNode(InputName);
				if (!InputNode)
				{
					InputNode = InOutBuilderToTransform.AddGraphInput(ClassInput);
				}
				else
				{
					InOutBuilderToTransform.SetGraphInputDefaults(InputName, ClassInput.GetDefaults());

				}

				check(InputNode);
				const FGuid InputNodeID = InputNode->GetID();
				const FMetasoundFrontendVertex* InputNodeOutputVertex = InOutBuilderToTransform.FindNodeOutput(InputNodeID, InputName);
				check(InputNodeOutputVertex);
				const FMetasoundFrontendVertex* ReferencedNodeInputVertex = InOutBuilderToTransform.FindNodeInput(InReferencedNodeID, InputName);
				check(ReferencedNodeInputVertex);
				
				ReferencedNodeInputVertices.Add(ReferencedNodeInputVertex);

				// If editor-only data, add template node and connections
#if WITH_EDITORONLY_DATA
				// template node takes on data type of concrete input node's output type
				const FName DataType = InputNode->Interface.Outputs.Last().TypeName;
				
				// Add inputs and space appropriately
				const INodeTemplate* InputTemplate = INodeTemplateRegistry::Get().FindTemplate(FInputNodeTemplate::ClassName);
				check(InputTemplate);

				FNodeTemplateGenerateInterfaceParams Params{ { }, { DataType } };
				const FMetasoundFrontendNode* TemplateNode = InOutBuilderToTransform.AddNodeByTemplate(*InputTemplate, MoveTemp(Params));

				check(TemplateNode);
				InputTemplateNodeIDs.Add(ReferencedNodeInputVertex, TemplateNode->GetID());

				const FGuid TemplateNodeInputVertexID = TemplateNode->Interface.Inputs.Last().VertexID;
				const FGuid TemplateNodeOutputVertexID = TemplateNode->Interface.Outputs.Last().VertexID;
				InOutBuilderToTransform.AddEdge(FMetasoundFrontendEdge
				{
					InputNodeID,
					InputNodeOutputVertex->VertexID,
					TemplateNode->GetID(),
					TemplateNodeInputVertexID
				});
				InOutBuilderToTransform.AddEdge(FMetasoundFrontendEdge
				{
					TemplateNode->GetID(),
					TemplateNodeOutputVertexID,
					InReferencedNodeID,
					ReferencedNodeInputVertex->VertexID
				});
#else // !WITH_EDITORONLY_DATA
				// If not editor-only data, just make connection directly between input and output node
				InOutBuilderToTransform.AddEdge(FMetasoundFrontendEdge
					{
						InputNodeID,
						InputNodeOutputVertex->VertexID,
						InReferencedNodeID,
						ReferencedNodeInputVertex->VertexID
					});

#endif  // !WITH_EDITORONLY_DATA
			}

#if WITH_EDITOR
			// Sort before adding nodes to graph layout & copy to preset (must be done after all
			// inputs/outputs are added but before setting locations to propagate effectively)
			const FMetasoundFrontendGraphClass& ParentRootGraph = ParentBuilder->GetConstDocumentChecked().RootGraph;
			FMetasoundFrontendInterfaceStyle Style = ParentRootGraph.GetDefaultInterface().GetInputStyle();

			// Sort vertices on referenced node
			// then use that order to order connected input nodes (which share the same names)
			auto GetInputDisplayName = [&InOutBuilderToTransform, &InReferencedNodeID](const FMetasoundFrontendVertex& Vertex)
				{
					return InOutBuilderToTransform.GetNodeInputDisplayName(InReferencedNodeID, Vertex.Name);
				};
			Style.SortVertices(ReferencedNodeInputVertices, GetInputDisplayName);

			InOutBuilderToTransform.SetInputStyle(MoveTemp(Style));

			// Set editor node locations, sorted to match referenced node input vertex order
			FVector2D InputNodeLocation = FVector2D::ZeroVector;
			for (const FMetasoundFrontendVertex* NodeInputVertex : ReferencedNodeInputVertices)
			{
				const FGuid* TemplateNodeID = InputTemplateNodeIDs.Find(NodeInputVertex);
				if (TemplateNodeID)
				{
					InOutBuilderToTransform.SetNodeLocation(*TemplateNodeID, InputNodeLocation);
					InputNodeLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
				}
			}
#endif // WITH_EDITOR
		}

		void FRebuildPresetRootGraph::AddAndConnectOutputs(const TArray<FMetasoundFrontendClassOutput>& InClassOutputs, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InReferencedNodeID) const
		{
			// Add outputs and space appropriately
			TArray<const FMetasoundFrontendVertex*> ReferencedNodeOutputVertices;

			for (const FMetasoundFrontendClassOutput& ClassOutput : InClassOutputs)
			{
				// Output may have already been added if interface member
				const FMetasoundFrontendNode* OutputNode = InOutBuilderToTransform.FindGraphOutputNode(ClassOutput.Name);
				if (!OutputNode)
				{
					OutputNode = InOutBuilderToTransform.AddGraphOutput(ClassOutput);
				}
				check(OutputNode);
				
				// Connect output node input vertex to corresponding referenced node output vertex. 
				const FMetasoundFrontendVertex* ReferencedNodeOutputVertex = InOutBuilderToTransform.FindNodeOutput(InReferencedNodeID, ClassOutput.Name);
				check(ReferencedNodeOutputVertex);
				const FMetasoundFrontendVertex* OutputNodeInputVertex = InOutBuilderToTransform.FindNodeInput(OutputNode->GetID(), ClassOutput.Name);
				check(OutputNodeInputVertex);

				InOutBuilderToTransform.AddEdge(FMetasoundFrontendEdge
				{
					InReferencedNodeID,
					ReferencedNodeOutputVertex->VertexID,
					OutputNode->GetID(),
					OutputNodeInputVertex->VertexID
				});
				ReferencedNodeOutputVertices.Add(ReferencedNodeOutputVertex);
			}

#if WITH_EDITOR
			// Sort before adding nodes to graph layout & copy to preset (must be done after all
			// inputs/outputs are added but before setting locations to propagate effectively)
			const FMetasoundFrontendGraphClass& ParentRootGraph = ParentBuilder->GetConstDocumentChecked().RootGraph;
			FMetasoundFrontendInterfaceStyle Style = ParentRootGraph.GetDefaultInterface().GetOutputStyle();

			// Sort vertices on referenced node
			// then use that order to order connected output nodes (which share the same names)
			auto GetOutputDisplayName = [&InOutBuilderToTransform, &InReferencedNodeID](const FMetasoundFrontendVertex& Vertex)
			{
				return InOutBuilderToTransform.GetNodeOutputDisplayName(InReferencedNodeID, Vertex.Name);
			};
			Style.SortVertices(ReferencedNodeOutputVertices, GetOutputDisplayName);

			InOutBuilderToTransform.SetOutputStyle(MoveTemp(Style));

			// Set output node locations
			FVector2D OutputNodeLocation = (2 * DisplayStyle::NodeLayout::DefaultOffsetX);
			for (const FMetasoundFrontendVertex* OutputVertex : ReferencedNodeOutputVertices)
			{
				const FName NodeName = OutputVertex->Name;
				const FGuid OutputNodeID = InOutBuilderToTransform.FindGraphOutputNode(NodeName)->GetID();

				// Set editor node locations
				const FGuid EdNodeGuid = FGuid::NewGuid(); // EdNodes are now never serialized and are transient, so just assign here
				InOutBuilderToTransform.SetNodeLocation(OutputNodeID, OutputNodeLocation);
				OutputNodeLocation += DisplayStyle::NodeLayout::DefaultOffsetY;
			}
#endif // WITH_EDITOR
		}

		TArray<FMetasoundFrontendClassInput> FRebuildPresetRootGraph::GenerateRequiredClassInputs(FMetaSoundFrontendDocumentBuilder& InDocumentToTransformBuilder) const
		{
			TArray<FMetasoundFrontendClassInput> ClassInputs;
			check(ParentBuilder);

			const FMetasoundFrontendDocument& ParentDocument = ParentBuilder->GetConstDocumentChecked();
			const FMetasoundFrontendDocument& DocumentToTransform = InDocumentToTransformBuilder.GetConstDocumentChecked();

			const FMetaSoundFrontendPresetTemplate* PresetTemplate = InDocumentToTransformBuilder.GetConstTemplateAs<FMetaSoundFrontendPresetTemplate>();
			check(PresetTemplate);

			// Iterate through all input nodes of referenced graph
			for (const FMetasoundFrontendClassInput& ParentClassInput : ParentDocument.RootGraph.GetDefaultInterface().Inputs)
			{
				// Copy class input and reset defaults and id
				FMetasoundFrontendClassInput NewClassInput = ParentClassInput;
				NewClassInput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(DocumentToTransform);
				NewClassInput.ResetDefaults(/*bInitializeDefaultPage=*/false);

				// Old assets may not have matching node ids between parent and preset inputs, so search by name first
				const FMetasoundFrontendClassInput* ExistingClassInput = InDocumentToTransformBuilder.FindGraphInput(ParentClassInput.Name);
				if (ExistingClassInput)
				{
					NewClassInput.NodeID = ExistingClassInput->NodeID;
				}
				else
				{
					// Try to search by id for the case of parent input rename
					const TArray<FMetasoundFrontendClassInput>& PresetInputs = DocumentToTransform.RootGraph.GetDefaultInterface().Inputs;
					ExistingClassInput = PresetInputs.FindByPredicate(
						[&](const FMetasoundFrontendClassInput& PresetInput)
					{
						return PresetInput.NodeID == ParentClassInput.NodeID;
					});
				}
				
				auto InheritDefaultsFromInput = [&NewClassInput](const FMetasoundFrontendClassInput& InInput)
				{
					InInput.IterateDefaults([&NewClassInput](const FGuid& PageID, const FMetasoundFrontendLiteral& Literal)
					{
						NewClassInput.AddDefault(PageID) = Literal;
					});
				};
				
				// Existing class input is invalid for copying defaults if the type has changed
				if (ExistingClassInput && ExistingClassInput->TypeName == ParentClassInput.TypeName)
				{
					bool bInherited = false;
					// If the input vertex already exists in the parent graph,
					// check if parent should be used or not from set of managed
					// input names.
					if (const FPresetVertexMetadata* VertexMetadata = PresetTemplate->FindConstVertexMetadata<FPresetVertexMetadata>(ExistingClassInput->NodeID))
					{
						if (!VertexMetadata->bOverrideInheritedDefault)
						{
							InheritDefaultsFromInput(ParentClassInput);
							bInherited = true;
						}
					}

					// Use existing defaults from preset
					if (!bInherited)
					{
						InheritDefaultsFromInput(*ExistingClassInput);
					}
				}
				else
				{
					InheritDefaultsFromInput(ParentClassInput);
				}

				ClassInputs.Add(MoveTemp(NewClassInput));
			}

			return ClassInputs;
		}
		
		TArray<FMetasoundFrontendClassOutput> FRebuildPresetRootGraph::GenerateRequiredClassOutputs(FMetaSoundFrontendDocumentBuilder& InDocumentToTransformBuilder) const
		{
			TArray<FMetasoundFrontendClassOutput> ClassOutputs;
			check(ParentBuilder);

			const FMetasoundFrontendDocument& ParentDocument = ParentBuilder->GetConstDocumentChecked();
			const FMetasoundFrontendDocument& DocumentToTransform = InDocumentToTransformBuilder.GetConstDocumentChecked();

			for (const FMetasoundFrontendClassOutput& ParentClassOutput : ParentDocument.RootGraph.GetDefaultInterface().Outputs)
			{
				// Copy class output and reset defaults and id
				FMetasoundFrontendClassOutput NewClassOutput = ParentClassOutput;
				NewClassOutput.VertexID = FDocumentIDGenerator::Get().CreateVertexID(DocumentToTransform);

				if (const FMetasoundFrontendClassOutput* ExistingClassOutput = InDocumentToTransformBuilder.FindGraphOutput(ParentClassOutput.Name))
				{
					NewClassOutput.NodeID = ExistingClassOutput->NodeID;
				}
				ClassOutputs.Add(MoveTemp(NewClassOutput));
			}
			return ClassOutputs;
		}

		bool FRenameRootGraphClass::Transform(FDocumentHandle InDocument) const
		{
			return false;
		}

		bool FRenameRootGraphClass::Transform(FMetasoundFrontendDocument& InOutDocument) const
		{
			return false;
		}
	} // namespace Frontend
} // namespace Metasound
