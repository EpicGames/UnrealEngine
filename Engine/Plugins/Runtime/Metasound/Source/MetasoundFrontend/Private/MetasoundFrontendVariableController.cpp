// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendVariableController.h"
#include "MetasoundFrontendInvalidController.h"

namespace Metasound
{
	namespace Frontend
	{
		FVariableController::FVariableController(const FVariableController::FInitParams& InParams)
		: VariablePtr(InParams.VariablePtr)
		, OwningGraph(InParams.OwningGraph)
		{
		}

		bool FVariableController::IsValid() const
		{
			if (nullptr != VariablePtr.Get())
			{
				return OwningGraph->IsValid();
			}
			return false;
		}

		FGuid FVariableController::GetID() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->ID;
			}
			return Metasound::FrontendInvalidID;
		}
		
		/** Returns the data type name associated with this output. */
		const FName& FVariableController::GetDataType() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->TypeName;
			}
			return Invalid::GetInvalidName();
		}

		/** Returns the human readable name associated with this output. */
		const FName& FVariableController::GetName() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->Name;
			}
			return Invalid::GetInvalidName();
		}

		void FVariableController::SetName(const FName& InName)
		{
			if (FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				Variable->Name = InName;
			}
		}
		
#if WITH_EDITOR
		/** Returns the human readable name associated with this output. */
		FText FVariableController::GetDisplayName() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->DisplayName;
			}
			return Invalid::GetInvalidText();
		}

		void FVariableController::SetDisplayName(const FText& InDisplayName)
		{
			if (FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				Variable->DisplayName = InDisplayName;
			}
		}

		/** Returns the human readable description associated with this output. */
		FText FVariableController::GetDescription() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->Description;
			}
			return Invalid::GetInvalidText();
		}

		void FVariableController::SetDescription(const FText& InDescription)
		{
			if (FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				Variable->Description = InDescription;
			}
		}
#endif // WITH_EDITOR

		FNodeHandle FVariableController::FindMutatorNode()
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				if (Metasound::FrontendInvalidID != Variable->MutatorNodeID)
				{
					return OwningGraph->GetNodeWithID(Variable->MutatorNodeID);
				}
			}

			return INodeController::GetInvalidHandle();
		}

		FConstNodeHandle FVariableController::FindMutatorNode() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				if (Metasound::FrontendInvalidID != Variable->MutatorNodeID)
				{
					return OwningGraph->GetNodeWithID(Variable->MutatorNodeID);
				}
			}

			return INodeController::GetInvalidHandle();
		}
		TArray<FNodeHandle> FVariableController::FindAccessorNodes()
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return GetNodeArray(Variable->AccessorNodeIDs);
			}

			return TArray<FNodeHandle>();
		}
		TArray<FConstNodeHandle> FVariableController::FindAccessorNodes() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return GetNodeArray(Variable->AccessorNodeIDs);
			}

			return TArray<FConstNodeHandle>();
		}

		TArray<FNodeHandle> FVariableController::FindDeferredAccessorNodes()
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return GetNodeArray(Variable->DeferredAccessorNodeIDs);
			}

			return TArray<FNodeHandle>();
		}

		TArray<FConstNodeHandle> FVariableController::FindDeferredAccessorNodes() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return GetNodeArray(Variable->DeferredAccessorNodeIDs);
			}
			return TArray<FConstNodeHandle>();
		}
		
		/** Returns a FGraphHandle to the node which owns this output. */
		FGraphHandle FVariableController::GetOwningGraph()
		{
			return OwningGraph;
		}
		
		/** Returns a FConstGraphHandle to the node which owns this output. */
		FConstGraphHandle FVariableController::GetOwningGraph() const
		{
			return OwningGraph;
		}

		/** Returns the value for the given variable instance if set. */
		const FMetasoundFrontendLiteral& FVariableController::GetLiteral() const
		{
			if (const FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				return Variable->Literal;
			}

			return Invalid::GetInvalidLiteral();
		}

		/** Sets the value for the given variable instance */
		bool FVariableController::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
		{
			if (FMetasoundFrontendVariable* Variable = VariablePtr.Get())
			{
				Variable->Literal = InLiteral;
				return true;
			}

			return false;
		}

		TArray<FNodeHandle> FVariableController::GetNodeArray(const TArray<FGuid>& InNodeIDs)
		{
			TArray<FNodeHandle> Nodes;
			for (const FGuid& ID : InNodeIDs)
			{
				FNodeHandle Node = OwningGraph->GetNodeWithID(ID);
				if (Node->IsValid())
				{
					Nodes.Add(Node);
				}
			}
			return Nodes;
		}

		TArray<FConstNodeHandle> FVariableController::GetNodeArray(const TArray<FGuid>& InNodeIDs) const
		{
			TArray<FConstNodeHandle> Nodes;
			for (const FGuid& ID : InNodeIDs)
			{
				FConstNodeHandle Node = OwningGraph->GetNodeWithID(ID);
				if (Node->IsValid())
				{
					Nodes.Add(Node);
				}
			}
			return Nodes;
		}

		FDocumentAccess FVariableController::ShareAccess() 
		{
			FDocumentAccess Access;
			return Access;
		}

		FConstDocumentAccess FVariableController::ShareAccess() const 
		{
			FConstDocumentAccess Access;
			return Access;
		}
	}
}

