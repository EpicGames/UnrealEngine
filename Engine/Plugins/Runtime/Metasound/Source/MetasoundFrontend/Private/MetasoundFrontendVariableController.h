// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "Misc/Guid.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	namespace Frontend
	{
		class FVariableController : public IVariableController
		{
		public:
			struct FInitParams
			{
				FVariableAccessPtr VariablePtr;
				FGraphHandle OwningGraph;
			};

			UE_API FVariableController(const FInitParams& InParams);
			virtual ~FVariableController() = default;

			/** Returns true if the controller is in a valid state. */
			UE_API virtual bool IsValid() const override;

			UE_API virtual FGuid GetID() const override;
			
			/** Returns the data type name associated with this variable. */
			UE_API virtual const FName& GetDataType() const override;

			/** Returns the name associated with this variable. */
			UE_API virtual const FName& GetName() const override;

			/** Sets the name associated with this variable. */
			UE_API virtual void SetName(const FName& InName) override;
			
#if WITH_EDITOR
			/** Returns the human readable name associated with this variable. */
			UE_API virtual FText GetDisplayName() const override;

			/** Sets the human readable name associated with this variable. */
			UE_API virtual void SetDisplayName(const FText& InDisplayName) override;

			/** Returns the human readable description associated with this variable. */
			UE_API virtual FText GetDescription() const override;

			/** Sets the human readable description associated with this variable. */
			UE_API virtual void SetDescription(const FText& InDescription) override;
#endif // WITH_EDITOR

			/** Returns the mutator node associated with this variable. */
			UE_API virtual FNodeHandle FindMutatorNode() override;

			/** Returns the mutator node associated with this variable. */
			UE_API virtual FConstNodeHandle FindMutatorNode() const override;

			/** Returns the accessor nodes associated with this variable. */
			UE_API virtual TArray<FNodeHandle> FindAccessorNodes() override;

			/** Returns the accessor nodes associated with this variable. */
			UE_API virtual TArray<FConstNodeHandle> FindAccessorNodes() const override;

			/** Returns the deferred accessor nodes associated with this variable. */
			UE_API virtual TArray<FNodeHandle> FindDeferredAccessorNodes() override;

			/** Returns the deferred accessor nodes associated with this variable. */
			UE_API virtual TArray<FConstNodeHandle> FindDeferredAccessorNodes() const override;
			
			/** Returns a FGraphHandle to the node which owns this variable. */
			UE_API virtual FGraphHandle GetOwningGraph() override;
			
			/** Returns a FConstGraphHandle to the node which owns this variable. */
			UE_API virtual FConstGraphHandle GetOwningGraph() const override;

			/** Returns the value for the given variable instance if set. */
			UE_API virtual const FMetasoundFrontendLiteral& GetLiteral() const override;

			/** Sets the value for the given variable instance */
			UE_API virtual bool SetLiteral(const FMetasoundFrontendLiteral& InLiteral) override;

		private:
			UE_API virtual FConstDocumentAccess ShareAccess() const override;
			UE_API virtual FDocumentAccess ShareAccess() override;

			UE_API TArray<FNodeHandle> GetNodeArray(const TArray<FGuid>& InNodeIDs);
			UE_API TArray<FConstNodeHandle> GetNodeArray(const TArray<FGuid>& InNodeIDs) const;

			FVariableAccessPtr VariablePtr;
			FGraphHandle OwningGraph;
		};
	}
}

#undef UE_API
