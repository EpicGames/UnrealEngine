// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGraphNode.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "Templates/SharedPointer.h"

namespace Metasound::Frontend
{
	FGraphNode::FGraphOperatorFactoryAdapter::FGraphOperatorFactoryAdapter(const IGraph& InGraph)
	: Graph(&InGraph)
	, GraphFactory(InGraph.GetDefaultOperatorFactory())
	{
	}

	TUniquePtr<IOperator> FGraphNode::FGraphOperatorFactoryAdapter::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		// Create default data references for inputs which do not have data references
		// already provided. These need to be created here, but the defaults
		// cannot be piped into the wrapped graph. The wrapped graph cannot
		// be modified. 
		FInputVertexInterfaceData InputData = InParams.InputData;
		CreateDefaultsIfNotBound(InParams.OperatorSettings, InputData);

		FBuildOperatorParams ForwardParams 
		{
			*Graph,  // Point to correct INode instance
			InParams.OperatorSettings,
			InputData,
			InParams.Environment,
			InParams.Builder,
			InParams.GraphRenderCost
		};
		return GraphFactory->CreateOperator(ForwardParams, OutResults);
	}

	FGraphNode::FGraphNode(const FNodeInitData& InNodeInitData, TSharedRef<const IGraph> InGraphToWrap)
	: FGraphNode(FNodeData(InNodeInitData.InstanceName, InNodeInitData.InstanceID, InGraphToWrap->GetVertexInterface()), MoveTemp(InGraphToWrap))
	{
	}

	FGraphNode::FGraphNode(FNodeData InNodeData, TSharedRef<const IGraph> InGraphToWrap)
	: NodeData(MoveTemp(InNodeData))
	, Factory(MakeShared<FGraphOperatorFactoryAdapter>(*InGraphToWrap))
	, Graph(MoveTemp(InGraphToWrap))
	{
	}

	const FName& FGraphNode::GetInstanceName() const
	{
		// Use the instance name of underlying graph because it refers
		// to the actual asset name.
		return Graph->GetInstanceName();
	}

	const FGuid& FGraphNode::GetInstanceID() const
	{
		return NodeData.ID;
	}

	const FNodeClassMetadata& FGraphNode::GetMetadata() const
	{
		return Graph->GetMetadata();
	}

	const FVertexInterface& FGraphNode::GetVertexInterface() const
	{
		return NodeData.Interface;
	}

	void FGraphNode::SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		if (FInputDataVertex* Vertex = NodeData.Interface.GetInputInterface().Find(InVertexName))
		{
			Vertex->SetDefaultLiteral(InLiteral);
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Could not set default input. Could not find input vertex %ls on node %ls of node class %ls", *InVertexName.ToString(), *GetInstanceName().ToString(), *GetMetadata().ClassName.ToString());
		}
	}

	TSharedPtr<const IOperatorData> FGraphNode::GetOperatorData() const
	{
		return NodeData.OperatorData;
	}

	FOperatorFactorySharedRef FGraphNode::GetDefaultOperatorFactory() const
	{
		return Factory;
	}
}

