// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundBuilderInterface.h"
#include "MetasoundNodeInterface.h"
#include "Templates/SharedPointer.h"

namespace Metasound::Frontend
{
	// FGraphNode is used to create unique INodes based off of a IGraph.
	//
	// Individual nodes need to reflect their InstanceName and InstanceID, but otherwise
	// they simply encapsulate a shared set of behavior. To minimize memory usage, a single
	// shared IGraph is used for all nodes referring to the same IGraph.
	class FGraphNode : public INode
	{
		// This adapter class forwards the correct FBuilderOperatorParams
		// to the graph's operator creation method. Many operator creation
		// methods downcast the supplied INode in `FBuilderOperatorParams`
		// and so it is required that it point to the correct runtime instance
		// when calling CreateOperator(...)
		class FGraphOperatorFactoryAdapter : public IOperatorFactory 
		{
		public:
			FGraphOperatorFactoryAdapter(const IGraph& InGraph);
			virtual ~FGraphOperatorFactoryAdapter() = default;

			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;

		private:
			const IGraph* Graph; // Only store pointer because owning node keeps wrapped IGraph alive.  
			FOperatorFactorySharedRef GraphFactory;
		};

	public:
		FGraphNode(const FNodeInitData& InNodeInitData, TSharedRef<const IGraph> InGraphToWrap);

		FGraphNode(FNodeData InNodeData, TSharedRef<const IGraph> InGraphToWrap);

		virtual const FName& GetInstanceName() const override;

		virtual const FGuid& GetInstanceID() const override;

		virtual const FNodeClassMetadata& GetMetadata() const override;

		virtual const FVertexInterface& GetVertexInterface() const override;

		virtual void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral) override;

		virtual TSharedPtr<const IOperatorData> GetOperatorData() const override;

		virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override;

	private:

		FNodeData NodeData;
		TSharedRef<FGraphOperatorFactoryAdapter> Factory;
		TSharedRef<const IGraph> Graph;
	};
}

