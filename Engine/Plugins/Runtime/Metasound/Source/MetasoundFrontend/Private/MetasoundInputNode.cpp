// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundInputNode.h"

#include "Internationalization/Text.h"
#include "MetasoundDataReference.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace MetasoundInputNodePrivate
	{
		FNonExecutableInputOperatorBase::FNonExecutableInputOperatorBase(const FVertexName& InVertexName, FAnyDataReference&& InDataRef)
		: VertexName(InVertexName)
		, DataRef(MoveTemp(InDataRef))
		{
		}

		void FNonExecutableInputOperatorBase::BindInputs(FInputVertexInterfaceData& InOutVertexData)
		{
			InOutVertexData.BindVertex(VertexName, DataRef);
		}

		void FNonExecutableInputOperatorBase::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
		{
			InOutVertexData.BindVertex(VertexName, DataRef);
		}

		IOperator::FExecuteFunction FNonExecutableInputOperatorBase::GetExecuteFunction()
		{
			return nullptr;
		}

		IOperator::FPostExecuteFunction FNonExecutableInputOperatorBase::GetPostExecuteFunction()
		{
			return nullptr;
		}

		IOperator::FResetFunction FNonExecutableInputOperatorBase::GetResetFunction() 
		{
			return nullptr;
		}
	}

	FLazyName FInputNode::ConstructorVariant("Constructor");
	FName FInputNode::GetVariantName(EVertexAccessType InVertexAccess)
	{
		if (EVertexAccessType::Value == InVertexAccess)
		{
			return ConstructorVariant;
		}
		else
		{
			return FName();
		}
	}

	FVertexInterface FInputNode::CreateVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess, const FLiteral& InLiteral)
	{
		return FVertexInterface(
			FInputVertexInterface(
				FInputDataVertex(InVertexName, InDataTypeName, FDataVertexMetadata{ FText::GetEmpty() }, InVertexAccess, InLiteral)
			),
			FOutputVertexInterface(
				FOutputDataVertex(InVertexName, InDataTypeName, FDataVertexMetadata{ FText::GetEmpty() }, InVertexAccess)
			)
		);
	}


	FVertexInterface FInputNode::CreateDefaultVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess, const FLiteral* InDefaultLiteral)
	{
		if (InDefaultLiteral)
		{
			return CreateVertexInterface(InVertexName, InDataTypeName, InVertexAccess, *InDefaultLiteral);
		}
		else
		{
			return CreateVertexInterface(InVertexName, InDataTypeName, InVertexAccess, FLiteral());
		}
	}

	FText FInputNode::GetInputDescription()
	{
		return METASOUND_LOCTEXT("Metasound_InputNodeDescription", "Input into the parent MetaSound graph.");
	}

	FNodeClassMetadata FInputNode::GetNodeMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess)
	{
		return CreateNodeClassMetadata(InVertexName, InDataTypeName, InVertexAccess);
	}

	FNodeClassMetadata FInputNode::CreateNodeClassMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess)
	{
		FNodeClassMetadata Info;

		Info.ClassName = { "Input", InDataTypeName, GetVariantName(InVertexAccess) };
		Info.MajorVersion = 1;
		Info.MinorVersion = 0;
		Info.Description = GetInputDescription();
		Info.Author = PluginAuthor;
		Info.PromptIfMissing = PluginNodeMissingPrompt;
		Info.DefaultInterface = CreateDefaultVertexInterface(InVertexName, InDataTypeName, InVertexAccess);

		return Info;
	}

	FInputNode::FInputNode(FInputNodeConstructorParams&& InParams, const FName& InDataTypeName, EVertexAccessType InVertexAccess, FOperatorFactorySharedRef InFactory)
	: FInputNode(MoveTemp(InFactory), FNodeData{InParams.NodeName, InParams.InstanceID, CreateVertexInterface(InParams.VertexName, InDataTypeName, InVertexAccess, InParams.InitParam)}, MakeShared<const FNodeClassMetadata>(CreateNodeClassMetadata(InParams.VertexName, InDataTypeName, InVertexAccess)))
	{
	}

	FInputNode::FInputNode(FOperatorFactorySharedRef InFactory, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
	: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
	, Factory(MoveTemp(InFactory))
	{
	}

	const FVertexName& FInputNode::GetVertexName() const
	{
		const FInputVertexInterface& Inputs = GetVertexInterface().GetInputInterface();
		check(Inputs.Num() == 1);
		return Inputs.At(0).VertexName;
	}

	TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> FInputNode::GetDefaultOperatorFactory() const
	{
		return Factory;
	}
}

#undef LOCTEXT_NAMESPACE // MetasoundFrontend
