// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HAL/Platform.h"

#include "IStructSerializerBackend.h"
#include "IStructDeserializerBackend.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonReader.h"

#include "UObject/Class.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"

// This file is largely copied from JsonStructSerializerBackend / JsonStructDeserializerBackend.
// The primary difference is that it exposes template args for which character encoding to use,
// as well as our printing policy.

namespace Metasound
{
	typedef ANSICHAR DefaultCharType;

	struct StructDeserializerBackendUtilities
	{
		/**
		* Clears the value of the given property.
		*
		* @param Property The property to clear.
		* @param Outer The property that contains the property to be cleared, if any.
		* @param Data A pointer to the memory holding the property's data.
		* @param ArrayIndex The index of the element to clear (if the property is an array).
		* @return true on success, false otherwise.
		* @see SetPropertyValue
		*/
		static bool ClearPropertyValue(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
		{
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer);

			if (ArrayProperty != nullptr)
			{
				if (ArrayProperty->Inner != Property)
				{
					return false;
				}

				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
				ArrayIndex = ArrayHelper.AddValue();
			}

			Property->ClearValue_InContainer(Data, ArrayIndex);

			return true;
		}


		/**
		* Gets a pointer to object of the given property.
		*
		* @param Property The property to get.
		* @param Outer The property that contains the property to be get, if any.
		* @param Data A pointer to the memory holding the property's data.
		* @param ArrayIndex The index of the element to set (if the property is an array).
		* @return A pointer to the object represented by the property, null otherwise..
		* @see ClearPropertyValue
		*/
		static void* GetPropertyValuePtr(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
		{
			check(Property);

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer))
			{
				if (ArrayProperty->Inner != Property)
				{
					return nullptr;
				}

				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
				int32 Index = ArrayHelper.AddValue();

				return ArrayHelper.GetRawPtr(Index);
			}

			if (ArrayIndex >= Property->ArrayDim)
			{
				return nullptr;
			}

			return Property->template ContainerPtrToValuePtr<void>(Data, ArrayIndex);
		}

		/**
		* Sets the value of the given property.
		*
		* @param Property The property to set.
		* @param Outer The property that contains the property to be set, if any.
		* @param Data A pointer to the memory holding the property's data.
		* @param ArrayIndex The index of the element to set (if the property is an array).
		* @return true on success, false otherwise.
		* @see ClearPropertyValue
		*/
		template<typename PropertyType, typename ValueType>
		static bool SetPropertyValue(PropertyType* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const ValueType& Value)
		{
			if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
			{
				Property->SetPropertyValue(Ptr, Value);
				return true;
			}

			return false;
		}
	};

	namespace JsonStructSerializerBackend
	{
		// Writes a property value to the serialization output.
		template<typename JsonWriterType, typename ValueType>
		static void WritePropertyValue(const TSharedRef<JsonWriterType> JsonWriter, const FStructSerializerState& State, const ValueType& Value)
		{
			if ((State.ValueProperty == nullptr) ||
				(State.ValueProperty->ArrayDim > 1) ||
				State.ValueProperty->GetOwner<FArrayProperty>() ||
				State.ValueProperty->GetOwner<FSetProperty>())
			{
				JsonWriter->WriteValue(Value);
			}
			else if (State.KeyProperty != nullptr)
			{
				FString KeyString;
				State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
				JsonWriter->WriteValue(KeyString, Value);
			}
			else
			{
				JsonWriter->WriteValue(State.ValueProperty->GetName(), Value);
			}
		}

		// Writes a null value to the serialization output.
		static void WriteNull(const TSharedRef<TJsonWriter<UCS2CHAR>> JsonWriter, const FStructSerializerState& State)
		{
			if ((State.ValueProperty == nullptr) ||
				(State.ValueProperty->ArrayDim > 1) ||
				State.ValueProperty->GetOwner<FArrayProperty>() ||
				State.ValueProperty->GetOwner<FSetProperty>())
			{
				JsonWriter->WriteNull();
			}
			else if (State.KeyProperty != nullptr)
			{
				FString KeyString;
				State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
				JsonWriter->WriteNull(KeyString);
			}
			else
			{
				JsonWriter->WriteNull(State.ValueProperty->GetName());
			}
		}
	}

	/**
	 * Implements a writer for UStruct serialization using Json.
	 *
	 * Optionally, CharType and PrettyPrintPolicy can be subsituted using template arguments.
	 */
	template <class CharType, class PrintPolicy = TPrettyJsonPrintPolicy<CharType>>
	class TJsonStructSerializerBackend
		: public IStructSerializerBackend
	{
	public:

		/**
		 * Creates and initializes a new instance with the given flags.
		 *
		 * @param InArchive The archive to serialize into.
		 * @param InFlags The flags that control the serialization behavior (typically EStructSerializerBackendFlags::Default).
		 */
		TJsonStructSerializerBackend(FArchive& InArchive, const EStructSerializerBackendFlags InFlags)
			: JsonWriter(TJsonWriter<CharType, PrintPolicy>::Create(&InArchive))
			, Flags(InFlags)
		{ }

		~TJsonStructSerializerBackend() = default;

	public:

		// IStructSerializerBackend interface

		virtual void BeginArray(const FStructSerializerState& State) override
		{
			if (State.ValueProperty->GetOwner<FArrayProperty>())
			{
				JsonWriter->WriteArrayStart();
			}
			else if (State.KeyProperty != nullptr)
			{
				FString KeyString;
				State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
				JsonWriter->WriteArrayStart(KeyString);
			}
			else
			{
				JsonWriter->WriteArrayStart(State.ValueProperty->GetName());
			}
		}

		virtual void BeginStructure(const FStructSerializerState& State) override
		{
			if (State.ValueProperty != nullptr)
			{
				if (State.ValueProperty->GetOwner<FArrayProperty>() || State.ValueProperty->GetOwner<FSetProperty>())
				{
					JsonWriter->WriteObjectStart();
				}
				else if (State.KeyProperty != nullptr)
				{
					FString KeyString;
					State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
					JsonWriter->WriteObjectStart(KeyString);
				}
				else
				{
					JsonWriter->WriteObjectStart(State.ValueProperty->GetName());
				}
			}
			else
			{
				JsonWriter->WriteObjectStart();
			}
		}

		virtual void EndArray(const FStructSerializerState& State) override
		{
			JsonWriter->WriteArrayEnd();
		}

		virtual void EndStructure(const FStructSerializerState& State) override
		{
			JsonWriter->WriteObjectEnd();
		}

		virtual void WriteComment(const FString& Comment) override
		{
			// Json does not support comments
		}

		virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override
		{
			using namespace JsonStructSerializerBackend;

			// booleans
			if (State.FieldType == FBoolProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, CastFieldChecked<FBoolProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}

			// unsigned bytes & enumerations
			else if (State.FieldType == FEnumProperty::StaticClass())
			{
				FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(State.ValueProperty);

				WritePropertyValue(JsonWriter, State, EnumProperty->GetEnum()->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<void>(State.ValueData, ArrayIndex))));
			}
			else if (State.FieldType == FByteProperty::StaticClass())
			{
				FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(State.ValueProperty);

				if (ByteProperty->IsEnum())
				{
					WritePropertyValue(JsonWriter, State, ByteProperty->Enum->GetNameStringByValue(ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex)));
				}
				else
				{
					WritePropertyValue(JsonWriter, State, (double)ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
				}
			}

			// floating point numbers
			else if (State.FieldType == FDoubleProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, CastFieldChecked<FDoubleProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FFloatProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, CastFieldChecked<FFloatProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}

			// signed integers
			else if (State.FieldType == FIntProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FIntProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FInt8Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FInt8Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FInt16Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FInt64Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}

			// unsigned integers
			else if (State.FieldType == FUInt16Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FUInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FUInt32Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FUInt32Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FUInt64Property::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, (double)CastFieldChecked<FUInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}

			// names, strings & text
			else if (State.FieldType == FNameProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, CastFieldChecked<FNameProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex).ToString());
			}
			else if (State.FieldType == FStrProperty::StaticClass())
			{
				WritePropertyValue(JsonWriter, State, CastFieldChecked<FStrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
			}
			else if (State.FieldType == FTextProperty::StaticClass())
			{
				const FText& TextValue = CastFieldChecked<FTextProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
				if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteTextAsComplexString))
				{
					FString TextValueString;
					FTextStringHelper::WriteToBuffer(TextValueString, TextValue);
					WritePropertyValue(JsonWriter, State, TextValueString);
				}
				else
				{
					WritePropertyValue(JsonWriter, State, TextValue.ToString());
				}
			}

			// classes & objects
			else if (State.FieldType == FSoftClassProperty::StaticClass())
			{
				FSoftObjectPtr const& Value = CastFieldChecked<FSoftClassProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
				WritePropertyValue(JsonWriter, State, Value.IsValid() ? Value->GetPathName() : FString());
			}
			else if (State.FieldType == FWeakObjectProperty::StaticClass())
			{
				FWeakObjectPtr const& Value = CastFieldChecked<FWeakObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
				WritePropertyValue(JsonWriter, State, Value.IsValid() ? Value.Get()->GetPathName() : FString());
			}
			else if (State.FieldType == FSoftObjectProperty::StaticClass())
			{
				FSoftObjectPtr const& Value = CastFieldChecked<FSoftObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
				WritePropertyValue(JsonWriter, State, Value.ToString());
			}
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
			{
				// @TODO: Could this be expanded to include everything derived from FObjectPropertyBase?
				// Generic handling for a property type derived from FObjectProperty that is obtainable as a pointer and will be stored using its path.
				// This must come after all the more specialized handlers for object property types.
				UObject* const Value = ObjectProperty->GetObjectPropertyValue_InContainer(State.ValueData, ArrayIndex);
				WritePropertyValue(JsonWriter, State, Value ? Value->GetPathName() : FString());
			}

			// unsupported property type
			else
			{
				UE_LOGF(LogSerialization, Verbose, "FJsonStructSerializerBackend: Property %ls cannot be serialized, because its type (%ls) is not supported", *State.ValueProperty->GetFName().ToString(), *State.ValueType->GetFName().ToString());
			}
		}

	protected:

		// Allow access to the internal JsonWriter to subclasses
		TSharedRef<TJsonWriter<CharType, PrintPolicy>>& GetWriter()
		{
			return JsonWriter;
		}

	private:

		/** Holds the Json writer used for the actual serialization. */
		TSharedRef<TJsonWriter<CharType, PrintPolicy>> JsonWriter;

		/** Flags controlling the serialization behavior. */
		EStructSerializerBackendFlags Flags;
	};


	/**
	* Implements a reader for UStruct deserialization using Json.
	*
	* Optionally, CharType that we're decoding from can be substituted using the CharType template argument.
	 */
	template <class CharType>
	class TJsonStructDeserializerBackend
		: public IStructDeserializerBackend
	{
	public:

		/**
		 * Creates and initializes a new instance.
		 *
		 * @param Archive The archive to deserialize from.
		 */
		TJsonStructDeserializerBackend(FArchive& Archive)
			: JsonReader(TJsonReader<CharType>::Create(&Archive))
		{ }

		~TJsonStructDeserializerBackend() = default;

	public:

		// IStructDeserializerBackend interface

		virtual const FString& GetCurrentPropertyName() const override
		{
			return JsonReader->GetIdentifier();
		}

		virtual FString GetDebugString() const override
		{
			return FString::Printf(TEXT("Line: %u, Ch: %u"), JsonReader->GetLineNumber(), JsonReader->GetCharacterNumber());
		}

		virtual const FString& GetLastErrorMessage() const override
		{
			return JsonReader->GetErrorMessage();
		}

		virtual bool GetNextToken(EStructDeserializerBackendTokens& OutToken) override
		{
			if (!JsonReader->ReadNext(LastNotation))
			{
				return false;
			}

			switch (LastNotation)
			{
			case EJsonNotation::ArrayEnd:
				OutToken = EStructDeserializerBackendTokens::ArrayEnd;
				break;

			case EJsonNotation::ArrayStart:
				OutToken = EStructDeserializerBackendTokens::ArrayStart;
				break;

			case EJsonNotation::Boolean:
			case EJsonNotation::Null:
			case EJsonNotation::Number:
			case EJsonNotation::String:
			{
				OutToken = EStructDeserializerBackendTokens::Property;
			}
			break;

			case EJsonNotation::Error:
				OutToken = EStructDeserializerBackendTokens::Error;
				break;

			case EJsonNotation::ObjectEnd:
				OutToken = EStructDeserializerBackendTokens::StructureEnd;
				break;

			case EJsonNotation::ObjectStart:
				OutToken = EStructDeserializerBackendTokens::StructureStart;
				break;

			default:
				OutToken = EStructDeserializerBackendTokens::None;
			}

			return true;
		}

		virtual bool ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) override
		{
			switch (LastNotation)
			{
				// boolean values
			case EJsonNotation::Boolean:
			{
				bool BoolValue = JsonReader->GetValueAsBoolean();

				if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(BoolProperty, Outer, Data, ArrayIndex, BoolValue);
				}

				const FCoreTexts& CoreTexts = FCoreTexts::Get();

				UE_LOGF(LogSerialization, Verbose, "Boolean field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), BoolValue ? *(CoreTexts.True.ToString()) : *(CoreTexts.False.ToString()), *Property->GetClass()->GetName(), *GetDebugString());

				return false;
			}
			break;

			// numeric values
			case EJsonNotation::Number:
			{
				double NumericValue = JsonReader->GetValueAsNumber();

				if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (int8)NumericValue);
				}

				if (FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(DoubleProperty, Outer, Data, ArrayIndex, (double)NumericValue);
				}

				if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FloatProperty, Outer, Data, ArrayIndex, (float)NumericValue);
				}

				if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(IntProperty, Outer, Data, ArrayIndex, (int32)NumericValue);
				}

				if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(UInt32Property, Outer, Data, ArrayIndex, (uint32)NumericValue);
				}

				if (FInt16Property* Int16Property = CastField<FInt16Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int16Property, Outer, Data, ArrayIndex, (int16)NumericValue);
				}

				if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FInt16Property, Outer, Data, ArrayIndex, (uint16)NumericValue);
				}

				if (FInt64Property* Int64Property = CastField<FInt64Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int64Property, Outer, Data, ArrayIndex, (int64)NumericValue);
				}

				if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FInt64Property, Outer, Data, ArrayIndex, (uint64)NumericValue);
				}

				if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int8Property, Outer, Data, ArrayIndex, (int8)NumericValue);
				}

				UE_LOGF(LogSerialization, Verbose, "Numeric field %ls with value '%f' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), NumericValue, *Property->GetClass()->GetName(), *GetDebugString());

				return false;
			}
			break;

			// null values
			case EJsonNotation::Null:
				return StructDeserializerBackendUtilities::ClearPropertyValue(Property, Outer, Data, ArrayIndex);

				// strings, names, enumerations & object/class reference
			case EJsonNotation::String:
			{
				const FString& StringValue = JsonReader->GetValueAsString();

				if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(StrProperty, Outer, Data, ArrayIndex, StringValue);
				}

				if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(NameProperty, Outer, Data, ArrayIndex, FName(*StringValue));
				}

				if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
				{
					FText TextValue;
					if (!FTextStringHelper::ReadFromBuffer(*StringValue, TextValue))
					{
						TextValue = FText::FromString(StringValue);
					}
					return StructDeserializerBackendUtilities::SetPropertyValue(TextProperty, Outer, Data, ArrayIndex, TextValue);
				}

				if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
				{
					if (!ByteProperty->Enum)
					{
						return false;
					}

					int32 Value = ByteProperty->Enum->GetValueByName(*StringValue);
					if (Value == INDEX_NONE)
					{
						return false;
					}

					return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)Value);
				}

				if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
				{
					int64 Value = EnumProperty->GetEnum()->GetValueByName(*StringValue);
					if (Value == INDEX_NONE)
					{
						return false;
					}

					if (void* ElementPtr = StructDeserializerBackendUtilities::GetPropertyValuePtr(EnumProperty, Outer, Data, ArrayIndex))
					{
						EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ElementPtr, Value);
						return true;
					}

					return false;
				}

				if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(ClassProperty, Outer, Data, ArrayIndex, LoadObject<UClass>(nullptr, *StringValue, nullptr, LOAD_NoWarn));
				}

				if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(SoftClassProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(LoadObject<UClass>(nullptr, *StringValue, nullptr, LOAD_NoWarn)));
				}

				if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(ObjectProperty, Outer, Data, ArrayIndex, StaticFindObject(ObjectProperty->PropertyClass, nullptr, *StringValue));
				}

				if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(WeakObjectProperty, Outer, Data, ArrayIndex, FWeakObjectPtr(StaticFindObject(WeakObjectProperty->PropertyClass, nullptr, *StringValue)));
				}

				if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(SoftObjectProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
				}

				UE_LOGF(LogSerialization, Verbose, "String field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), *StringValue, *Property->GetClass()->GetName(), *GetDebugString());

				return false;
			}
			break;
			}

			return true;
		}

		virtual void SkipArray() override
		{
			JsonReader->SkipArray();
		}

		virtual void SkipStructure() override
		{
			JsonReader->SkipObject();
		}

	protected:
		FString& GetLastIdentifier()
		{
			return LastIdentifier;
		}

		EJsonNotation GetLastNotation()
		{
			return LastNotation;
		}

		TSharedRef<TJsonReader<CharType>>& GetReader()
		{
			return JsonReader;
		}

	private:

		/** Holds the name of the last read Json identifier. */
		FString LastIdentifier;

		/** Holds the last read Json notation. */
		EJsonNotation LastNotation;

		/** Holds the Json reader used for the actual reading of the archive. */
		TSharedRef<TJsonReader<CharType>> JsonReader;
	};
}
