// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundLiteralNode.h"

#define LOCTEXT_NAMESPACE "MetasoundLiteralNode"
namespace Metasound
{
	namespace LiteralNodeNames
	{
		DEFINE_METASOUND_PARAM(OutputValue, "Value", "Value")
	}

	FVertexInterface FLiteralNode::CreateVertexInterface(const FName& InDataTypeName, EVertexAccessType InAccessType)
	{
		using namespace LiteralNodeNames; 
		const FDataVertexMetadata OutputMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(OutputValue) // display name
		};

		return FVertexInterface(
			FInputVertexInterface(),
			FOutputVertexInterface(
				FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputValue), InDataTypeName, OutputMetadata, InAccessType)
			)
		);
	}

	FNodeClassMetadata FLiteralNode::GetNodeMetadata(const FName& InDataTypeName, EVertexAccessType InAccessType)
	{
		FNodeClassMetadata Info;

		Info.ClassName = {"Literal", InDataTypeName, ""};
		Info.MajorVersion = 1;
		Info.MinorVersion = 0;
#if WITH_EDITOR
		Info.DisplayName = FText::Format(LOCTEXT("Metasound_LiteralNodeDisplayNameFormat", "Literal {0}"), FText::FromName(InDataTypeName));
		Info.Description = LOCTEXT("Metasound_LiteralNodeDescription", "Literal accessible within a parent Metasound graph.");
#endif // WITH_EDITOR
		Info.Author = PluginAuthor;
		Info.PromptIfMissing = PluginNodeMissingPrompt;
		Info.DefaultInterface = CreateVertexInterface(InDataTypeName, InAccessType);

		return Info;
	}

	FLiteralNode::FLiteralNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FName& InDataTypeName, EVertexAccessType InAccessType, FOperatorFactorySharedRef InFactory)
	: FNode(InInstanceName, InInstanceID, GetNodeMetadata(InDataTypeName, InAccessType))
	, Interface(CreateVertexInterface(InDataTypeName, InAccessType))
	, Factory(InFactory)
	{
	}

	const FVertexInterface& FLiteralNode::GetVertexInterface() const
	{
		return Interface;
	}

	TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> FLiteralNode::GetDefaultOperatorFactory() const
	{
		return Factory;
	}
}
#undef LOCTEXT_NAMESPACE // MetasoundLiteralNode
