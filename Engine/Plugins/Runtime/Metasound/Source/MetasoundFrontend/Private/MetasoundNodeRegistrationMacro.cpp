// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeRegistrationMacro.h"

#include "MetasoundFrontendDocument.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundFrontend.h"

namespace Metasound::Frontend
{
	namespace NodeRegistrationPrivate
	{
		FNodeRegistryEntryBase::FNodeRegistryEntryBase(const Metasound::FNodeClassMetadata& InMetadata, const FModuleInfo& InOwningModuleInfo)
		: ClassMetadata(MakeShared<FNodeClassMetadata>(InMetadata))
		, FrontendClass(GenerateClass(InMetadata))
#if WITH_EDITORONLY_DATA
		, PluginName(InOwningModuleInfo.PluginName)
		, ModuleName(InOwningModuleInfo.ModuleName)
#endif
		{
		}

		const FMetasoundFrontendClass& FNodeRegistryEntryBase::GetFrontendClass() const
		{
			return FrontendClass;
		}

		const TSet<FMetasoundFrontendVersion>* FNodeRegistryEntryBase::GetImplementedInterfaces() const
		{
			return nullptr;
		}

		FVertexInterface FNodeRegistryEntryBase::GetDefaultVertexInterface() const
		{
			return FVertexInterface(ClassMetadata->DefaultInterface);
		}

		const FClassInterface& FNodeRegistryEntryBase::GetClassInterface() const
		{
			return ClassMetadata->DefaultInterface;
		}

		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FNodeRegistryEntryBase::CreateFrontendNodeConfiguration() const
		{
			return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>();
		}

		bool FNodeRegistryEntryBase::IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const
		{
			// No node configuration supported for this node type, so only compatible if setting to invalid (null) configuration
			return !InNodeConfiguration.IsValid();
		}

		TSharedRef<const FNodeClassMetadata> FNodeRegistryEntryBase::GetNodeClassMetadata() const
		{
			return ClassMetadata;
		}
#if WITH_EDITORONLY_DATA
		FName FNodeRegistryEntryBase::GetPluginName() const
		{
			return PluginName;
		}

		FName FNodeRegistryEntryBase::GetModuleName() const
		{
			return ModuleName;
		}
#endif

		bool RegisterNodeInternal(TUniquePtr<INodeClassRegistryEntry>&& InEntry)
		{
			Frontend::FNodeRegistryKey Key = INodeClassRegistry::Get()->RegisterNode(MoveTemp(InEntry));
			const bool bSuccessfullyRegisteredNode = Key.IsValid();
			ensureAlwaysMsgf(bSuccessfullyRegisteredNode, TEXT("Registering node class failed. Please check the logs."));

			return bSuccessfullyRegisteredNode;
		}

	}

	bool UnregisterNode(const FNodeClassRegistryKey& InRegistryKey, const FModuleInfo& InModuleInfo)
	{
		INodeClassRegistry& Registry = INodeClassRegistry::GetChecked();

#if WITH_EDITORONLY_DATA && !UE_BUILD_SHIPPING
		if (Registry.IsNodeRegistered(InRegistryKey))
		{
			/** Check that the module info used to register the node matches the module
			* info that is used to unregister the node. */
			const FName ExpectedPlugin = Registry.GetOwningPluginName(InRegistryKey);
			const FName ExpectedModule = Registry.GetOwningModuleName(InRegistryKey);
			const bool bIsMatchingModuleInfo = (InModuleInfo.PluginName == ExpectedPlugin) && (InModuleInfo.ModuleName == ExpectedModule);

			UE_CLOGF(!bIsMatchingModuleInfo, LogMetaSound, Error, "Owning Module Info does not match between registration and unregistration of node %ls. Expected unregistration to come from plugin %ls and module %ls, but encountered it from plugin %ls and module %ls", *InRegistryKey.ToString(), *ExpectedPlugin.ToString(), *ExpectedModule.ToString(), *InModuleInfo.PluginName.ToString(), *InModuleInfo.ModuleName.ToString());
		}
#endif

		return Registry.UnregisterNode(InRegistryKey);
	}

	bool UnregisterNode(const FNodeClassMetadata& InMetadata, const FModuleInfo& InModuleInfo)
	{
		return UnregisterNode(FNodeClassRegistryKey(InMetadata), InModuleInfo);
	}
}
