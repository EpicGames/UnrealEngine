// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOutputNode.h"
#include "MetasoundFrontendDataTypeRegistry.h"

#define LOCTEXT_NAMESPACE "MetasoundGraphCore"

namespace Metasound
{
	namespace OutputNodePrivate
	{
		static const FLazyName ValueVertexAccessVariantName("Constructor");
		static const FLazyName ReferenceVertexAccessVariantName("");

		class FOutputOperator : public FNoOpOperator
		{
			public:
				FOutputOperator(const FVertexName& InVertexName, const FAnyDataReference& InDataReference);
				virtual ~FOutputOperator() = default;

				virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override;
				virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override;

			private:
				FVertexName VertexName;
				FAnyDataReference DataReference;
		};

		class FOutputOperatorFactory : public IOperatorFactory
		{
		public:

			FOutputOperatorFactory(const FVertexName& InVertexName);
			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;

		private:
			FVertexName VertexName;
		};

		FOutputOperator::FOutputOperator(const FVertexName& InVertexName, const FAnyDataReference& InDataReference)
		: VertexName(InVertexName)
		, DataReference(InDataReference)
		{
		}

		void FOutputOperator::BindInputs(FInputVertexInterfaceData& InVertexData) 
		{
			InVertexData.BindVertex(VertexName, DataReference);
		}

		void FOutputOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData) 
		{
			InVertexData.BindVertex(VertexName, DataReference);
		}

		FOutputOperatorFactory::FOutputOperatorFactory(const FVertexName& InVertexName)
		: VertexName(InVertexName)
		{
		}

		TUniquePtr<IOperator> FOutputOperatorFactory::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace OutputNodePrivate;

			const FInputDataVertex& Vertex = InParams.InputData.GetVertex(VertexName);

			// Use data reference if it is passed in. 
			if (const FAnyDataReference* Ref = InParams.InputData.FindDataReference(VertexName))
			{
				checkf(IsCastable(Ref->GetDataTypeName(), Vertex.DataTypeName), TEXT("Mismatched data type names on output node (%s). Expected (%s), received (%s)."), *VertexName.ToString(), *Vertex.DataTypeName.ToString(), *(Ref->GetDataTypeName().ToString()));
				return MakeUnique<FOutputOperator>(VertexName, *Ref);
			}
			else 
			{
				// Make data reference if none are passed in. 
				Frontend::IDataTypeRegistry& DataTypeRegistry = Frontend::IDataTypeRegistry::Get();

				if (const Frontend::IDataTypeRegistryEntry* Entry = DataTypeRegistry.FindDataTypeRegistryEntry(Vertex.DataTypeName))
				{
					FLiteral Literal = Vertex.GetDefaultLiteral();
					if (!Literal.IsValid())
					{
						// If the literal on the interface is invalid, fallback to using
						// the default literal for the data type
						Literal = DataTypeRegistry.CreateDefaultLiteral(Vertex.DataTypeName);
					}

					// Make a EDataReferenceAccessType::Value reference because this data is not going to be mutated by anything.
					TOptional<FAnyDataReference> DataReference = Entry->CreateDataReference(EDataReferenceAccessType::Value, Literal, InParams.OperatorSettings);
					if (DataReference.IsSet())
					{
						return MakeUnique<FOutputOperator>(VertexName, *DataReference);
					}
				}
			}

			// Do not make output operator if no data reference is available. 
			OutResults.Errors.Emplace(MakeUnique<FMissingOutputNodeInputReferenceError>(InParams.Node));
			return TUniquePtr<IOperator>(nullptr);
		}
	} // namespace OutputNodePrivate


	FName FOutputNode::GetVariantName(EVertexAccessType InVertexAccessType)
	{
		using namespace OutputNodePrivate;

		if (EVertexAccessType::Value == InVertexAccessType)
		{
			return ValueVertexAccessVariantName;
		}
		else
		{
			return ReferenceVertexAccessVariantName;
		}
	}

	FVertexInterface FOutputNode::CreateVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType)
	{
		static const FText VertexDescription = METASOUND_LOCTEXT("Metasound_OutputVertexDescription", "Output from the parent Metasound graph.");

		return FVertexInterface(
			FInputVertexInterface(
				FInputDataVertex(InVertexName, InDataTypeName, FDataVertexMetadata{VertexDescription}, InVertexAccessType)
			),
			FOutputVertexInterface(
				FOutputDataVertex(InVertexName, InDataTypeName, FDataVertexMetadata{VertexDescription}, InVertexAccessType)
			)
		);
	}

	FNodeClassMetadata FOutputNode::CreateNodeClassMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType)
	{
		FNodeClassMetadata Info;

		Info.ClassName = { "Output", InDataTypeName, GetVariantName(InVertexAccessType)};
		Info.MajorVersion = 1;
		Info.MinorVersion = 0;
		Info.Description = METASOUND_LOCTEXT("Metasound_OutputNodeDescription", "Output from the parent Metasound graph.");
		Info.Author = PluginAuthor;
		Info.PromptIfMissing = PluginNodeMissingPrompt;
		Info.DefaultInterface = CreateVertexInterface(InVertexName, InDataTypeName, InVertexAccessType);

		return Info;
	}

	FOutputNode::FOutputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
	: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
	, Factory(MakeShared<OutputNodePrivate::FOutputOperatorFactory, ESPMode::ThreadSafe>(InVertexName))
	{
	}

	FOutputNode::FOutputNode(const FVertexName& InVertexName, const FGuid& InInstanceID, const FVertexName& InInstanceName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType)
	: FOutputNode(InVertexName,
		FNodeData{InInstanceName, InInstanceID, CreateVertexInterface(InVertexName, InDataTypeName, InVertexAccessType)},
		MakeShared<FNodeClassMetadata>(CreateNodeClassMetadata(InVertexName, InDataTypeName, InVertexAccessType)))
	{
	}

	TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> FOutputNode::GetDefaultOperatorFactory() const
	{
		return Factory;
	}
}

#undef LOCTEXT_NAMESPACE // MetasoundOutputNode
