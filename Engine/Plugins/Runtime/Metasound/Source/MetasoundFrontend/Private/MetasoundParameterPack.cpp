// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundParameterPack.h"

#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#include <limits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetasoundParameterPack)

DEFINE_LOG_CATEGORY_STATIC(LogMetasoundParamPack, Log, All);

REGISTER_METASOUND_DATATYPE(FMetasoundParameterStorageWrapper, "MetasoundParameterPack", Metasound::ELiteralType::UObjectProxy, UMetasoundParameterPack);

template<>
struct Metasound::TEnableArrayNodes<FMetasoundParameterStorageWrapper>
{
	static constexpr bool Value = false;
};

namespace MetasoundParameterPack
{
	FName RoutingName("ParamPack");
}

TSharedPtr<Audio::IProxyData> UMetasoundParameterPack::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	return MakeShared<FMetasoundParameterPackProxy>(ParameterStorage);
}

Metasound::FSendAddress UMetasoundParameterPack::CreateSendAddressFromEnvironment(const Metasound::FMetasoundEnvironment& InEnvironment)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Metasound::FMetaSoundParameterTransmitter::CreateSendAddressFromEnvironment(InEnvironment, MetasoundParameterPack::RoutingName, Metasound::GetMetasoundDataTypeName<FMetasoundParameterStorageWrapper>());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMetasoundFrontendClassInput UMetasoundParameterPack::GetClassInput()
{
	FMetasoundFrontendClassInput ClassInput;
	ClassInput.Name = MetasoundParameterPack::RoutingName;
	ClassInput.TypeName = Metasound::GetMetasoundDataTypeName<FMetasoundParameterStorageWrapper>();

	constexpr UObject* NullObjectPtr = nullptr;
	ClassInput.InitDefault() = FAudioParameter { MetasoundParameterPack::RoutingName, NullObjectPtr };

	return ClassInput;
}

UMetasoundParameterPack* UMetasoundParameterPack::MakeMetasoundParameterPack()
{
	return NewObject<UMetasoundParameterPack>();
}

ESetParamResult UMetasoundParameterPack::SetBool(FName Name, bool InValue, bool OnlyIfExists)
{
	return SetParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<bool>(), InValue, OnlyIfExists);
}

ESetParamResult UMetasoundParameterPack::SetInt(FName Name, int32 InValue, bool OnlyIfExists)
{
	return SetParameter<int32>(Name, Metasound::GetMetasoundDataTypeName<int32>(), InValue, OnlyIfExists);
}

ESetParamResult UMetasoundParameterPack::SetFloat(FName Name, float InValue, bool OnlyIfExists)
{
	return SetParameter<float>(Name, Metasound::GetMetasoundDataTypeName<float>(), InValue, OnlyIfExists);
}

ESetParamResult UMetasoundParameterPack::SetString(FName Name, const FString& InValue, bool OnlyIfExists)
{
	return SetParameter<FString>(Name, Metasound::GetMetasoundDataTypeName<FString>(), InValue, OnlyIfExists);
}

ESetParamResult UMetasoundParameterPack::SetTrigger(FName Name, bool OnlyIfExists)
{
	return SetParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>(), true, OnlyIfExists);
}

bool UMetasoundParameterPack::GetBool(FName Name, ESetParamResult& Result) const
{
	return GetParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<bool>(), Result);
}

int32 UMetasoundParameterPack::GetInt(FName Name, ESetParamResult& Result) const
{
	return GetParameter<int32>(Name, Metasound::GetMetasoundDataTypeName<int32>(), Result);
}

float UMetasoundParameterPack::GetFloat(FName Name, ESetParamResult& Result) const
{
	return GetParameter<float>(Name, Metasound::GetMetasoundDataTypeName<float>(), Result);
}

FString UMetasoundParameterPack::GetString(FName Name, ESetParamResult& Result) const
{
	return GetParameter<FString>(Name, Metasound::GetMetasoundDataTypeName<FString>(), Result);
}

bool UMetasoundParameterPack::GetTrigger(FName Name, ESetParamResult& Result) const
{
	return GetParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>(), Result);
}

bool* UMetasoundParameterPack::GetBoolParameterPtr(FName Name) const
{
	return GetParameterPtr<bool>(Name, Metasound::GetMetasoundDataTypeName<bool>());
}

int32* UMetasoundParameterPack::GetIntParameterPtr(FName Name) const
{
	return GetParameterPtr<int32>(Name, Metasound::GetMetasoundDataTypeName<int32>());
}

float* UMetasoundParameterPack::GetFloatParameterPtr(FName Name) const
{
	return GetParameterPtr<float>(Name, Metasound::GetMetasoundDataTypeName<float>());
}

FString* UMetasoundParameterPack::GetStringParameterPtr(FName Name) const
{
	return GetParameterPtr<FString>(Name, Metasound::GetMetasoundDataTypeName<FString>());
}

bool* UMetasoundParameterPack::GetTriggerParameterPtr(FName Name) const
{
	return GetParameterPtr<bool>(Name, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>());
}

bool UMetasoundParameterPack::HasBool(FName Name) const
{
	return HasParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<bool>());
}

bool UMetasoundParameterPack::HasInt(FName Name) const
{
	return HasParameter<int32>(Name, Metasound::GetMetasoundDataTypeName<int32>());
}

bool UMetasoundParameterPack::HasFloat(FName Name) const
{
	return HasParameter<float>(Name, Metasound::GetMetasoundDataTypeName<float>());
}

bool UMetasoundParameterPack::HasString(FName Name) const
{
	return HasParameter<FString>(Name, Metasound::GetMetasoundDataTypeName<FString>());
}

bool UMetasoundParameterPack::HasTrigger(FName Name) const
{
	return HasParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>());
}

void UMetasoundParameterPack::AddBoolParameter(FName Name, bool InValue)
{
	AddParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<bool>(), InValue);
}

void UMetasoundParameterPack::AddIntParameter(FName Name, int32 InValue)
{
	AddParameter<int32>(Name, Metasound::GetMetasoundDataTypeName<int32>(), InValue);
}

void UMetasoundParameterPack::AddFloatParameter(FName Name, float InValue)
{
	AddParameter<float>(Name, Metasound::GetMetasoundDataTypeName<float>(), InValue);
}

void UMetasoundParameterPack::AddStringParameter(FName Name, const FString& InValue)
{
	AddParameter<FString>(Name, Metasound::GetMetasoundDataTypeName<FString>(), InValue);
}

void UMetasoundParameterPack::AddTriggerParameter(FName Name, bool InValue)
{
	AddParameter<bool>(Name, Metasound::GetMetasoundDataTypeName<Metasound::FTrigger>(), InValue);
}

FSharedMetasoundParameterStoragePtr UMetasoundParameterPack::GetParameterStorage() const
{
	return ParameterStorage;
}

FSharedMetasoundParameterStoragePtr UMetasoundParameterPack::GetCopyOfParameterStorage() const
{
	if (!ParameterStorage.IsValid())
	{
		return nullptr;
	}
	return MakeShared<FMetasoundParameterPackStorage>(*ParameterStorage);
}
