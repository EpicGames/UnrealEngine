// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundParameterTransmitter.h"

#include "HAL/IConsoleManager.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundLog.h"


namespace Metasound
{
	int32 MetaSoundParameterEnableWarningOnIgnoredParameterCVar = 0;

	FAutoConsoleVariableRef CVarMetaSoundParameterEnableWarningOnIgnoredParameter(
		TEXT("au.MetaSound.Parameter.EnableWarningOnIgnoredParameter"),
		MetaSoundParameterEnableWarningOnIgnoredParameterCVar,
		TEXT("If enabled, a warning will be logged when a parameters sent to a MetaSound is ignored.\n")
		TEXT("0: Disabled (default), !0: Enabled"),
		ECVF_Default);


	namespace Frontend
	{
		FLiteral ConvertParameterToLiteral(FAudioParameter&& InValue)
		{
			switch (InValue.ParamType)
			{
				case EAudioParameterType::Trigger:
				case EAudioParameterType::Boolean:
				{
					return FLiteral(InValue.BoolParam);
				}

				case EAudioParameterType::BooleanArray:
				{
					return FLiteral(MoveTemp(InValue.ArrayBoolParam));
				}

				case EAudioParameterType::Float:
				{
					return FLiteral(InValue.FloatParam);
				}

				case EAudioParameterType::FloatArray:
				{
					return FLiteral(MoveTemp(InValue.ArrayFloatParam));
				}

				case EAudioParameterType::Integer:
				{
					return FLiteral(InValue.IntParam);
				}

				case EAudioParameterType::IntegerArray:
				{
					return FLiteral(MoveTemp(InValue.ArrayIntParam));
				}

				case EAudioParameterType::None:
				{
					return FLiteral();
				}

				case EAudioParameterType::NoneArray:
				{
					TArray<FLiteral::FNone> InitArray;
					InitArray.Init(FLiteral::FNone(), InValue.IntParam);
					return FLiteral(MoveTemp(InitArray));
				}

				case EAudioParameterType::Object:
				{
					if (InValue.ObjectProxies.IsEmpty())
					{
						return FLiteral();
					}

					return FLiteral(MoveTemp(InValue.ObjectProxies[0]));
				}

				case EAudioParameterType::ObjectArray:
				{
					return FLiteral(MoveTemp(InValue.ObjectProxies));
				}

				case EAudioParameterType::String:
				{
					return FLiteral(MoveTemp(InValue.StringParam));
				}

				case EAudioParameterType::StringArray:
				{
					return FLiteral(MoveTemp(InValue.ArrayStringParam));
				}

				default:
				{
					static_assert(static_cast<int32>(EAudioParameterType::COUNT) == 13, "Possible missing switch case coverage");
					checkNoEntry();
				}
			}

			return FLiteral();
		}

		FLiteral ConvertParameterToLiteral(const FAudioParameter& InValue)
		{
			switch (InValue.ParamType)
			{
				case EAudioParameterType::Trigger:
				case EAudioParameterType::Boolean:
				{
					return FLiteral(InValue.BoolParam);
				}

				case EAudioParameterType::BooleanArray:
				{
					return FLiteral(InValue.ArrayBoolParam);
				}

				case EAudioParameterType::Float:
				{
					return FLiteral(InValue.FloatParam);
				}

				case EAudioParameterType::FloatArray:
				{
					return FLiteral(InValue.ArrayFloatParam);
				}

				case EAudioParameterType::Integer:
				{
					return FLiteral(InValue.IntParam);
				}

				case EAudioParameterType::IntegerArray:
				{
					return FLiteral(InValue.ArrayIntParam);
				}

				case EAudioParameterType::None:
				{
					return FLiteral();
				}

				case EAudioParameterType::NoneArray:
				{
					TArray<FLiteral::FNone> InitArray;
					InitArray.Init(FLiteral::FNone(), InValue.IntParam);
					return FLiteral(InitArray);
				}

				case EAudioParameterType::Object:
				{
					if (InValue.ObjectProxies.IsEmpty())
					{
						return FLiteral();
					}

					return FLiteral(InValue.ObjectProxies.Last());
				}

				case EAudioParameterType::ObjectArray:
				{
					return FLiteral(InValue.ObjectProxies);
				}

				case EAudioParameterType::String:
				{
					return FLiteral(InValue.StringParam);
				}

				case EAudioParameterType::StringArray:
				{
					return FLiteral(InValue.ArrayStringParam);
				}

				default:
				{
					static_assert(static_cast<int32>(EAudioParameterType::COUNT) == 13, "Possible missing switch case coverage");
					checkNoEntry();
				}
			}

			return FLiteral();
		}

		FName ConvertParameterToDataType(EAudioParameterType InParameterType)
		{
			switch (InParameterType)
			{
				case EAudioParameterType::Trigger:
				case EAudioParameterType::Boolean:
					return GetMetasoundDataTypeName<bool>();
				case EAudioParameterType::BooleanArray:
					return GetMetasoundDataTypeName<TArray<bool>>();
				case EAudioParameterType::Float:
					return GetMetasoundDataTypeName<float>();
				case EAudioParameterType::FloatArray:
					return GetMetasoundDataTypeName<TArray<float>>();
				case EAudioParameterType::Integer:
					return GetMetasoundDataTypeName<int32>();
				case EAudioParameterType::IntegerArray:
					return GetMetasoundDataTypeName<TArray<int32>>();
				case EAudioParameterType::String:
					return GetMetasoundDataTypeName<FString>();
				case EAudioParameterType::StringArray:
					return GetMetasoundDataTypeName<TArray<FString>>();

				case EAudioParameterType::Object:
				case EAudioParameterType::ObjectArray:
					// TODO: Add support for objects

				case EAudioParameterType::None:
				case EAudioParameterType::NoneArray:
				default:
					ensureAlwaysMsgf(false, TEXT("Failed to convert AudioParameterType to POD MetaSound DataType"));
					static_assert(static_cast<int32>(EAudioParameterType::COUNT) == 13, "Possible missing case coverage");
					return FName();
			}
		}
	} // namespace Frontend

	FSendAddress FMetaSoundParameterTransmitter::CreateSendAddressFromEnvironment(const FMetasoundEnvironment& InEnvironment, const FVertexName& InVertexName, const FName& InTypeName)
	{
		using namespace Frontend;

		uint64 InstanceID = -1;
		if (ensure(InEnvironment.Contains<uint64>(CoreInterface::Environment::InstanceID)))
		{
			InstanceID = InEnvironment.GetValue<uint64>(CoreInterface::Environment::InstanceID);
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CreateSendAddressFromInstanceID(InstanceID, InVertexName, InTypeName);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FSendAddress FMetaSoundParameterTransmitter::CreateSendAddressFromInstanceID(uint64 InInstanceID, const FVertexName& InVertexName, const FName& InTypeName)
	{
		return FSendAddress(InVertexName, InTypeName, InInstanceID);
	}

	FMetaSoundParameterTransmitter::FMetaSoundParameterTransmitter(FMetaSoundParameterTransmitter::FInitParams&& InInitParams)
		: Audio::FParameterTransmitterBase(MoveTemp(InInitParams.DefaultParams))
		, InstanceID(InInitParams.InstanceID)
		, DebugMetaSoundName(InInitParams.DebugMetaSoundName)
		, AvailableParameterNames(MoveTemp(InInitParams.ValidParameterNames))
		, DataChannel(MoveTemp(InInitParams.DataChannel))
	{
	}

	void FMetaSoundParameterTransmitter::AddAvailableParameter(FName InName)
	{
		AvailableParameterNames.Add(InName);
	}

	void FMetaSoundParameterTransmitter::RemoveAvailableParameter(FName InName)
	{
		AvailableParameterNames.Remove(InName);
	}

	bool FMetaSoundParameterTransmitter::SetVirtualizedParameters(TArray<FAudioParameter>&& InParameters)
	{
		check(IsInAudioThread());

		bool bSuccess = true;

		// Remove triggers
		for (int32 ParamIndex = InParameters.Num() - 1; ParamIndex >= 0; --ParamIndex)
		{
			// Triggers are transient and are not applied for virtualized sounds. 
			// If a cached value is desired, use SetBoolParameter
			// (see comment for IAudioParameterControllerInterface::SetTriggerParameter)
			FAudioParameter& Param = InParameters[ParamIndex];
			if (Param.ParamType == EAudioParameterType::Trigger)
			{
				InParameters.RemoveAtSwap(ParamIndex, 1, EAllowShrinking::No);
			}
		}

		if (!InParameters.IsEmpty())
		{
			bSuccess &= FParameterTransmitterBase::SetParameters(MoveTemp(InParameters));
		}

		InParameters.Reset();
		return bSuccess;
	}

	bool FMetaSoundParameterTransmitter::SetParameters(TArray<FAudioParameter>&& InParameters)
	{
		check(IsInAudioThread());

		// Don't set parameters directly if the active sound 
		// is currently virtualized (to prevent accumulation of unneeded updates) 
		if (IsVirtualized())
		{
			return SetVirtualizedParameters(MoveTemp(InParameters));
		}

		bool bSuccess = true;

		for (int32 ParamIndex = InParameters.Num() - 1; ParamIndex >= 0; --ParamIndex)
		{
			FAudioParameter& Param = InParameters[ParamIndex];
			const FName ParamName = Param.ParamName;
			if (!SetParameterWithLiteral(ParamName, Frontend::ConvertParameterToLiteral(Param)))
			{
				InParameters.RemoveAtSwap(ParamIndex, EAllowShrinking::No);
				bSuccess = false;
			}
		}

		if (!InParameters.IsEmpty())
		{
			bSuccess &= FParameterTransmitterBase::SetParameters(MoveTemp(InParameters));
		}

		InParameters.Reset();
		return bSuccess;
	}

	bool FMetaSoundParameterTransmitter::SetParameterWithLiteral(FName InParameterName, const FLiteral& InLiteral)
	{
		if (AvailableParameterNames.Contains(InParameterName))
		{
			if (DataChannel.IsValid())
			{
				DataChannel->Enqueue(FParameter{InParameterName, InLiteral});
			}

			return true;
		}

		// Enable / disable via CVAR to avoid log spam.
		if (MetaSoundParameterEnableWarningOnIgnoredParameterCVar)
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to set parameter %ls in asset %ls on instance %lld with value %ls. No runtime modifiable input with that name exists on instance.", *InParameterName.ToString(), *DebugMetaSoundName.ToString(), InstanceID, *LexToString(InLiteral));
		}

		return false;
	}
}
