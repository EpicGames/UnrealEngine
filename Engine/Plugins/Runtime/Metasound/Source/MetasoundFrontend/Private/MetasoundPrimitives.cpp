// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundPrimitives.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundRouter.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"

#include <type_traits>

namespace Metasound
{
	// Disable audio buffer arrays.
	template<>
	struct TEnableAutoArrayTypeRegistration<FAudioBuffer>
	{
		static constexpr bool Value = false;
	};

	// Disable constructor pins of audio buffers
	template<>
	struct TEnableConstructorVertex<FAudioBuffer>
	{
		static constexpr bool Value = false;
	};

	// Disable arrays of triggers
	template<>
	struct TEnableAutoArrayTypeRegistration<FTrigger>
	{
		static constexpr bool Value = false;
	};

	// Disable constructor pins of triggers 
	template<>
	struct TEnableConstructorVertex<FTrigger>
	{
		static constexpr bool Value = false;
	};

	// Disable auto-conversion using the FAudioBuffer(int32) constructor
	template<typename FromDataType>
	struct TEnableAutoConverterNodeRegistration<FromDataType, FAudioBuffer>
	{
		static constexpr bool Value = !std::is_arithmetic<FromDataType>::value;
	};

	// Disable auto-conversions based on FTrigger implicit converters
	template<typename ToDataType>
	struct TEnableAutoConverterNodeRegistration<FTrigger, ToDataType>
	{
		static constexpr bool Value = !std::is_arithmetic<ToDataType>::value;
	};

	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<bool>
	{
		static constexpr bool Value = true;
	};
	
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<int32>
	{
		static constexpr bool Value = true;
	};
	
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<float>
	{
		static constexpr bool Value = true;
	};
	
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<FString>
	{
		static constexpr bool Value = true;
	};
	
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<FTrigger>
	{
		static constexpr bool Value = true;
	};
	
	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<FTime>
	{
		static constexpr bool Value = true;
	};

	// Enable send/receive node registration for data types which existed before
	// send/receive were deprecated in order to support old UMetaSound assets. 
	template<>
	struct TEnableTransmissionNodeRegistration<FAudioBuffer>
	{
		static constexpr bool Value = true;
	};
}

REGISTER_METASOUND_DATATYPE(bool, "Bool", ::Metasound::ELiteralType::Boolean)
REGISTER_METASOUND_DATATYPE(int32, "Int32", ::Metasound::ELiteralType::Integer)
REGISTER_METASOUND_DATATYPE(float, "Float", ::Metasound::ELiteralType::Float)
REGISTER_METASOUND_DATATYPE(FString, "String", ::Metasound::ELiteralType::String)
REGISTER_METASOUND_DATATYPE(Metasound::FTrigger, "Trigger", ::Metasound::ELiteralType::Boolean)
REGISTER_METASOUND_DATATYPE(Metasound::FTime, "Time", ::Metasound::ELiteralType::Float)
REGISTER_METASOUND_DATATYPE(Metasound::FSendAddress, "Transmission:Address", ::Metasound::ELiteralType::String)

// FAudioBuffer is declared and defined in the MetaSoundGraphCore module. Here we only
// enqueue the registration command to avoid linker issues caused by the interplay
// of the REGISTER_METASOUND_DATATYPE and DEFINE_METASOUND_DATA_TYPE macros.
ENQUEUE_METASOUND_DATATYPE_REGISTRATION_COMMAND(Metasound::FAudioBuffer, "Audio");
