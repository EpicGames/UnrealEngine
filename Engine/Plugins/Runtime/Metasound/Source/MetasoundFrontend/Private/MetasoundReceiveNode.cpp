// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundReceiveNode.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace ReceiveNodeInfo
	{
		DEFINE_METASOUND_PARAM(AddressInput, "Address", "Address")
		DEFINE_METASOUND_PARAM(DefaultDataInput, "Default", "Default")
		DEFINE_METASOUND_PARAM(Output, "Out", "Out")

		const FDataVertexMetadata AddressInputMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(AddressInput) // display name
		};
		const FDataVertexMetadata DefaultDataInputMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(DefaultDataInput) // display name
		};
		const FDataVertexMetadata OutputMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(Output) // display name
		};

		FNodeClassName GetClassNameForDataType(const FName& InDataTypeName)
		{
			return FNodeClassName { "Receive", InDataTypeName, FName() };
		}

		int32 GetCurrentMajorVersion() { return 1; }

		int32 GetCurrentMinorVersion() { return 0; }
	}

	namespace ReceiveNodePrivate
	{
		Frontend::FNodeClassRegistryKey CreateNodeClassRegistryKey(const FName& InDataTypeName)
		{
			using namespace ReceiveNodeInfo;
			return Frontend::FNodeClassRegistryKey{EMetasoundFrontendClassType::External, GetClassNameForDataType(InDataTypeName), GetCurrentMajorVersion(), GetCurrentMinorVersion()};
		}
	}
}

#undef LOCTEXT_NAMESPACE
