// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundRouter.h"

#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundTrace.h"
#include "HAL/IConsoleManager.h"

namespace Metasound
{
	bool FTransmissionAddress::operator==(const FTransmissionAddress& InOther) const
	{
		return IsEqual(InOther);
	}

	bool FTransmissionAddress::operator!=(const FTransmissionAddress& InOther) const
	{
		return !IsEqual(InOther);
	}

	const FLazyName FSendAddress::AddressType("MetaSoundSendAddress");

	FSendAddress::FSendAddress(const FString& InChannelName)
	: ChannelName(*InChannelName)
	, DataType()
	, InstanceID(INDEX_NONE)
	{
	}

	FSendAddress::FSendAddress(const FName& InChannelName, const FName& InDataType, uint64 InInstanceID)
	: ChannelName(InChannelName)
	, DataType(InDataType)
	, InstanceID(InInstanceID)
	{
	}

	FName FSendAddress::GetAddressType() const
	{
		return FSendAddress::AddressType;
	}

	TUniquePtr<FTransmissionAddress> FSendAddress::Clone() const 
	{
		return MakeUnique<FSendAddress>(*this);
	}

	FName FSendAddress::GetDataType() const
	{
		return DataType;
	}

	const FName& FSendAddress::GetChannelName() const
	{
		return ChannelName;
	}

	uint64 FSendAddress::GetInstanceID() const
	{
		return InstanceID;
	}

	FString FSendAddress::ToString() const
	{
		return FString::Format(TEXT("SendAddress {0}:{1}[Type={2}]"), {ChannelName.ToString(), InstanceID, DataType.ToString()});
	}

	uint32 FSendAddress::GetHash() const
	{
		uint32 HashedChannel = HashCombineFast(GetTypeHashHelper(DataType), GetTypeHashHelper(ChannelName));
		HashedChannel = HashCombineFast(HashedChannel, GetTypeHashHelper(InstanceID));
		return HashedChannel;
	}

	bool FSendAddress::IsEqual(const FTransmissionAddress& InOther) const
	{
		if (const FSendAddress* OtherSendAddress = CastAddressType<FSendAddress>(InOther))
		{
			return (InstanceID == OtherSendAddress->InstanceID) && (ChannelName == OtherSendAddress->ChannelName) && (DataType == OtherSendAddress->DataType); 
		}
		return false;
	}

	IReceiver::~IReceiver()
	{
		DataChannel->OnReceiverDestroyed();
	}

	ISender::~ISender()
	{
		DataChannel->OnSenderDestroyed();
	}

	FDataTransmissionCenter& FDataTransmissionCenter::Get()
	{
		static FDataTransmissionCenter Singleton;
		return Singleton;
	}

	TUniquePtr<ISender> FDataTransmissionCenter::RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams)
	{
		return GlobalRouter.RegisterNewSender(InAddress, InitParams);
	}

	TUniquePtr<IReceiver> FDataTransmissionCenter::RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams)
	{
		return GlobalRouter.RegisterNewReceiver(InAddress, InitParams);
	}

	bool FDataTransmissionCenter::UnregisterDataChannel(const FTransmissionAddress& InAddress)
	{
		return GlobalRouter.UnregisterDataChannel(InAddress);
	}

	bool FDataTransmissionCenter::UnregisterDataChannelIfUnconnected(const FTransmissionAddress& InAddress)
	{
		return GlobalRouter.UnregisterDataChannelIfUnconnected(InAddress);
	}

	bool FDataTransmissionCenter::PushLiteral(FName DataType, FName GlobalChannelName, const FLiteral& InParam)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GlobalRouter.PushLiteral(DataType, GlobalChannelName, InParam);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FAddressRouter::FAddressRouter(const FAddressRouter& Other)
		: DataChannelMap(Other.DataChannelMap)
	{
	}

	TSharedPtr<IDataChannel, ESPMode::ThreadSafe> FAddressRouter::FindDataChannel(const FTransmissionAddress& InAddress)
	{
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> Channel;

		{
			FScopeLock ScopeLock(&DataChannelMapMutationLock);

			if (TSharedRef<IDataChannel, ESPMode::ThreadSafe>* ExistingChannelPtr = DataChannelMap.Find(InAddress))
			{
				Channel = *ExistingChannelPtr;
			}
		}

		return Channel;
	}

	TSharedPtr<IDataChannel, ESPMode::ThreadSafe> FAddressRouter::GetDataChannel(const FTransmissionAddress& InAddress, const FOperatorSettings& InOperatorSettings)
	{
		METASOUND_LLM_SCOPE;
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> DataChannel = FindDataChannel(InAddress);

		if (!DataChannel.IsValid())
		{
			FScopeLock ScopeLock(&DataChannelMapMutationLock);

			// This is the first time we're seeing this, add it to the map.
			DataChannel = Metasound::Frontend::IDataTypeRegistry::Get().CreateDataChannel(InAddress.GetDataType(), InOperatorSettings);
			if (DataChannel.IsValid())
			{
				DataChannelMap.Add(InAddress, DataChannel.ToSharedRef());
			}
		}

		return DataChannel;
	}

	TUniquePtr<ISender> FAddressRouter::RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams)
	{
		METASOUND_LLM_SCOPE;
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> DataChannel = GetDataChannel(InAddress, InitParams.OperatorSettings);

		if (DataChannel.IsValid())
		{
			return DataChannel->NewSender(InitParams);
		}
		else
		{
			return TUniquePtr<ISender>(nullptr);
		}
	}

	bool FAddressRouter::PushLiteral(const FName& InDataTypeName, const FName& InChannelName, const FLiteral& InParam)
	{
		FSendAddress Address{ InChannelName, InDataTypeName };

		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> Channel = FindDataChannel(Address);
		if (Channel.IsValid())
		{
			return Channel->PushLiteral(InParam);
		}
		else
		{
			return false;
		}
	}

	bool FAddressRouter::UnregisterDataChannel(const FTransmissionAddress& InAddress)
	{
		METASOUND_LLM_SCOPE;
		FScopeLock ScopeLock(&DataChannelMapMutationLock);

		if (TSharedRef<IDataChannel, ESPMode::ThreadSafe>* Channel = DataChannelMap.Find(InAddress))
		{
			if (const int32 NumReceiversActive = Channel->Get().GetNumActiveReceivers())
			{
				UE_LOGF(LogMetaSound, Verbose, "DataChannel '%ls' shutting down with %d receivers active.", *InAddress.ToString(), NumReceiversActive);
			}

			if (const int32 NumSendersActive = Channel->Get().GetNumActiveSenders())
			{
				UE_LOGF(LogMetaSound, Verbose, "DataChannel '%ls' shutting down with %d senders active.", *InAddress.ToString(), NumSendersActive);
			}
		}

		return DataChannelMap.Remove(InAddress) > 0;
	}

	bool FAddressRouter::UnregisterDataChannelIfUnconnected(const FTransmissionAddress& InAddress)
	{
		METASOUND_LLM_SCOPE;
		FScopeLock ScopeLock(&DataChannelMapMutationLock);

		if (TSharedRef<IDataChannel, ESPMode::ThreadSafe>* Channel = DataChannelMap.Find(InAddress))
		{
			if (0 == Channel->Get().GetNumActiveReceivers())
			{
				if (0 == Channel->Get().GetNumActiveSenders())
				{
					return DataChannelMap.Remove(InAddress) > 0;
				}
			}
		}

		return false;
	}

	TUniquePtr<IReceiver> FAddressRouter::RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams)
	{
		METASOUND_LLM_SCOPE;
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> DataChannel = GetDataChannel(InAddress, InitParams.OperatorSettings);

		if (DataChannel.IsValid())
		{
			return DataChannel->NewReceiver(InitParams);
		}
		else
		{
			return TUniquePtr<IReceiver>(nullptr);
		}
	}

	FAddressRouter::FTransmissionAddressKey::FTransmissionAddressKey(const FTransmissionAddress& InCopy)
	: PImpl(InCopy.Clone())
	{}

	FAddressRouter::FTransmissionAddressKey::FTransmissionAddressKey(const FTransmissionAddressKey& InCopy)
	: PImpl(InCopy.PImpl->Clone())
	{}

	FAddressRouter::FTransmissionAddressKey::FTransmissionAddressKey(TUniquePtr<FTransmissionAddress>&& InImpl)
	: PImpl(MoveTemp(InImpl))
	{
	}

	FAddressRouter::FTransmissionAddressKey& FAddressRouter::FTransmissionAddressKey::operator=(const FTransmissionAddress& InOther)
	{
		PImpl = InOther.Clone();
		return *this;
	}

	FName FAddressRouter::FTransmissionAddressKey::GetAddressType() const
	{
		return PImpl->GetAddressType();
	}

	FName FAddressRouter::FTransmissionAddressKey::GetDataType() const
	{
		return PImpl->GetDataType();
	}

	TUniquePtr<FTransmissionAddress> FAddressRouter::FTransmissionAddressKey::Clone() const
	{
		return PImpl->Clone();
	}

	FString FAddressRouter::FTransmissionAddressKey::ToString() const
	{
		return PImpl->ToString();
	}

	uint32 FAddressRouter::FTransmissionAddressKey::GetHash() const
	{
		return GetTypeHash(*PImpl);
	}

	bool FAddressRouter::FTransmissionAddressKey::IsEqual(const FTransmissionAddress& InOther) const
	{
		return InOther == *PImpl;
	}
}
