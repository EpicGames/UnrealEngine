// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundSendNode.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"
namespace Metasound::SendNodePrivate
{
	constexpr int32 SendNodeMajorVersion = 1;
	constexpr int32 SendNodeMinorVersion = 0;

	
	FVertexInterface CreateVertexInterface(const FName& InDataTypeName)
	{
		using namespace SendVertexNames; 
		static const FDataVertexMetadata AddressInputMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(AddressInput) // display name
		};

		return FVertexInterface(
			FInputVertexInterface(
				TInputDataVertex<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), AddressInputMetadata),
				FInputDataVertex(GetSendInputName(InDataTypeName), InDataTypeName, FDataVertexMetadata{ FText::GetEmpty() })
			),
			FOutputVertexInterface(
			)
		);
	}

	const FName& GetSendInputName(const FName& InDataTypeName)
	{
		return InDataTypeName;
	}

	FNodeClassName CreateNodeClassName(const FName& InDataTypeName)
	{
		return FNodeClassName{ "Send", InDataTypeName, FName() };
	}

	Frontend::FNodeClassRegistryKey CreateNodeClassRegistryKey(const FName& InDataTypeName)
	{
		return Frontend::FNodeClassRegistryKey{EMetasoundFrontendClassType::External, CreateNodeClassName(InDataTypeName), SendNodeMajorVersion, SendNodeMinorVersion};
	}

	FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FText& InDataTypeDisplayText)
	{
		const FVertexName& InputName = GetSendInputName(InDataTypeName);
		FNodeClassMetadata Info;

		Info.ClassName = CreateNodeClassName(InDataTypeName);
		Info.MajorVersion = SendNodeMajorVersion;
		Info.MinorVersion = SendNodeMinorVersion;
		Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_SendNodeDisplayNameFormat", "Send {0}", InDataTypeDisplayText);
		Info.Description = METASOUND_LOCTEXT("Metasound_SendNodeDescription", "Sends data from a send node with the same name.");
		Info.Author = PluginAuthor;
		Info.PromptIfMissing = PluginNodeMissingPrompt;
		Info.DefaultInterface = CreateVertexInterface(InDataTypeName);
		Info.CategoryHierarchy = { METASOUND_LOCTEXT("Metasound_TransmissionNodeCategory", "Transmission") };
		Info.Keywords = { };

		// Then send & receive nodes do not work as expected, particularly 
		// around multiple-consumer scenarios. Deprecate them to avoid
		// metasound assets from relying on send & receive nodes. 
		EnumAddFlags(Info.AccessFlags, ENodeClassAccessFlags::Deprecated);

		return Info;
	}
} // namespace Metasound::SendNodePrivate

#undef LOCTEXT_NAMESPACE
