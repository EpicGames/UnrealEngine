// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundTrigger.h"

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"


namespace Metasound
{
	FTrigger::FTrigger(const FOperatorSettings& InSettings, bool bShouldTrigger)
	:	NumFramesPerBlock(InSettings.GetNumFramesPerBlock())
	,	SampleRate(InSettings.GetSampleRate())
	{
		if (bShouldTrigger)
		{
			TriggerFrame(0);
		}
	}

	FTrigger::FTrigger(const FOperatorSettings& InSettings, int32 InFrameToTrigger)
	:	NumFramesPerBlock(InSettings.GetNumFramesPerBlock())
	,	SampleRate(InSettings.GetSampleRate())
	{
		TriggerFrame(InFrameToTrigger);
	}

	FTrigger::FTrigger(const FOperatorSettings& InSettings)
		: FTrigger(InSettings, false)
	{}

	void FTrigger::AssignRawParameter(const void* ParamPackPayload)
	{
		// We don't even need to check the payload. If the parameter pack contained
		// a Trigger bool, it must be true. There is no concept of "un-trigger" :-) 
		TriggerFrame(0);
	}

	void FTrigger::TriggerFrame(int32 InFrameToTrigger)
	{
		if (ensureMsgf(InFrameToTrigger >= 0, TEXT("Do not trigger negative frames [%d]"), InFrameToTrigger))
		{
			// Insert trigger frame index into sorted triggered frames
			TriggeredFrames.Insert(InFrameToTrigger, Algo::LowerBound(TriggeredFrames, InFrameToTrigger));

			if (InFrameToTrigger < NumFramesPerBlock)
			{
				UpdateLastTriggerIndexInBlock();
			}

			bHasTrigger = true;
		}
	}

	void FTrigger::AdvanceBlock()
	{
		Advance(NumFramesPerBlock);
	}

	void FTrigger::Advance(int32 InNumFrames)
	{
		if (bHasTrigger)
		{
			const int32 NumToRemove = Algo::LowerBound(TriggeredFrames, InNumFrames);

			if (NumToRemove)
			{
				TriggeredFrames.RemoveAt(0, NumToRemove, EAllowShrinking::No);
			}

			const int32 Num = TriggeredFrames.Num();

			if (Num > 0)
			{
				for (int32 i = 0; i < Num; i++)
				{
					TriggeredFrames[i] -= InNumFrames;
				}
			}
			else
			{
				bHasTrigger = false;
			}

			UpdateLastTriggerIndexInBlock();
		}
	}

	int32 FTrigger::Num() const
	{
		return TriggeredFrames.Num();
	}

	int32 FTrigger::NumTriggeredInBlock() const
	{
		return LastTriggerIndexInBlock;
	}

	int32 FTrigger::operator[](int32 InTriggerIndex) const
	{
		return TriggeredFrames[InTriggerIndex];
	}

	int32 FTrigger::First() const
	{
		return TriggeredFrames.IsEmpty() ? INDEX_NONE : TriggeredFrames[0];
	}

	int32 FTrigger::Last() const
	{
		return TriggeredFrames.IsEmpty() ? INDEX_NONE : TriggeredFrames.Last();
	}

	bool FTrigger::IsTriggered() const
	{
		return bHasTrigger;
	}

	bool FTrigger::IsTriggeredInBlock() const
	{
		return LastTriggerIndexInBlock > 0;
	}

	FTrigger::operator bool() const 
	{
		return IsTriggeredInBlock();
	}

	void FTrigger::Reset()
	{
		TriggeredFrames.Reset();
		LastTriggerIndexInBlock = 0;
		bHasTrigger = false;
	}

	void FTrigger::RemoveAfter(int32 InStartingFrame)
	{
		if (InStartingFrame <= 0)
		{
			Reset();
		}
		else if (TriggeredFrames.Num() > 0)
		{
			// Algo::UpperBound Returns index of first element > InStartingFrame
			int32 BeginningIndex = Algo::UpperBound(TriggeredFrames, InStartingFrame);
			int32 NumToRemove = TriggeredFrames.Num() - BeginningIndex;

			if (NumToRemove > 0)
			{
				TriggeredFrames.RemoveAt(BeginningIndex, NumToRemove);
				UpdateLastTriggerIndexInBlock();
				bHasTrigger = TriggeredFrames.Num() > 0;
			}
		}
	}

	void FTrigger::UpdateLastTriggerIndexInBlock()
	{
		LastTriggerIndexInBlock = Algo::LowerBound(TriggeredFrames, NumFramesPerBlock);
	}
}

const TCHAR* LexToString(const Metasound::FTrigger& Trigger)
{
	if (Trigger)
	{
		return TEXT("Metasound::FTrigger_True");
	}

	return TEXT("Metasound::FTrigger_False");
}
