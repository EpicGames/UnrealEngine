// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTemplates/MetasoundFrontendNodeTemplateAudioAnalyzer.h"

#include "MetasoundFrontendDocumentBuilder.h"

namespace Metasound::Frontend
{
	namespace AudioAnalyzerNodeTemplatePrivate
	{
		class FAudioAnalyzerNodeTemplateTransform : public INodeTemplateTransform
		{
		public:
			FAudioAnalyzerNodeTemplateTransform() = default;
			virtual ~FAudioAnalyzerNodeTemplateTransform() = default;

			virtual bool Transform(const FGuid& InPageID, const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const override;
		};

		bool FAudioAnalyzerNodeTemplateTransform::Transform(const FGuid& InPageID, const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const
		{
			// Strip it out:
			return OutBuilder.RemoveNode(InNodeID, &InPageID);
		}
	} // namespace AudioAnalyzerNodeTemplatePrivate

	const FMetasoundFrontendClassName FAudioAnalyzerNodeTemplate::ClassName{ "UE", "Audio Analyzer", "" };

	const FMetasoundFrontendVersionNumber FAudioAnalyzerNodeTemplate::VersionNumber{ 1, 0 };
	
	FMetasoundFrontendNodeInterface FAudioAnalyzerNodeTemplate::GenerateNodeInterface(FNodeTemplateGenerateInterfaceParams InParams) const
	{
		static const FName VertexName = "Value";
		const FName DataType = GetMetasoundDataTypeName<FAudioBuffer>();

		FMetasoundFrontendNodeInterface NewInterface;
		NewInterface.Inputs.Add(FMetasoundFrontendVertex{ VertexName, DataType, FGuid::NewGuid() });
		return NewInterface;
	}

	const FMetasoundFrontendClassName& FAudioAnalyzerNodeTemplate::GetClassName() const
	{
		return ClassName;
	}

	TUniquePtr<INodeTemplateTransform> FAudioAnalyzerNodeTemplate::GenerateNodeTransform() const
	{
		return TUniquePtr<INodeTemplateTransform>(new AudioAnalyzerNodeTemplatePrivate::FAudioAnalyzerNodeTemplateTransform());
	}

	const FMetasoundFrontendClass& FAudioAnalyzerNodeTemplate::GetFrontendClass() const
	{
		auto CreateFrontendClass = []()
		{
			FMetasoundFrontendClass Class;
			Class.Metadata.SetClassName(ClassName);
#if WITH_EDITOR
			Class.Metadata.SetSerializeText(false);
			Class.Metadata.SetAuthor(Metasound::PluginAuthor);
#endif // WITH_EDITOR
			Class.Metadata.SetType(EMetasoundFrontendClassType::Template);
			Class.Metadata.SetVersion(VersionNumber);

			return Class;
		};

		static const FMetasoundFrontendClass FrontendClass = CreateFrontendClass();
		return FrontendClass;
	}
	
	EMetasoundFrontendVertexAccessType FAudioAnalyzerNodeTemplate::GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		return EMetasoundFrontendVertexAccessType::Unset;
	}

	EMetasoundFrontendVertexAccessType FAudioAnalyzerNodeTemplate::GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		return EMetasoundFrontendVertexAccessType::Unset;
	}

	const FMetasoundFrontendVersionNumber& FAudioAnalyzerNodeTemplate::GetVersionNumber() const
	{
		return VersionNumber;
	}

	bool FAudioAnalyzerNodeTemplate::IsInputAccessTypeDynamic() const
	{
		return false;
	}

	bool FAudioAnalyzerNodeTemplate::IsInputConnectionUserModifiable() const
	{
		return true;
	}

	bool FAudioAnalyzerNodeTemplate::IsOutputAccessTypeDynamic() const
	{
		return false;
	}

	bool FAudioAnalyzerNodeTemplate::IsOutputConnectionUserModifiable() const
	{
		return false;
	}

	bool FAudioAnalyzerNodeTemplate::IsValidNodeInterface(const FMetasoundFrontendNodeInterface& InNodeInterface) const
	{
		if (InNodeInterface.Inputs.Num() != 1)
		{
			return false;
		}

		if (InNodeInterface.Outputs.Num() != 0)
		{
			return false;
		}

		if (InNodeInterface.Inputs.Last().TypeName != GetMetasoundDataTypeName<FAudioBuffer>())
		{
			return false;
		}

		return true;
	}
}
