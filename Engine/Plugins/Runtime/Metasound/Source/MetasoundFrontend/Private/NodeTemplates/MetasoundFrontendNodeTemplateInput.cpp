// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTemplates/MetasoundFrontendNodeTemplateInput.h"

#include "Algo/AnyOf.h"
#include "Internationalization/Text.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundInputNode.h"

#if WITH_EDITOR
#include "MetasoundFrontendController.h"
#endif // WITH_EDITOR


namespace Metasound::Frontend
{
	namespace InputNodeTemplatePrivate
	{
		// Creates an input template node, sets node position (should only ever be one in style location) from and connects it to the associated input with the given name.
		const FMetasoundFrontendNode* InitTemplateNode(
			const INodeTemplate& InTemplate,
			FName InputName,
			FMetaSoundFrontendDocumentBuilder& InOutBuilder,
			const FMetasoundFrontendVertexHandle& InputNodeVertex,
			const TArray<FMetasoundFrontendVertexHandle>& ConnectedVertices, const FGuid* InPageID = nullptr)
		{
			FMetasoundFrontendEdge NewEdge;
			FName TypeName;
#if WITH_EDITORONLY_DATA
			TMap<FGuid, FVector2D> Locations;
#endif // WITH_EDITORONLY_DATA

			// Cache data from Input node pointer as needed as subsequent call to create new template node may invalidate the input node's pointer
			{
				const FMetasoundFrontendNode* InputNode = InOutBuilder.FindGraphInputNode(InputName, InPageID);
				check(InputNode);
				TypeName = InputNode->Interface.Outputs.Last().TypeName;
				NewEdge.FromNodeID = InputNode->GetID(),
				NewEdge.FromVertexID = InputNode->Interface.Outputs.Last().VertexID;

#if WITH_EDITORONLY_DATA
				Locations = InputNode->Style.Display.Locations;
				ensure(Locations.Num() <= 1);
#endif // WITH_EDITORONLY_DATA
			}

			FNodeTemplateGenerateInterfaceParams Params { { }, { TypeName } };
			
			const FMetasoundFrontendNode* TemplateNode = InOutBuilder.AddNodeByTemplate(InTemplate, MoveTemp(Params), FGuid::NewGuid(), InPageID);
			check(TemplateNode);
			NewEdge.ToNodeID = TemplateNode->GetID();
			NewEdge.ToVertexID = TemplateNode->Interface.Inputs.Last().VertexID;

#if WITH_EDITORONLY_DATA
			if (Locations.IsEmpty())
			{
				// If connections are present, add a location for safety.  Try adding near existing node.
				if (!ConnectedVertices.IsEmpty())
				{
					UE_LOGF(LogMetaSound, Display, "Template node being generated for input '%ls' had no editor location set.  Procedurally placing near connected node.", *InputName.ToString());
					FVector2D NewLocation;
					if (const FMetasoundFrontendNode* Node = InOutBuilder.FindNode(ConnectedVertices.Last().NodeID))
					{
						if (!Node->Style.Display.Locations.IsEmpty())
						{
							auto It = Node->Style.Display.Locations.CreateConstIterator();
							const TPair<FGuid, FVector2D>& Pair = *It;
							const FGuid ConnectedVertexID = ConnectedVertices.Last().VertexID;
							const uint32 Index = Node->Interface.Inputs.IndexOfByPredicate([&ConnectedVertexID](const FMetasoundFrontendVertex& Input)
							{
								return Input.VertexID == ConnectedVertexID;
							});
							NewLocation = Pair.Value;
							NewLocation -= DisplayStyle::NodeLayout::DefaultOffsetX;
							// Offset Y position based on connected input index to avoid overlapping nodes
							NewLocation += Index * DisplayStyle::NodeLayout::DefaultOffsetY;
						}
					}
					
					InOutBuilder.SetNodeLocation(NewEdge.ToNodeID, NewLocation, nullptr, InPageID);
				}
			}
			else
			{
				for (const TPair<FGuid, FVector2D>& Pair : Locations)
				{
					InOutBuilder.SetNodeLocation(NewEdge.ToNodeID, Pair.Value, nullptr, InPageID);
				}
			}
#endif // WITH_EDITORONLY_DATA

			// Add edge between input node and new template node
			InOutBuilder.AddEdge(MoveTemp(NewEdge), InPageID);

			FMetasoundFrontendEdge EdgeToRemove { InputNodeVertex.NodeID, InputNodeVertex.VertexID };
			for (const FMetasoundFrontendVertexHandle& ConnectedVertex : ConnectedVertices)
			{
				// Swap connections from input node to connected node to now be from template node to connected node
				EdgeToRemove.ToNodeID = ConnectedVertex.NodeID;
				EdgeToRemove.ToVertexID = ConnectedVertex.VertexID;
				InOutBuilder.RemoveEdge(EdgeToRemove);

				InOutBuilder.AddEdge(FMetasoundFrontendEdge
				{
					TemplateNode->GetID(),
					TemplateNode->Interface.Outputs.Last().VertexID,
					ConnectedVertex.NodeID,
					ConnectedVertex.VertexID
				});
			}
			return TemplateNode;
		};
	} // namespace InputNodeTemplatePrivate

	const FMetasoundFrontendClassName FInputNodeTemplate::ClassName { "UE", "Input", "Template" };

	const FMetasoundFrontendVersionNumber FInputNodeTemplate::VersionNumber = { 1, 0 } ;

#if WITH_EDITOR
	const FMetasoundFrontendNode* FInputNodeTemplate::CreateNode(FMetaSoundFrontendDocumentBuilder& InOutBuilder, FName InputName, const FGuid* InPageID)
	{
		if (const FMetasoundFrontendClassInput* Input = InOutBuilder.FindGraphInput(InputName))
		{
			const INodeTemplate* ThisTemplate = INodeTemplateRegistry::Get().FindTemplate(ClassName);
			check(ThisTemplate);
					
			const FMetasoundFrontendNode* InputNode = InOutBuilder.FindGraphInputNode(InputName);
			if (!InputNode)
			{
				return nullptr;
			}

			const FGuid& InputNodeOutputVertexID = InputNode->Interface.Outputs.Last().VertexID;
			FMetasoundFrontendVertexHandle InputNodeVertex { InputNode->GetID(), InputNodeOutputVertexID };

			return InputNodeTemplatePrivate::InitTemplateNode(*ThisTemplate, InputName, InOutBuilder, InputNodeVertex, /*ConnectedVertices*/{}, InPageID);
		}

		return nullptr;
	}
#endif // WITH_EDITOR

	const TArray<FMetasoundFrontendClassInputDefault>* FInputNodeTemplate::FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const
	{
		// Just returns the default of the given node's class input and not walk to values provided by connected input like reroutes.
		return FNodeTemplateBase::FindNodeClassInputDefaults(InBuilder, InPageID, InNodeID, VertexName);
	}

	const FMetasoundFrontendClassName& FInputNodeTemplate::GetClassName() const
	{
		return FInputNodeTemplate::ClassName;
	}

#if WITH_EDITOR
	FText FInputNodeTemplate::GetNodeDisplayName(const IMetaSoundDocumentInterface& Interface, const FGuid& InPageID, const FGuid& InNodeID) const
	{
		return { };
	}
#endif // WITH_EDITOR

	const FMetasoundFrontendClass& FInputNodeTemplate::GetFrontendClass() const
	{
		auto CreateFrontendClass = []()
		{
			using namespace Metasound;

			FMetasoundFrontendClass Class;
			Class.Metadata.SetClassName(ClassName);

#if WITH_EDITOR
			Class.Metadata.SetSerializeText(false);
			Class.Metadata.SetAuthor(PluginAuthor);
			Class.Metadata.SetDescription(FInputNode::GetInputDescription());

			FMetasoundFrontendClassStyleDisplay& StyleDisplay = Class.Style.Display;
			StyleDisplay.bShowInputNames = false;
			StyleDisplay.bShowOutputNames = true;
			StyleDisplay.bShowLiterals = false;
			StyleDisplay.bShowName = true;
#endif // WITH_EDITOR

			Class.Metadata.SetType(EMetasoundFrontendClassType::Template);
			Class.Metadata.SetVersion(VersionNumber);

			return Class;
		};

		static const FMetasoundFrontendClass FrontendClass = CreateFrontendClass();
		return FrontendClass;
	}

	const FInputNodeTemplate& FInputNodeTemplate::GetChecked()
	{
		const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(GetRegistryKey());
		checkf(Template, TEXT("Failed to find InputNodeTemplate, which is required for migrating editor document data"));
		return static_cast<const FInputNodeTemplate&>(*Template);
	}

	const FNodeRegistryKey& FInputNodeTemplate::GetRegistryKey()
	{
		static const FNodeRegistryKey RegistryKey(EMetasoundFrontendClassType::Template, ClassName, VersionNumber);
		return RegistryKey;
	}

	EMetasoundFrontendVertexAccessType FInputNodeTemplate::GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		const FMetasoundFrontendNode* ConnectedInputNode = nullptr;
		if (const FMetasoundFrontendVertex* ConnectedInputOutput = InBuilder.FindNodeOutputConnectedToNodeInput(InNodeID, InVertexID, &ConnectedInputNode, &InPageID))
		{
			check(ConnectedInputNode);
			const FMetasoundFrontendClass* InputClass = InBuilder.FindDependency(ConnectedInputNode->ClassID);
			check(InputClass);

			return InputClass->GetInterfaceForNode(*ConnectedInputNode).Outputs.Last().AccessType;
		}

		return EMetasoundFrontendVertexAccessType::Unset;
	}

	EMetasoundFrontendVertexAccessType FInputNodeTemplate::GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		if (const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID, &InPageID))
		{
			const FMetasoundFrontendVertex& Input = Node->Interface.Inputs.Last();
			return GetNodeInputAccessType(InBuilder, InPageID, InNodeID, Input.VertexID);
		}

		return EMetasoundFrontendVertexAccessType::Unset;
	}

#if WITH_EDITOR
	FText FInputNodeTemplate::GetOutputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName OutputName) const
	{
		const FMetasoundFrontendNode* OwningNode = InBuilder.FindNode(InNodeID, &InPageID);
		if (!OwningNode)
		{
			return FText::FromName(OutputName);
		}

		const FMetasoundFrontendNode* ConnectedInputNode = nullptr;
		const FMetasoundFrontendVertex* ConnectedOutput = InBuilder.FindNodeOutputConnectedToNodeInput(InNodeID, OwningNode->Interface.Inputs.Last().VertexID, &ConnectedInputNode, &InPageID);
		if (ensureMsgf(ConnectedInputNode, TEXT("Input template node should always be connected to associated input node's only output")))
		{
			FName NodeName = ConnectedInputNode->Name;
			FText DisplayName;
			if (const FMetasoundFrontendClassInput* Input = InBuilder.FindGraphInput(NodeName))
			{
				DisplayName = Input->Metadata.GetDisplayName();
			}

			constexpr bool bIncludeNamespace = true;
			return INodeTemplate::ResolveMemberDisplayName(NodeName, DisplayName, bIncludeNamespace);
		}

		return FRerouteNodeTemplate::GetOutputVertexDisplayName(InBuilder, InPageID, InNodeID, OutputName);
	}

	bool FInputNodeTemplate::HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage) const
	{
		return true;
	}
#endif // WITH_EDITOR

	bool FInputNodeTemplate::IsInputAccessTypeDynamic() const
	{
		return true;
	}

	bool FInputNodeTemplate::IsInputConnectionUserModifiable() const
	{
		return false;
	}

	bool FInputNodeTemplate::IsOutputAccessTypeDynamic() const
	{
		return true;
	}

#if WITH_EDITOR
	bool FInputNodeTemplate::Inject(FMetaSoundFrontendDocumentBuilder& InOutBuilder, bool bForceNodeCreation) const
	{
		bool bInjectedNodes = false;

		const TArray<FMetasoundFrontendClassInput>& Inputs = InOutBuilder.GetConstDocumentChecked().RootGraph.GetDefaultInterface().Inputs;
		for (const FMetasoundFrontendClassInput& Input : Inputs)
		{
			// Potentially not used input, which is perfectly valid so early out
			const FMetasoundFrontendNode* InputNode = InOutBuilder.FindGraphInputNode(Input.Name);
			if (!InputNode)
			{
				continue;
			}

			const FGuid& InputNodeOutputVertexID = InputNode->Interface.Outputs.Last().VertexID;

			TArray<const FMetasoundFrontendNode*> ConnectedInputNodes;
			TArray<const FMetasoundFrontendVertex*> ConnectedInputVertices = InOutBuilder.FindNodeInputsConnectedToNodeOutput(InputNode->GetID(), InputNodeOutputVertexID, &ConnectedInputNodes);

			FMetasoundFrontendVertexHandle InputNodeVertex { InputNode->GetID(), InputNodeOutputVertexID };

			bool bHasTemplateConnection = false;

			TArray<FMetasoundFrontendVertexHandle> ConnectedVertices;
			for (int32 Index = 0; Index < ConnectedInputVertices.Num(); ++Index)
			{
				// Ignore edges already connected to input template nodes & cache connected vertex pair
				// as adding a template node in the subsequent step may invalidate these connected node/vertex
				// pointers.
				const FMetasoundFrontendVertex* ConnectedVertex = ConnectedInputVertices[Index];
				const FMetasoundFrontendNode* ConnectedNode = ConnectedInputNodes[Index];
				const FMetasoundFrontendClass* Class = InOutBuilder.FindDependency(ConnectedNode->ClassID);
				if (Class->Metadata.GetClassName() == FInputNodeTemplate::ClassName)
				{
					bHasTemplateConnection = true;
				}
				else
				{
					ConnectedVertices.Add(FMetasoundFrontendVertexHandle { ConnectedNode->GetID(), ConnectedVertex->VertexID });
				}
			}

			if (ConnectedVertices.IsEmpty())
			{
				if (bForceNodeCreation && !bHasTemplateConnection)
				{
					bInjectedNodes = true;
					InputNodeTemplatePrivate::InitTemplateNode(*this, Input.Name, InOutBuilder, InputNodeVertex, ConnectedVertices);
				}
			}
			else
			{
				bInjectedNodes = true;
				InputNodeTemplatePrivate::InitTemplateNode(*this, Input.Name, InOutBuilder, InputNodeVertex, ConnectedVertices);
			}
		}

		return bInjectedNodes;
	}

#endif // WITH_EDITOR
} // namespace Metasound::Frontend
