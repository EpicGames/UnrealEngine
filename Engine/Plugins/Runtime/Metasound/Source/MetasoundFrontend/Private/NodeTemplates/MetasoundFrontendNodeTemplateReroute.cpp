// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodeTemplates/MetasoundFrontendNodeTemplateReroute.h"

#include "Algo/AnyOf.h"
#include "MetasoundAssetManager.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendTransform.h"


namespace Metasound::Frontend
{
	namespace ReroutePrivate
	{
		struct FRerouteNodeVertexPair
		{
			const FMetasoundFrontendNode* Node= nullptr;
			const FMetasoundFrontendVertex* Vertex = nullptr;

			bool IsValid() const
			{
				return Node && Vertex;
			}
		};

		class FRerouteNodeTemplateTransform : public INodeTemplateTransform
		{
		public:
			FRerouteNodeTemplateTransform() = default;
			virtual ~FRerouteNodeTemplateTransform() = default;

			virtual bool Transform(const FGuid& InPageID, const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const override;
		};

		bool FRerouteNodeTemplateTransform::Transform(const FGuid& InPageID, const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const
		{
			FMetasoundFrontendEdge InputEdge;
			TArray<FMetasoundFrontendEdge> OutputEdges;

			const FMetasoundFrontendNode* Node = OutBuilder.FindNode(InNodeID, &InPageID);
			if (ensureMsgf(Node, TEXT("Failed to find node with ID '%s' when reroute template node transform was given a valid ID for builder '%s'."),
				*InNodeID.ToString(),
				*OutBuilder.GetDebugName()))
			{
				if (!ensureMsgf(Node->Interface.Inputs.Num() == 1, TEXT("Reroute nodes must only have one input")))
				{
					return false;
				}

				if (!ensureMsgf(Node->Interface.Outputs.Num() == 1, TEXT("Reroute nodes must only have one output")))
				{
					return false;
				}

				// Copy input edge to mutate from fields and avoid pointer going out of scope when template node is removed below
				{
					const FMetasoundFrontendVertex& InputVertex = Node->Interface.Inputs.Last();
					TArray<const FMetasoundFrontendEdge*> InputEdges = OutBuilder.FindEdges(Node->GetID(), InputVertex.VertexID, &InPageID);
					if (!InputEdges.IsEmpty())
					{
						InputEdge = *InputEdges.Last();
					}
				}

				// Copy output edges to mutate from fields and avoid pointer going out of scope when swapping below
				{
					const FMetasoundFrontendVertex& OutputVertex = Node->Interface.Outputs.Last();
					TArray<const FMetasoundFrontendEdge*> CurrentOutputEdges = OutBuilder.FindEdges(Node->GetID(), OutputVertex.VertexID, &InPageID);
					Algo::Transform(CurrentOutputEdges, OutputEdges, [](const FMetasoundFrontendEdge* CurrentEdge)
					{
						check(CurrentEdge);
						return *CurrentEdge;
					});
				}

				// Remove the template node
				OutBuilder.RemoveNode(Node->GetID(), &InPageID);

				// Add new connections from reroute source node to reroute destination node. Either could be another reroute,
				// which is valid because said node will subsequently get processed.
				if (InputEdge.GetFromVertexHandle().IsSet())
				{
					bool bModified = !OutputEdges.IsEmpty();
					for (FMetasoundFrontendEdge& OutputEdge : OutputEdges)
					{
						OutputEdge.FromNodeID = InputEdge.FromNodeID;
						OutputEdge.FromVertexID = InputEdge.FromVertexID;
						OutBuilder.AddEdge(MoveTemp(OutputEdge), &InPageID);
					}

					return bModified;
				}
			}

			return false;
		}

		FRerouteNodeVertexPair FindReroutedOutputVertex(
			const FMetasoundFrontendClassName& InClassName,
			const FMetaSoundFrontendDocumentBuilder& InBuilder,
			const FGuid& InPageID, const FMetasoundFrontendNode& InOutputOwningNode, const FMetasoundFrontendVertex& InOutputVertex)
		{
			const FMetasoundFrontendVertex* ReroutedVertex = &InOutputVertex;
			const FMetasoundFrontendNode* ReroutedNode = &InOutputOwningNode;

			if (const FMetasoundFrontendClass* Class = InBuilder.FindDependency(InOutputOwningNode.ClassID))
			{
				if (Class->Metadata.GetClassName() == InClassName)
				{
					const FGuid& OutputOwningNodeID = InOutputOwningNode.GetID();
					TArray<const FMetasoundFrontendVertex*> Inputs = InBuilder.FindNodeInputs(OutputOwningNodeID, { }, &InPageID);
					if (!Inputs.IsEmpty())
					{
						if (const FMetasoundFrontendVertex* RerouteInput = Inputs.Last())
						{
							if (const FMetasoundFrontendVertex* OutputVertex = InBuilder.FindNodeOutputConnectedToNodeInput(OutputOwningNodeID, RerouteInput->VertexID, &ReroutedNode, &InPageID))
							{
								check(ReroutedNode);
								return FindReroutedOutputVertex(InClassName, InBuilder, InPageID, *ReroutedNode, *OutputVertex);
							}
						}
					}

					return { };
				}
			}

			return FRerouteNodeVertexPair { ReroutedNode, ReroutedVertex };
		}

		void FindReroutedInputVertices(
			const FMetasoundFrontendClassName& InClassName,
			const FMetaSoundFrontendDocumentBuilder& InBuilder,
			const FGuid& InPageID,
			const FMetasoundFrontendNode& InInputOwningNode,
			const FMetasoundFrontendVertex& InInputVertex,
			TArray<const FMetasoundFrontendNode*>& InOutReroutedInputOwningNodes,
			TArray<const FMetasoundFrontendVertex*>& InOutReroutedInputVertices)
		{
			using namespace Frontend;

			if (const FMetasoundFrontendClass* Class = InBuilder.FindDependency(InInputOwningNode.ClassID))
			{
				if (Class->Metadata.GetClassName() == InClassName)
				{
					const FGuid& InputOwningNodeID = InInputOwningNode.GetID();
					TArray<const FMetasoundFrontendVertex*> Outputs = InBuilder.FindNodeOutputs(InputOwningNodeID);
					for (const FMetasoundFrontendVertex* Output : Outputs)
					{
						check(Output);

						TArray<const FMetasoundFrontendNode*> ConnectedInputNodes;
						TArray<const FMetasoundFrontendVertex*> ConnectedInputVertices = InBuilder.FindNodeInputsConnectedToNodeOutput(InputOwningNodeID, Output->VertexID, &ConnectedInputNodes, &InPageID);
						check(ConnectedInputNodes.Num() == ConnectedInputVertices.Num());

						for (int32 Index = 0; Index < ConnectedInputNodes.Num(); ++Index)
						{
							const FMetasoundFrontendNode* ConnectedInputOwningNode = ConnectedInputNodes[Index];
							check(ConnectedInputOwningNode);
							const FMetasoundFrontendVertex* ConnectedInputVertex = ConnectedInputVertices[Index];
							check(ConnectedInputVertex);

							FindReroutedInputVertices(InClassName, InBuilder, InPageID, *ConnectedInputOwningNode, *ConnectedInputVertex, InOutReroutedInputOwningNodes, InOutReroutedInputVertices);
						}
					}

					return;
				}
			}

			InOutReroutedInputOwningNodes.Add(&InInputOwningNode);
			InOutReroutedInputVertices.Add(&InInputVertex);
		}
	} // namespace ReroutePrivate

	const FMetasoundFrontendClassName FRerouteNodeTemplate::ClassName { "UE", "Reroute", "" };

	const FMetasoundFrontendVersionNumber FRerouteNodeTemplate::VersionNumber { 1, 0 };

	const FMetasoundFrontendClassName& FRerouteNodeTemplate::GetClassName() const
	{
		return FRerouteNodeTemplate::ClassName;
	}

#if WITH_EDITOR
	FText FRerouteNodeTemplate::GetNodeDisplayName(const IMetaSoundDocumentInterface& DocumentInterface, const FGuid& InPageID, const FGuid& InNodeID) const
	{
		return { };
	}
#endif // WITH_EDITOR

	FMetasoundFrontendNodeInterface FRerouteNodeTemplate::GenerateNodeInterface(FNodeTemplateGenerateInterfaceParams InParams) const
	{
		FName DataType;

		if (!InParams.InputsToConnect.IsEmpty())
		{
			DataType = InParams.InputsToConnect.Last();
		}

		if (!InParams.OutputsToConnect.IsEmpty())
		{
			const FName OutputDataType = InParams.OutputsToConnect.Last();
			if (DataType.IsNone())
			{
				DataType = OutputDataType;
			}
			else
			{
				checkf(DataType == OutputDataType, TEXT("Cannot generate MetasoundFrontendNodeInterface via reroute template with params of unmatched or unset input/output DataType"));
			}
		}

		static const FName VertexName = "Value";
		FMetasoundFrontendNodeInterface NewInterface;
		NewInterface.Inputs.Add(FMetasoundFrontendVertex { VertexName, DataType, FGuid::NewGuid() });
		NewInterface.Outputs.Add(FMetasoundFrontendVertex { VertexName, DataType, FGuid::NewGuid() });

		return NewInterface;

	}

	TUniquePtr<INodeTemplateTransform> FRerouteNodeTemplate::GenerateNodeTransform() const
	{
		using namespace ReroutePrivate;
		return TUniquePtr<INodeTemplateTransform>(new FRerouteNodeTemplateTransform());
	}

	const FMetasoundFrontendClass& FRerouteNodeTemplate::GetFrontendClass() const
	{
		auto CreateFrontendClass = []()
		{
			FMetasoundFrontendClass Class;
			Class.Metadata.SetClassName(ClassName);

#if WITH_EDITOR
			Class.Metadata.SetSerializeText(false);
			Class.Metadata.SetAuthor(Metasound::PluginAuthor);
			Class.Metadata.SetDescription(Metasound::PluginNodeMissingPrompt);

			FMetasoundFrontendClassStyleDisplay& StyleDisplay = Class.Style.Display;
			StyleDisplay.ImageName = "MetasoundEditor.Graph.Node.Class.Reroute";
			StyleDisplay.bShowInputNames = false;
			StyleDisplay.bShowOutputNames = false;
			StyleDisplay.bShowLiterals = false;
			StyleDisplay.bShowName = false;
#endif // WITH_EDITOR

			Class.Metadata.SetType(EMetasoundFrontendClassType::Template);
			Class.Metadata.SetVersion(VersionNumber);


			return Class;
		};

		static const FMetasoundFrontendClass FrontendClass = CreateFrontendClass();
		return FrontendClass;
	}

	const TArray<FMetasoundFrontendClassInputDefault>* FRerouteNodeTemplate::FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const
	{
		// Recursive search down DAG for first connected non-reroute node's input class input
		if (const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID, &InPageID))
		{
			// Should only ever be one
			const FMetasoundFrontendVertex& RerouteOutput = Node->Interface.Outputs.Last();

			TArray<const FMetasoundFrontendNode*> ConnectedNodes;
			TArray<const FMetasoundFrontendVertex*> ConnectedInputs = InBuilder.FindNodeInputsConnectedToNodeOutput(InNodeID, RerouteOutput.VertexID, &ConnectedNodes, &InPageID);
			for (int32 Index = 0; Index < ConnectedNodes.Num(); ++Index)
			{
				const FMetasoundFrontendNode* ConnectedNode = ConnectedNodes[Index];
				if (const FMetasoundFrontendClass* ConnectedNodeClass = InBuilder.FindDependency(ConnectedNode->ClassID))
				{
					const FMetasoundFrontendVertex* ConnectedInput = ConnectedInputs[Index];
					if (ConnectedNodeClass->Metadata.GetClassName() == GetClassName())
					{
						return this->FindNodeClassInputDefaults(InBuilder, InPageID, ConnectedNode->GetID(), ConnectedInput->Name);
					}

					return InBuilder.FindNodeClassInputDefaults(ConnectedNode->GetID(), ConnectedInput->Name, &InPageID);
				}
			}
		}

		return nullptr;
	}

	EMetasoundFrontendVertexAccessType FRerouteNodeTemplate::GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		// Recursive search down DAG for first connected non-reroute node's input access type
		if (const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID, &InPageID))
		{
			// Should only ever be one
			const FMetasoundFrontendVertex& RerouteOutput = Node->Interface.Outputs.Last();

			TArray<const FMetasoundFrontendNode*> ConnectedNodes;
			TArray<const FMetasoundFrontendVertex*> ConnectedInputs = InBuilder.FindNodeInputsConnectedToNodeOutput(InNodeID, RerouteOutput.VertexID, &ConnectedNodes, &InPageID);
			for (int32 Index = 0; Index < ConnectedNodes.Num(); ++Index)
			{
				const FMetasoundFrontendNode* ConnectedNode = ConnectedNodes[Index];
				if (const FMetasoundFrontendClass* ConnectedNodeClass = InBuilder.FindDependency(ConnectedNode->ClassID))
				{
					const FMetasoundFrontendVertex* ConnectedInput = ConnectedInputs[Index];
					if (ConnectedNodeClass->Metadata.GetClassName() == GetClassName())
					{
						return this->GetNodeInputAccessType(InBuilder, InPageID, ConnectedNode->GetID(), ConnectedInput->VertexID);
					}

					return InBuilder.GetNodeInputAccessType(ConnectedNode->GetID(), ConnectedInput->VertexID, &InPageID);
				}
			}
		}

		return EMetasoundFrontendVertexAccessType::Unset;
	}

	EMetasoundFrontendVertexAccessType FRerouteNodeTemplate::GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const
	{
		// Depth-first recursive search for first connected non-reroute node's output access type
		if (const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID))
		{
			// Should only ever be one
			const FMetasoundFrontendVertex& RerouteInput = Node->Interface.Inputs.Last();

			const FMetasoundFrontendNode* ConnectedNode = nullptr;
			if (const FMetasoundFrontendVertex* ConnectedOutput = InBuilder.FindNodeOutputConnectedToNodeInput(InNodeID, RerouteInput.VertexID, &ConnectedNode, &InPageID))
			{
				if (const FMetasoundFrontendClass* ConnectedNodeClass = InBuilder.FindDependency(ConnectedNode->ClassID))
				{
					if (ConnectedNodeClass->Metadata.GetClassName() == ClassName)
					{
						return this->GetNodeOutputAccessType(InBuilder, InPageID, ConnectedNode->GetID(), ConnectedOutput->VertexID);
					}

					return InBuilder.GetNodeOutputAccessType(ConnectedNode->GetID(), ConnectedOutput->VertexID, &InPageID);
				}
			}
		}

		return EMetasoundFrontendVertexAccessType::Unset;
	}

	const FNodeRegistryKey& FRerouteNodeTemplate::GetRegistryKey()
	{
		static const FNodeRegistryKey RegistryKey = FNodeRegistryKey(EMetasoundFrontendClassType::Template, ClassName, VersionNumber);
		return RegistryKey;
	}

	const FMetasoundFrontendVersionNumber& FRerouteNodeTemplate::GetVersionNumber() const
	{
		return VersionNumber;
	}

#if WITH_EDITOR
	bool FRerouteNodeTemplate::HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage) const
	{
		const FMetasoundFrontendNode* Node = InBuilder.FindNode(InNodeID);
		if (!Node)
		{
			return false;
		}

		TArray<const FMetasoundFrontendVertex*> Outputs = InBuilder.FindNodeOutputs(InNodeID, { }, &InPageID);
		const bool bConnectedToNonRerouteOutputs = Algo::AnyOf(Outputs, [&InBuilder, &InPageID, &Node](const FMetasoundFrontendVertex* OutputVertex)
		{
			using namespace ReroutePrivate;
			return FindReroutedOutputVertex(ClassName, InBuilder, InPageID, *Node, *OutputVertex).IsValid();
		});

		TArray<const FMetasoundFrontendVertex*> Inputs = InBuilder.FindNodeInputs(InNodeID, { }, &InPageID);
		const bool bConnectedToNonRerouteInputs = Algo::AnyOf(Inputs, [&InBuilder, &InPageID, &Node](const FMetasoundFrontendVertex* InputVertex)
		{
			using namespace ReroutePrivate;
			TArray<const FMetasoundFrontendVertex*> InputVertices;
			TArray<const FMetasoundFrontendNode*> InputVerticesOwningNodes;
			FindReroutedInputVertices(ClassName, InBuilder, InPageID, *Node, *InputVertex, InputVerticesOwningNodes, InputVertices);
			return !InputVertices.IsEmpty();
		});

		const bool bHasRequiredConnections = bConnectedToNonRerouteOutputs || bConnectedToNonRerouteOutputs == bConnectedToNonRerouteInputs;
		if (!bHasRequiredConnections && OutMessage)
		{
			*OutMessage = TEXT("Reroute node(s) missing non-reroute input connection(s).");
		}

		return bHasRequiredConnections;
	}

	bool FRerouteNodeTemplate::OnEdgeAdded(const FTemplateNodeEdgeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		using namespace ReroutePrivate;

		const FRerouteNodeVertexPair RerouteOutputPair = FindReroutedOutputVertex(GetClassName(), TransactionContext.Builder, TransactionContext.PageID, *TransactionContext.FromNode, *TransactionContext.FromOutput);
		if (RerouteOutputPair.IsValid())
		{
			check(RerouteOutputPair.Node && RerouteOutputPair.Vertex);

			if (RerouteOutputPair.Node != TransactionContext.FromNode)
			{
				if (const FMetasoundFrontendClass* RerouteClass = TransactionContext.Builder.FindDependency(RerouteOutputPair.Node->ClassID))
				{
					if (RerouteClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
					{
						if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(RerouteClass->Metadata.GetClassName()))
						{
							FTemplateNodeEdgeTransactionContext RerouteContext = TransactionContext;
							RerouteContext.FromNode = RerouteOutputPair.Node;
							RerouteContext.FromOutput = RerouteOutputPair.Vertex;
							return Template->OnEdgeAdded(RerouteContext, OutTransactor);
						}
					}
				}
			}

			TArray<const FMetasoundFrontendNode*> ReroutedInputOwningNodes;
			TArray<const FMetasoundFrontendVertex*> ReroutedInputVertices;

			FindReroutedInputVertices(
				ClassName,
				TransactionContext.Builder,
				TransactionContext.PageID,
				*TransactionContext.ToNode,
				*TransactionContext.ToInput,
				ReroutedInputOwningNodes,
				ReroutedInputVertices);

			for (int32 Index = 0; Index < ReroutedInputOwningNodes.Num(); ++Index)
			{
				bool bHandled = false;
				if (const FMetasoundFrontendClass* RerouteClass = TransactionContext.Builder.FindDependency(ReroutedInputOwningNodes[Index]->ClassID))
				{
					if (RerouteClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
					{
						if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(RerouteClass->Metadata.GetClassName()))
						{
							FTemplateNodeEdgeTransactionContext RerouteContext = TransactionContext;
							RerouteContext.FromNode = RerouteOutputPair.Node;
							RerouteContext.FromOutput = RerouteOutputPair.Vertex;
							RerouteContext.ToNode = ReroutedInputOwningNodes[Index];
							RerouteContext.ToInput = ReroutedInputVertices[Index];
							bHandled = Template->OnEdgeAdded(RerouteContext, OutTransactor);
						}
					}
				}

				if (!bHandled)
				{
					OutTransactor.AddDataEdge(
						RerouteOutputPair.Node->GetID(),
						RerouteOutputPair.Vertex->Name,
						ReroutedInputOwningNodes[Index]->GetID(),
						ReroutedInputVertices[Index]->Name);
				}
			}
		}

		// Return handled even if no rerouted edge was found. In this context, this means
		// the reroute was not routed to anything of consequence relative to the actively
		// executing graph, so ignore the edge removal in the frontend graph.
		return true;
	}

	bool FRerouteNodeTemplate::OnNodeAdded(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		// Reroutes do not apply transaction as they are essentially pass through nodes that can provide edge context on demand but do not apply operators with any execution.
		return true;
	}

	bool FRerouteNodeTemplate::OnRemoveSwappingEdge(const FTemplateNodeEdgeRemoveTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		using namespace ReroutePrivate;

		const FRerouteNodeVertexPair RerouteOutputPair = FindReroutedOutputVertex(GetClassName(), TransactionContext.Builder, TransactionContext.PageID, *TransactionContext.FromNode, *TransactionContext.FromOutput);
		if (RerouteOutputPair.IsValid())
		{
			if (RerouteOutputPair.Node != TransactionContext.FromNode)
			{
				check(RerouteOutputPair.Node && RerouteOutputPair.Vertex);
				if (const FMetasoundFrontendClass* RerouteClass = TransactionContext.Builder.FindDependency(RerouteOutputPair.Node->ClassID))
				{
					if (RerouteClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
					{
						if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(RerouteClass->Metadata.GetClassName()))
						{
							FTemplateNodeEdgeRemoveTransactionContext RerouteContext = TransactionContext;
							RerouteContext.FromNode = RerouteOutputPair.Node;
							RerouteContext.FromOutput = RerouteOutputPair.Vertex;
							return Template->OnRemoveSwappingEdge(RerouteContext, OutTransactor);
						}
					}
				}
			}

			TArray<const FMetasoundFrontendNode*> ReroutedInputOwningNodes;
			TArray<const FMetasoundFrontendVertex*> ReroutedInputVertices;

			FindReroutedInputVertices(
				// We use the direct reroute ClassName because, for now, no inheriting classes of reroute are supported.
				// Recursion should probably 
				ClassName,
				TransactionContext.Builder,
				TransactionContext.PageID,
				*TransactionContext.ToNode,
				*TransactionContext.ToInput,
				ReroutedInputOwningNodes,
				ReroutedInputVertices);

			for (int32 Index = 0; Index < ReroutedInputOwningNodes.Num(); ++Index)
			{
				bool bHandled = false;
				if (const FMetasoundFrontendClass* RerouteClass = TransactionContext.Builder.FindDependency(ReroutedInputOwningNodes[Index]->ClassID))
				{
					if (RerouteClass->Metadata.GetType() == EMetasoundFrontendClassType::Template)
					{
						if (const INodeTemplate* Template = INodeTemplateRegistry::Get().FindTemplate(RerouteClass->Metadata.GetClassName()))
						{
							FTemplateNodeEdgeRemoveTransactionContext ReroutedContext = TransactionContext;
							ReroutedContext.FromNode = RerouteOutputPair.Node;
							ReroutedContext.FromOutput = RerouteOutputPair.Vertex;
							ReroutedContext.ToNode = ReroutedInputOwningNodes[Index];
							ReroutedContext.ToInput = ReroutedInputVertices[Index];
							bHandled = Template->OnRemoveSwappingEdge(ReroutedContext, OutTransactor);
						}
					}
				}

				if (!bHandled)
				{
					if (const FMetasoundFrontendLiteral* DefaultLiteral = TransactionContext.ResolveActiveNodeInputDefault(RerouteOutputPair.Node->GetID(), RerouteOutputPair.Vertex->Name))
					{
						OutTransactor.RemoveDataEdge(
							RerouteOutputPair.Node->GetID(),
							RerouteOutputPair.Vertex->Name,
							ReroutedInputOwningNodes[Index]->GetID(),
							ReroutedInputVertices[Index]->Name,
							DefaultLiteral->ToLiteral(ReroutedInputVertices[Index]->TypeName),
							&FMetaSoundFrontendDocumentBuilder::CreateDataReference);
					}
				}
			}
		}

		// Return handled even if no rerouted edge was found. In this context, this means
		// the reroute was not routed to anything of consequence relative to the actively
		// executing graph, so ignore the edge removal in the frontend graph.
		return true;
	}

	bool FRerouteNodeTemplate::OnRemoveSwappingNode(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const
	{
		// Reroutes do not apply transaction as they are essentially pass through nodes that can provide edge context on demand but do not apply operators with any execution.
		return true;
	}
#endif // WITH_EDITOR

	bool FRerouteNodeTemplate::IsInputAccessTypeDynamic() const
	{
		return true;
	}

	bool FRerouteNodeTemplate::IsInputConnectionUserModifiable() const
	{
		return true;
	}

	bool FRerouteNodeTemplate::IsOutputConnectionUserModifiable() const
	{
		return true;
	}

	bool FRerouteNodeTemplate::IsOutputAccessTypeDynamic() const
	{
		return true;
	}

	bool FRerouteNodeTemplate::IsValidNodeInterface(const FMetasoundFrontendNodeInterface& InNodeInterface) const
	{
		if (InNodeInterface.Inputs.Num() != 1)
		{
			return false;
		}
			
		if (InNodeInterface.Outputs.Num() != 1)
		{
			return false;
		}

		const FName DataType = InNodeInterface.Inputs.Last().TypeName;
		if (DataType != InNodeInterface.Outputs.Last().TypeName)
		{
			return false;
		}

		return IDataTypeRegistry::Get().IsRegistered(DataType);
	}
} // namespace Metasound::Frontend
