// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API


#define METASOUND_ANALYZER_PATH_SEPARATOR TEXT("/")

namespace Metasound
{
	namespace Frontend
	{
		// String serializable (as key) channel of analyzer or its internal members
		// that can be written to or read from using the Transmission System.
		class FAnalyzerAddress : public FTransmissionAddress
		{
		public:
			UE_API FAnalyzerAddress(const FString& InAddressString);
			FAnalyzerAddress() = default;

			virtual ~FAnalyzerAddress() = default;

			UE_API virtual FName GetAddressType() const override;

			UE_API virtual FName GetDataType() const override;

			UE_API virtual TUniquePtr<FTransmissionAddress> Clone() const override;

			// Converts AnalyzerAddress to String representation using the PathSeparator
			UE_API virtual FString ToString() const override;

			UE_API virtual uint32 GetHash() const override;

			UE_API virtual bool IsEqual(const FTransmissionAddress& InOther) const override;

			// Active Instance ID to monitor
			uint64 InstanceID = TNumericLimits<uint64>::Max();

			// ID of Node being monitored
			FGuid NodeID;

			// Name of output to monitor (not to be confused with the Analyzer's members,
			// which are specific to the analyzer instance being addressed)
			FVertexName OutputName;

			// DataType of the given channel
			FName DataType;

			// Name of Analyzer
			FName AnalyzerName;

			// Instance ID of analyzer (allowing for multiple analyzer of the same type to be
			// addressed at the same output).
			FGuid AnalyzerInstanceID;

			// Optional name used to specify a channel for a given analyzer's inputs/outputs.
			// If not provided (i.e. 'none'), single input & output are assumed to share
			// the same name. Useful if the analyzer requires outputting multiple analysis values.
			// Can potentially be used as an input as well to modify analyzer settings.
			FName AnalyzerMemberName;

			/**
			 * Specifies whether to use the data transmission center for passing values to views.
			 */
			bool UseDataTransmissionCenter = true;
		};
	} // namespace Frontend

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(Frontend::FAnalyzerAddress, METASOUNDFRONTEND_API, FAnalyzerAddressTypeInfo, FAnalyzerAddressReadRef, FAnalyzerAddressWriteRef)
} // namespace Metasound

#undef UE_API
