// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Templates/UniquePtr.h"


namespace Metasound
{
	namespace Frontend
	{
		class IVertexAnalyzerFactory
		{
		public:
			virtual ~IVertexAnalyzerFactory() = default;

			virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const = 0;
			virtual FName GetDataType() const = 0;
			virtual TUniquePtr<IVertexAnalyzer> CreateAnalyzer(const FCreateAnalyzerParams& InParams) const = 0;
		};

		template<typename TAnalyzerClass>
		class TVertexAnalyzerFactory : public IVertexAnalyzerFactory
		{
		public:
			static const FName& GetAnalyzerName() { return TAnalyzerClass::GetAnalyzerName(); }

			virtual ~TVertexAnalyzerFactory() = default;

			virtual FName GetDataType() const override { return TAnalyzerClass::GetDataType(); }
			virtual TUniquePtr<IVertexAnalyzer> CreateAnalyzer(const FCreateAnalyzerParams& InParams) const override
			{
				return TUniquePtr<IVertexAnalyzer>(new TAnalyzerClass(InParams));
			}
		};
	} // namespace Frontend
} // namespace Metasound
