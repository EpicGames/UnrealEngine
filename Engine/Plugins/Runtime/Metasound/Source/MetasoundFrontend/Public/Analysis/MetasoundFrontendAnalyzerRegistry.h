// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	// Forward Declarations
	class FDataReferenceCollection;

	namespace Frontend
	{
		// Forward Declarations
		class IVertexAnalyzerFactory;

		class IVertexAnalyzerRegistry
		{
		public:
			virtual ~IVertexAnalyzerRegistry() = default;
			
			static UE_API IVertexAnalyzerRegistry& Get();

			virtual const IVertexAnalyzerFactory* FindAnalyzerFactory(FName InAnalyzerName) const = 0;
			virtual void RegisterAnalyzerFactory(FName AnalyzerName, TUniquePtr<IVertexAnalyzerFactory>&& Factory) = 0;
		};
	} // namespace Frontend
} // namespace Metasound

#define METASOUND_REGISTER_VERTEX_ANALYZER_FACTORY(ANALYZER_CLASS) \
	{ \
	Frontend::IVertexAnalyzerRegistry& AnalyzerRegistry = Frontend::IVertexAnalyzerRegistry::Get(); \
	AnalyzerRegistry.RegisterAnalyzerFactory( \
		ANALYZER_CLASS::GetAnalyzerName(), \
		MakeUnique<ANALYZER_CLASS::FFactory>()); \
	}

#undef UE_API
