// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "MetasoundAssetBase.h"
#include "MetasoundRouter.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
class FMetasoundAssetBase;
class UAudioComponent;

namespace Metasound
{
	// Forward Declarations
	class FOperatorSettings;

	namespace Frontend
	{
		// Pairs an IReceiver with a given AnalyzerAddress, which enables
		// watching a particular analyzer result on any given thread.
		class FMetasoundAnalyzerView
		{
			TMap<FName, TSharedPtr<IReceiver>> OutputReceivers;

		public:
			const FAnalyzerAddress AnalyzerAddress = { };

			FMetasoundAnalyzerView() = default;
			UE_API FMetasoundAnalyzerView(FAnalyzerAddress&& InAnalyzerAddress);

			UE_API void BindToAllOutputs(const FOperatorSettings& InOperatorSettings);
			UE_API bool UnbindOutput(FName InOutputName);

			template <typename DataType>
			bool TryGetOutputData(FName InOutputName, DataType& OutValue)
			{
				TSharedPtr<IReceiver>* Receiver = OutputReceivers.Find(InOutputName);
				if (Receiver && Receiver->IsValid())
				{
					TReceiver<DataType>& TypedReceiver = (*Receiver)->GetAs<TReceiver<DataType>>();
					if (TypedReceiver.CanPop())
					{
						TypedReceiver.Pop(OutValue);
						return true;
					}
				}

				return false;
			}

			struct FBoundOutputDescription
			{
				FName Name;
				FName TypeName;
			};

			UE_API TArray<FBoundOutputDescription> GetBoundOutputDescriptions() const;
		};
	}
}

#undef UE_API
