// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound
{
	namespace Frontend
	{
		using FNodeVertexDataReferenceMap = TMap<FGuid, FDataReferenceCollection>;

		class FGraphAnalyzerAddress : public FTransmissionAddress
		{
			uint64 InstanceID = INDEX_NONE;

		public:
			FGraphAnalyzerAddress(uint64 InInstanceID)
				: InstanceID(InInstanceID)
			{
			}

			virtual ~FGraphAnalyzerAddress() = default;

			virtual FName GetAddressType() const override
			{
				return "GraphAnalyzer";
			}

			virtual FName GetDataType() const override
			{
				return GetMetasoundDataTypeName<TArray<FAnalyzerAddress>>();
			}

			virtual TUniquePtr<FTransmissionAddress> Clone() const override
			{
				return TUniquePtr<FTransmissionAddress>(new FGraphAnalyzerAddress(*this));
			}

			virtual FString ToString() const override
			{
				return FString::Printf(TEXT("%s%s%lld"), *GetAddressType().ToString(), METASOUND_ANALYZER_PATH_SEPARATOR, InstanceID);
			}

			virtual uint32 GetHash() const override
			{
				return static_cast<uint32>(InstanceID);
			}

			virtual bool IsEqual(const FTransmissionAddress& InOther) const override
			{
				if (InOther.GetAddressType() != GetAddressType())
				{
					return false;
				}

				const FGraphAnalyzerAddress& OtherAddr = static_cast<const FGraphAnalyzerAddress&>(InOther);
				return OtherAddr.InstanceID == InstanceID;
			}
		};

		// Handles intrinsic analysis operations within a given graph
		// should the graph's operator be enabled for analysis.
		class FGraphAnalyzer
		{
			const FOperatorSettings OperatorSettings;
			const uint64 InstanceID = INDEX_NONE;

			TUniquePtr<IReceiver> ActiveAnalyzerReceiver;
			TArray<TUniquePtr<Frontend::IVertexAnalyzer>> Analyzers;
			FNodeVertexDataReferenceMap InternalDataReferences;

		public:
			UE_API FGraphAnalyzer(const FOperatorSettings& InSettings, uint64 InInstanceID, FNodeVertexDataReferenceMap&& InGraphReferences);
			~FGraphAnalyzer() = default;

			// Execute analysis for the current block
			UE_API void Execute();
		};
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
