// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerView.h"
#include "Analysis/MetasoundFrontendGraphAnalyzer.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "MetasoundAssetBase.h"
#include "MetasoundRouter.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
class FMetasoundAssetBase;
class UAudioComponent;

namespace Metasound
{
	namespace Frontend
	{
		class FMetasoundGraphAnalyzerView
		{
			// Sender in charge of supplying expected vertex analyzers currently being analyzed.
			TUniquePtr<ISender> ActiveAnalyzerSender;

			// Set of active analyzer addresses used for what analyzers should be active on the associated graph instance
			TSet<FAnalyzerAddress> ActiveAnalyzers;

			UE_API const FMetasoundAssetBase& GetMetaSoundAssetChecked() const;
			UE_API void SendActiveAnalyzers();

			uint64 InstanceID = INDEX_NONE;
			const FMetasoundAssetBase* MetaSoundAsset = nullptr;

			FOperatorSettings OperatorSettings = { 48000, 100 };

			using FMetasoundGraphAnalyzerOutputKey = TTuple<FGuid /* NodeID */, FName /* OutputName */>;
			TMap<FMetasoundGraphAnalyzerOutputKey, TArray<FMetasoundAnalyzerView>> AnalyzerViews;

			FMetasoundGraphAnalyzerView(const FMetasoundGraphAnalyzerView&) = delete;
			FMetasoundGraphAnalyzerView(FMetasoundGraphAnalyzerView&&) = delete;
			FMetasoundGraphAnalyzerView& operator=(const FMetasoundGraphAnalyzerView&) = delete;
			FMetasoundGraphAnalyzerView& operator=(FMetasoundGraphAnalyzerView&&) = delete;

		public:
			FMetasoundGraphAnalyzerView() = default;

			UE_API FMetasoundGraphAnalyzerView(const FMetasoundAssetBase& InAssetBase, uint64 InInstanceID, const FOperatorSettings& InOperatorSettings);
			UE_API ~FMetasoundGraphAnalyzerView();

			UE_API void AddAnalyzerForAllSupportedOutputs(FName InAnalyzerName, bool bInRequiresConnection = true);
			UE_API FGuid AddAnalyzerForSpecifiedOutput(const FGuid& InNodeID, FVertexName InOutputName, FName InAnalyzerName, FName InAnalyzerMemberName);
			UE_API void RemoveAnalyzerForAllSupportedOutputs(FName InAnalyzerName);
			UE_API void RemoveAnalyzerInstance(FName InAnalyzerName, const FGuid& InAnalyzerInstanceID);
			UE_API bool HasAnalyzerInstance(FName InAnalyzerName, const FGuid& InAnalyzerInstanceID) const;
			UE_API TArray<FMetasoundAnalyzerView*> GetAnalyzerViews(FName InAnalyzerName);
			UE_API TArray<const FMetasoundAnalyzerView*> GetAnalyzerViews(FName InAnalyzerName) const;
			UE_API TArray<FMetasoundAnalyzerView*> GetAnalyzerViewsForOutput(const FGuid& InNodeID, FName InOutputName, FName InAnalyzerName);
			UE_API TArray<const FMetasoundAnalyzerView*> GetAnalyzerViewsForOutput(const FGuid& InNodeID, FName InOutputName, FName InAnalyzerName) const;
		};
	}
}

#undef UE_API
