// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "MetasoundDataReference.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundOutputStorage.h"

#include "Misc/Guid.h"

namespace Metasound
{
	namespace Frontend
	{
		// Output of analyzer (not to be confused with the node vertex output which an
		// analyzer may be processing).  Used to signify by an analyzer what information
		// is sent to other threads using an analyzer view for logging, displaying, processing,
		// etc. resulting analyzer data.
		struct FAnalyzerOutput
		{
			FName Name;
			FName DataType;
		};

		// Parameters used to generate an analyzer instance
		struct FCreateAnalyzerParams
		{
			// Address of analyzer
			const FAnalyzerAddress& AnalyzerAddress;

			// OperatorSettings used for analyzer execution
			const FOperatorSettings& OperatorSettings;

			// Data reference to vertex (currently only output vertices
			// support analysis) writing data to be analyzed.
			const FAnyDataReference& VertexDataReference;
		};

		// Analyzer that watches a data reference associated with a particular
		// node vertex. Currently, only output node vertex analysis is supported.
		class IVertexAnalyzer
		{
		public:
			virtual ~IVertexAnalyzer() = default;

			// Returns analyzer address
			virtual const FAnalyzerAddress& GetAnalyzerAddress() const = 0;

			// Returns the data reference id for the analyzer's input
			virtual FDataReferenceID GetDataReferenceId() const = 0;

			// Re-bind the analyzer's input
			virtual void SetDataReference(const FAnyDataReference& NewDataRef) = 0;

			// Executes analysis
			virtual void Execute() = 0;

			DECLARE_DELEGATE_TwoParams(FOnOutputDataChanged, FName, TSharedPtr<IOutputStorage>);
			FOnOutputDataChanged OnOutputDataChanged;
		};

		// Bound output from an analyzer (not to be confused with an output vertex)
		// sent internally using the Transmission System. Each bound output corresponds to
		// a DataChannel with an analyzer view or views potentially receiving analysis results.
		class IBoundAnalyzerOutput
		{
		public:
			virtual ~IBoundAnalyzerOutput() = default;

			// Returns the data reference associated with the analyzer's output
			virtual FAnyDataReference GetDataReference() const = 0;
			
			// Pushes data to the sender to be forwarded to all actively associated analyser views.
			virtual void PushData() = 0;

			// Gets the latest value to pass to IVertexAnalyzer::OnOutputDataChanged
			virtual TSharedPtr<IOutputStorage> CreateOutputData() const = 0;
		};

		// Templatized implementation of a bound analyzer output (see IBoundAnalyzerOutput)
		template<typename DataType>
		class TBoundAnalyzerOutput final : public IBoundAnalyzerOutput
		{
		public:
			TBoundAnalyzerOutput(const FAnalyzerAddress& InAnalyzerOutputAddress, const FOperatorSettings& InOperatorSettings, TDataReadReference<DataType> InData)
				: Address(InAnalyzerOutputAddress)
				, DataRef(InData)
			{
				Sender = FDataTransmissionCenter::Get().RegisterNewSender<DataType>(InAnalyzerOutputAddress, FSenderInitParams { InOperatorSettings, 0 });
				ensure(Sender);
			}

			virtual ~TBoundAnalyzerOutput() override
			{
				// Only unregister the data channel if we had a sender using that 
				// data channel. This protects against removing the data channel 
				// multiple times. Multiple removals of data channels has caused
				// race conditions between newly created transmitters and transmitters
				// being cleaned up.
				if (Sender.IsValid())
				{
					Sender.Reset();
					FDataTransmissionCenter::Get().UnregisterDataChannel(Address);
				}
			}

			virtual FAnyDataReference GetDataReference() const override
			{
				return DataRef;
			}

			virtual void PushData() override
			{
				if (Sender.IsValid())
				{
					Sender->Push(*DataRef);
				}
			}

			virtual TSharedPtr<IOutputStorage> CreateOutputData() const override
			{
				return MakeShared<TOutputStorage<DataType>>(*DataRef);
			}

		private:
			FAnalyzerAddress Address;
			TDataReadReference<DataType> DataRef;
			TUniquePtr<TSender<DataType>> Sender;
		};

		// Base implementation of a vertex analyzer.
		class FVertexAnalyzerBase : public IVertexAnalyzer
		{
		public:
			FVertexAnalyzerBase(const FAnalyzerAddress& InAnalyzerAddress, const FAnyDataReference& InDataReference)
				: AnalyzerAddress(InAnalyzerAddress)
				, VertexDataReference(InDataReference)
			{
			}

			virtual ~FVertexAnalyzerBase() = default;

			virtual const FAnalyzerAddress& GetAnalyzerAddress() const override
			{
				return AnalyzerAddress;
			}

			virtual FDataReferenceID GetDataReferenceId() const override final
			{
				return Metasound::GetDataReferenceID(VertexDataReference);
			}

			virtual void SetDataReference(const FAnyDataReference& NewDataRef) override final
			{
				VertexDataReference = NewDataRef;
			}

		protected:
			using FBoundOutputDataPtr = TSharedPtr<IBoundAnalyzerOutput>;

			// Marks all output channels as dirty, refreshing all bound data.
			// Likely best called after all computation is complete within an
			// execution call, to be implemented by the child class.
			void MarkOutputDirty()
			{
				for (TPair<FName, FBoundOutputDataPtr>& Pair : BoundOutputData)
				{
					Pair.Value->PushData();
					if (OnOutputDataChanged.IsBound())
					{
						OnOutputDataChanged.Execute(Pair.Key, Pair.Value->CreateOutputData());
					}
				}
			}

			// Unbinds a particular named analyzer output
			bool UnbindOutputData(FName InAnalyzerOutputName)
			{
				return BoundOutputData.Remove(InAnalyzerOutputName) > 0;
			}

			// Binds a particular named analyzer output to be updated when MarkOutputDirty is called.
			template<typename DataType>
			void BindOutputData(FName InAnalyzerOutputName, const FOperatorSettings& InOperatorSettings, TDataReadReference<DataType> InData)
			{
				FAnalyzerAddress OutputAddress = AnalyzerAddress;
				OutputAddress.AnalyzerMemberName = InAnalyzerOutputName;
				OutputAddress.DataType = GetMetasoundDataTypeName<DataType>();
				FBoundOutputDataPtr BoundOutputDataPtr = FBoundOutputDataPtr(new TBoundAnalyzerOutput<DataType>(OutputAddress, InOperatorSettings, InData));
				BoundOutputData.Emplace(InAnalyzerOutputName, MoveTemp(BoundOutputDataPtr));
			}

			// Returns the most recent vertex data.
			template<typename DataType>
			const DataType& GetVertexData() const
			{
				return *VertexDataReference.GetDataReadReference<DataType>();
			}

		private:
			FAnalyzerAddress AnalyzerAddress;
			TMap<FName, FBoundOutputDataPtr> BoundOutputData;
			FAnyDataReference VertexDataReference;
		};
	} // namespace Frontend
} // namespace Metasound
