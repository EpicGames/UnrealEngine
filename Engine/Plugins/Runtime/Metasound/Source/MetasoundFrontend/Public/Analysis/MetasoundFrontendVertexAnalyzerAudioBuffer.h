// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Containers/Array.h"
#include "MetasoundDataReferenceCollection.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound
{
	namespace Frontend
	{
		class FVertexAnalyzerAudioBuffer : public FVertexAnalyzerBase
		{
		public:
			static UE_API const FName& GetAnalyzerName();
			static UE_API const FName& GetDataType();

			struct FOutputs
			{
				static UE_API const FAnalyzerOutput& GetValue();
			};

			class FFactory : public TVertexAnalyzerFactory<FVertexAnalyzerAudioBuffer>
			{
			public:
				virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override
				{
					static const TArray<FAnalyzerOutput> Outputs { FOutputs::GetValue() };
					return Outputs;
				}
			};

			UE_API FVertexAnalyzerAudioBuffer(const FCreateAnalyzerParams& InParams);
			virtual ~FVertexAnalyzerAudioBuffer() = default;

			UE_API virtual void Execute() override;

		private:
			FAudioBufferWriteRef AudioBuffer;
		};
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
