// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Containers/Array.h"
#include "DSP/EnvelopeFollower.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound
{
	namespace Frontend
	{
		class FVertexAnalyzerEnvelopeFollower : public FVertexAnalyzerBase
		{
		public:
			static UE_API const FName& GetAnalyzerName();
			static UE_API const FName& GetDataType();

			struct FOutputs
			{
				static UE_API const FAnalyzerOutput& GetValue();
			};

			class FFactory : public TVertexAnalyzerFactory<FVertexAnalyzerEnvelopeFollower>
			{
			public:
				virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override
				{
					static const TArray<FAnalyzerOutput> Outputs { FOutputs::GetValue() };
					return Outputs;
				}
			};

			UE_API FVertexAnalyzerEnvelopeFollower(const FCreateAnalyzerParams& InParams);
			virtual ~FVertexAnalyzerEnvelopeFollower() = default;

			UE_API virtual void Execute() override;

		private:
			Audio::FEnvelopeFollower EnvelopeFollower;
			TDataWriteReference<float> EnvelopeValue;
		};
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
