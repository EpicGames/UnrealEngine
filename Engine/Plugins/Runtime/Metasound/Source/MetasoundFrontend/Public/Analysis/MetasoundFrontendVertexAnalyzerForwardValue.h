// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerAddress.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Containers/Array.h"
#include "MetasoundDataReference.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"
#include "MetasoundTrigger.h"


namespace Metasound
{
	namespace Frontend
	{
		// Templatized implementation of the simplest/cheapest form of an analyzer,
		// which just forwards a given value to an associated view or views. Can be
		// used in conjunction with an additional process, task, etc. handled by a
		// view to avoid further processing/expense within the graph execution thread
		// context.
		template <typename ChildClass, typename DataType>
		class TVertexAnalyzerForwardValue : public FVertexAnalyzerBase
		{
			DataType LastValue;
			bool HasExecuted = false;

		public:
			static const FName& GetDataType()
			{
				return GetMetasoundDataTypeName<DataType>();
			}

			struct FOutputs
			{
				static const FAnalyzerOutput& GetValue()
				{
					auto MakeValue = []()
					{
						return FAnalyzerOutput { "Value", GetMetasoundDataTypeName<DataType>() };
					};

					static FAnalyzerOutput Value = MakeValue();
					return Value;
				}
			};

			class FFactory : public TVertexAnalyzerFactory<ChildClass>
			{
			public:
				virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override
				{
					static const TArray<FAnalyzerOutput> Outputs { ChildClass::FOutputs::GetValue() };
					return Outputs;
				}
			};

			TVertexAnalyzerForwardValue(const FCreateAnalyzerParams& InParams)
 				: FVertexAnalyzerBase(InParams.AnalyzerAddress, InParams.VertexDataReference)
			{
				TDataReadReference<DataType> DataRef = InParams.VertexDataReference.GetDataReadReference<DataType>();
				FVertexAnalyzerBase::BindOutputData<DataType>(ChildClass::FOutputs::GetValue().Name, InParams.OperatorSettings, MoveTemp(DataRef));
			}

			virtual ~TVertexAnalyzerForwardValue() = default;

			virtual void Execute() override
			{
				const DataType& Value = GetVertexData<DataType>();

				if (!HasExecuted)
				{
					HasExecuted = true;
					UpdateValue(Value);
				}
				else if (LastValue != Value)
				{
					UpdateValue(Value);
				}
			}

		private:
			void UpdateValue(const DataType& Value)
			{
				LastValue = Value;
				MarkOutputDirty();
			}
		};

		#define METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(ClassType, AnalyzerFName, DataType) \
			class ClassType : public TVertexAnalyzerForwardValue<ClassType, DataType> \
			{ \
			public: \
				static const FName& GetAnalyzerName() \
				{ \
					static const FName AnalyzerName = AnalyzerFName; return AnalyzerName; \
				} \
				ClassType(const FCreateAnalyzerParams& InParams) \
					: TVertexAnalyzerForwardValue(InParams) { } \
				virtual ~ClassType() = default; \
			};

		// Primitive forward value types
		METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(FVertexAnalyzerForwardBool, "UE.Forward.Bool", bool)
		METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(FVertexAnalyzerForwardFloat, "UE.Forward.Float", float)
		METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(FVertexAnalyzerForwardInt, "UE.Forward.Int32", int32)
		METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(FVertexAnalyzerForwardString, "UE.Forward.String", FString)
		METASOUND_DECLARE_FORWARD_VALUE_ANALYZER(FVertexAnalyzerForwardTime, "UE.Forward.Time", FTime)
	} // namespace Frontend
} // namespace Metasound
