// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Containers/Array.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/EnvelopeFollower.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"


namespace Metasound
{
	namespace Frontend
	{
		class FVertexAnalyzerTriggerDensity : public FVertexAnalyzerBase
		{
		public:
			METASOUNDFRONTEND_API static const FName& GetAnalyzerName();
			METASOUNDFRONTEND_API static const FName& GetDataType();

			struct FOutputs
			{
				METASOUNDFRONTEND_API static const FAnalyzerOutput& GetValue();
			};

			class FFactory : public TVertexAnalyzerFactory<FVertexAnalyzerTriggerDensity>
			{
			public:
				virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override
				{
					static const TArray<FAnalyzerOutput> Outputs { FOutputs::GetValue() };
					return Outputs;
				}
			};

			METASOUNDFRONTEND_API FVertexAnalyzerTriggerDensity(const FCreateAnalyzerParams& InParams);
			METASOUNDFRONTEND_API virtual ~FVertexAnalyzerTriggerDensity();

			METASOUNDFRONTEND_API virtual void Execute() override;

		private:

			Audio::FEnvelopeFollower EnvelopeFollower;
			TDataWriteReference<float> EnvelopeValue;
			int32 NumFramesPerBlock = 0;
			Audio::FAlignedFloatBuffer ScratchBuffer;
		};
	} // namespace Frontend
} // namespace Metasound
