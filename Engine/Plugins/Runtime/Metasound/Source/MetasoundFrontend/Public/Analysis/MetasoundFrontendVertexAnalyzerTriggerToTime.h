// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundDataReference.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	namespace Frontend
	{
		class FVertexAnalyzerTriggerToTime final : public FVertexAnalyzerBase
		{
		public:
			static UE_API const FName& GetAnalyzerName();
			static UE_API const FName& GetDataType();

			struct FOutputs
			{
				static UE_API const FAnalyzerOutput& GetValue();
			};

			class FFactory final : public TVertexAnalyzerFactory<FVertexAnalyzerTriggerToTime>
			{
			public:
				UE_API virtual const TArray<FAnalyzerOutput>& GetAnalyzerOutputs() const override;
			};

			UE_API explicit FVertexAnalyzerTriggerToTime(const FCreateAnalyzerParams& InParams);
			virtual ~FVertexAnalyzerTriggerToTime() override = default;

			UE_API virtual void Execute() override;

		private:
			const FSampleRate SampleRate;
			const int32 FramesPerBlock;
			const double SecondsPerBlock;
			FTime BlockStartTime{ 0.0f };
			TDataWriteReference<FTime> LastTriggerTime;
		};
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
