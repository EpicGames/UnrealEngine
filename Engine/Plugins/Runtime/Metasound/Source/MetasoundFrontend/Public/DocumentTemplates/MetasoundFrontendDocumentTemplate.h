// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "Delegates/DelegateCombinations.h"
#include "Misc/DataValidation.h"
#endif // WITH_EDITOR

#include "Internationalization/Text.h"

#include "MetasoundFrontendDocumentTemplate.generated.h"

#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;
class UMetaSoundFrontendMemberMetadata;

enum class EMetasoundFrontendClassType : uint8;

#if WITH_EDITOR
namespace Metasound::Frontend
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConfigPropertyChangedDelegate, const FPropertyChangedEvent& /* InEvent */);
} // namespace Metasound::Frontend
#endif // WITH_EDITOR

// Struct housed on a MetaSoundDocument and used to define procedural behavior applied when making
// modifications to a MetaSound prior to serialization. Templates enable MetaSounds to be
// procedurally generated at edit and cook-time to adhere to a specific design or topology anytime
// a given configuration or its respective properties are changed, enabling faster and more accurate
// iteration where fully-customized MetaSound edit behavior is not desired.
USTRUCT(BlueprintType, meta = (Hidden))
struct FMetaSoundFrontendDocumentTemplate
{
	GENERATED_BODY()

	virtual ~FMetaSoundFrontendDocumentTemplate() = default;

	// Executes procedural configuration of document template. Returns true if template applied
	// to document, false if not (resulting in a follow-up call to reset the document).
	UE_INTERNAL UE_API virtual bool ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder);

#if WITH_EDITOR
	struct FEditorOptions
	{
		// Asset class types this template supports as a selection (passed on initial creation) when user
		// creates a configured MetaSound from an action (eg. right-click action in the Content Browser right-click
		// menu). Defaults to all MetaSound asset types (eg. Source & Patch) if unset.
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		TArray<TSoftClassPtr<UObject>> AssetActionClasses;

		// MetaSound asset types this template can be set to when user creates a configured MetaSound
		// from an action (eg. right-click action in the Content Browser right-click menu). Defaults to
		// all MetaSound asset types (eg. Source & Patch) if unset.
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		TArray<TSoftClassPtr<UObject>> MetaSoundClasses;

		// ToolTip shown when selecting document template from list of available configurations
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		FText ToolTip;

		// Whether or not template supports custom tabs (i.e. custom template widgets)
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bCustomTabEnabled = true;

		// Whether or not interface editing is enabled in MetaSound asset editors
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bInterfaceEditingEnabled = false;

		// Whether or not member editing is enabled in MetaSound asset editors
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bMemberEditingEnabled = false;

		// Whether or not graph page editing is enabled in MetaSound asset editors
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bPageGraphEditingEnabled = false;

		// Whether or not graph editor is visible by default (user can overwrite if desired)
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bGraphEditorVisible = false;

		// Whether or not to display template properties in the details panel.
		UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
		bool bTemplatePropertiesVisible = true;

		// Default options if none are specified
		UE_API static const FEditorOptions& GetDefaultOptions();
	};

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual const FEditorOptions& GetEditorOptions() const;

	UE_API virtual EDataValidationResult IsDataValid(const FMetaSoundFrontendDocumentBuilder& Builder, FDataValidationContext& InOutContext) const;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder);

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder);

	UE_API Metasound::Frontend::FOnConfigPropertyChangedDelegate& GetPropertyChangedDelegate() const;

private:
	mutable Metasound::Frontend::FOnConfigPropertyChangedDelegate PropertyChangedDelegate;
#endif // WITH_EDITOR
};

// Convenience, less verbose aliases
namespace Metasound::Frontend
{
	using FMetaSoundTemplate = FMetaSoundFrontendDocumentTemplate;
} // namespace Metasound::Frontend;

#undef UE_API
