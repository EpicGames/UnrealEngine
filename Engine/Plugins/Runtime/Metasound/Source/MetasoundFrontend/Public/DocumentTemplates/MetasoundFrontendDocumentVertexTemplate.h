// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentTemplate.h"

#include "MetasoundFrontendDocumentVertexTemplate.generated.h"

#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;


// Vertex metadata displayed on MetaSound inputs & outputs pertaining to a given configuration.
USTRUCT(BlueprintType)
struct FMetaSoundFrontendDocumentTemplateVertexMetadata
{
	GENERATED_BODY()

public:
	FMetaSoundFrontendDocumentTemplateVertexMetadata() = default;

	virtual ~FMetaSoundFrontendDocumentTemplateVertexMetadata() = default;

#if WITH_EDITOR
	// If true, default values are directly editable if the parent configuration lists member editing as enabled.
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	virtual bool IsDefaultEditable() const { return true; }
#endif // WITH_EDITOR

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	virtual void OnDefaultReset(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) { }

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	virtual void OnDefaultUpdated(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) { }

#if WITH_EDITOR
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	virtual void OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder, const FMetasoundFrontendClassVertex& InVertex) { }
#endif // WITH_EDITOR
};

// Template of a MetaSound that stores vertex (input or output) Metadata to display alongside standard properties in MetaSound vertex editors.
USTRUCT(BlueprintType, meta = (Hidden))
struct FMetaSoundFrontendDocumentVertexTemplate : public FMetaSoundFrontendDocumentTemplate
{
	GENERATED_BODY()

	virtual ~FMetaSoundFrontendDocumentVertexTemplate() = default;

	// Creates and stores new VertexMetadata. Overwrites if metadata already exists with
	// default constructed VertexMetadata if template supports Metadata.
	UE_INTERNAL FMetaSoundFrontendDocumentTemplateVertexMetadata* AddVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex);

	// Creates and stores new VertexMetadata with the provided type and name, asserting that the type created by the overloaded template
	// matches the template type. Overwrites if metadata already exists with default constructed VertexMetadata. Returns null if template
	// does not support Metadata.
	template <typename TTemplateVertexMetadata>
	UE_INTERNAL TTemplateVertexMetadata* AddVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex)
	{
		if (TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>* Entry = VertexMetadata.Find(InVertex.NodeID))
		{
			return &Entry->GetMutable<TTemplateVertexMetadata>();
		}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> NewMetadata = ConstructVertexMetadata(Builder, InVertex);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

		if (NewMetadata.IsValid())
		{
			return &VertexMetadata.Add(InVertex.NodeID, MoveTemp(NewMetadata)).GetMutable<TTemplateVertexMetadata>();
		}

		return nullptr;
	}

	// Returns whether or not there is existing Metadata with the given NodeID.
	UE_API bool ContainsVertexMetadata(const FGuid& NodeID) const;

	// Helper function to returns VertexMetadata with the provided class type and name and cast to underlying config type.
	template <typename TTemplateVertexMetadata = FMetaSoundFrontendDocumentTemplateVertexMetadata>
	UE_INTERNAL TTemplateVertexMetadata* FindVertexMetadata(const FGuid& NodeID)
	{
		if (TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>* Entry = VertexMetadata.Find(NodeID))
		{
			return Entry->GetMutablePtr<TTemplateVertexMetadata>();
		}

		return nullptr;
	}

	// Helper function to returns VertexMetadata with the provided class type and name and cast to underlying config type.
	template <typename TTemplateVertexMetadata = FMetaSoundFrontendDocumentTemplateVertexMetadata>
	const TTemplateVertexMetadata* FindConstVertexMetadata(const FGuid& NodeID) const
	{
		if (const TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>* Entry = VertexMetadata.Find(NodeID))
		{
			return Entry->GetPtr<TTemplateVertexMetadata>();
		}

		return nullptr;
	}

	const UScriptStruct* GetVertexMetadataStruct(const FGuid& NodeID) const
	{
		if (const TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>* Entry = VertexMetadata.Find(NodeID))
		{
			return Entry->GetScriptStruct();
		}

		return nullptr;
	}

	// Removes VertexMetadata with the provided NodeID. Returns if Metadata was found and removed or not.
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API bool RemoveVertexMetadata(const FGuid& NodeID);

protected:
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> ConstructVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) const;

private:
	UPROPERTY()
	TMap<FGuid, TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata>> VertexMetadata;
};

// Convenience, less verbose aliases
namespace Metasound::Frontend
{
	using FDocumentVertexTemplate = FMetaSoundFrontendDocumentVertexTemplate;
	using FTemplateVertexMetadata = FMetaSoundFrontendDocumentTemplateVertexMetadata;
} // namespace Metasound::Frontend;

#undef UE_API
