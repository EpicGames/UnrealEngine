// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentVertexTemplate.h"
#include "UObject/NameTypes.h"

#include "MetasoundFrontendPresetTemplate.generated.h"

#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
class UMetaSoundFrontendMemberMetadata;
enum class EMetasoundFrontendClassType : uint8;


USTRUCT(BlueprintType, DisplayName = "Preset")
struct FMetaSoundFrontendPresetTemplate final : public FMetaSoundFrontendDocumentVertexTemplate
{
	GENERATED_BODY()

	virtual ~FMetaSoundFrontendPresetTemplate() = default;

#if WITH_EDITORONLY_DATA
	UE_INTERNAL UE_API virtual bool ConfigureDocument(FMetaSoundFrontendDocumentBuilder& OutBuilder) override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual const FMetaSoundFrontendDocumentTemplate::FEditorOptions& GetEditorOptions() const override;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API EDataValidationResult IsDataValid(const FMetaSoundFrontendDocumentBuilder& Builder, FDataValidationContext& InOutContext) const override;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnAssetInitialized(TArray<UObject*> SelectedObjects, FMetaSoundFrontendDocumentBuilder& OutBuilder) override;

	UE_EXPERIMENTAL(5.8, "MetaSound Templates are in active development and subject to change")
	UE_API virtual void OnPropertyChanged(const FPropertyChangedEvent& InEvent, FMetaSoundFrontendDocumentBuilder& OutBuilder) override;
#endif // WITH_EDITOR

protected:
	UE_INTERNAL UE_API virtual TInstancedStruct<FMetaSoundFrontendDocumentTemplateVertexMetadata> ConstructVertexMetadata(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) const override;

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Preset Options")
	TScriptInterface<IMetaSoundDocumentInterface> Parent;
};

USTRUCT(BlueprintType)
struct FMetaSoundFrontendPresetVertexMetadata : public FMetaSoundFrontendDocumentTemplateVertexMetadata
{
	GENERATED_BODY()

	virtual ~FMetaSoundFrontendPresetVertexMetadata() = default;

#if WITH_EDITOR
	virtual bool IsDefaultEditable() const override;
#endif // WITH_EDITOR

	virtual void OnDefaultReset(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) override;
	virtual void OnDefaultUpdated(const FMetaSoundFrontendDocumentBuilder& Builder, const FMetasoundFrontendClassVertex& InVertex) override;

	// Enables overriding the input's inherited default value otherwise provided by the parent graph.
	// Setting to true disables the configuration utilizing the input's default value if updated on the parent asset.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PresetOptions)
	bool bOverrideInheritedDefault = false;
};

namespace Metasound::Frontend
{
	using FPresetTemplate = FMetaSoundFrontendPresetTemplate;
	using FPresetVertexMetadata = FMetaSoundFrontendPresetVertexMetadata;
} // namespace Metasound::Frontend;
#undef UE_API