// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "MetasoundFrontendDocument.h"

namespace Metasound
{
	namespace Frontend
	{
		METASOUNDFRONTEND_API bool IsSubsetOfInterface(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendInterface& InSupersetInterface);

		METASOUNDFRONTEND_API bool IsSubsetOfClassInterface(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendClassInterface& InSupersetClassInterface);

		UE_DEPRECATED(5.6, "Use IsSubsetOfClassInterface instead")
		METASOUNDFRONTEND_API bool IsSubsetOfClass(const FMetasoundFrontendInterface& InSubsetInterface, const FMetasoundFrontendClass& InSupersetClass);


		METASOUNDFRONTEND_API bool IsEquivalentInterface(const FMetasoundFrontendInterface& InInputInterface, const FMetasoundFrontendInterface& InTargetInterface);

		METASOUNDFRONTEND_API bool IsEqualInterface(const FMetasoundFrontendInterface& InInputInterface, const FMetasoundFrontendInterface& InTargetInterface);

		UE_DEPRECATED(5.6, "Use InputOutputDifferenceCount which accepts a FMetasoundFrontendClassInterface instead")
		METASOUNDFRONTEND_API int32 InputOutputDifferenceCount(const FMetasoundFrontendClass& InClass, const FMetasoundFrontendInterface& InInterface);

		METASOUNDFRONTEND_API int32 InputOutputDifferenceCount(const FMetasoundFrontendClassInterface& InClass, const FMetasoundFrontendInterface& InInterface);

		METASOUNDFRONTEND_API int32 InputOutputDifferenceCount(const FMetasoundFrontendInterface& InInterfaceA, const FMetasoundFrontendInterface& InInterfaceB);

		UE_DEPRECATED(5.6, "All interface functionality is in the MetaSoundFrontendDocumentBuilder API")
		METASOUNDFRONTEND_API bool DeclaredInterfaceVersionsMatch(const FMetasoundFrontendDocument& InDocumentA, const FMetasoundFrontendDocument& InDocumentB);


		METASOUNDFRONTEND_API const FMetasoundFrontendInterface* FindMostSimilarInterfaceSupportingEnvironment(const FMetasoundFrontendGraphClass& InRootGraph, const TArray<FMetasoundFrontendClass>& InDependencies, const TArray<FMetasoundFrontendGraphClass>& InSubgraphs, const TArray<FMetasoundFrontendInterface>& InCandidateInterfaces);

		METASOUNDFRONTEND_API const FMetasoundFrontendInterface* FindMostSimilarInterfaceSupportingEnvironment(const FMetasoundFrontendDocument& InDocument, const TArray<FMetasoundFrontendInterface>& InCandidateInterfaces);
	} // namespace Frontend
} // namespace Metasound
