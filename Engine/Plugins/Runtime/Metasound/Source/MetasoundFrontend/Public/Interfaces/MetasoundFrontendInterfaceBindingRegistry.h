// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "UObject/NoExportTypes.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound::Frontend
{
	class FInterfaceBindingRegistryEntry
	{
		const FMetasoundFrontendVersion OutputInterfaceVersion;
		const TArray<FMetasoundFrontendInterfaceVertexBinding> VertexBindings;
		const int32 BindingPriority = 0;

	public:
		UE_API FInterfaceBindingRegistryEntry(const FMetasoundFrontendInterfaceBinding& InBinding);
		virtual ~FInterfaceBindingRegistryEntry() = default;

		// Returns the output interface version corresponding with the given binding entry.
		UE_API const FMetasoundFrontendVersion& GetOutputInterfaceVersion() const;

		// Returns array of FName pairs that describe how the output and input interfaces are to bind
		UE_API const TArray<FMetasoundFrontendInterfaceVertexBinding>& GetVertexBindings() const;

		// If two binding registry entries match output & input interfaces versions, integer corresponds
		// to which binding takes priority for a given bind operation. Lower numbers are higher priority.
		UE_API int32 GetBindingPriority() const;
	};


	// Registry of interface bindings, a way to characterize how MetaSound node instances that implement
	// a give set of interfaces "bind" to other nodes that implement a second set set of interfaces. Binding
	// is an abstracted way to connect two nodes together using multiple edges as defined through this registry.
	// Accessing this registry is currently not thread safe and only expected to be occur on the Game Thread.
	class IInterfaceBindingRegistry
	{
	public:
		// Returns binding registry singleton.
		static UE_API IInterfaceBindingRegistry& Get();

		// Find all bindings associated with the given input interface version. Returns true if interface binding is found, false if not.
		virtual bool FindInterfaceBindingEntries(const FMetasoundFrontendVersion& InInputInterfaceVersion, TArray<const FInterfaceBindingRegistryEntry*>& OutEntries) const = 0;

		// Add interface bindings to the registry.
		virtual void RegisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, TUniquePtr<FInterfaceBindingRegistryEntry>&& InEntry) = 0;

		// Removes an interface binding from registry.
		virtual bool UnregisterInterfaceBinding(const FMetasoundFrontendVersion& InInputInterfaceVersion, const FMetasoundFrontendVersion& InOutputInterfaceVersion) = 0;

		// Removes all interface bindings associated with the given input interface if found. Returns true if found and removed, false if not found.
		virtual bool UnregisterAllInterfaceBindings(const FMetasoundFrontendVersion& InInputInterfaceVersion) = 0;
	};
} // namespace Metasound::Frontend

#undef UE_API
