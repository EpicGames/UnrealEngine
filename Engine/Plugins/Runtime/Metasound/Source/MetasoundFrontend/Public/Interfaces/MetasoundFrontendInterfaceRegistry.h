// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"


#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
	// Forward Declarations
	class IBuilderVersionTransform;

	// Interface keys are actively being deprecated: Frontend Version is now directly used as hashable interface identifier")
	using FInterfaceRegistryKey = FString;
	using FRegistryTransactionID = int32;

	UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
	UE_API bool IsValidInterfaceRegistryKey(const FInterfaceRegistryKey& InKey);

	UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
	UE_API FInterfaceRegistryKey GetInterfaceRegistryKey(const FMetasoundFrontendVersion& InInterfaceVersion);

	UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
	UE_API FInterfaceRegistryKey GetInterfaceRegistryKey(const FMetasoundFrontendInterface& InInterface);

	class IInterfaceRegistryEntry
	{
	public:
		virtual ~IInterfaceRegistryEntry() = default;

		// MetaSound Interface definition
		virtual const FMetasoundFrontendInterface& GetInterface() const = 0;

		// Whether or not entry is deprecated or not. If false, entry is checked for validity on registration in editor builds.
		virtual bool IsDeprecated() const { return false; }

		// Name of routing system used to update interface inputs (ex. ParameterInterface or DataReference).
		virtual FName GetRouterName() const = 0;

		UE_DEPRECATED(5.7, "Transform accessor must be thread safe, and thus a shared pointer is passed via 'GetUpdateTransform'")
		virtual bool UpdateRootGraphInterface(FDocumentHandle InDocument) const { return false; };

		UE_DEPRECATED(5.8, "Transform accessor must be thread safe, and thus a shared pointer is passed via 'GetUpdateTransform'")
		virtual bool UpdateRootGraphInterface(FMetaSoundFrontendDocumentBuilder& InDocumentBuilder) const { return false; }

		// Returns transform indicating how to update a given document if versioning is required to this interface from a deprecated version.
		// If not specified, update simply adds the given interface as a new interface to the given builder's document.
		virtual TSharedRef<IBuilderVersionTransform> GetUpdateTransform() const = 0;
	};

	class FInterfaceRegistryTransaction
	{
	public:
		using FTimeType = uint64;

		/** Describes the type of transaction. */
		enum class ETransactionType : uint8
		{
			InterfaceRegistration,     //< Something was added to the registry.
			InterfaceUnregistration,  //< Something was removed from the registry.
			Invalid
		};

		UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
		UE_API FInterfaceRegistryTransaction(ETransactionType InType, const FInterfaceRegistryKey& InKey, const FMetasoundFrontendVersion& InInterfaceVersion, FTimeType InTimestamp);

		UE_API FInterfaceRegistryTransaction(ETransactionType InType, const FMetasoundFrontendVersion& InInterfaceVersion, FTimeType InTimestamp);

		UE_API ETransactionType GetTransactionType() const;
		UE_API const FMetasoundFrontendVersion& GetInterfaceVersion() const;

		UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
		UE_API const FInterfaceRegistryKey& GetInterfaceRegistryKey() const;

		UE_API FTimeType GetTimestamp() const;

	private:
		ETransactionType Type;
		FMetasoundFrontendVersion InterfaceVersion;
		FTimeType Timestamp;
	};

	class IInterfaceRegistry
	{
	public:
		static UE_API IInterfaceRegistry& Get();

		virtual ~IInterfaceRegistry() = default;

		// Register an interface
		virtual bool RegisterInterface(TUniquePtr<IInterfaceRegistryEntry>&& InEntry) = 0;

		virtual bool ContainsInput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InInput) const = 0;

		virtual bool ContainsInterface(const FMetasoundFrontendVersion& InVersion) const = 0;

		virtual bool ContainsOutput(const FMetasoundFrontendVersion& InVersion, const FMetasoundFrontendVertex& InOutput) const = 0;

		UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier. In addition, direct access to entry is not thread safe, use FindInterface call instead.")
		virtual const IInterfaceRegistryEntry* FindInterfaceRegistryEntry(const FInterfaceRegistryKey& InKey) const { return nullptr; }

		UE_DEPRECATED(5.8, "Interface keys are deprecated: Frontend Version is now directly used as hashable interface identifier")
		virtual bool FindInterface(const FInterfaceRegistryKey& InKey, FMetasoundFrontendInterface& OutInterface) const { return false; }

		// Find an interface with the given version. Returns true if interface is found, false if not.
		virtual bool FindInterface(const FMetasoundFrontendVersion& InVersion, FMetasoundFrontendInterface& OutInterface) const = 0;

		// Returns the name of the router for the given interface.
		virtual bool FindInterfaceRouter(const FMetasoundFrontendVersion& InVersion, FName& OutRouter) const = 0;

		// Find the interface update transform associated with the given version if found.
		virtual TSharedPtr<IBuilderVersionTransform> FindInterfaceUpdateTransform(const FMetasoundFrontendVersion& InVersion) const = 0;
	};
} // namespace Metasound::Frontend

#undef UE_API
