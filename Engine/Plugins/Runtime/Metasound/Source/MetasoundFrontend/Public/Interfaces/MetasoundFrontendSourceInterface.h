// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioParameterInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendTransform.h"
#include "UObject/NameTypes.h"
#include "MetasoundFrontendController.h"

#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
struct FMetasoundFrontendVersion;

namespace Metasound
{
	namespace Frontend
	{
		namespace SourceOneShotInterface
		{
			namespace Outputs
			{
				METASOUNDFRONTEND_API const extern FName OnFinished;
			}

			METASOUNDFRONTEND_API const FMetasoundFrontendVersion& GetVersion();

			METASOUNDFRONTEND_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		}

		namespace SourceStartTimeInterface
		{
			namespace Inputs
			{
				METASOUNDFRONTEND_API const extern FName StartTime;
			}

			METASOUNDFRONTEND_API const FMetasoundFrontendVersion& GetVersion();

			METASOUNDFRONTEND_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		}

		namespace SourceInterfaceV1_0
		{
			namespace Inputs
			{
				METASOUNDFRONTEND_API const extern FName OnPlay;
			}

			namespace Outputs
			{
				METASOUNDFRONTEND_API const extern FName OnFinished;
			}

			namespace Environment
			{
				METASOUNDFRONTEND_API const extern FName DeviceID;
				METASOUNDFRONTEND_API const extern FName GraphName;
				METASOUNDFRONTEND_API const extern FName IsPreview;
				METASOUNDFRONTEND_API const extern FName SoundUniqueID;
				METASOUNDFRONTEND_API const extern FName TransmitterID;
			}

			METASOUNDFRONTEND_API const FMetasoundFrontendVersion& GetVersion();

			METASOUNDFRONTEND_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);
		}

		namespace SourceInterface
		{
			namespace Inputs
			{ 
				METASOUNDFRONTEND_API const extern FName OnPlay;
			}

			namespace Environment
			{
				METASOUNDFRONTEND_API const extern FName DeviceID;
				METASOUNDFRONTEND_API const extern FName GraphName;
				METASOUNDFRONTEND_API const extern FName IsPreview;
				METASOUNDFRONTEND_API const extern FName SoundUniqueID;
				METASOUNDFRONTEND_API const extern FName TransmitterID;
				METASOUNDFRONTEND_API const extern FName AudioMixerNumOutputFrames;
				METASOUNDFRONTEND_API const extern FName AudioMixerSampleRate;
				METASOUNDFRONTEND_API const extern FName SourceFormatName;
			}

			METASOUNDFRONTEND_API const FMetasoundFrontendVersion& GetVersion();

			METASOUNDFRONTEND_API Audio::FParameterInterfacePtr CreateInterface(const UClass& InClass);

			class FUpdateInterface : public Frontend::IDocumentTransform
			{
			public:
				UE_API virtual bool Transform(Frontend::FDocumentHandle InDocument) const override;
			};
		}
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
