// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/Requires.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 1
#else
#define METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 0
#endif

namespace Metasound
{
	namespace Frontend
	{
		class FAccessPoint;

		/** The access token mirrors the lifespan of a object in a non-intrusive
		 * manner. A TAccessPtr<> can be created to reference an object, but use
		 * an access token to determine whether the referenced object is accessible.
		 * When the access token is destroyed, the access pointer becomes invalid.
		 */
		struct FAccessToken {};

		/** TAccessPtr
		 *
		 * TAccessPtr is used to determine whether an object has been destructed or not.
		 * It is useful when an object cannot be wrapped in a TSharedPtr. A TAccessPtr
		 * is functionally similar to a TWeakPtr, but it cannot pin the object. 
		 * In order to determine whether an object as accessible, the TAccessPtr
		 * executes a TFunctions<> which returns object. If a nullptr is returned,
		 * then the object is not accessible. 
		 *
		 * Object pointers  held within a TAccessPtr must only be accessed on 
		 * the thread where the objects gets destructed to avoid having the object 
		 * destructed while in use.
		 *
		 * If the TAccessPtr's underlying object is accessed when the pointer is invalid,
		 * a fallback object will be returned. 
		 *
		 * @tparam Type - The type of object to track.
		 */
		template<typename Type>
		class TAccessPtr
		{
			enum class EConstCast { Tag };
			enum class EDerivedCopy { Tag };

		public:
			using FTokenType = FAccessToken;
			using ObjectType = Type;

			static Type FallbackObject;

			/** Returns a pointer to the accessible object. If the object is 
			 * inaccessible, a nullptr is returned. 
			 */
			Type* Get() const
			{
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				CachedObjectPtr = GetObject();
				return CachedObjectPtr;
#else
				return GetObject();
#endif
			}

			/** Returns an access pointer to a member of the wrapped object. 
			 *
			 * @tparam AccessPtrType - The access pointer type to return.
			 * @tparam FunctionType - A type which is callable accepts a reference to the wrapped object and returns a pointer to the member.
			 *
			 * @param InGetMember - A FunctionType accepts a reference to the wrapped object and returns a pointer to the member.
			 */
			template<typename AccessPtrType, typename FunctionType>
			AccessPtrType GetMemberAccessPtr(FunctionType InGetMember) const
			{
				using MemberType = typename AccessPtrType::ObjectType;

				TFunction<MemberType*()> GetMemberFromObject = [GetObject=this->GetObject, GetMember=MoveTemp(InGetMember)]() -> MemberType*
				{
					if (Type* Object = GetObject())
					{
						return GetMember(*Object);
					}
					return static_cast<MemberType*>(nullptr);
				};

				return AccessPtrType(GetMemberFromObject);
			}

			TAccessPtr()
			: GetObject([]() { return static_cast<Type*>(nullptr); })
			{
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				Get();
#endif
			}

			/** Creates a access pointer using an access token. */
			TAccessPtr(TWeakPtr<FTokenType> AccessToken, Type& InRef)
			{
				Type* RefPtr = &InRef;

				GetObject = [AccessToken=MoveTemp(AccessToken), RefPtr]() -> Type*
				{
					Type* Object = nullptr;
					if (AccessToken.IsValid())
					{
						Object = RefPtr;
					}
					return Object;
				};
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				Get();
#endif
			}


			/** Creates an access pointer from another using a const casts. */
			template<typename OtherType>
			TAccessPtr(const TAccessPtr<OtherType>& InOther, EConstCast InTag)
			{
				GetObject = [GetOtherObject=InOther.GetObject]() -> Type*
				{
					return const_cast<Type*>(GetOtherObject());
				};
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				Get();
#endif
			} 

			/** Creates an access pointer from another using a static cast. */
			template <
				typename OtherType
				UE_REQUIRES(std::is_convertible_v<ObjectType*, Type*>)
			>
			TAccessPtr(const TAccessPtr<OtherType>& InOther, EDerivedCopy InTag=EDerivedCopy::Tag)
			{
				GetObject = [GetOtherObject=InOther.GetObject]() -> Type*
				{
					return static_cast<Type*>(GetOtherObject());
				};
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				Get();
#endif
			}

			TAccessPtr(const TAccessPtr<Type>& InOther) = default;
			TAccessPtr<Type>& operator=(const TAccessPtr<Type>& InOther) = default;

			TAccessPtr(TAccessPtr<Type>&& InOther) = default;
			TAccessPtr& operator=(TAccessPtr<Type>&& InOther) = default;

		protected:

			template<typename RelatedAccessPtrType, typename RelatedType>
			friend RelatedAccessPtrType MakeAccessPtr(const FAccessPoint& InAccessPoint, RelatedType& InRef);

			template<typename ToAccessPtrType, typename FromAccessPtrType> 
			friend ToAccessPtrType ConstCastAccessPtr(const FromAccessPtrType& InAccessPtr);

			template<typename OtherType>
			friend class TAccessPtr;

#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
			mutable Type* CachedObjectPtr = nullptr;
#endif
			TFunction<Type*()> GetObject;

			TAccessPtr(TFunction<Type*()> InGetObject)
			: GetObject(InGetObject)
			{
#if METASOUND_FRONTEND_ACCESSPTR_DEBUG_INFO 
				Get();
#endif
			}

		};

		template<typename Type>
		Type TAccessPtr<Type>::FallbackObject = Type();


		/** FAccessPoint acts as a lifecycle tracker for the TAccessPtrs it creates. 
		 * When this object is destructed, all associated TAccessPtrs will become invalid.
		 */
		class FAccessPoint
		{
		public:
			using FTokenType = FAccessToken;

			FAccessPoint() 
			{
				Token = MakeShared<FTokenType>();
			}

			FAccessPoint(const FAccessPoint&) 
			{
				// Do not copy token from other access point on copy.
				Token = MakeShared<FTokenType>();
			}

			// Do not copy token from other access point on assignment
			FAccessPoint& operator=(const FAccessPoint&) 
			{
				return *this;
			}

		private:
			template<typename AccessPtrType, typename Type>
			friend AccessPtrType MakeAccessPtr(const FAccessPoint& InAccessPoint, Type& InRef);

			FAccessPoint(FAccessPoint&&) = delete;
			FAccessPoint& operator=(FAccessPoint&&) = delete;

			TSharedPtr<FTokenType> Token;
		};

		template<typename AccessPtrType, typename Type>
		AccessPtrType MakeAccessPtr(const FAccessPoint& InAccessPoint, Type& InRef)
		{
			return AccessPtrType(InAccessPoint.Token, InRef);
		}

		template<typename ToAccessPtrType, typename FromAccessPtrType> 
		ToAccessPtrType ConstCastAccessPtr(const FromAccessPtrType& InAccessPtr)
		{
			return ToAccessPtrType(InAccessPtr, ToAccessPtrType::EConstCast::Tag);
		}
	}
}
