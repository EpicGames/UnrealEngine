// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Internationalization/Text.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEnvironment.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	namespace MetasoundArrayNodesPrivate
	{
		// Convenience function for make FNodeClassMetadata of array nodes.
		METASOUNDFRONTEND_API FNodeClassName CreateArrayNodeClassName(const FName& InOperatorName, const FName& InDataTypeName);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface, int32 MajorVersion=1, int32 MinorVersion=0, bool bIsDeprecated=false);

		METASOUNDFRONTEND_API FVertexInterface CreateArrayNumInterface(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArrayGetInterface(const FName& InArrayDataTypeName, const FName& InDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArraySetInterface(const FName& InArrayDataTypeName, const FName& InDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArraySubsetInterface(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArrayConcatInterface(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArrayLastIndexInterface(const FName& InArrayDataTypeName);

		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArrayNumNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArrayGetNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArraySetNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArraySubsetNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArrayConcatNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArrayLastIndexNodeClassRegistryKey(const FName& InArrayDataTypeName);

		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayNumNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayGetNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArraySetNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArraySubsetNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayConcatNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayLastIndexNodeClassMetadata(const FName& InArrayDataTypeName, const FText& InArrayDataTypeDisplayText);

		// Retrieve the ElementType from an ArrayType
		template<typename ArrayType>
		struct TArrayElementType
		{
			// Default implementation has Type. 
		};

		// ElementType specialization for TArray types.
		template<typename ElementType>
		struct TArrayElementType<TArray<ElementType>>
		{
			using Type = ElementType;
		};
	}

	namespace ArrayNodeVertexNames
	{
		extern METASOUNDFRONTEND_API const FLazyName InputInitialArrayName;
		extern METASOUNDFRONTEND_API const FText InputInitialArrayTooltip;
		extern METASOUNDFRONTEND_API const FText InputInitialArrayDisplayName; 

		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputArray)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputLeftArray)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputRightArray)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputTriggerGet)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputTriggerSet)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputIndex)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputStartIndex)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputEndIndex)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, InputValue)

		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputNum)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputValue)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputArrayConcat)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputArraySet)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputArraySubset)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputLastIndex)
	}


	/** TArrayNumOperator gets the number of elements in an Array. The operator
	 * uses the FNodeFacade and defines the vertex, metadata and vertex interface
	 * statically on the operator class. */
	template<typename ArrayType>
	class TArrayNumOperator : public TExecutableOperator<TArrayNumOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static const FNodeClassMetadata Metadata = MetasoundArrayNodesPrivate::CreateArrayNumNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			// Get the input array or construct an empty one. 
			FArrayDataReadReference Array = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputArray), InParams.OperatorSettings);

			return MakeUnique<TArrayNumOperator>(Array);
		}

		TArrayNumOperator(FArrayDataReadReference InArray)
		: Array(InArray)
		, Num(TDataWriteReference<int32>::CreateNew())
		{
			// Initialize value for downstream nodes.
			*Num = Array->Num();
		}

		virtual ~TArrayNumOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputArray), Array);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputNum), Num);
		}

		void Execute()
		{
			*Num = Array->Num();
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			Execute();
		}

	private:

		FArrayDataReadReference Array;
		TDataWriteReference<int32> Num;
	};

	template<typename ArrayType>
	using TArrayNumNode = TNodeFacade<TArrayNumOperator<ArrayType>>;

	/** TArrayGetOperator copies a value from the array to the output when
	 * a trigger occurs. Initially, the output value is default constructed and
	 * will remain that way until until a trigger is encountered.
	 */
	template<typename ArrayType>
	class TArrayGetOperator : public TExecutableOperator<TArrayGetOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;


		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static FNodeClassMetadata Metadata = CreateArrayGetNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeName<ElementType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		struct FInitParams
		{
			TDataReadReference<FTrigger> Trigger;
			FArrayDataReadReference Array;
			TDataReadReference<int32> Index;
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			FString GraphName;
#endif
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			// Input Trigger
			TDataReadReference<FTrigger> Trigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerGet), InParams.OperatorSettings);
			
			// Input Array
			FArrayDataReadReference Array = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputArray), InParams.OperatorSettings);

			// Input Index
			TDataReadReference<int32> Index = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputIndex), InParams.OperatorSettings);
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			FString GraphName;
			if (InParams.Environment.Contains<FString>(Frontend::SourceInterface::Environment::GraphName))
			{
				GraphName = InParams.Environment.GetValue<FString>(Frontend::SourceInterface::Environment::GraphName);
			}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

			FInitParams OperatorInitParams
			{
				Trigger
				, Array
				, Index
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				, GraphName
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
			};

			return MakeUnique<TArrayGetOperator>(InParams.OperatorSettings, MoveTemp(OperatorInitParams));
		}



		TArrayGetOperator(const FOperatorSettings& InSettings, FInitParams&& InParams)
		: Trigger(InParams.Trigger)
		, Array(InParams.Array)
		, Index(InParams.Index)
		, Value(TDataWriteReferenceFactory<ElementType>::CreateExplicitArgs(InSettings))
#if WITH_METASOUND_DEBUG_ENVIRONMENT
		, GraphName(InParams.GraphName)
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
		{
			const int32 IndexValue = *Index;
			const ArrayType& ArrayRef = *Array;

			if ((IndexValue >= 0) && (IndexValue < ArrayRef.Num()))
			{
				*Value = ArrayRef[IndexValue];
			}
		}

		virtual ~TArrayGetOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerGet), Trigger);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputArray), Array);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIndex), Index);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), Value);
		}

		void Execute()
		{
			// Only perform get on trigger.
			if (*Trigger)
			{
				const int32 IndexValue = *Index;
				const ArrayType& ArrayRef = *Array;

				if ((IndexValue >= 0) && (IndexValue < ArrayRef.Num()))
				{
					*Value = ArrayRef[IndexValue];
				}
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Attempt to get value at invalid index [ArraySize:%d, Index:%d] in MetaSound Graph \"%ls\".", ArrayRef.Num(), IndexValue, *GraphName);
				}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			const int32 IndexValue = *Index;
			const ArrayType& ArrayRef = *Array;

			if ((IndexValue >= 0) && (IndexValue < ArrayRef.Num()))
			{
				*Value = ArrayRef[IndexValue];
			}
			else
			{
				*Value = TDataTypeFactory<ElementType>::CreateExplicitArgs(InParams.OperatorSettings);
			}
		}

	private:

		TDataReadReference<FTrigger> Trigger;
		FArrayDataReadReference Array;
		TDataReadReference<int32> Index;
		TDataWriteReference<ElementType> Value;

#if WITH_METASOUND_DEBUG_ENVIRONMENT
		FString GraphName;
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

	};

	template<typename ArrayType>
	using TArrayGetNode = TNodeFacade<TArrayGetOperator<ArrayType>>;

	/** TArraySetOperator sets an element in an array to a specific value. */
	template<typename ArrayType>
	class TArraySetOperator : public TExecutableOperator<TArraySetOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using FArrayDataWriteReference = TDataWriteReference<ArrayType>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static FNodeClassMetadata Metadata = CreateArraySetNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeName<ElementType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		struct FInitParams
		{
			TDataReadReference<FTrigger> Trigger;
			FArrayDataReadReference InitArray;
			FArrayDataWriteReference Array;
			TDataReadReference<int32> Index;
			TDataReadReference<ElementType> Value;
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			FString GraphName;
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			TDataReadReference<FTrigger> Trigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerSet), InParams.OperatorSettings);

			FArrayDataReadReference InitArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(InputInitialArrayName, InParams.OperatorSettings);
			FArrayDataWriteReference Array = TDataWriteReferenceFactory<ArrayType>::CreateExplicitArgs(InParams.OperatorSettings, *InitArray);

			TDataReadReference<int32> Index = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputIndex), InParams.OperatorSettings);

			TDataReadReference<ElementType> Value = InputData.GetOrCreateDefaultDataReadReference<ElementType>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			FString GraphName;
			if (InParams.Environment.Contains<FString>(Frontend::SourceInterface::Environment::GraphName))
			{
				GraphName = InParams.Environment.GetValue<FString>(Frontend::SourceInterface::Environment::GraphName);
			}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

			FInitParams OperatorInitParams 
			{
				Trigger
				, InitArray
				, Array 
				, Index 
				, Value
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				, GraphName
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
			};

			return MakeUnique<TArraySetOperator>(InParams.OperatorSettings, MoveTemp(OperatorInitParams));
		}

		TArraySetOperator(const FOperatorSettings& InSettings, FInitParams&& InParams)
		: OperatorSettings(InSettings)
		, Trigger(InParams.Trigger)
		, InitArray(InParams.InitArray)
		, Array(InParams.Array)
		, Index(InParams.Index)
		, Value(InParams.Value)
#if WITH_METASOUND_DEBUG_ENVIRONMENT
		, GraphName(InParams.GraphName)
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
		{
		}

		virtual ~TArraySetOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerSet), Trigger);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputArray), InitArray);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIndex), Index);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), Value);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputArraySet), Array);
		}

		void Execute()
		{
			if (*Trigger)
			{
				const int32 IndexValue = *Index;
				ArrayType& ArrayRef = *Array;

				if ((IndexValue >= 0) && (IndexValue < ArrayRef.Num()))
				{
					ArrayRef[IndexValue] = *Value;
				}
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Attempt to set value at invalid index [ArraySize:%d, Index:%d] in MetaSound Graph \"%ls\".", ArrayRef.Num(), IndexValue, *GraphName);
				}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
			}
		}

		void Reset(const IOperator::FResetParams& Inparams)
		{
			*Array = *InitArray;
		}

	private:
		FOperatorSettings OperatorSettings;

		TDataReadReference<FTrigger> Trigger;
		FArrayDataReadReference InitArray;
		FArrayDataWriteReference Array;
		TDataReadReference<int32> Index;
		TDataReadReference<ElementType> Value;

#if WITH_METASOUND_DEBUG_ENVIRONMENT
		FString GraphName;
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

	};

	template<typename ArrayType>
	using TArraySetNode = TNodeFacade<TArraySetOperator<ArrayType>>;

	/** TArrayConcatOperator concatenates two arrays on trigger. */
	template<typename ArrayType>
	class TArrayConcatOperator : public TExecutableOperator<TArrayConcatOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using FArrayDataWriteReference = TDataWriteReference<ArrayType>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static const FNodeClassMetadata Metadata = CreateArrayConcatNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			TDataReadReference<FTrigger> Trigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerGet), InParams.OperatorSettings);

			FArrayDataReadReference LeftArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputLeftArray), InParams.OperatorSettings);
			FArrayDataReadReference RightArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputRightArray), InParams.OperatorSettings);

			return MakeUnique<TArrayConcatOperator>(Trigger, LeftArray, RightArray);
		}


		TArrayConcatOperator(TDataReadReference<FTrigger> InTrigger, FArrayDataReadReference InLeftArray, FArrayDataReadReference InRightArray)
		: Trigger(InTrigger)
		, LeftArray(InLeftArray)
		, RightArray(InRightArray)
		, OutArray(TDataWriteReference<ArrayType>::CreateNew())
		{
		}

		virtual ~TArrayConcatOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerGet), Trigger);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLeftArray), LeftArray);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRightArray), RightArray);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputArrayConcat), OutArray);
		}

		void Execute()
		{
			if (*Trigger)
			{
				*OutArray = *LeftArray;
				OutArray->Append(*RightArray);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			OutArray->Reset();
		}

	private:

		TDataReadReference<FTrigger> Trigger;
		FArrayDataReadReference LeftArray;
		FArrayDataReadReference RightArray;
		FArrayDataWriteReference OutArray;
	};

	template<typename ArrayType>
	using TArrayConcatNode = TNodeFacade<TArrayConcatOperator<ArrayType>>;

	/** TArraySubsetOperator slices an array on trigger. */
	template<typename ArrayType>
	class TArraySubsetOperator : public TExecutableOperator<TArraySubsetOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using FArrayDataWriteReference = TDataWriteReference<ArrayType>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static const FNodeClassMetadata Metadata = CreateArraySubsetNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			TDataReadReference<FTrigger> Trigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerGet), InParams.OperatorSettings);

			FArrayDataReadReference InArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputArray), InParams.OperatorSettings);

			TDataReadReference<int32> StartIndex = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputStartIndex), InParams.OperatorSettings);
			TDataReadReference<int32> EndIndex = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputEndIndex), InParams.OperatorSettings);

			TDataWriteReference<ArrayType> OutputArray = TDataWriteReferenceFactory<ArrayType>::CreateExplicitArgs(InParams.OperatorSettings);

			return MakeUnique<TArraySubsetOperator>(Trigger, InArray, StartIndex, EndIndex, OutputArray);
		}


		TArraySubsetOperator(TDataReadReference<FTrigger> InTrigger, FArrayDataReadReference InInputArray, TDataReadReference<int32> InStartIndex, TDataReadReference<int32> InEndIndex, TDataWriteReference<ArrayType> InOutputArray)
		: Trigger(InTrigger)
		, InputArray(InInputArray)
		, StartIndex(InStartIndex)
		, EndIndex(InEndIndex)
		, OutputArray(InOutputArray)
		{
		}

		virtual ~TArraySubsetOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerGet), Trigger);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputArray), InputArray);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartIndex), StartIndex);
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEndIndex), EndIndex);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputArraySubset), OutputArray);
		}

		void Execute()
		{
			if (*Trigger)
			{
				OutputArray->Reset();

				const ArrayType& InputArrayRef = *InputArray;
				const int32 StartIndexValue = FMath::Max(0, *StartIndex);
				const int32 EndIndexValue = FMath::Min(InputArrayRef.Num(), *EndIndex + 1);

				if (StartIndexValue < EndIndexValue)
				{
					const int32 Num = EndIndexValue - StartIndexValue;
					OutputArray->Append(&InputArrayRef[StartIndexValue], Num);
				}
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			OutputArray->Reset();
		}

	private:

		TDataReadReference<FTrigger> Trigger;
		FArrayDataReadReference InputArray;
		TDataReadReference<int32> StartIndex;
		TDataReadReference<int32> EndIndex;
		FArrayDataWriteReference OutputArray;
	};

	template<typename ArrayType>
	using TArraySubsetNode = TNodeFacade<TArraySubsetOperator<ArrayType>>;

	/** TArrayLastIndex gets last index of an array. */
	template<typename ArrayType>
	class TArrayLastIndexOperator : public TExecutableOperator<TArrayLastIndexOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace MetasoundArrayNodesPrivate;

			static const FNodeClassMetadata Metadata = CreateArrayLastIndexNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			// Get the input array or construct an empty one. 
			FArrayDataReadReference Array = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputArray), InParams.OperatorSettings);

			return MakeUnique<TArrayLastIndexOperator>(Array);
		}

		TArrayLastIndexOperator(FArrayDataReadReference InArray)
			: Array(InArray)
			, LastIndex(TDataWriteReference<int32>::CreateNew())
		{
			// Initialize value for downstream nodes.
			*LastIndex = Array->Num() - 1;
		}

		virtual ~TArrayLastIndexOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;
			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputArray), Array);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace ArrayNodeVertexNames;

			InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputLastIndex), LastIndex);
		}

		void Execute()
		{
			*LastIndex = Array->Num() - 1;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			Execute();
		}

	private:

		FArrayDataReadReference Array;
		TDataWriteReference<int32> LastIndex;
	};

	template<typename ArrayType>
	using TArrayLastIndexNode = TNodeFacade<TArrayLastIndexOperator<ArrayType>>;

} // namespace Metasound
#undef LOCTEXT_NAMESPACE
