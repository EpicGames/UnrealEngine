// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundArrayNodes.h"
#include "MetasoundArrayShuffleNode.h"
#include "MetasoundArrayRandomNode.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundNodeRegistrationMacro.h"

#include <type_traits>

namespace Metasound
{
	template<typename ... ElementType>
	struct TEnableArrayNodes
	{
		static constexpr bool Value = true;
	};

	namespace MetasoundArrayNodesPrivate
	{
		// TArrayNodeSupport acts as a configuration sturct to determine whether
		// a particular TArrayNode can be instantiated for a specific ArrayType.
		//
		// Some ArrayNodes require that the array elements have certain properties
		// such as default element constructors, element copy constructors, etc.
		template<typename ArrayType>
		struct TArrayNodeSupport
		{
		private:
			using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;

			static constexpr bool bIsElementParsableAndAssignable = TIsParsable<ElementType>::Value && std::is_copy_assignable<ElementType>::value;

			static constexpr bool bEnabled = TEnableArrayNodes<ElementType>::Value;

		public:
			
			// Array num is supported for all array types.
			static constexpr bool bIsArrayNumSupported = bEnabled;

			// Array last index is supported for all array types.
			static constexpr bool bIsArrayLastIndexSupported = bEnabled;

			// Element must be default parsable to create get operator because a
			// value must be returned even if the index is invalid. Also values are
			// assigned by copy.
			static constexpr bool bIsArrayGetSupported = bEnabled && bIsElementParsableAndAssignable;

			// Element must be copy assignable to set the value.
			static constexpr bool bIsArraySetSupported = bEnabled && std::is_copy_assignable<ElementType>::value && std::is_copy_constructible<ElementType>::value;

			// Elements must be copy constructible
			static constexpr bool bIsArrayConcatSupported = bEnabled && std::is_copy_constructible<ElementType>::value;

			// Elements must be copy constructible
			static constexpr bool bIsArraySubsetSupported = bEnabled && std::is_copy_constructible<ElementType>::value;

			// Array shuffle is supported for all types that get is supported for.
			static constexpr bool bIsArrayShuffleSupported = bEnabled && bIsElementParsableAndAssignable;

			// Random get is supported for all types that get is supported for.
			static constexpr bool bIsArrayRandomGetSupported = bEnabled && bIsElementParsableAndAssignable;
		};
	}

	namespace Frontend
	{
		/** Registers all available array nodes which can be instantiated for the given
		 * ArrayType. Some nodes cannot be instantiated due to limitations of the 
		 * array elements.
		 */
		template<typename ArrayType>
		bool RegisterArrayNodes(const FModuleInfo& InModuleInfo)
		{
			using namespace MetasoundArrayNodesPrivate;

			bool bSuccess = true;

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayNumSupported)
			{
				bSuccess = bSuccess && Frontend::RegisterNode<Metasound::TArrayNumNode<ArrayType>>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayGetSupported)
			{
				using FGetNodeType = typename Metasound::TArrayGetNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FGetNodeType>(InModuleInfo);
			}
			
			if constexpr (TArrayNodeSupport<ArrayType>::bIsArraySetSupported)
			{
				using FSetNodeType = typename Metasound::TArraySetNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FSetNodeType>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArraySubsetSupported)
			{
				using FSubsetNodeType = typename Metasound::TArraySubsetNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FSubsetNodeType>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayConcatSupported)
			{
				using FConcatNodeType = typename Metasound::TArrayConcatNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FConcatNodeType>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayShuffleSupported)
			{
				using FShuffleNodeType = typename Metasound::TArrayShuffleNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FShuffleNodeType>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayRandomGetSupported)
			{
				using FRandomGetNodeType = typename Metasound::TArrayRandomGetNode<ArrayType>;
				bSuccess = bSuccess && Frontend::RegisterNode<FRandomGetNodeType>(InModuleInfo);
			}

			if constexpr (TArrayNodeSupport<ArrayType>::bIsArrayLastIndexSupported)
			{
				bSuccess = bSuccess && Frontend::RegisterNode<Metasound::TArrayLastIndexNode<ArrayType>>(InModuleInfo);
			}

			return bSuccess;
		}

		UE_INTERNAL
		METASOUNDFRONTEND_API bool UnregisterArrayNodes(const FName& InArrayDataTypeName, const FModuleInfo& InModuleInfo);
	}
}

