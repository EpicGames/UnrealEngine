// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Concepts/GetTypeHashable.h"
#include "Containers/CircularQueue.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Templates/Models.h"
#include "MetasoundArrayNodes.h"
#include "MetasoundEnum.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundParamHelper.h"
#include "MetasoundTrigger.h"

#define UE_API METASOUNDFRONTEND_API

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace MetasoundArrayHashPrivate
{
	template<typename ElementType>
	FGuid GetArrayContentHashGuid(const TArray<ElementType>& InArray)
	{
		if constexpr (TModels_V<CGetTypeHashable, ElementType>)
		{
			uint32 A = GetTypeHash(Metasound::GetMetasoundDataTypeName<ElementType>());
			uint32 B = A; uint32 C = A; uint32 D = A;
			for (int32 i = 0; i < InArray.Num(); i++)
			{
				const int32 Pos = i % 4;
				switch (Pos)
				{
				case 0: 
					A = HashCombineFast(A, GetTypeHash(InArray[i]));
					break;
				case 1:
					B = HashCombineFast(B, GetTypeHash(InArray[i]));
					break;
				case 2:
					C = HashCombineFast(C, GetTypeHash(InArray[i]));
					break;
				case 3:
					D = HashCombineFast(D, GetTypeHash(InArray[i]));
					break;
				}
			}
			return FGuid(A, B, C, D);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Array Random Get: Please implement \"uint32 GetTypeHash(const T&)\" for type %ls to use Same Data for Shared State Behavior.", *Metasound::GetMetasoundDataTypeString<ElementType>());
			return FGuid();
		}
	}
}

namespace Metasound
{
	namespace ArrayNodeRandomGetVertexNames
	{
		METASOUND_PARAM(InputTriggerNextValue, "Next", "Trigger to get the next value in the randomized array.")
		METASOUND_PARAM(InputTriggerResetSeed, "Reset", "Trigger to reset the seed for the randomized array.")
		METASOUND_PARAM(InputRandomArray, "In Array", "Input array to randomize.")
		METASOUND_PARAM(InputWeights, "Weights", "Input array of weights to use for random selection. Will repeat if this array is shorter than the input array to select from.")
		METASOUND_PARAM(InputSeed, "Seed", "Seed to use for the random stream. Set to -1 to use a random seed.")
		METASOUND_PARAM(InputNoRepeatOrder, "No Repeats", "The number of elements to track to avoid repeating in a row. This is clamped to be within half the array size. The output will end up repeating a clear pattern if set close to the array size. Set to -1 to automatically set to half the array size (which is the maximum no-repeats behavior)")
		METASOUND_PARAM(InputEnableSharedState, "Enable Shared State", "Set to enabled to share state with other Random Get (Array) nodes. Does not apply when previewing in the MetaSound editor; use PIE or game.")
		METASOUND_PARAM(InputSharedStateBehavior, "Shared State Behavior", "The behavior for how state is shared with other Random Get (Array) nodes. Only applied when Enable Shared State is true.")
		METASOUND_PARAM(OutputTriggerOnNext, "On Next", "Triggers when the \"Next\" input is triggered.")
		METASOUND_PARAM(OutputTriggerOnReset, "On Reset", "Triggers when the \"Reset\" input is triggered.")
		METASOUND_PARAM(ShuffleOutputValue, "Value", "Value of the current random element.")
		METASOUND_PARAM(OutputIndex, "Index", "Array index of the current random element.")
	}

	namespace ArrayRandomNodePrivate
	{
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateArrayRandomNodeClassRegistryKey(const FName& InArrayDataTypeName);
		METASOUNDFRONTEND_API FVertexInterface CreateArrayRandomNodeInterface(const FName& InArrayDataTypeName, const FName& InElementDataTypeName);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateArrayRandomNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText);
	}

	enum class ESharedStateBehaviorType : int32
	{
		SameNode,
		SameNodeInComposition, 
		SameData
	};

	DECLARE_METASOUND_ENUM(ESharedStateBehaviorType, ESharedStateBehaviorType::SameNodeInComposition, METASOUNDFRONTEND_API,
		FEnumSharedStateBehaviorType, FEnumSharedStateBehaviorTypeInfo, FEnumSharedStateBehaviorTypeReadRef, FSharedStateBehaviorTypeWriteRef);

	class FArrayRandomGet
	{
	public:
		FArrayRandomGet() = default;
		UE_API FArrayRandomGet(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder);
		~FArrayRandomGet() = default;

		UE_DEPRECATED(5.5, "Use UpdateState instead")
		UE_API void Init(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder);
		UE_API void UpdateState(int32 InSeed, int32 InMaxIndex, const TArray<float>& InWeights, int32 InNoRepeatOrder);
		UE_API void SetSeed(int32 InSeed);
		UE_API void SetNoRepeatOrder(int32 InNoRepeatOrder);
		UE_API void SetRandomWeights(const TArray<float>& InRandomWeights);
		UE_API void ResetSeed();
		UE_API int32 NextValue();

		int32 GetNoRepeatOrder() const { return NoRepeatOrder; }
		int32 GetMaxIndex() const { return MaxIndex; }

	private:
		UE_API float ComputeTotalWeight();

		// The current index into the array of indicies (wraps between 0 and ShuffleIndices.Num())
		TArray<int32> PreviousIndices;
		TUniquePtr<TCircularQueue<int32>> PreviousIndicesQueue;
		int32 NoRepeatOrder = INDEX_NONE;

		// Array of indices (in order 0 to Num)
		int32 MaxIndex = 0;
		TArray<float> RandomWeights;

		// Random stream to use to randomize the shuffling
		FRandomStream RandomStream;
		int32 Seed = INDEX_NONE;
		bool bRandomStreamInitialized = false;
	};

	struct InitSharedStateArgs
	{
		FGuid SharedStateId;
		int32 Seed = INDEX_NONE;
		int32 NumElements = 0;
		int32 NoRepeatOrder = 0;
		bool bIsPreviewSound = false;
		TArray<float> Weights;
	};

	class FSharedStateRandomGetManager
	{
	public:
		static UE_API FSharedStateRandomGetManager& Get();

		UE_API void InitSharedState(InitSharedStateArgs& InArgs);
		// Initialize or update state for a given shared state id. No lock, so call this function within one if needed
		UE_API void InitOrUpdate(InitSharedStateArgs& InStateArgs);

		// Get the next array index 
		// Init or update state with the given args, then return next value (within a single lock operation)
		UE_API int32 NextValue(const FGuid& InSharedStateId, InitSharedStateArgs& InStateArgs);
		UE_API int32 NextValue(const FGuid& InSharedStateId);

		UE_API void SetSeed(const FGuid& InSharedStateId, int32 InSeed);
		UE_API void SetNoRepeatOrder(const FGuid& InSharedStateId, int32 InNoRepeatOrder);
		UE_API void SetRandomWeights(const FGuid& InSharedStateId, const TArray<float>& InRandomWeights);

		// Init or update state with the given args, then reset seed (within a single lock operation)
		UE_API void ResetSeed(const FGuid& InSharedStateId, InitSharedStateArgs& InStateArgs);
		UE_API void ResetSeed(const FGuid& InSharedStateId);


	private:
		FSharedStateRandomGetManager() = default;
		~FSharedStateRandomGetManager() = default;

		FCriticalSection CritSect;

		TMap<FGuid, TUniquePtr<FArrayRandomGet>> RandomGets;
	};

	template<typename ArrayType>
	class TArrayRandomGetOperator : public TExecutableOperator<TArrayRandomGetOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using FArrayWeightReadReference = TDataReadReference<TArray<float>>;
		using WeightsArrayType = TArray<float>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;
		using FElementTypeWriteReference = TDataWriteReference<ElementType>;

		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ArrayNodeRandomGetVertexNames;
			using namespace ArrayRandomNodePrivate;

			static const FVertexInterface DefaultInterface = CreateArrayRandomNodeInterface(GetMetasoundDataTypeName<ArrayType>,GetMetasoundDataTypeName<ElementType>());

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace ArrayRandomNodePrivate;
			static const FNodeClassMetadata Metadata = CreateArrayRandomNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeName<ElementType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeRandomGetVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef InTriggerNext = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerNextValue), InParams.OperatorSettings);
			FTriggerReadRef InTriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerResetSeed), InParams.OperatorSettings);
			FArrayDataReadReference InInputArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputRandomArray), InParams.OperatorSettings);
			FArrayWeightReadReference InInputWeightsArray = InputData.GetOrCreateDefaultDataReadReference<WeightsArrayType>(METASOUND_GET_PARAM_NAME(InputWeights), InParams.OperatorSettings);
			FInt32ReadRef InSeedValue = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSeed), InParams.OperatorSettings);
			FInt32ReadRef InNoRepeatOrder = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputNoRepeatOrder), InParams.OperatorSettings);
			FBoolReadRef bInEnableSharedState = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputEnableSharedState), InParams.OperatorSettings);
			FEnumSharedStateBehaviorType InSharedStateBehavior = InputData.GetOrCreateDefaultValue<FEnumSharedStateBehaviorType>(METASOUND_GET_PARAM_NAME(InputSharedStateBehavior), InParams.OperatorSettings);

			return MakeUnique<TArrayRandomGetOperator<ArrayType>>(InParams, InTriggerNext, InTriggerReset, InInputArray, InInputWeightsArray, InSeedValue, InNoRepeatOrder, bInEnableSharedState, InSharedStateBehavior);
		}

		TArrayRandomGetOperator(
			const FBuildOperatorParams& InParams,
			const FTriggerReadRef& InTriggerNext,
			const FTriggerReadRef& InTriggerReset,
			const FArrayDataReadReference& InInputArray,
			const TDataReadReference<WeightsArrayType>& InInputWeightsArray,
			const FInt32ReadRef& InSeedValue,
			const FInt32ReadRef& InNoRepeatOrder,
			const FBoolReadRef& bInEnableSharedState,
			const FEnumSharedStateBehaviorType InSharedStateBehavior)
			: TriggerNext(InTriggerNext)
			, TriggerReset(InTriggerReset)
			, InputArray(InInputArray)
			, InputWeightsArray(InInputWeightsArray)
			, SeedValue(InSeedValue)
			, NoRepeatOrder(InNoRepeatOrder)
			, bEnableSharedState(bInEnableSharedState)
			, SharedStateBehavior(InSharedStateBehavior)
			, TriggerOnNext(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, TriggerOnReset(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OutValue(TDataWriteReferenceFactory<ElementType>::CreateAny(InParams.OperatorSettings))
			, OutIndex(FInt32WriteRef::CreateNew(INDEX_NONE))			
		{
			NodeId = InParams.Node.GetInstanceID();
			Reset(InParams);
		}

		virtual ~TArrayRandomGetOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ArrayNodeRandomGetVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerNextValue), TriggerNext);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerResetSeed), TriggerReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRandomArray), InputArray);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputWeights), InputWeightsArray);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSeed), SeedValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputNoRepeatOrder), NoRepeatOrder);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEnableSharedState), bEnableSharedState);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(InputSharedStateBehavior), SharedStateBehavior);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ArrayNodeRandomGetVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnNext), TriggerOnNext);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnReset), TriggerOnReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ShuffleOutputValue), OutValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputIndex), OutIndex);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace Frontend;
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			if (InParams.Environment.Contains<FString>(SourceInterface::Environment::GraphName))
			{
				GraphName = *InParams.Environment.GetValue<FString>(SourceInterface::Environment::GraphName);
			}

			TOptional<FName> EnumName = FEnumSharedStateBehaviorType::ToName(SharedStateBehavior);
			if (EnumName.IsSet())
			{
				DebugSharedStateBehaviorString = EnumName.GetValue().ToString();
			}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
	   		if (InParams.Environment.Contains<bool>(SourceInterface::Environment::IsPreview))
			{
				bIsPreviewSound = InParams.Environment.GetValue<bool>(SourceInterface::Environment::IsPreview);
			}
			else
			{
				bIsPreviewSound = false;
			}
			
			*OutValue = TDataTypeFactory<ElementType>::CreateAny(InParams.OperatorSettings);
			*OutIndex = INDEX_NONE;
			TriggerOnNext->Reset();
			TriggerOnReset->Reset();

			// Cache shared state id for shared state behavior types that cannot be changed after node init
			if (InParams.Environment.Contains<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy))
			{
				const TArray<FGuid>& GraphHierarchy = InParams.Environment.GetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy);
				if (SharedStateBehavior == ESharedStateBehaviorType::SameNode)
				{
					check(GraphHierarchy.Num() > 0);
					// Hash node id with this node's graph id because node ids are not guaranteed to be unique 
					// (they are not regenerated when duplicating assets)
					SharedStateId = GetSameNodeSharedStateId(NodeId, GraphHierarchy.Last());
				}
				else if (SharedStateBehavior == ESharedStateBehaviorType::SameNodeInComposition)
				{
					SharedStateId = GetSameNodeInCompositionId(NodeId, GraphHierarchy);
				}
			}
			else
			{
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				if (!bHasLoggedMissingGraphHierarchyWarning)
				{
					UE_LOGF(LogMetaSound, Warning, "Array Random Get: Graph Hierarchy environment variable needed for Same Node or Same Node in Composition shared state id not found (Graph '%ls')", *GraphName);
					bHasLoggedMissingGraphHierarchyWarning = true;
				}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
			}
		}

		void Execute()
		{
			TriggerOnNext->AdvanceBlock();
			TriggerOnReset->AdvanceBlock();

			const ArrayType& InputArrayRef = *InputArray;
			if (InputArrayRef.Num() == 0)
			{
#if WITH_METASOUND_DEBUG_ENVIRONMENT
				if (!bHasLoggedEmptyArrayWarning)
				{
					UE_LOGF(LogMetaSound, Verbose, "Array Random Get: empty array input (Graph '%ls')", *GraphName);
					bHasLoggedEmptyArrayWarning = true;
				}
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT
				// Pass through triggers
				TriggerReset->ExecuteBlock(
					[&](int32 StartFrame, int32 EndFrame)
					{
					},
					[this](int32 StartFrame, int32 EndFrame)
					{
						TriggerOnReset->TriggerFrame(StartFrame);
					}
				);

				TriggerNext->ExecuteBlock(
					[&](int32 StartFrame, int32 EndFrame)
					{
					},
					[this](int32 StartFrame, int32 EndFrame)
					{
						TriggerOnNext->TriggerFrame(StartFrame);
					}
				);

				return;
			}

 			TriggerReset->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					ExecuteTriggerReset(StartFrame);
				}
			);
 
			TriggerNext->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					ExecuteTriggerNext(StartFrame);
				}
			);
		}

	private:
		void CreateSharedStateArgs(InitSharedStateArgs& InOutStateArgs)
		{
			InOutStateArgs.SharedStateId = SharedStateId;
			InOutStateArgs.Seed = *SeedValue;
			InOutStateArgs.NumElements = (*InputArray).Num();
			InOutStateArgs.NoRepeatOrder = *NoRepeatOrder;
			InOutStateArgs.bIsPreviewSound = bIsPreviewSound;
			InOutStateArgs.Weights = *InputWeightsArray;
		}

		void ExecuteTriggerReset(int32 StartFrame)
		{
			const ArrayType& InputArrayRef = *InputArray;
			if (*bEnableSharedState && !bIsPreviewSound)
			{
				// Update shared state id for array content hash 
				if (SharedStateBehavior == ESharedStateBehaviorType::SameData)
				{
					SharedStateId = MetasoundArrayHashPrivate::GetArrayContentHashGuid(*InputArray);
				}

				FSharedStateRandomGetManager& RGM = FSharedStateRandomGetManager::Get();
				InitSharedStateArgs StateArgs;
				CreateSharedStateArgs(StateArgs);

				// Update and reset seed as one operation
				RGM.ResetSeed(SharedStateId, StateArgs);
			}
			else // No shared state
			{
				if (!ArrayRandomGet.IsValid())
				{
					ArrayRandomGet = MakeUnique<FArrayRandomGet>(*SeedValue, InputArrayRef.Num(), *InputWeightsArray, *NoRepeatOrder);
				}
				else
				{
					ArrayRandomGet->UpdateState(*SeedValue, InputArrayRef.Num(), *InputWeightsArray, *NoRepeatOrder);
				}
				ArrayRandomGet->ResetSeed();
			}
			TriggerOnReset->TriggerFrame(StartFrame);
		}
		
		void ExecuteTriggerNext(int32 StartFrame)
		{
			const ArrayType& InputArrayRef = *InputArray;
			if (*bEnableSharedState && !bIsPreviewSound)
			{
				// Update shared state id for array content hash 
				if (SharedStateBehavior == ESharedStateBehaviorType::SameData)
				{
					SharedStateId = MetasoundArrayHashPrivate::GetArrayContentHashGuid(*InputArray);
				}

				FSharedStateRandomGetManager& RGM = FSharedStateRandomGetManager::Get();
				InitSharedStateArgs StateArgs;
				CreateSharedStateArgs(StateArgs);

				// Update and get next value as one operation
				*OutIndex = RGM.NextValue(SharedStateId, StateArgs);
			}
			else // No shared state
			{
				// Initialize or update state
				if (!ArrayRandomGet.IsValid())
				{
					ArrayRandomGet = MakeUnique<FArrayRandomGet>(*SeedValue, InputArrayRef.Num(), *InputWeightsArray, *NoRepeatOrder);
				}
				else
				{
					ArrayRandomGet->UpdateState(*SeedValue, InputArrayRef.Num(), *InputWeightsArray, *NoRepeatOrder);
				}
				// Get next value
				*OutIndex = ArrayRandomGet->NextValue();
			}

			check(*OutIndex != INDEX_NONE);
#if WITH_METASOUND_DEBUG_ENVIRONMENT
			UE_LOG(LogMetaSound, VeryVerbose, TEXT("Array Random Get Execute Next: \
				Index chosen: %u, Graph: '%s', NumRepeats: %d, Array Size: %u, Seed: %d, Type: %s\
				 Node Id: %s, Shared State Enabled: %u, Shared State Behavior: %s, Shared State Id: %s"), \
				*OutIndex, *GraphName, *NoRepeatOrder, InputArrayRef.Num(), *SeedValue, *Metasound::GetMetasoundDataTypeString<ElementType>(), \
				*NodeId.ToString(), *bEnableSharedState, *DebugSharedStateBehaviorString, *SharedStateId.ToString());
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

			// The input array size may have changed, so make sure it's wrapped into range of the input array
			*OutValue = InputArrayRef[*OutIndex % InputArrayRef.Num()];

			TriggerOnNext->TriggerFrame(StartFrame);
		}

		// Hash combine the current node id with another id
		FGuid GetSameNodeSharedStateId(const FGuid& InNodeId, const FGuid& InOtherId) const
		{
			return FGuid(
				HashCombineFast(InNodeId.A, InOtherId.A),
				HashCombineFast(InNodeId.B, InOtherId.B),
				HashCombineFast(InNodeId.C, InOtherId.C),
				HashCombineFast(InNodeId.D, InOtherId.D)
			);
		}

		// Hash combine the current node id with the graph hierarchy ids
		FGuid GetSameNodeInCompositionId(const FGuid& InNodeId, const TArray<FGuid>& InGraphHierarchy) const
		{
			uint32 A = InNodeId.A; uint32 B = InNodeId.B; uint32 C = InNodeId.C; uint32 D = InNodeId.D;
			for (int i = 0; i < InGraphHierarchy.Num(); ++i)
			{
				A = HashCombineFast(A, InGraphHierarchy[i].A);
				B = HashCombineFast(B, InGraphHierarchy[i].B);
				C = HashCombineFast(C, InGraphHierarchy[i].C);
				D = HashCombineFast(D, InGraphHierarchy[i].D);
			}
			return FGuid(A, B, C, D);
		}
		
		// Inputs
		FTriggerReadRef TriggerNext;
		FTriggerReadRef TriggerReset;
		FArrayDataReadReference InputArray;
		TDataReadReference<WeightsArrayType> InputWeightsArray;
		FInt32ReadRef SeedValue;
		FInt32ReadRef NoRepeatOrder;
		FBoolReadRef bEnableSharedState;
		FEnumSharedStateBehaviorType SharedStateBehavior;

		// Outputs
		FTriggerWriteRef TriggerOnNext;
		FTriggerWriteRef TriggerOnReset;
		TDataWriteReference<ElementType> OutValue;
		FInt32WriteRef OutIndex;

#if WITH_METASOUND_DEBUG_ENVIRONMENT
		FString GraphName;
		bool bHasLoggedEmptyArrayWarning = false;
		bool bHasLoggedMissingGraphHierarchyWarning = false;
		FString DebugSharedStateBehaviorString;
#endif // WITH_METASOUND_DEBUG_ENVIRONMENT

		// Data
		TUniquePtr<FArrayRandomGet> ArrayRandomGet;
		FGuid NodeId;
		FGuid SharedStateId;
		bool bIsPreviewSound = false;
	};

	template<typename ArrayType>
	using TArrayRandomGetNode = TNodeFacade<TArrayRandomGetOperator<ArrayType>>;
}
#undef LOCTEXT_NAMESPACE

#undef UE_API
