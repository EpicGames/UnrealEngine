// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Internationalization/Text.h"
#include "MetasoundArrayNodes.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "Misc/ScopeLock.h"

#include <type_traits>

#define UE_API METASOUNDFRONTEND_API

#define LOCTEXT_NAMESPACE "MetasoundFrontend"


namespace Metasound
{
	/** Shuffle Node Vertex Names */
	namespace ArrayNodeShuffleVertexNames
	{
		METASOUND_PARAM(InputTriggerNext, "Next", "Trigger to get the next value in the shuffled array.")
		METASOUND_PARAM(InputTriggerShuffle, "Shuffle", "Trigger to shuffle the array manually.")
		METASOUND_PARAM(InputTriggerReset, "Reset Seed", "Trigger to reset the random seed stream of the shuffle node.")
		METASOUND_PARAM(InputShuffleArray, "In Array", "Input Array.")
		METASOUND_PARAM(InputShuffleSeed, "Seed", "Seed to use for the the random shuffle.")
		METASOUND_PARAM(InputAutoShuffle, "Auto Shuffle", "Set to true to automatically shuffle when the array has been read.")
		METASOUND_PARAM(InputShuffleEnableSharedState, "Enable Shared State", "Set to enabled shared state across instances of this metasound.")

		METASOUND_PARAM(OutputTriggerOnNext, "On Next", "Triggers when the \"Next\" input is triggered.")
		METASOUND_PARAM(OutputTriggerOnShuffle, "On Shuffle", "Triggers when the \"Shuffle\" input is triggered or if the array is auto-shuffled.")
		METASOUND_PARAM(OutputTriggerOnResetSeed, "On Reset Seed", "Triggers when the \"Reset Seed\" input is triggered.")
		METASOUND_PARAM(OutputShuffledValue, "Value", "Value of the current shuffled element.")
	}

	namespace ArrayShuffleNodePrivate
	{
		UE_API Frontend::FNodeClassRegistryKey CreateArrayShuffleNodeClassRegistryKey(const FName& InArrayDataTypeName);
		UE_API FVertexInterface CreateArrayShuffleNodeInterface(const FName& InArrayDataTypeName, const FName& InElementDataTypeName);
		UE_API FNodeClassMetadata CreateArrayShuffleNodeClassMetadata(const FName& InArrayDataTypeName, const FName& InElementDataTypeName, const FText& InArrayDataTypeDisplayText);
	}

	class FArrayIndexShuffler
	{
	public:
		FArrayIndexShuffler() = default;
		UE_API FArrayIndexShuffler(int32 InSeed, int32 MaxIndices);

		UE_API void Init(int32 InSeed, int32 MaxIndices);
		UE_API void SetSeed(int32 InSeed);
		UE_API void ResetSeed();

		// Returns the next value in the array indices. Returns true if the array was re-shuffled automatically.
		UE_API bool NextValue(bool bAutoShuffle, int32& OutIndex);

		// Shuffle the array with the given max indices
		UE_API void ShuffleArray();

	private:
		// Helper function to swap the current index with a random index
		UE_API void RandomSwap(int32 InCurrentIndex, int32 InStartIndex, int32 InEndIndex);

		// The current index into the array of indicies (wraps between 0 and ShuffleIndices.Num())
		int32 CurrentIndex = 0;

		// The previously returned value. Used to avoid repeating the last value on shuffle.
		int32 PrevValue = INDEX_NONE;

		// Array of indices (in order 0 to Num), shuffled
		TArray<int32> ShuffleIndices;

		// Random stream to use to randomize the shuffling
		FRandomStream RandomStream;
	};

	class FSharedStateShuffleManager
	{
	public:
		static FSharedStateShuffleManager& Get()
		{
			static FSharedStateShuffleManager GSM;
			return GSM;
		}

		void InitSharedState(uint32 InSharedStateId, int32 InSeed, int32 InNumElements)
		{
			FScopeLock Lock(&CritSect);

			if (!Shufflers.Contains(InSharedStateId))
			{
				Shufflers.Add(InSharedStateId, MakeUnique<FArrayIndexShuffler>(InSeed, InNumElements));
			}
		}

		bool NextValue(uint32 InSharedStateId, bool bAutoShuffle, int32& OutIndex)
		{
			FScopeLock Lock(&CritSect);
			TUniquePtr<FArrayIndexShuffler>* Shuffler = Shufflers.Find(InSharedStateId);
			return (*Shuffler)->NextValue(bAutoShuffle, OutIndex);
		}

		void SetSeed(uint32 InSharedStateId, int32 InSeed)
		{
			FScopeLock Lock(&CritSect);
			TUniquePtr<FArrayIndexShuffler>* Shuffler = Shufflers.Find(InSharedStateId);
			(*Shuffler)->SetSeed(InSeed);
		}

		void ResetSeed(uint32 InSharedStateId)
		{
			FScopeLock Lock(&CritSect);
			TUniquePtr<FArrayIndexShuffler>* Shuffler = Shufflers.Find(InSharedStateId);
			(*Shuffler)->ResetSeed();
		}

		void ShuffleArray(uint32 InSharedStateId)
		{
			FScopeLock Lock(&CritSect);
			TUniquePtr<FArrayIndexShuffler>* Shuffler = Shufflers.Find(InSharedStateId);
			(*Shuffler)->ShuffleArray();
		}

	private:
		FSharedStateShuffleManager() = default;
		~FSharedStateShuffleManager() = default;

		FCriticalSection CritSect;

		TMap<uint32, TUniquePtr<FArrayIndexShuffler>> Shufflers;	
	};

	/** TArrayShuffleOperator shuffles an array on trigger and outputs values sequentially on "next". It avoids repeating shuffled elements and supports auto-shuffling.*/
	template<typename ArrayType>
	class TArrayShuffleOperator : public TExecutableOperator<TArrayShuffleOperator<ArrayType>>
	{
	public:
		using FArrayDataReadReference = TDataReadReference<ArrayType>;
		using ElementType = typename MetasoundArrayNodesPrivate::TArrayElementType<ArrayType>::Type;
		using FElementTypeWriteReference = TDataWriteReference<ElementType>;

		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ArrayNodeShuffleVertexNames;
			using namespace ArrayShuffleNodePrivate;

			static const FVertexInterface DefaultInterface = CreateArrayShuffleNodeInterface(GetMetasoundDataTypeName<ArrayType>,GetMetasoundDataTypeName<ElementType>());

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			using namespace ArrayShuffleNodePrivate;
			static const FNodeClassMetadata Metadata = CreateArrayShuffleNodeClassMetadata(GetMetasoundDataTypeName<ArrayType>(), GetMetasoundDataTypeName<ElementType>(), GetMetasoundDataTypeDisplayText<ArrayType>());

			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ArrayNodeShuffleVertexNames;
			using namespace MetasoundArrayNodesPrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TDataReadReference<FTrigger> InTriggerNext = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerNext), InParams.OperatorSettings);
			TDataReadReference<FTrigger> InTriggerShuffle = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerShuffle), InParams.OperatorSettings);
			TDataReadReference<FTrigger> InTriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerReset), InParams.OperatorSettings);
			FArrayDataReadReference InInputArray = InputData.GetOrCreateDefaultDataReadReference<ArrayType>(METASOUND_GET_PARAM_NAME(InputShuffleArray), InParams.OperatorSettings);
			TDataReadReference<int32> InSeedValue = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputShuffleSeed), InParams.OperatorSettings);
			TDataReadReference<bool> bInAutoShuffle = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputAutoShuffle), InParams.OperatorSettings);
			TDataReadReference<bool> bInEnableSharedState = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputShuffleEnableSharedState), InParams.OperatorSettings);

			return MakeUnique<TArrayShuffleOperator>(InParams, InTriggerNext, InTriggerShuffle, InTriggerReset, InInputArray, InSeedValue, bInAutoShuffle, bInEnableSharedState);
		}

		TArrayShuffleOperator(
			const FBuildOperatorParams& InParams,
			const TDataReadReference<FTrigger>& InTriggerNext, 
			const TDataReadReference<FTrigger>& InTriggerShuffle,
			const TDataReadReference<FTrigger>& InTriggerReset,
			const FArrayDataReadReference& InInputArray,
			const TDataReadReference<int32>& InSeedValue,
			const TDataReadReference<bool>& bInAutoShuffle, 
			const TDataReadReference<bool>& bInEnableSharedState)
			: TriggerNext(InTriggerNext)
			, TriggerShuffle(InTriggerShuffle)
			, TriggerReset(InTriggerReset)
			, InputArray(InInputArray)
			, SeedValue(InSeedValue)
			, bAutoShuffle(bInAutoShuffle)
			, bEnableSharedState(bInEnableSharedState)
			, TriggerOnShuffle(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OutValue(TDataWriteReferenceFactory<ElementType>::CreateAny(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual ~TArrayShuffleOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ArrayNodeShuffleVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerNext), TriggerNext);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerShuffle), TriggerShuffle);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerReset), TriggerReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputShuffleArray), InputArray);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputShuffleSeed), SeedValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAutoShuffle), bAutoShuffle);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputShuffleEnableSharedState), bEnableSharedState);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ArrayNodeShuffleVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnNext), TriggerNext);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnShuffle), TriggerOnShuffle);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnResetSeed), TriggerReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputShuffledValue), OutValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace Frontend;

			SharedStateUniqueId = INDEX_NONE;
			if (InParams.Environment.Contains<uint32>(SourceInterface::Environment::SoundUniqueID))
			{
				// Get the environment variable for the unique ID of the sound
				SharedStateUniqueId = InParams.Environment.GetValue<uint32>(SourceInterface::Environment::SoundUniqueID);
			}

			PrevSeedValue = *SeedValue;

			InitializeShufflers(InputArray->Num());

			TriggerOnShuffle->Reset();
			*OutValue = TDataTypeFactory<ElementType>::CreateAny(InParams.OperatorSettings);
		}

		void Execute()
		{
			TriggerOnShuffle->AdvanceBlock();

			const ArrayType& InputArrayRef = *InputArray;
			// Don't do anything if our array is empty
			if (InputArrayRef.Num() == 0)
			{
				return;
			}

			if (PrevArraySize != InputArrayRef.Num())
			{
				InitializeShufflers(InputArrayRef.Num());
			}
 
			// Check for a seed change
			if (PrevSeedValue != *SeedValue)
			{
				Seed(*SeedValue);
			}

			TriggerReset->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					Reset();
				}
			);

			TriggerShuffle->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					Shuffle();

					TriggerOnShuffle->TriggerFrame(StartFrame);
				}
			);

			TriggerNext->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					const ArrayType& InputArrayRef = *InputArray;

					int32 OutShuffleIndex = INDEX_NONE;
					bool bShuffleTriggered = GetNextIndex(OutShuffleIndex);

					// The input array size may have changed, so make sure it's wrapped into range of the input array
					*OutValue = InputArrayRef[OutShuffleIndex % InputArrayRef.Num()];

					// Trigger out if the array was auto-shuffled
					if (bShuffleTriggered)
					{
						TriggerOnShuffle->TriggerFrame(StartFrame);
					}
				}
			);
		}

	private:
		void InitializeShufflers(int32 InSize)
		{
			PrevArraySize = InSize;

			if (InSize > 0)
			{
				if (SharedStateUniqueId != INDEX_NONE)
				{
					FSharedStateShuffleManager& SM = FSharedStateShuffleManager::Get();
					SM.InitSharedState(SharedStateUniqueId, PrevSeedValue, InSize);
				}
				ArrayIndexShuffler = MakeUnique<FArrayIndexShuffler>(PrevSeedValue, InSize);
			}
		}


		void Seed(int32 InNewSeed)
		{
			PrevSeedValue = InNewSeed;

			if (SharedStateUniqueId != INDEX_NONE)
			{
				FSharedStateShuffleManager& SM = FSharedStateShuffleManager::Get();
				SM.SetSeed(SharedStateUniqueId, InNewSeed);
			}

			if (ArrayIndexShuffler.IsValid())
			{
				ArrayIndexShuffler->SetSeed(InNewSeed);
			}
		}

		void Reset()
		{
			if (SharedStateUniqueId != INDEX_NONE)
			{
				FSharedStateShuffleManager& SM = FSharedStateShuffleManager::Get();
				SM.ResetSeed(SharedStateUniqueId);
			}
		
			if (ArrayIndexShuffler.IsValid())
			{
				ArrayIndexShuffler->ResetSeed();
			}
		}

		bool GetNextIndex(int32& OutIndex) 
		{
			if (*bEnableSharedState && SharedStateUniqueId != INDEX_NONE)
			{
				FSharedStateShuffleManager& SM = FSharedStateShuffleManager::Get();
				return SM.NextValue(SharedStateUniqueId, *bAutoShuffle, OutIndex);
			}
			else if (ArrayIndexShuffler.IsValid())
			{
				return ArrayIndexShuffler->NextValue(*bAutoShuffle, OutIndex);
			}
			else
			{
				checkNoEntry();
				OutIndex = 0;
				return false;
			}
		}

		void Shuffle()
		{
			if (SharedStateUniqueId != INDEX_NONE)
			{
				FSharedStateShuffleManager& SM = FSharedStateShuffleManager::Get();
				SM.ShuffleArray(SharedStateUniqueId);
			}

			if (ArrayIndexShuffler.IsValid())
			{
				ArrayIndexShuffler->ShuffleArray();
			}
		}

		// Inputs
		TDataReadReference<FTrigger> TriggerNext;
		TDataReadReference<FTrigger> TriggerShuffle;
		TDataReadReference<FTrigger> TriggerReset;
		FArrayDataReadReference InputArray;
		TDataReadReference<int32> SeedValue;
		TDataReadReference<bool> bAutoShuffle;
		TDataReadReference<bool> bEnableSharedState;

		// Outputs
		TDataWriteReference<FTrigger> TriggerOnShuffle;
		TDataWriteReference<ElementType> OutValue;

		// Data
		TUniquePtr<FArrayIndexShuffler> ArrayIndexShuffler;
		int32 PrevSeedValue = INDEX_NONE;
		int32 PrevArraySize = 0;
		uint32 SharedStateUniqueId = INDEX_NONE;
	};

	template<typename ArrayType>
	using TArrayShuffleNode = TNodeFacade<TArrayShuffleOperator<ArrayType>>;
}

#undef LOCTEXT_NAMESPACE

#undef UE_API
