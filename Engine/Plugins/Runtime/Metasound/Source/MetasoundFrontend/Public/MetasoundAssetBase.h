// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundAccessPtr.h"
#include "MetasoundAssetManager.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundGraph.h"
#include "MetasoundLog.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundVertex.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
class UEdGraph;
struct FMetaSoundFrontendDocumentBuilder;
class IConsoleVariable;
typedef TMulticastDelegate<void(IConsoleVariable*), FDefaultDelegateUserPolicy> FConsoleVariableMulticastDelegate;

class FMetasoundAssetProxy;
using FMetasoundAssetProxyPtr = TSharedPtr<FMetasoundAssetProxy>;

namespace Metasound
{
	class IGraph;
	class FGraph;
	namespace Frontend
	{
		// Forward Declarations
		class IInterfaceRegistryEntry;

		METASOUNDFRONTEND_API TRange<float> GetBlockRateClampRange();
		METASOUNDFRONTEND_API float GetBlockRateOverride();
		METASOUNDFRONTEND_API FConsoleVariableMulticastDelegate& GetBlockRateOverrideChangedDelegate();

		METASOUNDFRONTEND_API TRange<int32> GetSampleRateClampRange();
		METASOUNDFRONTEND_API int32 GetSampleRateOverride();
		METASOUNDFRONTEND_API FConsoleVariableMulticastDelegate& GetSampleRateOverrideChangedDelegate();
		class FProxyDataCache;

	} // namespace Frontend
} // namespace Metasound

/** FMetasoundAssetBase is a mix-in subclass for UObjects which utilize MetaSound objects. As MetaSounds
 * can now be generated dynamically via the Builder API, the name does not accurately reflect this classes
 * current implementation.  While it currently has some support for asset, editor graph & document accessors (actively
 * being deprecated), its primary use is to support all access to runtime-specific, MetaSound features and associated
 * data, such as proxy generation and runtime node class registration.  For forward support of the MetaSound document
 * model and supported accessors, see 'IMetaSoundDocumentInterface'.
 */
class FMetasoundAssetBase : public IAudioProxyDataFactory
{
public:
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	static UE_API const FString FileExtension;

	FMetasoundAssetBase() = default;
	virtual ~FMetasoundAssetBase() = default;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const = 0;

	// Returns the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @return Editor graph associated with this metasound uobject.
	virtual UEdGraph* GetGraph() const = 0;
	virtual UEdGraph& GetGraphChecked() const = 0;
	virtual void MigrateEditorGraph(FMetaSoundFrontendDocumentBuilder& OutBuilder) = 0;

	// Sets the graph associated with this Metasound. Graph is required to be referenced on
	// Metasound UObject for editor serialization purposes.
	// @param Editor graph associated with this metasound object.
	virtual void SetGraph(UEdGraph* InGraph) = 0;

	UE_DEPRECATED(5.6, "AssetClass tags now directly serialized using UObject GetAssetRegistryTags call")
	virtual void SetRegistryAssetClassInfo(const Metasound::Frontend::FNodeClassInfo& InClassInfo) { }
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.7, "Use method which requires a FMetaSoundAssetRegistrationOptions")
	UE_API void UpdateAndRegisterForExecution();

	// Registers the root graph of the given asset with the MetaSound Frontend. Unlike 'UpdateAndRegisterForSerialization", this call
	// generates all necessary runtime data to execute the given graph (i.e. INodes).
	UE_API virtual void UpdateAndRegisterForExecution(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions& InRegistrationOptions);

	// Unregisters the root graph of the given asset with the MetaSound Frontend.
	UE_API void UnregisterGraphWithFrontend();

#if WITH_EDITORONLY_DATA
	// Updates and registers this and referenced MetaSound document objects with the NodeClass Registry. AutoUpdates and
	// optimizes aforementioned documents for serialization. Unlike 'UpdateAndRegisterForRuntime', does not generate required
	// runtime data for graph execution. 
	UE_DEPRECATED(5.7, "Use method which passes in FMetaSoundAssetCookOptions")
	UE_API void UpdateAndRegisterForSerialization(FName CookPlatformName = { });

	// Updates and registers this and referenced MetaSound document objects with the NodeClass Registry. AutoUpdates and
	// optimizes aforementioned documents for serialization. Unlike 'UpdateAndRegisterForRuntime', does not generate required
	// runtime data for graph execution. 
	UE_API void UpdateAndRegisterForSerialization(const Metasound::Frontend::FMetaSoundAssetCookOptions& InCookOptions);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Rebuild dependent asset classes
	UE_API void RebuildReferencedAssetClasses();
#endif // WITH_EDITOR

	// Returns whether an interface with the given version is declared by the given asset's document.
	UE_API bool IsInterfaceDeclared(const FMetasoundFrontendVersion& InVersion) const;

	// Returns all the class keys of this asset's referenced assets
	virtual const TSet<FString>& GetReferencedAssetClassKeys() const = 0;

	// Returns set of class references set call to serialize in the editor
	// Used at runtime load register referenced classes.
	virtual TArray<FMetasoundAssetBase*> GetReferencedAssets() const = 0;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Dynamic loading of referenced assets is deprecated")
	virtual const TSet<FSoftObjectPath>& GetAsyncReferencedAssetClassPaths() const
	{
		static TSet<FSoftObjectPath> Empty;
		return Empty;
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "This is deprecated behavior and will become UE_INTERNAL API in a future release for back compat.")
	virtual void OnAsyncReferencedAssetsLoaded(const TArray<FMetasoundAssetBase*>& InAsyncReferences) { }
#endif // WITH_EDITORONLY_DATA

	UE_API bool AddingReferenceCausesLoop(const FMetasoundAssetBase& InMetaSound) const;

	UE_API bool IsReferencedAsset(const FMetasoundAssetBase& InAssetToCheck) const;

	UE_API bool IsRegistered() const;

	UE_DEPRECATED(5.8, "Feature no longer supported")
	UE_API bool ImportFromJSON(const FString& InJSON);

	UE_DEPRECATED(5.8, "Feature no longer supported")
	UE_API bool ImportFromJSONAsset(const FString& InAbsolutePath);

	// Soft Deprecated in favor of DocumentBuilder API. Returns handle for the root metasound graph of this asset.
	UE_API Metasound::Frontend::FDocumentHandle GetDocumentHandle(bool bReloadBuilder = true);
	UE_API Metasound::Frontend::FConstDocumentHandle GetDocumentHandle() const;

	// Soft Deprecated in favor of DocumentBuilder API. Returns handle for the root metasound graph of this asset.
	UE_API Metasound::Frontend::FGraphHandle GetRootGraphHandle();
	UE_API Metasound::Frontend::FConstGraphHandle GetRootGraphHandle() const;

	UE_API virtual const FMetasoundFrontendDocument& GetConstDocumentChecked() const;

	// Soft deprecated.  Document layer should not be directly mutated via asset base in anticipation
	// of moving all mutable document calls to the Frontend/Subsystem Document Builder API.
	UE_API FMetasoundFrontendDocument& GetDocumentChecked();

	UE_API const Metasound::Frontend::FGraphRegistryKey& GetGraphRegistryKey() const;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Implementation is now internal and moved to VersioningManager.")
	UE_API bool VersionAsset(FMetaSoundFrontendDocumentBuilder& Builder);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_DEPRECATED(5.7, "Moved to FMetaSoundFrontendDocumentBuilder")
	UE_API void CacheRegistryMetadata();

	UE_API FMetasoundFrontendDocumentModifyContext& GetModifyContext();

	UE_API const FMetasoundFrontendDocumentModifyContext& GetConstModifyContext() const;
#endif // WITH_EDITOR

	// Calls the outermost package and marks it dirty.
	UE_API bool MarkMetasoundDocumentDirty() const;

	struct FSendInfoAndVertexName
	{
		Metasound::FMetaSoundParameterTransmitter::FSendInfo SendInfo;
		Metasound::FVertexName VertexName;
	};

	// Returns the owning asset responsible for transactions applied to MetaSound
	virtual UObject* GetOwningAsset() = 0;

	// Returns the owning asset responsible for transactions applied to MetaSound
	virtual const UObject* GetOwningAsset() const = 0;

	UE_API FString GetOwningAssetName() const;

#if WITH_EDITORONLY_DATA
	UE_API void ClearVersionedOnLoad();
	UE_API bool GetVersionedOnLoad() const;
	UE_API void SetVersionedOnLoad();
#endif // WITH_EDITORONLY_DATA

protected:
	UE_API void OnNotifyBeginDestroy();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use SetReferencedAssets instead")
	virtual void SetReferencedAssetClasses(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetInfo>&& InAssetClasses) { }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void SetReferencedAssets(TSet<Metasound::Frontend::IMetaSoundAssetManager::FAssetRef>&& InAssetRefs) = 0;
#endif

#if WITH_EDITORONLY_DATA
	UE_API FText GetDisplayName(FString&& InTypeName) const;
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.6, "AccessPtrs are actively being deprecated. Writable access outside of the builder API "
		" is particularly problematic as in so accessing, the builder's caches are reset which can cause major "
		"editor performance regressions.")
	virtual Metasound::Frontend::FDocumentAccessPtr GetDocumentAccessPtr(bool bReloadBuilder = true) = 0;

	// Returns an access pointer to the document.
	virtual Metasound::Frontend::FConstDocumentAccessPtr GetDocumentConstAccessPtr() const = 0;

	// Ensures all referenced graph classes are registered (or re-registers depending on options).
	UE_API void RegisterAssetDependencies(const Metasound::Frontend::FMetaSoundAssetRegistrationOptions & InRegistrationOptions);

private:
#if WITH_EDITORONLY_DATA
	UE_API void UpdateAndRegisterReferencesForSerialization(const Metasound::Frontend::FMetaSoundAssetCookOptions& InCookOptions);

	bool bVersionedOnLoad = false;
#endif // WITH_EDITORONLY_DATA

	Metasound::Frontend::FGraphRegistryKey GraphRegistryKey;
	
	FMetasoundAssetProxyPtr Proxy;
};

class FMetasoundAssetProxy final : public Audio::TProxyData<FMetasoundAssetProxy>
{
public:
	IMPL_AUDIOPROXY_CLASS(FMetasoundAssetProxy);

	struct FParameters
	{
		TSet<FMetasoundFrontendVersion> Interfaces;
		Metasound::Frontend::FGraphRegistryKey GraphRegistryKey;
	};

	UE_API explicit FMetasoundAssetProxy(const FParameters& InParams);
	
	UE_API FMetasoundAssetProxy(const FMetasoundAssetProxy& Other);

	UE_API const Metasound::Frontend::FGraphRegistryKey& GetGraphRegistryKey() const;

	UE_API const Metasound::IGraph* GetGraph() const;
	
	UE_API const TSet<FMetasoundFrontendVersion>& GetInterfaces() const;

private:
	
	TSet<FMetasoundFrontendVersion> Interfaces;
	Metasound::Frontend::FGraphRegistryKey GraphRegistryKey;
};

#undef UE_API
