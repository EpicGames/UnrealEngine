// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundVertex.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MetasoundAssetKey.generated.h"

#define UE_API METASOUNDFRONTEND_API


USTRUCT()
struct FMetaSoundAssetKey
{
	GENERATED_BODY()

	UPROPERTY()
	FMetasoundFrontendClassName ClassName;

	UPROPERTY()
	FMetasoundFrontendVersionNumber Version;

	FMetaSoundAssetKey() = default;
	UE_API FMetaSoundAssetKey(const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InVersion);
	UE_API FMetaSoundAssetKey(const Metasound::Frontend::FNodeClassRegistryKey& RegKey);
	UE_API FMetaSoundAssetKey(const FMetasoundFrontendClassMetadata& InMetadata);

	inline friend bool operator==(const FMetaSoundAssetKey& InLHS, const FMetaSoundAssetKey& InRHS)
	{
		return (InLHS.ClassName == InRHS.ClassName) && (InLHS.Version == InRHS.Version);
	}

	inline friend bool operator<(const FMetaSoundAssetKey& InLHS, const FMetaSoundAssetKey& InRHS)
	{
		if (InLHS.ClassName == InRHS.ClassName)
		{
			return InLHS.Version < InRHS.Version;
		}

		return InLHS.ClassName < InRHS.ClassName;
	}

	friend inline uint32 GetTypeHash(const FMetaSoundAssetKey& InKey)
	{
		return HashCombineFast(GetTypeHash(InKey.ClassName), GetTypeHash(InKey.Version));
	}

	// Returns invalid asset key.
	static UE_API const FMetaSoundAssetKey& GetInvalid();

	// Returns whether or not key is valid.
	UE_API bool IsValid() const;

	// Returns whether or not ClassType is supported by asset/asset key.
	static UE_API bool IsValidType(EMetasoundFrontendClassType ClassType);

	// Returns string representation of asset key.
	UE_API FString ToString() const;
};

namespace Metasound::Frontend
{
	using FAssetKey UE_DEPRECATED(5.6, "Moved to global namespace as 'FMetaSoundAssetKey' and made a USTRUCT") = FMetaSoundAssetKey;
} // namespace Metasound::Frontend

#undef UE_API
