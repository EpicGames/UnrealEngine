// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundAssetKey.h"
#include "MetasoundAssetManagerTransaction.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilderRegistry.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundVertex.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MetasoundAssetManager.generated.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
class FMetasoundAssetBase;
class UEdGraph;

class IMetaSoundDocumentInterface;

struct FAssetData;
struct FMetasoundFrontendClassName;
struct FMetaSoundFrontendDocumentBuilder;


USTRUCT(BlueprintType, DisplayName = "MetaSound Document Info")
struct FMetaSoundDocumentInfo
{
	GENERATED_BODY()

	UE_API FMetaSoundDocumentInfo();
	UE_API FMetaSoundDocumentInfo(const IMetaSoundDocumentInterface& InDocInterface);
	UE_API FMetaSoundDocumentInfo(const FAssetData& InAssetData, bool& bOutIsValid);

	UE_API void ExportToContext(FAssetRegistryTagsContext& OutContext) const;

	// Version of document
	UPROPERTY(BlueprintReadOnly, Category = DocumentInfo)
	FMetasoundFrontendVersionNumber DocumentVersion;

	// Versions of referenced asset class keys
	UPROPERTY()
	TArray<FMetaSoundAssetKey> ReferencedAssetKeys;

	// Whether asset is a preset or not.
	UPROPERTY(BlueprintReadOnly, Category = DocumentInfo)
	uint8 bIsPreset : 1;
};

namespace Metasound::Frontend
{
	struct FMetaSoundAssetRegistrationOptions
	{
		// If true, forces a re-register of this class (and all class dependencies
		// if the following option 'bRegisterDependencies' is enabled).
		bool bForceReregister = true;

#if WITH_EDITOR
		// If true, forces flag to resync all view (editor) data pertaining to the given asset(s) being registered.
		bool bForceViewSynchronization = true;
#endif // WITH_EDITOR

		// If true, recursively attempts to register dependencies. (TODO: Determine if this option should be removed.
		// Must validate that failed dependency updates due to auto-update for ex. being disabled is handled gracefully
		// at runtime.)
		bool bRegisterDependencies = true;

		// Attempt to auto-update (Only runs if class not registered or set to force re-register.
		// Will not respect being set to true if project-level MetaSoundSettings specify to not run auto-update.)
		bool bAutoUpdate = true;

		// If true, warnings will be logged if updating a node results in existing connections being discarded.
		bool bAutoUpdateLogWarningOnDroppedConnection = false;

#if WITH_EDITOR
		// Soft deprecated. Preprocessing now handled contextually if cooking or serializing.
		bool bPreprocessDocument = true;

		// Attempt to rebuild referenced classes (only run if class not registered or set to force re-register)
		bool bRebuildReferencedAssetClasses = true;

		// No longer used. Memory management of document (i.e. copying or using object's version) inferred internally
		bool bRegisterCopyIfAsync = false;

		// If currently previewing the given MetaSound, registration request is ignored.
		bool bIgnoreIfLiveAuditioning = false;
#endif // WITH_EDITOR

		// Contains a ranking of the preferred page to use with the most preferred page ID being first. 
		TArrayView<const FGuid> PageOrder;
	};

	struct FMetaSoundAssetCookOptions
	{
		// If true, any unused page will get removed during cook.
		bool bStripUnusedPages = false; 

		// Which PageIDs are going to available for targeting after cook. 
		TArray<FGuid> PagesToTarget; 

		// The PageOrder to use when cooking.
		TArray<FGuid> PageOrder;
	};


	/** At runtime, contains a minimal set of information needed to further query additional
	  * class data from the Node Class Registries. At edit-time, contains this plus additional
	  * data useful for informing user of applicable asset classes within given edit contexts
	  * (see 'FMetaSoundEditorGraphSchema').
	  */
	struct FMetaSoundAssetClassInfo : public FMetaSoundClassInfo
	{
		FMetaSoundAssetClassInfo() = default;
		UE_API FMetaSoundAssetClassInfo(const IMetaSoundDocumentInterface& InDocInterface);

		// Attempts to transform AssetTag data from the given AssetData to this class info (asset may or
		// may not be loaded). bIsValid set to false if object isn't loaded and any fields fail to load/are
		// not serialized (i.e. tags are out-of-date).
		UE_API FMetaSoundAssetClassInfo(const FAssetData& InAssetData);

		// Exports tag data to the given RegistryContext
		UE_API virtual void ExportToContext(FAssetRegistryTagsContext& OutContext) const override;

		// If asset is loaded, retrieves asset key from loaded data. Otherwise, parses just the
		// tag data necessary to get the given asset's asset key. Does not attempt to load asset,
		// and will fail if data is not found and asset isn't loaded.
		static UE_API bool TryGetAssetKey(const FAssetData& InAssetData, FMetaSoundAssetKey& OutKey);

		// If asset is loaded, retrieves asset class name from loaded data. Otherwise, parses just the
		// tag data necessary to get the given asset's class name. Does not attempt to load asset,
		// and will fail if data is not found and asset isn't loaded.
		static UE_API bool TryGetAssetClassName(const FAssetData& InAssetData, FMetasoundFrontendClassName& OutClassName);

	protected:
		UE_API virtual void InitFromDocument(const IMetaSoundDocumentInterface& InDocInterface) override;

	private:
		static UE_API bool TryGetAssetClassTag(const FAssetData& InAssetData, FString& OutClassIDString);

	public:
		// Path to asset containing graph if external type and references asset class.
		FTopLevelAssetPath AssetPath;

#if WITH_EDITORONLY_DATA
		FMetaSoundDocumentInfo DocInfo;
#endif // WITH_EDITORONLY_DATA
	};


	class IMetaSoundAssetManager
	{
	public:
		virtual ~IMetaSoundAssetManager() = default;

		static UE_API IMetaSoundAssetManager* Get();
		static UE_API IMetaSoundAssetManager& GetChecked();
		static UE_API void Deinitialize();
		static UE_API void Initialize(TUniquePtr<IMetaSoundAssetManager>&& InInterface);

		// Passed template function pointer to execute in certain contexts when a MetaSound object is either already
		// loaded or has completed asynchronous load.  Due to some implementation restrictions when constructing
		// FLoadPackageAsyncDelegate, this has to be a copyable function (vs. TUniqueFunction).
		using FOnUpdatedAssetLoaded = TFunction<void(FMetaSoundAssetKey, UObject&)>;

		struct FAssetRef
		{
			FMetaSoundAssetKey Key;
			FTopLevelAssetPath Path;

			inline friend bool operator==(const FAssetRef& InLHS, const FAssetRef& InRHS)
			{
				return (InLHS.Key == InRHS.Key) && (InLHS.Path == InRHS.Path);
			}

			inline friend uint32 GetTypeHash(const FAssetRef& InInfo)
			{
				return HashCombineFast(GetTypeHash(InInfo.Key), GetTypeHash(InInfo.Path));
			}
		};

		struct UE_DEPRECATED(5.6, "Use FAssetRef instead") FAssetInfo
		{
			FNodeRegistryKey RegistryKey;
			FSoftObjectPath AssetPath;

			inline friend bool operator==(const FAssetInfo& InLHS, const FAssetInfo& InRHS)
			{
				return (InLHS.RegistryKey == InRHS.RegistryKey) && (InLHS.AssetPath == InRHS.AssetPath);
			}

			inline friend uint32 GetTypeHash(const FAssetInfo& InInfo)
			{
				return HashCombineFast(GetTypeHash(InInfo.RegistryKey), GetTypeHash(InInfo.AssetPath));
			}
		};

#if WITH_EDITORONLY_DATA
		struct FVersionAssetResults
		{
			TArray<FTopLevelAssetPath> FailedPackages;
			TArray<UPackage*> PackagesToReserialize;
			TArray<FTopLevelAssetPath> PackagesUpToDate;

			// Returns whether or not documents were found and loaded/attempted to version
			bool DocumentsFoundInPackages() const
			{
				return !PackagesToReserialize.IsEmpty() || !PackagesUpToDate.IsEmpty() || !FailedPackages.IsEmpty();
			}
		};

		// Adds missing assets using the provided asset's local reference class cache. Used
		// to prime system from asset attempting to register prior to asset scan being complete.
		// Returns true if references were added, false if it they are already found.
		virtual bool AddAssetReferences(FMetasoundAssetBase& InAssetBase) = 0;
#endif // WITH_EDITORONLY_DATA

		UE_DEPRECATED(5.6, "Moved to AddOrUpdateFromObject")
		virtual FMetaSoundAssetKey AddOrUpdateAsset(const UObject& InObject) { return { }; }

		UE_DEPRECATED(5.6, "Moved to AddOrLoadAndUpdateFromObjectAsync")
		virtual FMetaSoundAssetKey AddOrUpdateAsset(const FAssetData& InAssetData) { return { }; }

		// Add or Update a MetaSound Asset's entry data from a loaded MetaSound asset UObject.
		virtual FMetaSoundAssetKey AddOrUpdateFromObject(const UObject& InObject) = 0;

		// Add or Update a MetaSound Asset's entry data from an object, loading it if it isn't already.  On initial call, requests object load
		// asynchronously and runs provided function on successful completion. If asset is already loaded, runs provided function immediately
		// on entry update (synchronously).
		virtual void AddOrLoadAndUpdateFromObjectAsync(const FAssetData& InAssetData, FOnUpdatedAssetLoaded&& OnUpdatedAssetLoaded) = 0;

		// Add or Update a MetaSound Asset's entry data from AssetData. Potentially loads asset
		// and adds asynchronously if asset or associated tag schema is out-of-date.
		virtual void AddOrUpdateFromAssetData(const FAssetData& InAssetData) = 0;

		UE_DEPRECATED(5.7, "AutoUpdate is required now prior to MetaSounds executing runtime builds")
		virtual bool CanAutoUpdate(const FMetasoundFrontendClassName& InClassName) const { return true; }

		// Whether or not the asset manager has loaded the given asset
		virtual bool ContainsKey(const FMetaSoundAssetKey& InAssetKey) const = 0;

		// Whether or not the asset manager has loaded one or more assets with the given registry key.
		// Returns false if key is not valid asset key (ex. input or output class key, variable, etc.)
		virtual bool ContainsKey(const Metasound::Frontend::FNodeRegistryKey& InRegistryKey) const = 0;

		// Returns object (if loaded) associated with the given key (null if key not registered with the AssetManager)
		// If multiple assets are associated with the given key, the last one is returned. 
		virtual FMetasoundAssetBase* FindAsset(const FMetaSoundAssetKey& InAssetKey) const = 0;

		// Returns all class info associated with the given asset key (may be more than one due to errors in duplication
		// of assets, improper plugin migration, etc. but should usually just be a single entry) registered
		// with the manager. Info may not be returned if the asset has yet to be registered with the MetaSoundAssetManager
		// or may be out-of-date if edited after last registration attempt via AddOrUpdate... calls.
		virtual TArray<Frontend::FMetaSoundClassInfo> FindRegisteredClassInfo(const FMetaSoundAssetKey& InKey) const = 0;

		// Returns object (if loaded) associated with the given key as a Document Interface (null if key not registered with the AssetManager)
		virtual TScriptInterface<IMetaSoundDocumentInterface> FindAssetAsDocumentInterface(const FMetaSoundAssetKey& InKey) const = 0;

		// Returns path associated with the given key (returns invalid asset path if key not registered with the AssetManager or was not loaded from asset)
		// If multiple assets are associated with the given key, the last one is returned.
		virtual FTopLevelAssetPath FindAssetPath(const FMetaSoundAssetKey& InAssetKey) const = 0;
		
		// Returns all paths associated with the given key (returns empty array if key not registered with the AssetManager or was not loaded from asset)
		virtual TArray<FTopLevelAssetPath> FindAssetPaths(const FMetaSoundAssetKey& InAssetKey) const = 0;

		// Converts an object to an AssetBase if its a registered asset
		virtual FMetasoundAssetBase* GetAsAsset(UObject& InObject) const = 0;
		virtual const FMetasoundAssetBase* GetAsAsset(const UObject& InObject) const = 0;

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.6, "Use GetReferencedAssets instead")
		virtual TSet<FAssetInfo> GetReferencedAssetClasses(const FMetasoundAssetBase& InAssetBase) const { return {  }; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Generates all asset info associated with registered assets that are referenced by the provided asset's graph.
		virtual TSet<FAssetRef> GetReferencedAssets(const FMetasoundAssetBase& InAssetBase) const = 0;

		// Get assets this asset is a preset of, recursively for presets of presets.
		// This means finding all references including the last non preset, but not graphs used directly in non-preset composition. 
		// Ex. MetaSound A is a preset of MetaSound B, which is a preset of MetaSound C, which references D by composition -> When called on A, will return B and C.
		// If not a preset, returns false.
		virtual bool GetReferencedPresetHierarchy(FMetasoundAssetBase& InAsset, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const = 0;
#endif // WITH_EDITOR

		// Returns default registration options for MetaSound assets
		UE_INTERNAL virtual Frontend::FMetaSoundAssetRegistrationOptions GetRegistrationOptions() const = 0;

		// Returns whether or not the given ClassMetadata is both valid asset ClassMetadata and is registered.
		virtual bool IsAssetClass(const FMetasoundFrontendClassMetadata& ClassMetadata) const = 0;

		UE_DEPRECATED(5.6, "Moved to Iterate ClassInfo")
		virtual void IterateAssets(TFunctionRef<void(const FMetaSoundAssetKey, const TArray<FTopLevelAssetPath>&)> Iter) const { }

		// Set flag for logging active assets on shutdown. In certain cases (ex. validation), it is expected that assets are active at shutdown
		virtual void SetLogActiveAssetsOnShutdown(bool bLogActiveAssetsOnShutdown) = 0;

		// Attempts to retrieve the AssetID from the given ClassName if the ClassName is from a valid asset.
		virtual bool TryGetAssetIDFromClassName(const FMetasoundFrontendClassName& InClassName, FGuid& OutGuid) const = 0;

		// Attempts to load an FMetasoundAssetBase from the given path, or returns it if its already loaded
		virtual FMetasoundAssetBase* TryLoadAsset(const FSoftObjectPath& InObjectPath) const = 0;

		// Returns asset associated with the given key (null if key not registered with the AssetManager or was not loaded from asset)
		virtual FMetasoundAssetBase* TryLoadAssetFromKey(const FMetaSoundAssetKey& InAssetKey) const = 0;

		// Try to load referenced assets of the given asset or return them if they are already loaded (non-recursive).
		// @return - True if all referenced assets successfully loaded, false if not.
		virtual bool TryLoadReferencedAssets(const FMetasoundAssetBase& InAssetBase, TArray<FMetasoundAssetBase*>& OutReferencedAssets) const = 0;

#if WITH_EDITOR
		// Assigns a new arbitrary class name to the given document, which can cause references to be invalidated.
		// (See
		virtual bool ReassignClassName(TScriptInterface<IMetaSoundDocumentInterface> DocInterface) = 0;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.8, "This is a deprecated feature, and exists to port old soft object references to hard references.")
		virtual void RequestAsyncLoadReferencedAssets(FMetasoundAssetBase& InAssetBase, const TSet<FSoftObjectPath>* Paths = nullptr) { }
#endif // WITH_EDITORONLY_DATA

		UE_DEPRECATED(5.8, "Use 'ReregisterLoadedAssets' instead.")
		virtual void ReloadMetaSoundAssets() const { ReregisterLoadedAssets(); }

		// Removes object from MetaSound asset manager
		virtual void RemoveAsset(const UObject& InObject) = 0;

		// Removes object from MetaSound asset manager
		virtual void RemoveAsset(const FAssetData& InAssetData) = 0;

		// Updates the given MetaSound's asset record with the new name and optionally reregisters it with the Frontend Node Class Registry.
		virtual void RenameAsset(const FAssetData& InAssetData, const FString& InOldObjectPath) = 0;

		// Finds all currently loaded assets and re-registers them with the MetasoundUObjectRegistry. Ensures all paged data is targeted
		// appropriately and can be expensive.
		virtual void ReregisterLoadedAssets() const = 0;

#if WITH_EDITORONLY_DATA
		// Sets all access flags to the provided flags for the given assets. Optionally, recurses the given paths.
		// Populates the provided array with MetaSound packages whose document versions were out-of-date and consequently updated.
		// Returns resulting assets that were updated and those not.
		virtual FVersionAssetResults SetAccessFlagsOnAssetsInFolders(const TArray<FString>& FolderPaths, EMetasoundFrontendClassAccessFlags Flags, bool bRecursePaths) const = 0;

		// Versions all MetaSound asset tags & documents found within the given folder paths. Optionally, recurses the given paths.
		// Populates the provided array with MetaSound packages whose document versions were out-of-date and consequently updated.
		// Returns resulting versioned and unversioned paths.
		virtual FVersionAssetResults VersionAssetsInFolders(const TArray<FString>& FolderPaths, bool bRecursePaths = true) const = 0;

		// Waits until all async load requests related to this asset are complete.
		virtual void WaitUntilAsyncLoadReferencedAssetsComplete(const FMetasoundAssetBase& InAssetBase) = 0;
#endif // WITH_EDITORONLY_DATA
	};

	class FMetaSoundAssetManagerBase : public IMetaSoundAssetManager
	{
	public:
		UE_API FMetaSoundAssetManagerBase();
		virtual ~FMetaSoundAssetManagerBase() = default;

		UE_API IMetaSoundAssetTransactor& GetTransactor();

	private:
		TUniquePtr<IMetaSoundAssetTransactor> Transactor;
	};

} // namespace Metasound::Frontend

#undef UE_API
