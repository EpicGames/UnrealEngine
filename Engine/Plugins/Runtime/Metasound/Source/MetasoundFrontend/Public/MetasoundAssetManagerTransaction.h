// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "MetasoundAssetKey.h"
#include "MetasoundFrontendRegistryKey.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound::Frontend
{
	class FAssetManagerTransaction
	{
	public:
		using FTimeType = uint64;

		/** Describes the type of transaction. */
		enum class ETransactionType : uint8
		{
			AssetRegistration,     //< Something was added to the registry.
			AssetUnregistration,  //< Something was removed from the registry.
			Invalid
		};

		static FString LexToString(ETransactionType InTransactionType);

		FAssetManagerTransaction(ETransactionType InType, const FMetaSoundAssetKey& InKey, FTimeType InTimestamp);

		ETransactionType GetTransactionType() const;
		FNodeClassRegistryKey GetNodeRegistryKey() const;
		FTimeType GetTimestamp() const;

	private:
		ETransactionType Type;
		FNodeClassRegistryKey Key;
		FTimeType Timestamp;
	};

	class IMetaSoundAssetTransactor
	{
	public:
		virtual ~IMetaSoundAssetTransactor() = default;

		virtual void RegisterAsset(FMetaSoundAssetKey Key) = 0;
		virtual void UnregisterAsset(FMetaSoundAssetKey Key) = 0;
	};
} // namespace Metasound::Frontend
#undef UE_API