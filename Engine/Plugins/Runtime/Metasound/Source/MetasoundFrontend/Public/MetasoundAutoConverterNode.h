// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MetasoundBasicNode.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundBuildError.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundVertex.h"

#include <type_traits>

namespace Metasound
{
	namespace AutoConverterNodePrivate
	{
		struct FConvertDataTypeInfo
		{
			FName FromDataTypeName;
			FText FromDataTypeText;
			FName ToDataTypeName;
			FText ToDataTypeText;
			bool bIsFromEnum;
			bool bIsToEnum;
		};

		METASOUNDFRONTEND_API FVertexName GetInputName(const FConvertDataTypeInfo& InInfo);

		METASOUNDFRONTEND_API FVertexName GetOutputName(const FConvertDataTypeInfo& InInfo);

		METASOUNDFRONTEND_API FVertexInterface CreateVertexInterface(const FConvertDataTypeInfo& InInfo);

		METASOUNDFRONTEND_API FNodeClassName CreateAutoConverterNodeClassName(const FName& InFromDataType, const FName& InToDataType);
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateAutoConverterNodeClassRegistryKey(const FName& InFromDataType, const FName& InToDataType);
		METASOUNDFRONTEND_API FNodeClassMetadata CreateAutoConverterNodeMetadata(const FConvertDataTypeInfo& InInfo);

		template<typename FromDataType, typename ToDataType>
		FConvertDataTypeInfo MakeConvertDataTypeInfo()
		{
			return FConvertDataTypeInfo
			{
				GetMetasoundDataTypeName<FromDataType>(),
				GetMetasoundDataTypeDisplayText<FromDataType>(),
				GetMetasoundDataTypeName<ToDataType>(),
				GetMetasoundDataTypeDisplayText<ToDataType>(),
				TEnumTraits<FromDataType>::bIsEnum,
				TEnumTraits<ToDataType>::bIsEnum
			};
		}
	}

	// Determines whether an auto converter node will be registered to convert 
	// between two types. 
	template<typename TFromDataType, typename TToDataType>
	struct TIsAutoConvertible
	{
		static constexpr bool bIsConvertible = std::is_convertible<TFromDataType, TToDataType>::value;

		// Handle case of converting enums to/from integers.
		static constexpr bool bIsIntToEnumConversion = std::is_same<int32, TFromDataType>::value && TEnumTraits<TToDataType>::bIsEnum;
		static constexpr bool bIsEnumToIntConversion = TEnumTraits<TFromDataType>::bIsEnum && std::is_same<int32, TToDataType>::value;

		static constexpr bool Value = bIsConvertible || bIsIntToEnumConversion || bIsEnumToIntConversion;
	};

	// This convenience node can be registered and will invoke static_cast<ToDataType>(FromDataType) every time it is executed, 
	// with a special case for enum <-> int32 conversions. 
	template<typename FromDataType, typename ToDataType>
	class TAutoConverterNode : public FBasicNode
	{
		static_assert(TIsAutoConvertible<FromDataType, ToDataType>::Value,
		"Tried to create an auto converter node between two types we can't static_cast between.");
		
	public:
		static const FVertexName& GetInputName()
		{
			static const FVertexName InputName = GetMetasoundDataTypeName<FromDataType>();
			return InputName;
		}

		static const FVertexName& GetOutputName()
		{
			static const FVertexName OutputName = GetMetasoundDataTypeName<ToDataType>();
			return OutputName;
		}

		static FVertexInterface DeclareVertexInterface()
		{
			using namespace AutoConverterNodePrivate;
			return CreateVertexInterface(MakeConvertDataTypeInfo<FromDataType, ToDataType>());
		}

		static const FNodeClassMetadata& GetAutoConverterNodeMetadata()
		{
			using namespace AutoConverterNodePrivate;
			static const FNodeClassMetadata Info = CreateAutoConverterNodeMetadata(MakeConvertDataTypeInfo<FromDataType, ToDataType>());

			return Info;
		}

	private:
		/** FConverterOperator converts from "FromDataType" to "ToDataType" using
		 * a implicit conversion operators.
		 */
		class FConverterOperator : public TExecutableOperator<FConverterOperator>
		{
		public:

			FConverterOperator(TDataReadReference<FromDataType> InFromDataReference, TDataWriteReference<ToDataType> InToDataReference)
				: FromData(InFromDataReference)
				, ToData(InToDataReference)
			{
				Execute();
			}

			virtual ~FConverterOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				InVertexData.BindReadVertex(GetInputName(), FromData);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
			{
				InVertexData.BindReadVertex(GetOutputName(), ToData);
			}

			void Execute()
			{
				// enum -> int32
				if constexpr (TIsAutoConvertible<FromDataType, ToDataType>::bIsEnumToIntConversion)
				{
					// Convert from enum wrapper to inner enum type, then to int
					typename TEnumTraits<FromDataType>::InnerType InnerEnum = static_cast<typename TEnumTraits<FromDataType>::InnerType>(*FromData);
					*ToData = static_cast<ToDataType>(InnerEnum);
				}
				// int32 -> enum
				else if constexpr (TIsAutoConvertible<FromDataType, ToDataType>::bIsIntToEnumConversion)
				{
					const int32 FromInt = *FromData;
					// Convert from int to inner enum type
					typename TEnumTraits<ToDataType>::InnerType InnerEnum = static_cast<typename TEnumTraits<ToDataType>::InnerType>(FromInt);

					// Update tracking for previous int value we tried to convert, used to prevent log spam if it's an invalid enum value
					if (FromInt != PreviousIntValueForEnumConversion)
					{
						PreviousIntValueForEnumConversion = FromInt;
						bHasLoggedInvalidEnum = false;
					}

					// If int value is invalid for this enum, return enum default value
					TOptional<FName> EnumName = ToDataType::ToName(InnerEnum);
					if (!EnumName.IsSet())
					{
						if (!bHasLoggedInvalidEnum)
						{
							UE_LOGF(LogMetaSound, Warning, "Cannot convert int32 value '%d' to enum type '%ls'. No valid corresponding enum value exists, so returning enum default value instead.", FromInt, *GetMetasoundDataTypeDisplayText<ToDataType>().ToString());
							bHasLoggedInvalidEnum = true;
						}
						*ToData = static_cast<ToDataType>(TEnumTraits<ToDataType>::DefaultValue);
					}
					else
					{
						// Convert from inner enum type to int
						*ToData = static_cast<ToDataType>(InnerEnum);
					}
				}
				else
				{
					*ToData = static_cast<ToDataType>(*FromData);
				}
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				PreviousIntValueForEnumConversion = 0;
				bHasLoggedInvalidEnum = false;
				Execute();
			}

			private:
				TDataReadReference<FromDataType> FromData;
				TDataWriteReference<ToDataType> ToData;

				// To prevent log spam, keep track of whether we've logged an invalid enum value being converted already
				// and the previous int value (need both bool and int for the initial case)
				bool bHasLoggedInvalidEnum = false;
				int32 PreviousIntValueForEnumConversion = 0;
		};

		/** FConverterOperatorFactory creates an operator which converts from 
		 * "FromDataType" to "ToDataType". 
		 */
		class FCoverterOperatorFactory : public IOperatorFactory
		{
			public:
				FCoverterOperatorFactory() = default;

				virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
				{
					TDataWriteReference<ToDataType> WriteReference = TDataWriteReferenceFactory<ToDataType>::CreateExplicitArgs(InParams.OperatorSettings);
					TDataReadReference<FromDataType> ReadReference = InParams.InputData.GetOrCreateDefaultDataReadReference<FromDataType>(GetInputName(), InParams.OperatorSettings);

					return MakeUnique<FConverterOperator>(MoveTemp(ReadReference), MoveTemp(WriteReference));
				}
		};

		public:

			TAutoConverterNode(const FNodeInitData& InInitData)
				: TAutoConverterNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, DeclareVertexInterface()), MakeShared<const FNodeClassMetadata>(GetAutoConverterNodeMetadata()))
			{
			}

			TAutoConverterNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
				: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
				, Factory(MakeOperatorFactoryRef<FCoverterOperatorFactory>())
			{
			}

			virtual ~TAutoConverterNode() = default;

			virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override
			{
				return Factory;
			}

		private:
			FOperatorFactorySharedRef Factory;
	};
} // namespace Metasound
