// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MetasoundChannelAgnosticType.h"

#include "MetasoundChannelAgnosticAutoTypes.generated.h"

#define UE_API METASOUNDFRONTEND_API

UENUM()
enum class EMetasoundChannelAgnosticSourceFormatChooser : uint8
{
	Auto,
	Custom
};

UENUM()
enum class EMetasoundChannelAgnosticNodeFormatChooser : uint8
{
	Auto,
	Source,
	Custom
};

#undef UE_API 