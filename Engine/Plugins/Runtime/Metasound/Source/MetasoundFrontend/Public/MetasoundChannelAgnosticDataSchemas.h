// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#include "MetasoundChannelAgnosticDataSchemas.generated.h"

namespace Metasound
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")
	METASOUNDFRONTEND_API TArray<FName> LoadChannelAgnosticDataSchemas(const FString& InContentPath);
	
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")		
	METASOUNDFRONTEND_API void UnloadChannelAgnosticDataSchemas(const TArray<FName>&);

	// Put these here for now.
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")
	METASOUNDFRONTEND_API FName AddNamespaceToName(const FName InDatatype, const FName InNamespace);

	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
	METASOUNDFRONTEND_API FName AddCatNamespaceToName(const FName InDatatype);
}

USTRUCT()
struct FCatDataTypeSchema
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")
	GENERATED_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName BaseType;

	UPROPERTY()
	FName FamilyType;

	UPROPERTY()
	FString FriendlyName;

	UPROPERTY()
	bool bIsAbstract = false;

	UPROPERTY()
	bool bIsParentsDefault = false;
};

USTRUCT()
struct FCatSoundfieldDataSchema : public FCatDataTypeSchema
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
	GENERATED_BODY()

	UPROPERTY()
	int32 NumOrders = 1;
};

USTRUCT()
struct FCatDiscreteChannel
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
	GENERATED_BODY()

	UPROPERTY()
	float ElevationDegrees = 0;			// In Degrees. +180 to -180

	UPROPERTY()
	float AzimuthDegrees = 0;			// -180 to 180 azimuth.

	UPROPERTY()
	FName ID;							// ID of the Speaker. This could be unique ID or match the Short form of the Speaker ID. 

	UPROPERTY()
	bool bIsSpatialized = true;			// If this is off, azimuth+degrees do nothing. (for excluding from panner). 
};

UENUM()
UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
enum class ECatDiscreteOrderPolicy : uint32
{
	ChannelEnum,					// Format uses the order of the AudioMixerChannel (which is the same as WavFormatEx) 
	Explicit,						// If explicit, use the Order[] array here to define it. 
};

USTRUCT()
struct FCatDiscreteDataSchema : public FCatDataTypeSchema
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
	GENERATED_BODY()

	UPROPERTY()
	ECatDiscreteOrderPolicy OrderPolicy = ECatDiscreteOrderPolicy::ChannelEnum;
	
	UPROPERTY()
	TArray<FName> Order;
	
	UPROPERTY()
	TArray<FCatDiscreteChannel> Speakers;
};

USTRUCT()
struct FCatCompositeDataSchema : public FCatDataTypeSchema
{
	UE_EXPERIMENTAL(5.8, "Channel Angostic Types are Experimental")	
	GENERATED_BODY()
};