// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundPolymorphic.h"
#include "ChannelAgnostic/ChannelAgnosticType.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	class FChannelAgnosticType : public Audio::FChannelAgnosticType
	{
	public:
		UE_NONCOPYABLE(FChannelAgnosticType)
		using Super = Audio::FChannelAgnosticType;

		static FName GetRegisteredTypeName()
		{
			static const FName Name("Cat");			
			return Name;
		}				
		
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS		
				
		// Metasound Polymorphic Type Id+Name. (required for intrusive references to work).
		const FName& GetDataTypeName() const { return GetType().GetName(); }
		const void* GetDataTypeId() const { return &GetType(); }
		
		// Factory helper.
		UE_EXPERIMENTAL(5.8, "Channel Agnostic Types are Experimental")
		UE_API static TOptional<TDataWriteReference<FChannelAgnosticType>> CreateWriteReference(const FName InDerivedType, const FOperatorSettings& InOpSettings);
	
		UE_API static FName GetCatDefaultCatFormat();
		
		UE_API static TSharedRef<FChannelAgnosticType, ESPMode::NotThreadSafe> CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType=GetCatDefaultCatFormat());
public:
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
		
		UE_API FChannelAgnosticType(const int32 InNumFramesPerBlock, const FName& InChannelTypeName);
		UE_API FChannelAgnosticType(const FOperatorSettings& InSettings, const FName& InChannelTypeName);
	private:
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	};

	class FDiscreteChannelAgnosticType : public FChannelAgnosticType
	{
	public:
		UE_NONCOPYABLE(FDiscreteChannelAgnosticType)
		
		static FName GetRegisteredTypeName()
		{
			static const FName Name("Cat:Discrete");			
			return Name;
		}				
		using Super = FChannelAgnosticType;
			
		UE_EXPERIMENTAL(5.8, "Channel Agnostic Types are Experimental")
		UE_API static TOptional<TDataWriteReference<FDiscreteChannelAgnosticType>> CreateWriteReference(const FName InDerivedType, const FOperatorSettings& InOpSettings);
		
		UE_API static FName GetDefaultCatFormat();
		
		// Factory function for the default case.
		UE_API static TSharedRef<FDiscreteChannelAgnosticType, ESPMode::NotThreadSafe> CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType=GetDefaultCatFormat());
		
	public:
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
		
		FDiscreteChannelAgnosticType(const FOperatorSettings& InSettings, const FName InDerivedType)
			: Super(InSettings, InDerivedType) {}
	};
	class FSoundfieldChannelAgnosticType : public FChannelAgnosticType
	{
	public:
		UE_NONCOPYABLE(FSoundfieldChannelAgnosticType)
		
		static FName GetRegisteredTypeName()
		{
			static const FName Name("Cat:Soundfield");			
			return Name;
		}		
		using Super = FChannelAgnosticType;
		
		UE_API static FName GetDefaultCatFormat();
		UE_API static TSharedRef<FSoundfieldChannelAgnosticType, ESPMode::NotThreadSafe> CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType=GetDefaultCatFormat());
		
	public:
		FSoundfieldChannelAgnosticType(const FOperatorSettings& InSettings, const FName InDerivedType)
			: Super(InSettings, InDerivedType) 
		{}
		
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	};
	class FCompositeChannelAgnosticType : public FChannelAgnosticType
	{
	public:
		UE_NONCOPYABLE(FCompositeChannelAgnosticType)
		
		static FName GetRegisteredTypeName()
		{
			static const FName Name("Cat:Composite");			
			return Name;
		}
		using Super = FChannelAgnosticType;
		
		UE_API static FName GetDefaultCatFormat();
		UE_API static TSharedRef<FCompositeChannelAgnosticType, ESPMode::NotThreadSafe> CreateNew(const FOperatorSettings& InSettings, const FName InDerivedType=GetDefaultCatFormat());
	public:
		FCompositeChannelAgnosticType(const FOperatorSettings& InSettings, const FName InDerivedType)
			: Super(InSettings, InDerivedType) 
		{}	
		
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	};
	
}

namespace Metasound
{
	// Declare Metasound Data reference Types. 
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(Metasound::FChannelAgnosticType,				UE_API, FChannelAgnosticTypeTypeInfo,				FChannelAgnosticTypeReadRef,			FChannelAgnosticTypeWriteRef);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(Metasound::FCompositeChannelAgnosticType,	UE_API, FCompositeChannelAgnosticTypeTypeInfo,	FCompositeChannelAgnosticTypeReadRef,	FCompositeChannelAgnosticTypeWriteRef);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(Metasound::FDiscreteChannelAgnosticType,		UE_API, FDiscreteChannelAgnosticTypeTypeInfo,		FDiscreteChannelAgnosticTypeReadRef,	FDiscreteChannelAgnosticTypeWriteRef);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(Metasound::FSoundfieldChannelAgnosticType,	UE_API, FSoundfieldChannelAgnosticTypeTypeInfo,	FSoundfieldChannelAgnosticTypeReadRef,	FSoundChannelAgnosticTypeWriteRef);

	// Declare Polymorphic DataTypes.
	DECLARE_METASOUND_POLY_TYPE(Metasound::FChannelAgnosticType,			void, true);
	DECLARE_METASOUND_POLY_TYPE(Metasound::FCompositeChannelAgnosticType,	Metasound::FChannelAgnosticType, true);
	DECLARE_METASOUND_POLY_TYPE(Metasound::FDiscreteChannelAgnosticType,	Metasound::FChannelAgnosticType, true);
	DECLARE_METASOUND_POLY_TYPE(Metasound::FSoundfieldChannelAgnosticType,	Metasound::FChannelAgnosticType, true);

	// Disable auto-generated array nodes. CAT types are polymorphic/abstract and
	// should not expose array variants in the MetaSound editor.
	template<> struct TEnableAutoArrayTypeRegistration<FChannelAgnosticType> { static constexpr bool Value = false; };
	template<> struct TEnableAutoArrayTypeRegistration<FCompositeChannelAgnosticType> { static constexpr bool Value = false; };
	template<> struct TEnableAutoArrayTypeRegistration<FDiscreteChannelAgnosticType> { static constexpr bool Value = false; };
	template<> struct TEnableAutoArrayTypeRegistration<FSoundfieldChannelAgnosticType> { static constexpr bool Value = false; };
	// Public helper.
	UE_API bool IsExperimentalPluginEnabled();
}

#undef UE_API