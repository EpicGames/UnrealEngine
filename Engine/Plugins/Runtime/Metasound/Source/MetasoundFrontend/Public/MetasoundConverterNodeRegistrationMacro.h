// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"

#include "MetasoundDataReference.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundInputNode.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOutputNode.h"
#include "MetasoundVertex.h"

template<typename TNodeType, typename TFromType, typename TToType>
UE_DEPRECATED(5.8, "Converter node can be registered directly with the INodeClassRegistry in the owning modules startup/shutdown methods")
bool RegisterConversionNode(const Metasound::FVertexName& FromPin, const Metasound::FVertexName& ToPin, const Metasound::FNodeClassMetadata& InNodeMetadata)
{
	using FConverterNodeRegistryKey = ::Metasound::Frontend::FConverterNodeRegistryKey;
	using FConverterNodeInfo = ::Metasound::Frontend::FConverterNodeInfo;

	// if we reenter this code (because DECLARE_METASOUND_DATA_REFERENCE_TYPES was called twice with the same type),
	// we catch it here.
	static bool bAlreadyRegisteredThisConverter = false;
	if (bAlreadyRegisteredThisConverter)
	{
		UE_LOGF(LogMetaSound, Error, "Tried to call REGISTER_METASOUND_CONVERTER twice with the same class. ignoring the second call. Likely because REGISTER_METASOUND_CONVERTER is in a header that's used in multiple modules. Consider moving it to a private header or cpp file.")
		return false;
	}

	bAlreadyRegisteredThisConverter = true;

	// Get the FNames from our datatypes
	FName FromType = ::Metasound::TDataReferenceTypeInfo<TFromType>::TypeName;
	FName ToType = ::Metasound::TDataReferenceTypeInfo<TToType>::TypeName;

	FConverterNodeRegistryKey RegistryKey = { FromType, ToType };

	FConverterNodeInfo ConverterNodeInfo =
	{
		FromPin,
		ToPin,
		Metasound::Frontend::FNodeRegistryKey(InNodeMetadata)
	};

	return FMetasoundFrontendRegistryContainer::Get()->RegisterConversionNode(RegistryKey, ConverterNodeInfo);
}

// Convenience macro for token pasting the three parameters for the converter node into a unique static bool name:
#define MS_CONVERTER_BOOL(Node, FromDataType, ToDataType) MS_CONVERTER_BOOL_HIDDEN(Node, FromDataType, ToDataType) 
#define MS_CONVERTER_BOOL_HIDDEN(Node, FromDataType, ToDataType) Node ## FromDataType ## ToDataType

#define MS_INVOKE_CONVERTER_REGISTRATION_HIDDEN(NodeType, FromDataType, ToDataType, FromOutputPin, ToInputPin) FMetasoundFrontendRegistryContainer::Get()->EnqueueInitCommand([](){ ::Metasound::RegisterConversionNode<NodeType, FromDataType, ToDataType >(FString(TEXT( #FromOutputPin )), FString(TEXT( #ToInputPin ))); })

#define MS_INVOKE_CONSTRUCTOR_REGISTRATION(NodeType, FromDataType, ToDataType, FromOutputPin, ToInputPin) MS_CONVERTER_BOOL(NodeType, FromDataType, ToDataType) = MS_INVOKE_CONVERTER_REGISTRATION_HIDDEN(NodeType, FromDataType, ToDataType, FromOutputPin, ToInputPin)

// This macro exposes a previously registered node as a valid way to convert from one datatype to another.
// When a user tries to connect a FromDataType output to a ToDataType input pin, this node will pop up as a potential node to insert between the two.
#define REGISTER_METASOUND_CONVERTER(NodeType, FromDataType, ToDataType, FromOutputPin, ToInputPin) UE_DEPRECATED_MACRO(5.8, "REGISTER_METASOUND_CONVERTER will no longer be supported. Use explicit registration during module startup/shutdown.") static const bool MS_INVOKE_CONSTRUCTOR_REGISTRATION(NodeType, FromDataType, ToDataType, FromOutputPin, ToInputPin);
