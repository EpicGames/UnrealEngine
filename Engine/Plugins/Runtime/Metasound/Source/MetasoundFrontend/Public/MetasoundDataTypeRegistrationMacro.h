// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"
#include "HAL/PreprocessorHelpers.h"
#include "MetasoundArrayNodesRegistration.h"
#include "MetasoundAutoConverterNode.h"
#include "MetasoundConverterNodeRegistrationMacro.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundPolymorphic.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundEnum.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDataTypeTraits.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundInputNode.h"
#include "MetasoundLiteral.h"
#include "MetasoundLiteralNode.h"
#include "MetasoundLog.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOutputNode.h"
#include "MetasoundReceiveNode.h"
#include "MetasoundSendNode.h"
#include "MetasoundTransmissionRegistration.h"
#include "MetasoundVariableNodes.h"
#include "MetasoundParameterPackFixedArray.h"

#include "Templates/Casts.h"

#include <type_traits>

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	// Disable registering of arrays of variables. 
	template<typename UnderlyingType>
	struct TEnableAutoArrayTypeRegistration<TVariable<UnderlyingType>>
	{
		static constexpr bool Value = false;
	};
}

namespace Metasound::Frontend
{

	namespace MetasoundDataTypeRegistrationPrivate
	{
		// Returns the Array version of a literal type if it exists.
		template<ELiteralType LiteralType>
		struct TLiteralArrayEnum 
		{
			// Default to TArray default constructor by using
			// ELiteralType::None
			static constexpr ELiteralType Value = ELiteralType::None;
		};

		// Specialization for None->NoneArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::None>
		{
			static constexpr ELiteralType Value = ELiteralType::NoneArray;
		};
		 
		// Specialization for Boolean->BooleanArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::Boolean>
		{
			static constexpr ELiteralType Value = ELiteralType::BooleanArray;
		};
		
		// Specialization for Integer->IntegerArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::Integer>
		{
			static constexpr ELiteralType Value = ELiteralType::IntegerArray;
		};
		
		// Specialization for Float->FloatArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::Float>
		{
			static constexpr ELiteralType Value = ELiteralType::FloatArray;
		};
		
		// Specialization for String->StringArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::String>
		{
			static constexpr ELiteralType Value = ELiteralType::StringArray;
		};
		
		// Specialization for UObjectProxy->UObjectProxyArray
		template<>
		struct TLiteralArrayEnum<ELiteralType::UObjectProxy>
		{
			static constexpr ELiteralType Value = ELiteralType::UObjectProxyArray;
		};

		
		// Register the send and receive nodes of a datatype if it is supported
		template<typename TDataType>
		void RegisterSendAndReceiveNodes(const FModuleInfo& InModuleInfo)
		{
			if constexpr (TIsTransmittable<TDataType>::Value && TEnableTransmissionNodeRegistration<TDataType>::Value)
			{
				RegisterNode<Metasound::TSendNode<TDataType>>(InModuleInfo);
				RegisterNode<Metasound::TReceiveNode<TDataType>>(InModuleInfo);
			}
		}

	
		// Unregister the send and receive nodes related to a datatype.
		UE_INTERNAL
		UE_API void UnregisterSendAndReceiveNodes(const FName& InDataTypeName, const FModuleInfo& InModuleInfo);

		// Register an autoconverter node for a specific data type conversion. Only registers the node if it is supported and not already registered. 
		template<typename TFromDataType, typename TToDataType>
		void RegisterAutoConverterNode(INodeClassRegistry& InRegistry, const FModuleInfo& InModuleInfo)
		{
			if constexpr (TIsAutoConvertible<TFromDataType, TToDataType>::Value && TEnableAutoConverterNodeRegistration<TFromDataType, TToDataType>::Value && !std::is_same<TFromDataType, TToDataType>::value)
			{
				using namespace AutoConverterNodePrivate;

				const FName& FromDataTypeName = GetMetasoundDataTypeName<TFromDataType>();
				const FName& ToDataTypeName = GetMetasoundDataTypeName<TToDataType>();

				FConvertDataTypeInfo ConvertInfo = MakeConvertDataTypeInfo<TFromDataType, TToDataType>();
				const FNodeClassMetadata Metadata = CreateAutoConverterNodeMetadata(ConvertInfo);
				const Metasound::Frontend::FNodeClassRegistryKey Key{Metadata};

				if (!InRegistry.IsNodeRegistered(Key))
				{
					using FConverterNode = Metasound::TAutoConverterNode<TFromDataType, TToDataType>;
					RegisterNode<FConverterNode>(Metadata, InModuleInfo);

					FConverterNodeInfo ConverterNodeInfo
					{
						FConverterNode::GetInputName(), 
						FConverterNode::GetOutputName(), 
						FNodeClassRegistryKey{Metadata}
					};
					InRegistry.RegisterConversionNode(FConverterNodeClassRegistryKey{FromDataTypeName, ToDataTypeName}, ConverterNodeInfo);
				}
			}
		}

		// Unregister any autoconverter node that is registered for going converting between specific data types
		UE_INTERNAL
		UE_API void UnregisterAutoConverterNode(INodeClassRegistry& InRegistry, const FName& InFromDataTypeName, const FName& InToDataTypeName, const FModuleInfo& InModuleInfo);

		// Register a slew of basic converter nodes for basic types
		template<typename TDataType>
		void RegisterAutoConverterNodes(const FModuleInfo& InModuleInfo)
		{
			INodeClassRegistry& Registry = INodeClassRegistry::GetChecked();
			// Conversions to this data type:
			RegisterAutoConverterNode<bool, TDataType>(Registry, InModuleInfo);
			RegisterAutoConverterNode<int32, TDataType>(Registry, InModuleInfo);
			RegisterAutoConverterNode<float, TDataType>(Registry, InModuleInfo);
			RegisterAutoConverterNode<FString, TDataType>(Registry, InModuleInfo);

			// Conversions from this data type:
			RegisterAutoConverterNode<TDataType, bool>(Registry, InModuleInfo);
			RegisterAutoConverterNode<TDataType, int32>(Registry, InModuleInfo);
			RegisterAutoConverterNode<TDataType, float>(Registry, InModuleInfo);
			RegisterAutoConverterNode<TDataType, FString>(Registry, InModuleInfo);
		}

		// Unregister a slew of basic converter nodes for basic types.
		UE_INTERNAL
		UE_API void UnregisterAutoConverterNodes(const FName& InDataTypeName, const FModuleInfo& InModuleInfo);

		/** Creates the FDataTypeRegistryInfo for a data type.
		 * 
		 * @tparam TDataType - The data type to create info for.
		 * @tparam PreferredArgType - The preferred constructor argument type to use when creating an instance of the data type.
		 * @tparam UClassToUse - The preferred UObject class to use when constructing from an Audio::IProxyData.
		 */
		template<typename TDataType, ELiteralType PreferredArgType, typename UClassToUse>
		Frontend::FDataTypeRegistryInfo CreateDataTypeInfo()
		{
			Frontend::FDataTypeRegistryInfo RegistryInfo;

			RegistryInfo.DataTypeName = GetMetasoundDataTypeName<TDataType>();
			RegistryInfo.DataTypeDisplayText = GetMetasoundDataTypeDisplayText<TDataType>();
			RegistryInfo.PreferredLiteralType = PreferredArgType;

			RegistryInfo.bIsParsable = TLiteralTraits<TDataType>::bIsParsableFromAnyLiteralType;
			RegistryInfo.bIsArrayParseable = TLiteralTraits<TDataType>::bIsParseableFromAnyArrayLiteralType;

			RegistryInfo.bIsArrayType = TIsArrayType<TDataType>::Value;

			RegistryInfo.bIsDefaultParsable = TIsParsable<TDataType, FLiteral::FNone>::Value;
			RegistryInfo.bIsBoolParsable = TIsParsable<TDataType, bool>::Value;
			RegistryInfo.bIsIntParsable = TIsParsable<TDataType, int32>::Value;
			RegistryInfo.bIsFloatParsable = TIsParsable<TDataType, float>::Value;
			RegistryInfo.bIsStringParsable = TIsParsable<TDataType, FString>::Value;
			RegistryInfo.bIsProxyParsable = TIsParsable<TDataType, const TSharedPtr<Audio::IProxyData>&>::Value;

			RegistryInfo.bIsUniquePtrProxyParsable_DEPRECATED = TIsParsable<TDataType, const TUniquePtr<Audio::IProxyData>&>::Value;

			RegistryInfo.bIsDefaultArrayParsable = TIsParsable<TDataType, TArray<FLiteral::FNone>>::Value;
			RegistryInfo.bIsBoolArrayParsable = TIsParsable<TDataType, TArray<bool>>::Value;
			RegistryInfo.bIsIntArrayParsable = TIsParsable<TDataType, TArray<int32>>::Value;
			RegistryInfo.bIsFloatArrayParsable = TIsParsable<TDataType, TArray<float>>::Value;
			RegistryInfo.bIsStringArrayParsable = TIsParsable<TDataType, TArray<FString>>::Value;
			RegistryInfo.bIsProxyArrayParsable = TIsParsable<TDataType, const TArray<TSharedPtr<Audio::IProxyData>>& >::Value;

			RegistryInfo.bIsUniquePtrProxyArrayParsable_DEPRECATED = TIsParsable<TDataType, const TArray<TUniquePtr<Audio::IProxyData>>& >::Value;

			RegistryInfo.bIsEnum = TEnumTraits<TDataType>::bIsEnum;
			RegistryInfo.bIsExplicit = TIsExplicit<TDataType>::Value;
			RegistryInfo.bIsVariable = TIsVariable<TDataType>::Value;
			RegistryInfo.bIsTransmittable = TIsTransmittable<TDataType>::Value;
			RegistryInfo.bIsConstructorType = TIsConstructorVertexSupported<TDataType>::Value;
			
			if constexpr (std::is_base_of<UObject, UClassToUse>::value)
			{
				RegistryInfo.ProxyGeneratorClass = UClassToUse::StaticClass();
			}
			else
			{
				static_assert(std::is_same<UClassToUse, void>::value, "Only UObject derived classes can supply proxy interfaces.");
				RegistryInfo.ProxyGeneratorClass = nullptr;
			}

			using Base = typename TPolymorphicTraits<TDataType>::BaseType;
			RegistryInfo.bIsPolymorphic = TPolymorphicTraits<TDataType>::bIsPolymorphic;
			RegistryInfo.bIsAbstract = TPolymorphicTraits<TDataType>::bIsAbstract;
			RegistryInfo.ParentDataTypeName = GetMetasoundDataTypeName<Base>();
			return RegistryInfo;
		}

		/** Returns an IEnumDataTypeInterface pointer for the data type. If the
		 * data type has no IEnumDataTypeInterface, the returned pointer will be
		 * invalid.
		 *
		 * @tparam TDataType - The data type to create the interface for.
		 *
		 * @return A shared pointer to the IEnumDataTypeInterface. If the TDataType
		 *         does not have an IEnumDataTypeInterface, returns an invalid pointer.
		 */
		template<typename TDataType>
		TSharedPtr<Frontend::IEnumDataTypeInterface> GetEnumDataTypeInterface()
		{
			TSharedPtr<Frontend::IEnumDataTypeInterface> EnumInterfacePtr;

			using FEnumTraits = TEnumTraits<TDataType>;

			// Check if data type is an enum.
			if constexpr (FEnumTraits::bIsEnum)
			{
				using InnerType = typename FEnumTraits::InnerType;
				using FStringHelper = TEnumStringHelper<InnerType>;

				struct FEnumHandler : Metasound::Frontend::IEnumDataTypeInterface
				{
					FName GetNamespace() const override
					{
						return FStringHelper::GetNamespace();
					}

					int32 GetDefaultValue() const override
					{
						return static_cast<int32>(TEnumTraits<TDataType>::DefaultValue);
					}

					const TArray<FGenericInt32Entry>& GetAllEntries() const override
					{
						auto BuildIntEntries = []()
						{
							// Convert to int32 representation 
							TArray<FGenericInt32Entry> IntEntries;
							IntEntries.Reserve(FStringHelper::GetAllEntries().Num());
							for (const TEnumEntry<InnerType>& i : FStringHelper::GetAllEntries())
							{
								IntEntries.Emplace(i);
							}
							return IntEntries;
						};
						static const TArray<FGenericInt32Entry> IntEntries = BuildIntEntries();
						return IntEntries;
					}
				};

				EnumInterfacePtr = MakeShared<FEnumHandler>();
			}

			return EnumInterfacePtr;
		}

		template <typename T, typename V = void>
		struct HasRawParameterAssignmentOp { bool value = false; };

		template <typename T>
		struct HasRawParameterAssignmentOp<T, decltype(&T::AssignRawParameter, void())> { bool value = true; };

		// Base registry entry for any data type
		class FDataTypeRegistryEntryBase : public IDataTypeRegistryEntry
		{
		public:
		 	UE_API FDataTypeRegistryEntryBase(const Frontend::FDataTypeRegistryInfo& Info, const TSharedPtr<Frontend::IEnumDataTypeInterface>& EnumInterface);
			UE_API virtual ~FDataTypeRegistryEntryBase() override;
			
			UE_API virtual const Frontend::FDataTypeRegistryInfo& GetDataTypeInfo() const override;
			UE_API virtual TSharedPtr<const Frontend::IEnumDataTypeInterface> GetEnumInterface() const override;
			UE_API virtual TSharedPtr<IPolymorphicDataTypeInterface> GetPolymorphicInterface() const override;			
			UE_API virtual const FMetasoundFrontendClass& GetFrontendInputClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetInputClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendConstructorInputClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetConstructorInputClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendLiteralClass() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendOutputClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetOutputClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendConstructorOutputClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetConstructorOutputClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendVariableClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetVariableClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendVariableMutatorClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetVariableMutatorClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendVariableAccessorClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetVariableAccessorClassMetadata() const override;
			UE_API virtual const FMetasoundFrontendClass& GetFrontendVariableDeferredAccessorClass() const override;
			UE_API virtual TSharedPtr<const FNodeClassMetadata> GetVariableDeferredAccessorClassMetadata() const override;
			UE_API virtual const Frontend::IParameterAssignmentFunction& GetRawAssignmentFunction() const override;
			UE_API virtual Frontend::FLiteralAssignmentFunction GetLiteralAssignmentFunction() const override;
			UE_API virtual TUniquePtr<INode> CreateOutputNode(FNodeData InNodeData) const override;
			UE_API virtual TUniquePtr<INode> CreateConstructorOutputNode(FNodeData InNodeData) const override;

		protected:
			Frontend::FDataTypeRegistryInfo Info;
			FMetasoundFrontendClass InputClass;
			TSharedPtr<const FNodeClassMetadata> InputClassMetadata;
			FMetasoundFrontendClass ConstructorInputClass;
			TSharedPtr<const FNodeClassMetadata> ConstructorInputClassMetadata;
			FMetasoundFrontendClass OutputClass;
			TSharedPtr<const FNodeClassMetadata> OutputClassMetadata;
			FMetasoundFrontendClass ConstructorOutputClass;
			TSharedPtr<const FNodeClassMetadata> ConstructorOutputClassMetadata;
			
			FMetasoundFrontendClass LiteralClass;

			FMetasoundFrontendClass VariableClass;
			TSharedPtr<const FNodeClassMetadata> VariableClassMetadata;
			FMetasoundFrontendClass VariableMutatorClass;
			TSharedPtr<const FNodeClassMetadata> VariableMutatorClassMetadata;
			FMetasoundFrontendClass VariableAccessorClass;
			TSharedPtr<const FNodeClassMetadata> VariableAccessorClassMetadata;
			FMetasoundFrontendClass VariableDeferredAccessorClass;
			TSharedPtr<const FNodeClassMetadata> VariableDeferredAccessorClassMetadata;
			TSharedPtr<Frontend::IEnumDataTypeInterface> EnumInterface;
			TSharedPtr<IPolymorphicDataTypeInterface> PolymorphicInterface;
			Frontend::IParameterAssignmentFunction RawAssignmentFunction;
			Frontend::FLiteralAssignmentFunction LiteralAssignmentFunction = nullptr;

			typedef void CreateNodeClassMetadataFunc();

			UE_API void InitNodeClassesBase(
				const FNodeClassMetadata& InputClassMetadata,
				const FNodeClassMetadata& OutputClassMetadata,
				const FNodeClassMetadata& LiteralPrototypeMetadata,
				const FNodeClassMetadata& VariableClassMetadata,
				const FNodeClassMetadata& VariableMutatorClassMetadata,
				const FNodeClassMetadata& VariableAccessorClassMetadata,
				const FNodeClassMetadata& VariableDeferredAccessorClassMetadata
			);

			UE_API void InitNodeClassesBaseConstructor(
				const FNodeClassMetadata& ConstructorInputClassMetadata,
				const FNodeClassMetadata& ConstructorOutputClassMetadata
			);
		};


		/** Registers a data type with the MetaSound Frontend. This allows the data type
		 * to be used in Input and Output nodes by informing the Frontend how to
		 * instantiate an instance.
		 *
		 * @tparam TDataType - The data type to register.
		 * @tparam PreferredArgType - The preferred constructor argument type to use when creating an instance of the data type.
		 * @tparam UClassToUse - The preferred UObject class to use when constructing from an Audio::IProxyDataPtr. If the type is not
		 *                       constructible with an Audio::IProxyDataPtr, then this should be void.
		 *
		 * @return True on success, false on failure.
		 */
		template<typename TDataType, ELiteralType PreferredArgType = ELiteralType::None, typename UClassToUse = void>
		bool RegisterDataTypeWithFrontendInternal(const FModuleInfo& InModuleInfo)
		{
			static constexpr bool bIsParsable = TLiteralTraits<TDataType>::bIsParsableFromAnyLiteralType;
			static constexpr bool bIsConstructorType = TIsConstructorVertexSupported<TDataType>::Value;

			class FDataTypeRegistryEntry : public FDataTypeRegistryEntryBase
			{
				void InitRawAssignmentFunction()
				{
					if constexpr (HasRawParameterAssignmentOp<TDataType>().value)
					{
						this->RawAssignmentFunction = [](const void* src, void* dest) { reinterpret_cast<TDataType*>(dest)->AssignRawParameter(src); };
					}
					else if constexpr (std::is_copy_assignable_v<TDataType>)
					{
						if constexpr (!TIsArrayType<TDataType>::Value)
						{
							this->RawAssignmentFunction = [](const void* Src, void* Dest)
								{
									*(reinterpret_cast<TDataType*>(Dest)) = *(reinterpret_cast<const TDataType*>(Src)); 
								};
						}
						else
						{
							this->RawAssignmentFunction = [](const void* Src, void* Dest)
								{
									//***********************************************************************************************************
									// sanity check to be sure memory layout is the same regardless of number of elements in the fixed array...
									using FSmallFixedArray = TParamPackFixedArray<typename TDataType::ElementType, 1>;
									using FLargeFixedArray = TParamPackFixedArray<typename TDataType::ElementType, 200>;
									static_assert(offsetof(FSmallFixedArray,InlineData) > offsetof(FSmallFixedArray, NumValidElements));
									static_assert(offsetof(FSmallFixedArray,InlineData) == offsetof(FLargeFixedArray,InlineData));
									//***********************************************************************************************************
									TDataType& DestinationArray = *(reinterpret_cast<TDataType*>(Dest));
									const FSmallFixedArray& SourceArray = *(reinterpret_cast<const FSmallFixedArray*>(Src));
									SourceArray.CopyToArray(DestinationArray);
								};
						}
					}
				}

				void InitLiteralAssignmentFunction()
				{
					if constexpr (std::is_copy_assignable_v<TDataType> && bIsParsable && !std::is_abstract<TDataType>::value)
					{
						this->LiteralAssignmentFunction = [](const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral, const FAnyDataReference& OutDataRef)
						{
							*OutDataRef.GetWritableValue<TDataType>() = TDataTypeLiteralFactory<TDataType>::CreateExplicitArgs(InOperatorSettings, InLiteral);
						};
					}
				}

				void InitNodeClasses()
				{
					// Create class info using prototype node

					if constexpr (bIsParsable)
					{
						const FName DataTypeName = GetMetasoundDataTypeName<TDataType>();
						const FName UnnamedVertex;

						InitNodeClassesBase(
							TInputNode<TDataType, EVertexAccessType::Reference>::CreateNodeClassMetadata(UnnamedVertex),
							FOutputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Reference),
							TLiteralNode<TDataType>(TEXT(""), FGuid(), FLiteral()).GetMetadata(),
							TVariableNode<TDataType>::CreateNodeClassMetadata(),
							TVariableMutatorNode<TDataType>::CreateNodeClassMetadata(),
							TVariableAccessorNode<TDataType>::CreateNodeClassMetadata(),
							TVariableDeferredAccessorNode<TDataType>::CreateNodeClassMetadata()
						);

						if constexpr (bIsConstructorType)
						{
							InitNodeClassesBaseConstructor(
								TInputNode<TDataType, EVertexAccessType::Value>::CreateNodeClassMetadata(UnnamedVertex),
								FOutputNode::CreateNodeClassMetadata(UnnamedVertex, DataTypeName, EVertexAccessType::Value)
							);
						}
					}
				}
				UE_EXPERIMENTAL(5.7, "Polymorphic API is experimental")
				void InitPolymorphicInterface()
				{
					using FPolyTraits = TPolymorphicTraits<TDataType>;
					if constexpr (FPolyTraits::bIsPolymorphic)
					{
						using BaseType = FPolyTraits::BaseType;
						this->PolymorphicInterface = MakeShared<FPolymorphicDataTypeBase>(
							GetMetasoundDataTypeName<TDataType>(),
							GetMetasoundDataTypeName<BaseType>(),
							FPolyTraits::bIsAbstract);
					}
				}

			public:

				FDataTypeRegistryEntry()
					: FDataTypeRegistryEntryBase(CreateDataTypeInfo<TDataType, PreferredArgType, UClassToUse>(), GetEnumDataTypeInterface<TDataType>())
				{
					InitRawAssignmentFunction();
					InitLiteralAssignmentFunction();
					InitNodeClasses();
					PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
					InitPolymorphicInterface();
					PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
				}

				virtual TUniquePtr<INode> CreateInputNode(FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable)
					{
						if (InputClassMetadata)
						{
							const FInputVertexInterface& Inputs = InNodeData.Interface.GetInputInterface();
							if (ensure(Inputs.Num() == 1))
							{
								const FVertexName& VertexName = Inputs.At(0).VertexName;
								return MakeUnique<TInputNode<TDataType, EVertexAccessType::Reference>>(VertexName, MoveTemp(InNodeData), InputClassMetadata.ToSharedRef());
							}
						}
					}
					return TUniquePtr<INode>(nullptr);
				}

				virtual TUniquePtr<INode> CreateConstructorInputNode(FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable && bIsConstructorType)
					{
						if (ConstructorInputClassMetadata)
						{
							const FInputVertexInterface& Inputs = InNodeData.Interface.GetInputInterface();
							if (ensure(Inputs.Num() == 1))
							{
								const FVertexName& VertexName = Inputs.At(0).VertexName;
								return MakeUnique<TInputNode<TDataType, EVertexAccessType::Value>>(VertexName, MoveTemp(InNodeData), ConstructorInputClassMetadata.ToSharedRef());
							}
						}
					}
					return TUniquePtr<INode>(nullptr);
				}

				UE_DEPRECATED(5.6, "Direct creation of literal nodes will no longer be supported")
				virtual TUniquePtr<INode> CreateLiteralNode(FLiteralNodeConstructorParams&& InParams) const override
				{
					if constexpr (bIsParsable)
					{
						return MakeUnique<TLiteralNode<TDataType>>(InParams.NodeName, InParams.InstanceID, MoveTemp(InParams.Literal));
					}
					else
					{
						return TUniquePtr<INode>(nullptr);
					}
				}

				UE_DEPRECATED(5.6, "Direct creation of receive nodes will no longer be supported")
				virtual TUniquePtr<INode> CreateReceiveNode(const FNodeInitData& InParams) const override
				{
					if constexpr (bIsParsable)
					{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						return MakeUnique<TReceiveNode<TDataType>>(InParams);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
					else
					{
						return TUniquePtr<INode>(nullptr);
					}
				}

				virtual TUniquePtr<INode> CreateVariableNode(FLiteral InLiteral, FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable)
					{
						if (VariableClassMetadata)
						{
							return MakeUnique<TVariableNode<TDataType>>(MoveTemp(InLiteral), MoveTemp(InNodeData), VariableClassMetadata.ToSharedRef());
						}
					}
					return TUniquePtr<INode>(nullptr);
				}
 
				virtual TUniquePtr<INode> CreateVariableMutatorNode(FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable)
					{
						if (VariableMutatorClassMetadata)
						{
							return MakeUnique<TVariableMutatorNode<TDataType>>(MoveTemp(InNodeData), VariableMutatorClassMetadata.ToSharedRef());
						}
					}
					return TUniquePtr<INode>(nullptr);
				}

				virtual TUniquePtr<INode> CreateVariableAccessorNode(FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable)
					{
						if (VariableAccessorClassMetadata)
						{
							return MakeUnique<TVariableAccessorNode<TDataType>>(MoveTemp(InNodeData), VariableAccessorClassMetadata.ToSharedRef());
						}
					}
					return TUniquePtr<INode>(nullptr);
				}

				virtual TUniquePtr<INode> CreateVariableDeferredAccessorNode(FNodeData InNodeData) const override
				{
					if constexpr (bIsParsable)
					{
						if (VariableDeferredAccessorClassMetadata)
						{
							return MakeUnique<TVariableDeferredAccessorNode<TDataType>>(MoveTemp(InNodeData), VariableDeferredAccessorClassMetadata.ToSharedRef());
						}
					}
					return TUniquePtr<INode>(nullptr);
				}

				virtual TSharedPtr<Audio::IProxyData> CreateProxy(UObject* InObject) const override
				{
					// Only attempt to create proxy if the `UClassToUse` is not void.
					if constexpr (!std::is_same<UClassToUse, void>::value)
					{
						static_assert(std::is_base_of<IAudioProxyDataFactory, UClassToUse>::value, "If a Metasound Datatype uses a UObject as a literal, the UClass of that object needs to also derive from Audio::IProxyDataFactory. See USoundWave as an example.");
						if (Frontend::IDataTypeRegistry::Get().IsUObjectProxyFactory(InObject))
						{
							IAudioProxyDataFactory* ObjectAsFactory = Audio::CastToProxyDataFactory<UClassToUse>(InObject);
							if (ensureAlways(ObjectAsFactory))
							{
								Audio::FProxyDataInitParams ProxyInitParams;
								ProxyInitParams.NameOfFeatureRequestingProxy = "MetaSound";

								return ObjectAsFactory->CreateProxyData(ProxyInitParams);
							}
						}
					}

					return TSharedPtr<Audio::IProxyData>(nullptr);
				}

				virtual TOptional<FAnyDataReference> CreateDataReference(EDataReferenceAccessType InAccessType, const FLiteral& InLiteral, const FOperatorSettings& InOperatorSettings) const override
				{
					if constexpr (bIsParsable && !std::is_abstract<TDataType>::value)
					{
						switch (InAccessType)
						{
						case EDataReferenceAccessType::Read:
							return FAnyDataReference{ TDataReadReferenceLiteralFactory<TDataType>::CreateExplicitArgs(InOperatorSettings, InLiteral) };

						case EDataReferenceAccessType::Write:
							return FAnyDataReference{ TDataWriteReferenceLiteralFactory<TDataType>::CreateExplicitArgs(InOperatorSettings, InLiteral) };

						case EDataReferenceAccessType::Value:
							return FAnyDataReference{ TDataValueReferenceLiteralFactory<TDataType>::CreateExplicitArgs(InOperatorSettings, InLiteral) };

						default:
							break;
						}
					}
					return TOptional<FAnyDataReference>();
				}

				virtual TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FOperatorSettings& InOperatorSettings) const override
				{
					if constexpr (bIsParsable)
					{
						return FTransmissionDataChannelFactory::CreateDataChannel<TDataType>(InOperatorSettings);
					}
					else
					{
						return TSharedPtr<IDataChannel, ESPMode::ThreadSafe>(nullptr);
					}
				}
			};

			bool bSucceeded = IDataTypeRegistry::Get().RegisterDataType(MakeUnique<FDataTypeRegistryEntry>());

			if (bSucceeded)
			{
				RegisterAutoConverterNodes<TDataType>(InModuleInfo);
				RegisterSendAndReceiveNodes<TDataType>(InModuleInfo);
			}
			else
			{
				UE_LOGF(LogMetaSound, Error, "Failed to register MetaSound DataType %ls", *GetMetasoundDataTypeName<TDataType>().ToString());
			}
			
			return bSucceeded;
		}

		// Unregister a datatype as well as some autogenerated nodes. This should unregister all nodes that were registered
		// during RegisterDataTypeWIthFrontendInternal<>(...)
		UE_INTERNAL
		UE_API bool UnregisterDataTypeWithFrontendInternal(const FName& InDataTypeName, const FModuleInfo& InModuleInfo);

		/** Registers an array of a data type with the MetaSound Frontend. This allows 
		 * an array of the data type to be used in Input, Output, Send and Receive 
		 * nodes by informing the Frontend how to instantiate an instance. 
		 *
		 * @tparam TDataType - The data type to register.
		 * @tparam PreferredArgType - The preferred constructor argument type to use when creating an instance of the data type.
		 * @tparam UClassToUse - The preferred UObject class to use when constructing from an Audio::IProxyDataPtr. If the type is not
		 *                       constructible with an Audio::IProxyDataPtr, then this should be void.
		 *
		 * @return True on success, false on failure.
		 */
		template<typename TDataType, ELiteralType PreferredArgType>
		bool RegisterDataTypeArrayWithFrontend(const FModuleInfo& InModuleInfo) 
		{
			using namespace MetasoundDataTypeRegistrationPrivate;
			using TArrayType = TArray<TDataType>;

			if constexpr (TEnableAutoArrayTypeRegistration<TDataType>::Value && !CIsPolymorphicAndAbstract<TDataType>)
			{
				bool bSuccess = RegisterDataTypeWithFrontendInternal<TArrayType, TLiteralArrayEnum<PreferredArgType>::Value>(InModuleInfo);
				bSuccess = bSuccess && RegisterArrayNodes<TArrayType>(InModuleInfo);
				bSuccess = bSuccess && RegisterDataTypeWithFrontendInternal<TVariable<TArrayType>>(InModuleInfo);
				return bSuccess;
			}
			else
			{
				return true;
			}
		}

		template<typename TDataType>
		bool UnregisterDataTypeArrayWithFrontend(const FModuleInfo& InModuleInfo)
		{
			using namespace MetasoundDataTypeRegistrationPrivate;

			if constexpr (TEnableAutoArrayTypeRegistration<TDataType>::Value && !CIsPolymorphicAndAbstract<TDataType>)
			{
				using TArrayType = TArray<TDataType>;

				const FName& DataTypeName = GetMetasoundDataTypeName<TArrayType>();

				// Unregistration methods are called in reverse order of registration 
				// functions.
PRAGMA_DISABLE_INTERNAL_WARNINGS
				bool bSuccess = UnregisterDataTypeWithFrontendInternal(GetMetasoundDataTypeName<TVariable<TArrayType>>(), InModuleInfo);
				bSuccess = bSuccess && UnregisterArrayNodes(DataTypeName, InModuleInfo);
				bSuccess = bSuccess && UnregisterDataTypeWithFrontendInternal(DataTypeName, InModuleInfo);
PRAGMA_ENABLE_INTERNAL_WARNINGS
				return bSuccess;
			}
			else
			{
				return true;
			}
		}
	} // namespace MetasoundDataTypeRegistrationPrivate
	
	/** Registers a data type with the MetaSound Frontend. This allows the data type 
	 * to be used in Input, Output, Send and Receive  nodes by informing the 
	 * Frontend how to instantiate an instance. 
	 *
	 * @tparam TDataType - The data type to register.
	 * @tparam PreferredArgType - The preferred constructor argument type to use when creating an instance of the data type.
	 * @tparam UClassToUse - The preferred UObject class to use when constructing from an Audio::IProxyDataPtr. If the type is not
	 *                       constructible with an Audio::IProxyDataPtr, then this should be void.
	 *
	 * @return True on success, false on failure.
	 */
	template<typename TDataType, ELiteralType PreferredArgType = ELiteralType::None, typename UClassToUse = void>
	bool RegisterDataType(const FModuleInfo& InModuleInfo)
	{
		using namespace MetasoundDataTypeRegistrationPrivate;

		// Register TDataType as a metasound data type.
		bool bSuccess = RegisterDataTypeWithFrontendInternal<TDataType, PreferredArgType, UClassToUse>(InModuleInfo);
		bSuccess = bSuccess && RegisterDataTypeWithFrontendInternal<TVariable<TDataType>>(InModuleInfo);

		// Register TArray<TDataType> as a metasound data type.
		bSuccess = bSuccess && RegisterDataTypeArrayWithFrontend<TDataType, PreferredArgType>(InModuleInfo);

		return bSuccess;
	}

	/** Unregisters a data type with the MetaSound Frontend. 
	 *
	 * @tparam TDataType - The data type to register.
	 *
	 * @return True on success, false on failure.
	 */
	template<typename TDataType>
	bool UnregisterDataType(const FModuleInfo& InModuleInfo)
	{
		using namespace MetasoundDataTypeRegistrationPrivate;

PRAGMA_DISABLE_INTERNAL_WARNINGS
		// Unregistration is done in reverse order of registration
		bool bSuccess = UnregisterDataTypeArrayWithFrontend<TDataType>(InModuleInfo);
		bSuccess = bSuccess && UnregisterDataTypeWithFrontendInternal(GetMetasoundDataTypeName<TVariable<TDataType>>(), InModuleInfo);
		bSuccess = bSuccess && UnregisterDataTypeWithFrontendInternal(GetMetasoundDataTypeName<TDataType>(), InModuleInfo);
PRAGMA_ENABLE_INTERNAL_WARNINGS

		return bSuccess;
	}

	/** Registers a data type with the MetaSound Frontend. This allows the data type 
	 * to be used in Input, Output, Send and Receive  nodes by informing the 
	 * Frontend how to instantiate an instance. 
	 *
	 * @tparam TDataType - The data type to register.
	 * @tparam PreferredArgType - The preferred constructor argument type to use when creating an instance of the data type.
	 * @tparam UClassToUse - The preferred UObject class to use when constructing from an Audio::IProxyDataPtr. If the type is not
	 *                       constructible with an Audio::IProxyDataPtr, then this should be void.
	 *
	 * @return True on success, false on failure.
	 */
	template<typename TDataType, ELiteralType PreferredArgType = ELiteralType::None, typename UClassToUse = void>
	UE_DEPRECATED(5.7, "Use RegisterDataType(const FModuleInfo&) instead")
	bool RegisterDataType()
	{
		return RegisterDataType<TDataType, PreferredArgType, UClassToUse>(FModuleInfo{});
	}


	/** Registration info for a data type.
	 *
	 * @tparam DataType - The data type to be registered. 
	 */
	template<typename DataType>
	struct TMetasoundDataTypeRegistration
	{
		static_assert(std::is_same<DataType, typename std::decay<DataType>::type>::value, "DataType and decayed DataType must be the same");
		
		// To register a data type, an input node must be able to instantiate it.
		static constexpr bool bCanRegister = TInputNode<DataType, EVertexAccessType::Reference>::bCanRegister;

		// This is no longer used.
		static const bool bSuccessfullyRegistered = false;
	};
}

namespace Metasound
{
	template<typename DataType>
	using TMetasoundDataTypeRegistration UE_DEPRECATED(5.6, "Use Frontend::TMetasoundDataTypeRegistration") = Frontend::TMetasoundDataTypeRegistration<DataType>;

	template<typename TDataType, ELiteralType PreferredArgType = ELiteralType::None, typename UClassToUse = void>
	UE_DEPRECATED(5.6, "Use Frontend::RegisterDataType instead")
	bool RegisterDataTypeWithFrontend()
	{
		return Frontend::RegisterDataType<TDataType, PreferredArgType, UClassToUse>();
	}
}


#define CANNOT_REGISTER_METASOUND_DATA_TYPE_ASSERT_STRING(DataType) \
"To register " #DataType " to be used as a Metasounds input or output type, it needs a default constructor or one of the following constructors must be implemented:  " \
#DataType "(), " \
#DataType "(bool InValue), " \
#DataType "(int32 InValue), " \
#DataType "(float InValue), " \
#DataType "(const FString& InString)" \
#DataType "(const Audio::IProxyDataPtr& InData),  or " \
#DataType "(const TArray<Audio::IProxyDataPtr>& InProxyArray)."\
#DataType "(const ::Metasound::FOperatorSettings& InSettings), " \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, bool InValue), " \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, int32 InValue), " \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, float InValue), " \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, const FString& InString)" \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, const Audio::IProxyDataPtr& InData),  or " \
#DataType "(const ::Metasound::FOperatorSettings& InSettings, const TArray<Audio::IProxyDataPtr>& InProxyArray)."

#define ENQUEUE_METASOUND_DATATYPE_REGISTRATION_COMMAND(DataType, DataTypeName, ...) \
	static_assert(::Metasound::Frontend::TMetasoundDataTypeRegistration<DataType>::bCanRegister, CANNOT_REGISTER_METASOUND_DATA_TYPE_ASSERT_STRING(DataType)); \
	METASOUND_IMPLEMENT_REGISTRATION_ACTION(UE_JOIN(_DataTypeRegistration,__COUNTER__), (::Metasound::Frontend::RegisterDataType<DataType, ##__VA_ARGS__>), (::Metasound::Frontend::UnregisterDataType<DataType>))

// This should be used to expose a datatype as a potential input or output for a metasound graph.
// The first argument to the macro is the class to expose.
// the second argument is the display name of that type in the Metasound editor.
// Optionally, a Metasound::ELiteralType can be passed in to designate a preferred literal type-
// For example, if Metasound::ELiteralType::Float is passed in, we will default to using a float parameter to create this datatype.
// If no argument is passed in, we will infer a literal type to use.
// If 
// Metasound::ELiteralType::Invalid can be used to enforce that we don't provide space for a literal, in which case you should have a default constructor or a constructor that takes [const FOperatorSettings&] implemented.
// If you pass in a preferred arg type, please make sure that the passed in datatype has a matching constructor, since we won't check this until runtime.
#define REGISTER_METASOUND_DATATYPE(DataType, DataTypeName, ...) \
	DEFINE_METASOUND_DATA_TYPE(DataType, DataTypeName); \
	ENQUEUE_METASOUND_DATATYPE_REGISTRATION_COMMAND(DataType, DataTypeName, ##__VA_ARGS__)

// Helper macro for disabling auto array registration
#define DISABLE_METASOUND_ARRAYS_FOR_TYPE(TYPENAME)\
	template<>\
	struct TEnableAutoArrayTypeRegistration<TYPENAME>\
	{\
		static constexpr bool Value = false;\
	};


#undef UE_API
