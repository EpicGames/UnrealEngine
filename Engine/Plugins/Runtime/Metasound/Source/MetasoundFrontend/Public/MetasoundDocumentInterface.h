// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOperatorSettings.h"
#include "Templates/Function.h"
#include "UObject/TopLevelAssetPath.h"

#include "MetasoundDocumentInterface.generated.h"


#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;

// UInterface for all MetaSound UClasses that implement a MetaSound document
// as a means for accessing data via code, scripting, execution, or node
// class generation.
UINTERFACE(MinimalAPI, BlueprintType, meta = (DisplayName = "MetaSound Document Interface", CannotImplementInterfaceInBlueprint))
class UMetaSoundDocumentInterface : public UInterface
{
	GENERATED_BODY()
};

class IMetaSoundDocumentInterface : public IInterface
{
	GENERATED_BODY()

public:
	virtual FTopLevelAssetPath GetAssetPathChecked() const = 0;

	// Returns read-only reference to the the MetaSoundFrontendDocument
	// containing all MetaSound runtime & editor data.
	virtual const FMetasoundFrontendDocument& GetConstDocument() const = 0;

	// Returns the parent class registered with the MetaSound UObject registry.
	virtual const UClass& GetBaseMetaSoundUClass() const = 0;

	// Returns the builder class used to modify the given document.
	virtual const UClass& GetBuilderUClass() const = 0;

	// Returns the default access flags utilized when document is initialized.
	virtual EMetasoundFrontendClassAccessFlags GetDefaultAccessFlags() const = 0;

	// Conforms UProperty data outside the Frontend Document Model to the document's data.
	// Returns whether or not object data was modified.
	virtual bool ConformObjectToDocument() = 0;

	// Returns whether or not a document builder is currently active and can mutate the given interface's document
	virtual bool IsActivelyBuilding() const = 0;

private:
	virtual FMetasoundFrontendDocument& GetDocument() = 0;

	// Derived classes can implement these methods to react to a builder beginning
	// or finishing. Begin and Finish are tied to the lifetime of the active 
	// FMetaSoundFrontendDocumentBuilder.
	virtual void OnBeginActiveBuilder() = 0;
	virtual void OnFinishActiveBuilder() = 0;

	friend struct FMetaSoundFrontendDocumentBuilder;
};

// Forward declaration for building/backwards compat. Now found in separate header: MetasoundFrontendDocumentBuilderRegistry.h.
namespace Metasound::Frontend
{
	class IDocumentBuilderRegistry;
} // namespace Metasound::Frontend
#undef UE_API
