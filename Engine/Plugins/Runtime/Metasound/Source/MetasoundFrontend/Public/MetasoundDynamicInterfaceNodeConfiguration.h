// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundVertex.h"

#include "MetasoundDynamicInterfaceNodeConfiguration.generated.h"

#define UE_API METASOUNDFRONTEND_API

/**
 * Built-in FMetaSoundFrontendNodeConfiguration subclass that stores
 * sub-interface instance counts and variant type selections as serializable
 * UPROPERTY data. Node developers who declare a FClassInterface with
 * sub-interfaces and/or variants can use this configuration type (or have
 * it auto-applied) instead of writing a custom configuration subclass.
 *
 * Sub-interface counts are stored as a map of sub-interface name to instance
 * count. Variant selections are stored as a map of variant name to data type
 * name.
 *
 * OverrideDefaultInterface() looks up the FClassInterface from the node
 * registry and calls CreateVertexInterface() with the stored configurations,
 * producing a FMetasoundFrontendClassInterface with the correct vertex layout.
 */
USTRUCT()
struct FMetaSoundDynamicInterfaceNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	/** Sub-interface instance counts, keyed by sub-interface name. */
	UPROPERTY()
	TMap<FName, uint32> SubInterfaceCounts;

	/** Variant type selections, keyed by variant name. */
	UPROPERTY()
	TMap<FName, FName> VariantSelections;

	UE_API virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	UE_API virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

#undef UE_API
