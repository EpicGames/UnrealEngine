// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Transform.h"
#include "Internationalization/Text.h"
#include "MetasoundLog.h"
#include "Misc/Optional.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

#include <type_traits>

// NOTE: Metasound Enum types are defined outside of Engine so can't use the UENUM type reflection here.
// Basic reflection is provides using template specialization and Macros defined below.

// Example usage:
//
// 1. Declare an Enum class. 
// enum class EMyOtherTestEnum : int32
// {
//		Alpha = 500,
//		Beta = -666,
//		Gamma = 333
// };

// 2. Declare its wrapper types using the DECLARE_METASOUND_ENUM macro.
// DECLARE_METASOUND_ENUM(EMyOtherTestEnum, EMyOtherTestEnum::Gamma, METASOUNDSTANDARDNODES_API, FMyOtherTestEnumTypeInfo, FMyOtherTestEnumReadRef, FMyOtherTestEnumWriteRef)

// 3. Define it using the BEGIN/ENTRY/END macros
// DEFINE_METASOUND_ENUM_BEGIN(EMyOtherTestEnum)
// 		DEFINE_METASOUND_ENUM_ENTRY(EMyOtherTestEnum::Alpha, "AlphaDescription", "Alpha", "AlphaDescriptionTT", "Alpha tooltip"),
// 		DEFINE_METASOUND_ENUM_ENTRY(EMyOtherTestEnum::Beta, "BetaDescription", "Beta", "BetaDescriptioTT", "Beta tooltip"),
// 		DEFINE_METASOUND_ENUM_ENTRY(EMyOtherTestEnum::Gamma, "GammaDescription", "Gamma", "GammaDescriptionTT", "Gamma tooltip")
// DEFINE_METASOUND_ENUM_END()

namespace Metasound
{
	// Struct to hold each of the entries of the Enum.
	template<typename T>
	struct TEnumEntry
	{
		T Value;
		FName Name;
#if WITH_EDITOR
		FText DisplayName;
		FText Tooltip;
#endif

		// Allow implicit conversion to int32 entry
		operator TEnumEntry<int32>() const
		{
#if WITH_EDITOR
			return TEnumEntry<int32>{ static_cast<int32>(Value), Name, DisplayName, Tooltip };
#else
			return TEnumEntry<int32>{ static_cast<int32>(Value), Name };
#endif
		}	
	};

	/** CRTP base class for Enum String Helper type.
	 *  Provides common code for all specializations.
	 */ 
	template<typename Derived, typename EnumType>
	struct TEnumStringHelperBase
	{
		// Give a enum value e.g. 'EMyEnum::One', convert that to a Name (if its valid)
		static TOptional<FName> ToName(EnumType InValue)
		{
			for (const TEnumEntry<EnumType>& i : Derived::GetAllEntries())
			{
				if (i.Value == InValue)
				{
					return i.Name;
				}
			}
			return {};
		}
		// Give a Name "EMyEnum::One", convert that to a Enum Value
		static TOptional<EnumType> FromName(const FName	InName)
		{
			for (const TEnumEntry<EnumType>& i : Derived::GetAllEntries())
			{
				if (i.Name == InName)
				{
					return i.Value;
				}
			}
			return {};
		}
		// Return all possible names.
		static TArray<FName> GetAllNames()
		{
			TArray<FName> Names;
			for (const TEnumEntry<EnumType>& i : Derived::GetAllEntries())
			{
				Names.Emplace(i.Name);
			}
			return Names;
		}	
	};

	/** Metasound Enum String Helper
	 */
	template<typename T>
	struct TEnumStringHelper : TEnumStringHelperBase<TEnumStringHelper<T>, T>
	{
		static_assert(TIsEnum<T>::Value, "Please define a specialization of this class. The DECLARE_METASOUND_ENUM macros will do this for you");
	};

	/** Metasound Enum Wrapper
	 */
	template<typename EnumType, EnumType DefaultValue>
	class TEnum final
	{
	public:
		using InnerType = EnumType;
		using SerializedType = int32;
		using UnderlyingType = std::underlying_type_t<EnumType>;

		static constexpr bool EnumTypeIsSupported()
		{
			// make sure this is an enum
			if constexpr (!TIsEnum<EnumType>::Value)
			{
				return false;
			}

			// make sure the underlying type is integral
			if constexpr (!TIsIntegral<UnderlyingType>::Value)
			{
				return false;
			}

			// if this is a scoped enum make sure the underlying type will fit in the serialized type
			if constexpr (TIsEnumClass<EnumType>::Value)
			{
				constexpr bool SerializedIsSigned = TIsSigned<SerializedType>::Value;
				constexpr bool UnderlyingIsSigned = TIsSigned<UnderlyingType>::Value;

				// if the serialized type is signed, the underlying type must be
				// smaller than the serialized type if it is unsigned,
				// or the same size or smaller than the serialized type if it's signed
				if constexpr (SerializedIsSigned)
				{
					return UnderlyingIsSigned
						? sizeof(UnderlyingType) <= sizeof(SerializedType)
						: sizeof(UnderlyingType) < sizeof(SerializedType);
				}

				// otherwise the underlying type must be unsigned and at most the size of the serialized type
				// This is just here in case someone changes the serialized type
				return !UnderlyingIsSigned && sizeof(UnderlyingType) <= sizeof(SerializedType);
			}

			// make an exception for C-style enums so we don't break backward compatibility
			return true;
		}
		
		static_assert(EnumTypeIsSupported(), "EnumType is not supported for TEnum");
		
		// Default.
		explicit TEnum(EnumType InValue = DefaultValue)
		{
			// Try and convert to validate this is a valid value.
			TOptional<FName> Converted = ToName(InValue);
			if (Converted)
			{
				EnumValue = InValue;
			}
			else if (!bHasWarnedNameToEnumConversionFailure)
			{
				TArray<FString> ValueStrings;
				Algo::Transform(GetAllNames(), ValueStrings, [](const FName& Name) { return Name.ToString(); });

				UE_LOGF(LogMetaSound, Warning,
					"Cannot create valid enum from value '%ls'.\nPossible Values:\n%ls",
					*FString::FromInt((int32)(InValue)),
					*FString::Join(ValueStrings, TEXT("\n, "))
				);
				bHasWarnedNameToEnumConversionFailure = true;
			}

		}

		// From Int32 (this is the common path from a Literal).
		explicit TEnum(int32 InIntValue)
			: TEnum(static_cast<EnumType>(InIntValue))
		{
		}

		// From Name
		explicit TEnum(FName InValueName)
		{
			// Try and convert from Name to Value 
			TOptional<EnumType> Converted = NameToEnum(InValueName);
			if (Converted)
			{
				EnumValue = *Converted;
			}
			else
			{
				if (!bHasWarnedNameToEnumConversionFailure)
				{
					TArray<FString> ValueStrings;
					Algo::Transform(GetAllNames(), ValueStrings, [](const FName& Name) { return Name.ToString(); });

					UE_LOGF(LogMetaSound, Warning,
						"Cannot create valid enum value from string '%ls'.\nPossible Values:\n%ls",
						*InValueName.ToString(),
						*FString::Join(ValueStrings, TEXT("\n, "))
					);
				}
			}
		}

		// Slow, construct from FString to FName.
		explicit TEnum(const FString& InString)
			: TEnum(FName(*InString))
		{
		}

		EnumType Get() const
		{
			return EnumValue;
		}

		int32 ToInt() const
		{
			return static_cast<int32>(EnumValue);
		}

		// Convert to its FName (if possible).
		TOptional<FName> ToName() const 
		{
			return ToName(EnumValue);
		}

		// Conversion operator to automatically convert this to its underlying enum type.
		operator EnumType() const
		{
			return EnumValue;
		}

		// Convert from EnumValue to FName (if possible).
		static TOptional<FName> ToName(EnumType InValue)
		{
			return TEnumStringHelper<EnumType>::ToName(InValue);
		}

		// Convert from Name to EnumValue (if possible).
		static TOptional<EnumType> NameToEnum(FName InValue)
		{
			return TEnumStringHelper<EnumType>::FromName(InValue);
		}

		// Return all possible Names
		static TArray<FName> GetAllNames()
		{
			return TEnumStringHelper<EnumType>::GetAllNames();
		}

	private:
		// Keep the type in its fully typed form for debugging.
		EnumType EnumValue = DefaultValue;

		static bool bHasWarnedNameToEnumConversionFailure;
	};

	template<typename EnumType, EnumType DefaultValue>
	bool TEnum<EnumType, DefaultValue>::bHasWarnedNameToEnumConversionFailure = false;

	template<typename T>
	struct TEnumTraits
	{
		static constexpr bool bIsEnum = false;
		using InnerType = int32;
		static constexpr int32 DefaultValue = 0;
	};

	template<typename T, T D>
	struct TEnumTraits<TEnum<T, D>>
	{
		static constexpr bool bIsEnum = true;
		using InnerType = T;
		static constexpr T DefaultValue = D;
	};
}

// Helper macros: for defining/declaring a new Enum to be used in Metasounds.
// * Declares/defines a typedef FEnum<YourEnum> wrapper around the enum that you pass it
// * Declares/defines a specialization of TEnumStringHelper which is used to validate and convert between values/names.

// These helps generate a couple of boiler plate functions:
//  - GetNamespace() which will return the name of the Enum "EMyEnum" etc. 
//  - GetAllEntries() which returns all posible entries in the enum. 

// Definition macros are in MetasoundEnumRegistrationMacro.h

/** DECLARE_METASOUND_ENUM
 * @param ENUMNAME - The typename of your EnumType you want to use for Metasounds. e.g. MyEnum
 * @param DEFAULT - A fully qualified default Enum value. e.g. EMyEnum::One
 * @param API - The module API this is declared inside e.g. METASOUNDSTANDARDNODES_API
 * @param ENUMTYPEDEF - The name of the TEnum<YourType> wrapper type
 * @param TYPEINFO - The name of the TypeInfo type you want to define e.g. FMyEnumTypeInfo
 * @param READREF - The name of the Read Reference type you want to define. e.g FMyEnumReadRef
 * @param WRITEREF -The name of the Write Reference type you want to define e.g. FMyEnumWriteRef
 */
#define DECLARE_METASOUND_ENUM(ENUMNAME, DEFAULT, API, ENUMTYPEDEF, TYPEINFO, READREF, WRITEREF)\
	using ENUMTYPEDEF = Metasound::TEnum<ENUMNAME, DEFAULT>; \
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(ENUMTYPEDEF, API, TYPEINFO, READREF, WRITEREF);\
	template<> struct Metasound::TEnumStringHelper<ENUMNAME> : Metasound::TEnumStringHelperBase<Metasound::TEnumStringHelper<ENUMNAME>, ENUMNAME>\
	{\
		static FName GetNamespace()\
		{\
			static const FName ThisName { TEXT(#ENUMNAME) };\
			return ThisName;\
		}\
		static API TArrayView<const Metasound::TEnumEntry<ENUMNAME>> GetAllEntries();\
	};
