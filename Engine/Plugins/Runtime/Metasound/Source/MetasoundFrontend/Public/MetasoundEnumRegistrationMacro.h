// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundEnum.h"
#include "MetasoundFrontendDataTypeTraits.h"
#include "MetasoundDataTypeRegistrationMacro.h"

namespace Metasound
{
	// Enable registration of converter nodes which convert from int32 to enums
	template<typename FromDataType, typename EnumType, EnumType DefaultValue>
	struct TEnableAutoConverterNodeRegistration<FromDataType, TEnum<EnumType, DefaultValue>>
	{
		static constexpr bool Value = std::is_same<int32, FromDataType>::value;
	};

	// Enable registration of converter nodes which convert from enums to int32
	template<typename ToDataType, typename EnumType, EnumType DefaultValue>
	struct TEnableAutoConverterNodeRegistration<TEnum<EnumType, DefaultValue>, ToDataType>
	{
		static constexpr bool Value = std::is_same<int32, ToDataType>::value;
	};

	// Disable arrays of enums
	template<typename EnumType, EnumType DefaultValue>
	struct TEnableAutoArrayTypeRegistration<TEnum<EnumType, DefaultValue>>
	{
		static constexpr bool Value = false;
	};

	// Disable array nodes of enums
	template<typename EnumType, EnumType DefaultValue>
	struct TEnableArrayNodes<TEnum<EnumType, DefaultValue>>
	{
		static constexpr bool Value = false;
	};

	// Specialization of TIsTransmittable<> to disable transmission of enums.
	template<typename EnumType, EnumType DefaultValue>
	struct TEnableTransmissionNodeRegistration<TEnum<EnumType, DefaultValue>>
	{
	public:
		static constexpr bool Value = false;
	};
}
// Helper macros for use with the enum declaration macro in MetasoundEnum.h
/** DEFINE_METASOUND_ENUM_BEGIN
 * @param ENUMNAME - The typename of your raw EnumType you want to use for Metasounds. e.g. EMyType
 * @param ENUMTYPEDEF - The name of the TEnum<YourType> wrapper type
 * @param DATATYPENAMESTRING - The string that will the data type name "Enum:<string>" e.g. "MyEnum"
 */
#define DEFINE_METASOUND_ENUM_BEGIN(ENUMNAME,ENUMTYPEDEF,DATATYPENAMESTRING)\
	REGISTER_METASOUND_DATATYPE(ENUMTYPEDEF, "Enum:" DATATYPENAMESTRING, ::Metasound::ELiteralType::Integer);\
	TArrayView<const Metasound::TEnumEntry<ENUMNAME>> Metasound::TEnumStringHelper<ENUMNAME>::GetAllEntries()\
	{\
		static const Metasound::TEnumEntry<ENUMNAME> Entries[] = {

/** DEFINE_METASOUND_ENUM_ENTRY - defines a single Enum Entry
  * @param ENTRY - Fully Qualified Name of Entry of the Enum. (e.g. EMyType::One)
  * @param DISPLAYNAME_KEY - Display Name loc key 
  * @param DISPLAYNAME - Display Name text presented to User
  * @param TOOLTIP_KEY - Tooltip loc key 
  * @param TOOLTIP - Tooltip text 
  */
#if WITH_EDITOR
	#define DEFINE_METASOUND_ENUM_ENTRY(ENTRY, DISPLAYNAME_KEY, DISPLAYNAME, TOOLTIP_KEY, TOOLTIP) { ENTRY, TEXT(#ENTRY), LOCTEXT(DISPLAYNAME_KEY, DISPLAYNAME), LOCTEXT(TOOLTIP_KEY, TOOLTIP) }
	#define DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(ENTRY, DISPLAYNAME_KEY, DISPLAYNAME) { ENTRY, TEXT(#ENTRY), LOCTEXT(DISPLAYNAME_KEY, DISPLAYNAME) }
#else
	#define DEFINE_METASOUND_ENUM_ENTRY(ENTRY, DISPLAYNAME_KEY, DISPLAYNAME, TOOLTIP_KEY, TOOLTIP) { ENTRY, TEXT(#ENTRY) }
	#define DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(ENTRY, DISPLAYNAME_KEY, DISPLAYNAME) { ENTRY, TEXT(#ENTRY) }
#endif // WITH_EDITOR

/** DEFINE_METASOUND_ENUM_END - macro which ends the function body of the GetAllEntries function
   */
#define DEFINE_METASOUND_ENUM_END() \
		};\
		return Entries;\
	};
