// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundGraph.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"

// Forward Declarations
class FMetasoundAssetBase;

namespace Metasound
{	
	namespace Frontend
	{
		/** Generates a new FMetasoundFrontendClass from Node Metadata 
		 *
		 * @param InNodeMetadata - Metadata describing an external node.
		 *
		 * @return FrontendClass for natively-defined node.
		 */
		METASOUNDFRONTEND_API FMetasoundFrontendClass GenerateClass(const FNodeClassMetadata& InNodeMetadata, EMetasoundFrontendClassType ClassType=EMetasoundFrontendClassType::External);

		/** Generates a new FMetasoundFrontendClass from node lookup info.
		 *
		 * @param InInfo - Class info for a already registered external node.
		 *
		 * @return FrontendClass for natively-defined node.
		 */
		METASOUNDFRONTEND_API FMetasoundFrontendClass GenerateClass(const Metasound::Frontend::FNodeRegistryKey& InKey);

		/** Generates a new FMetasoundFrontendClass from Node init data
		 *
		 * @tparam NodeType - Type of node to instantiate.
		 * @param InNodeInitData - Data used to call constructor of node.
		 *
		 * @return FrontendClass for natively-defined node.
		 */
		template<typename NodeType>
		FMetasoundFrontendClass GenerateClass(const FNodeInitData& InNodeInitData)
		{
			TUniquePtr<INode> Node = MakeUnique<NodeType>(InNodeInitData);

			if (ensure(Node.IsValid()))
			{
				return GenerateClass(Node->GetMetadata());
			}

			return FMetasoundFrontendClass();
		}

		/** Generates a new FMetasoundFrontendClass from a NodeType
		 *
		 * @tparam NodeType - Type of node.
		 *
		 * @return FrontendClass for natively-defined node.
		 */
		template<typename NodeType>
		FMetasoundFrontendClass GenerateClass()
		{
			FNodeInitData InitData;
			InitData.InstanceName = "GeneratedClass";

			return GenerateClass<NodeType>(InitData);
		}

		// Takes a JSON string and deserializes it into a Metasound document struct.
		// @returns false if the file couldn't be found or parsed into a document.
		METASOUNDFRONTEND_API bool ImportJSONToMetasound(const FString& InJSON, FMetasoundFrontendDocument& OutMetasoundDocument);

		// Opens a json document at the given absolute path and deserializes it into a Metasound document struct.
		// @returns false if the file couldn't be found or parsed into a document.
		METASOUNDFRONTEND_API bool ImportJSONAssetToMetasound(const FString& InPath, FMetasoundFrontendDocument& OutMetasoundDocument);

		// These functions can be used to get the FVertexInterface from a FMetasoundFrontendClass 
		// for node registration in special cases.
		// Originally, nodes did not take in a FVertexInterface on construction,
		// but with the introduction of node configuration in 5.6, they now do take
		// in a FVertexInterface. 
		
		// If called on the game thread, no proxy data cache is needed for object literal creation, but if called from other threads, the proxy data cache must be provided. 
		FVertexInterface CreateDefaultVertexInterfaceFromClass(const FMetasoundFrontendClass& InNodeClass, const FProxyDataCache* InProxyDataCache = nullptr);
		// Does not create proxies, so no thread restrictions. 
		FVertexInterface CreateDefaultVertexInterfaceFromClassNoProxy(const FMetasoundFrontendClass& InNodeClass);

		// Struct that indicates whether an input and an output can be connected,
		// and whether an intermediate node is necessary to connect the two.
		struct FConnectability
		{
			enum class EConnectable
			{
				Yes,
				No,
				YesWithConverterNode
			};

			enum class EReason : uint8
			{
				None,
				IncompatibleDataTypes,
				CausesLoop,
				IncompatibleAccessTypes
			};

			EConnectable Connectable = EConnectable::No;

			EReason Reason = EReason::None;


			// If Connectable is EConnectable::YesWithConverterNode,
			// this will be a populated list of nodes we can use 
			// to convert between the input and output.
			TArray<FConverterNodeInfo> PossibleConverterNodeClasses;
		};
	} // namespace Frontend
} // namespace Metasound
