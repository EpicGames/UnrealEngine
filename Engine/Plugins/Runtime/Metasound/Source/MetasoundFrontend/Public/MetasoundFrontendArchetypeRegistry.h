// Copyright Epic Games, Inc. All Rights Reserved.
// DEPRECATED MetasoundFrontendArchetypeRegistry.h. Use Interfaces/MetasoundFrontendInterfaceRegistry.h instead
#pragma once

#include "Misc/CoreMiscDefines.h"

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.3, "Use Interfaces/MetasoundFrontendInterfaceRegistry.h instead instead of MetasoundFrontendArchetypeRegistry.h.")

#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
