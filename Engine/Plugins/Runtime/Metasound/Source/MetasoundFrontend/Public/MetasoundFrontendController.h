// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentAccessPtr.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundVertex.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"

#define UE_API METASOUNDFRONTEND_API


/* Metasound Controllers and Handles provide a object oriented interface for  manipulating Metasound Documents. 
 *
 * Each controller interface is associated with a single Metasound entity such as 
 * Document, Graph, Node, Input or Output. Access to these entities generally begins
 * with the DocumentController which provides access to Graphs. Graphs provide access
 * to Nodes and Nodes provide access to Inputs and Outputs. 
 *
 * A "Handle" is simply a TSharedRef<> of a controller.
 *
 * In general, the workflow for editing a Metasound graph will be:
 * 1) Load or create a metasound asset.
 * 2) Call UMetaSoundPatch::GetDocumentHandle() to get a handle to the document for that asset.
 * 
 * Typically the workflow for creating a Metasound subgraph will be
 * 1) Get a Metasound::Frontend::FGraphHandle (typically through the two steps described in the first paragraph)
 * 2) Build a FMetasoundFrontendClassMetadata struct with whatever name/author/description you want to give the subgraph,
 * 3) call Metasound::Frontend::FGraphHandle::CreateEmptySubgraphNode, which will return that subgraph as it as a FNodeHandle for that subgraph in the current graph.
 * 4) class Metasound::Frontend::FNodeHandle::AsGraph which provides access to edit the subgraphs internal structure as well as externally facing inputs and outputs.
 *
 * General note- these apis are NOT thread safe. 
 * Make sure that any FDocumentHandle, FGraphHandle, FNodeHandle, FInputHandle and FOutputHandle that access similar data are called on the same thread.
 */

struct FNodeTemplateGenerateInterfaceParams;

namespace Metasound
{
	namespace Frontend
	{
		// Forward declare 
		class IDocumentController;
		class IGraphController;
		class IInputController;
		class IMetaSoundAssetManager;
		class INodeController;
		class IOutputController;
		class IVariableController;

		// Metasound Frontend Handles are all TSharedRefs of various Metasound Frontend Controllers.
		using FInputHandle = TSharedRef<IInputController>;
		using FConstInputHandle = TSharedRef<const IInputController>;
		using FOutputHandle = TSharedRef<IOutputController>;
		using FConstOutputHandle = TSharedRef<const IOutputController>;
		using FVariableHandle = TSharedRef<IVariableController>;
		using FConstVariableHandle = TSharedRef<const IVariableController>;
		using FNodeHandle = TSharedRef<INodeController>;
		using FConstNodeHandle = TSharedRef<const INodeController>;
		using FGraphHandle = TSharedRef<IGraphController>;
		using FConstGraphHandle = TSharedRef<const IGraphController>;
		using FDocumentHandle = TSharedRef<IDocumentController>;
		using FConstDocumentHandle = TSharedRef<const IDocumentController>;


		// Container holding various access pointers to the FMetasoundFrontendDocument
		struct FConstDocumentAccess
		{
			FConstVertexAccessPtr ConstVertex;
			FConstClassInputAccessPtr ConstClassInput;
			FConstClassOutputAccessPtr ConstClassOutput;
			FConstNodeAccessPtr ConstNode;
			FConstClassAccessPtr ConstClass;
			FConstGraphClassAccessPtr ConstGraphClass;
			FConstGraphAccessPtr ConstGraph;
			FConstDocumentAccessPtr ConstDocument;
		};

		// Container holding various access pointers to the FMetasoundFrontendDocument
		struct FDocumentAccess : public FConstDocumentAccess
		{
			FVertexAccessPtr Vertex; 
			FClassInputAccessPtr ClassInput;
			FClassOutputAccessPtr ClassOutput;
			FNodeAccessPtr Node;
			FClassAccessPtr Class;
			FGraphClassAccessPtr GraphClass;
			FGraphAccessPtr Graph;
			FDocumentAccessPtr Document;
		};

		/** IDocumentAccessor describes an interface for various I*Controllers to interact with
		 * each other without exposing that interface publicly or requiring friendship 
		 * between various controller implementation classes. 
		 */
		class IDocumentAccessor
		{
			protected:
				/** Share access to FMetasoundFrontendDocument objects. 
				 *
				 * Derived classes must implement this method. In the implementation,
				 * derived classes should set the various TAccessPtrs on FDocumentAccess 
				 * to the TAccessPtrs held internal in the derived class.
				 *
				 * Sharing access simplifies operations involving multiple frontend controllers
				 * by providing direct access to the FMetasoundFrontendDocument objects to be
				 * edited. 
				 */
				virtual FDocumentAccess ShareAccess() = 0;

				/** Share access to FMetasoundFrontendDocument objects. 
				 *
				 * Derived classes must implement this method. In the implementation,
				 * derived classes should set the various TAccessPtrs on FDocumentAccess 
				 * to the TAccessPtrs held internal in the derived class.
				 *
				 * Sharing access simplifies operations involving multiple frontend controllers
				 * by providing direct access to the FMetasoundFrontendDocument objects to be
				 * edited. 
				 */
				virtual FConstDocumentAccess ShareAccess() const = 0;

				/** Returns the shared access from an IDocumentAccessor. */
				static UE_API FDocumentAccess GetSharedAccess(IDocumentAccessor& InDocumentAccessor);

				/** Returns the shared access from an IDocumentAccessor. */
				static UE_API FConstDocumentAccess GetSharedAccess(const IDocumentAccessor& InDocumentAccessor);
		};
		
		/* An IOutputController provides methods for querying and manipulating a metasound output vertex. */
		class IOutputController : public TSharedFromThis<IOutputController>, public IDocumentAccessor
		{
		public:
			static UE_API FOutputHandle GetInvalidHandle();

			IOutputController() = default;
			virtual ~IOutputController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			virtual FGuid GetID() const = 0;
			
			/** Returns the data type name associated with this output. */
			virtual const FName& GetDataType() const = 0;
			
			/** Returns the name associated with this output. */
			virtual const FVertexName& GetName() const = 0;

			/** Returns the vertex access type. */
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const = 0;
			
#if WITH_EDITOR
			/** Returns the human readable name associated with this output. */
			virtual FText GetDisplayName() const = 0;
			
			/** Returns the tooltip associated with this output. */
			virtual const FText& GetTooltip() const = 0;

			/** Returns all metadata associated with this output. */
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const = 0;
#endif // WITH_EDITOR

			/** Returns the ID of the node which owns this output. */
			virtual FGuid GetOwningNodeID() const = 0;
			
			/** Returns a FNodeHandle to the node which owns this output. */
			virtual FNodeHandle GetOwningNode() = 0;
			
			/** Returns a FConstNodeHandle to the node which owns this output. */
			virtual FConstNodeHandle GetOwningNode() const = 0;

			/** This should only be used as a means of fixing up vertex names for document model versioning transform(s). */
			virtual void SetName(const FVertexName& InName) = 0;

			/** Returns true if the output connections can be directly modified by 
			 * a user.  Returns false otherwise. */
			virtual bool IsConnectionUserModifiable() const = 0;

			/** Return true if the input is connect to an output. */
			virtual bool IsConnected() const = 0;

			/** Returns the currently connected output. If this input is not
			 * connected, the returned handle will be invalid. */
			virtual TArray<FInputHandle> GetConnectedInputs() = 0;

			/** Returns the currently connected output. If this input is not
			 * connected, the returned handle will be invalid. */
			virtual TArray<FConstInputHandle> GetConstConnectedInputs() const = 0;

			virtual bool Disconnect() = 0;

			/** Returns information describing connectability between this output and the supplied input. */
			virtual FConnectability CanConnectTo(const IInputController& InController) const = 0;
			
			/** Connects this output and the supplied input. 
			 * @return True on success, false on failure. 
			 */
			virtual bool Connect(IInputController& InController) = 0;
			
			/** Connects this output to the input using a converter node.
			 * @return True on success, false on failure. 
			 */
			virtual bool ConnectWithConverterNode(IInputController& InController, const FConverterNodeInfo& InNodeClassName) = 0;
			
			/** Disconnects this output from the input. 
			 * @return True on success, false on failure. 
			 */
			virtual bool Disconnect(IInputController& InController) = 0;

		};

		class IVariableController : public TSharedFromThis<IVariableController>, public IDocumentAccessor
		{
		public:
			static UE_API FVariableHandle GetInvalidHandle();

			IVariableController() = default;
			virtual ~IVariableController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			/** Returns the variable ID. */
			virtual FGuid GetID() const = 0;
			
			/** Returns the data type name associated with this variable. */
			virtual const FName& GetDataType() const = 0;

			/** Returns the name associated with this variable. */
			virtual const FName& GetName() const = 0;

			/** Sets the name associated with this variable. */
			virtual void SetName(const FName& InName) = 0;
			
#if WITH_EDITOR
			/** Returns the human readable name associated with this variable. */
			virtual FText GetDisplayName() const = 0;

			/** Sets the human readable name associated with this variable. */
			virtual void SetDisplayName(const FText& InDisplayName) = 0;
			
			/** Returns the human readable description associated with this variable. */
			virtual FText GetDescription() const = 0;

			/** Sets the human readable description associated with this variable. */
			virtual void SetDescription(const FText& InDescription) = 0;
#endif // WITH_EDITOR

			/** Returns the mutator node associated with this variable. */
			virtual FNodeHandle FindMutatorNode() = 0;

			/** Returns the mutator node associated with this variable. */
			virtual FConstNodeHandle FindMutatorNode() const = 0;

			/** Returns the accessor nodes associated with this variable. */
			virtual TArray<FNodeHandle> FindAccessorNodes() = 0;

			/** Returns the accessor nodes associated with this variable. */
			virtual TArray<FConstNodeHandle> FindAccessorNodes() const = 0;

			/** Returns the deferred accessor nodes associated with this variable. */
			virtual TArray<FNodeHandle> FindDeferredAccessorNodes() = 0;

			/** Returns the deferred accessor nodes associated with this variable. */
			virtual TArray<FConstNodeHandle> FindDeferredAccessorNodes() const = 0;
			
			/** Returns a FGraphHandle to the node which owns this variable. */
			virtual FGraphHandle GetOwningGraph() = 0;
			
			/** Returns a FConstGraphHandle to the node which owns this variable. */
			virtual FConstGraphHandle GetOwningGraph() const = 0;

			/** Returns the value for the given variable instance if set. */
			virtual const FMetasoundFrontendLiteral& GetLiteral() const = 0;

			/** Sets the value for the given variable instance */
			virtual bool SetLiteral(const FMetasoundFrontendLiteral& InLiteral) = 0;
		};

		/* An IInputController provides methods for querying and manipulating a metasound input vertex. */
		class IInputController : public TSharedFromThis<IInputController>, public IDocumentAccessor
		{
		public:
			static UE_API FInputHandle GetInvalidHandle();

			IInputController() = default;
			virtual ~IInputController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			virtual FGuid GetID() const = 0;

			/** Return true if the input is connect to an output. */
			virtual bool IsConnected() const = 0;

			/** Returns the data type name associated with this input. */
			virtual const FName& GetDataType() const = 0;

			/** Returns the data type name associated with this input. */
			virtual const FVertexName& GetName() const = 0;

			/** Returns the vertex access type. */
			virtual EMetasoundFrontendVertexAccessType GetVertexAccessType() const = 0;

#if WITH_EDITOR
			/** Returns the data type name associated with this input. */
			virtual FText GetDisplayName() const = 0;
#endif // WITH_EDITOR

			/** Clears the value for the given input instance if set. */
			virtual bool ClearLiteral() = 0;

			/** Returns the value for the given input instance if set. */
			virtual const FMetasoundFrontendLiteral* GetLiteral() const = 0;

			/** Sets the value for the given input instance (effectively overriding the class default). */
			virtual void SetLiteral(const FMetasoundFrontendLiteral& InLiteral) = 0;

			/** Returns the class default value of the given input. */
			virtual const FMetasoundFrontendLiteral* GetClassDefaultLiteral() const = 0;

#if WITH_EDITOR
			/** Returns the data type name associated with this input. */
			virtual const FText& GetTooltip() const = 0;

			/** Returns all metadata associated with this input. */
			virtual const FMetasoundFrontendVertexMetadata& GetMetadata() const = 0;
#endif // WITH_EDITOR

			/** Returns the ID of the node which owns this output. */
			virtual FGuid GetOwningNodeID() const = 0;
			
			/** Returns a FNodeHandle to the node which owns this output. */
			virtual FNodeHandle GetOwningNode() = 0;
			
			/** Returns a FConstNodeHandle to the node which owns this output. */
			virtual FConstNodeHandle GetOwningNode() const = 0;
			
			/** Returns the currently connected output. If this input is not
			 * connected, the returned handle will be invalid. */
			virtual FOutputHandle GetConnectedOutput() = 0;

			/** Returns the currently connected output. If this input is not
			 * connected, the returned handle will be invalid. */
			virtual FConstOutputHandle GetConnectedOutput() const = 0;

			/** This should only be used as a means of fixing up vertex names for document model versioning transform(s). */
			virtual void SetName(const FVertexName& InName) = 0;
			
			/** Returns true if the input connections can be directly modified by 
			 * a user.  Returns false otherwise. */
			virtual bool IsConnectionUserModifiable() const = 0;

			/** Returns information describing connectability between this input and the supplied output. */
			virtual FConnectability CanConnectTo(const IOutputController& InController) const = 0;

			/** Connects this input and the supplied output. 
			 * @return True on success, false on failure. 
			 */
			virtual bool Connect(IOutputController& InController) = 0;

			/** Connects this input to the output using a converter node.
			 * @return True on success, false on failure. 
			 */
			virtual bool ConnectWithConverterNode(IOutputController& InController, const FConverterNodeInfo& InNodeClassName) = 0;

			/** Disconnects this input from the given output. 
			 * @return True on success, false on failure. 
			 */
			virtual bool Disconnect(IOutputController& InController) = 0;

			/** Disconnects this input from any connected output. 
			 * @return True on success, false on failure. 
			 */
			virtual bool Disconnect() = 0;

			/** Clear object literals to prevent referencing assets
			 * on connected inputs that are not used in the graph. */
			virtual void ClearConnectedObjectLiterals() = 0;
		};

		/* An INodeController provides methods for querying and manipulating a Metasound node. */
		class INodeController : public TSharedFromThis<INodeController>, public IDocumentAccessor
		{

		public:
			static UE_API FNodeHandle GetInvalidHandle();

			using FVertexNameAndType = TTuple<FVertexName, FName>;

			INodeController() = default;
			virtual ~INodeController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			/** Returns all node inputs. */
			virtual TArray<FInputHandle> GetInputs() = 0;

			/** Returns all node inputs. */
			virtual TArray<FConstInputHandle> GetConstInputs() const = 0;

#if WITH_EDITOR
			/** Returns the display name of the given node (what to distinguish and label in visual arrays, such as context menus). */
			virtual FText GetDisplayName() const = 0;

			/** Sets the description of the node. */
			virtual void SetDescription(const FText& InDescription) = 0;

			/** Sets the display name of the node. */
			virtual void SetDisplayName(const FText& InDisplayName) = 0;

			/** Returns the title of the given node (what to label when displayed as visual node). */
			virtual const FText& GetDisplayTitle() const = 0;
#endif // WITH_EDITOR

			virtual void SetNodeName(const FVertexName& InName) = 0;

			/** Iterate over inputs */
			virtual void IterateInputs(TUniqueFunction<void(FInputHandle)> InFunction) = 0;
			virtual void IterateConstInputs(TUniqueFunction<void(FConstInputHandle)> InFunction) const = 0;

			/** Returns number of node inputs. */
			virtual int32 GetNumInputs() const = 0;

			/** Iterate over outputs */
			virtual void IterateOutputs(TUniqueFunction<void(FOutputHandle)> InFunction) = 0;
			virtual void IterateConstOutputs(TUniqueFunction<void(FConstOutputHandle)> InFunction) const = 0;

			/** Returns number of node outputs. */
			virtual int32 GetNumOutputs() const = 0;

			virtual FInputHandle GetInputWithVertexName(const FVertexName& InName) = 0;
			virtual FConstInputHandle GetConstInputWithVertexName(const FVertexName& InName) const = 0;

			/** Returns all node outputs. */
			virtual TArray<FOutputHandle> GetOutputs() = 0;

			/** Returns all node outputs. */
			virtual TArray<FConstOutputHandle> GetConstOutputs() const = 0;

			virtual FOutputHandle GetOutputWithVertexName(const FVertexName& InName) = 0;
			virtual FConstOutputHandle GetConstOutputWithVertexName(const FVertexName& InName) const = 0;

			virtual bool IsInterfaceMember() const = 0;
			virtual const FMetasoundFrontendVersion& GetInterfaceVersion() const = 0;

			/**
			  * Replaces this node with a new node of the provided version number, and attempts to 
			  * rebuild edges where possible with matching vertex names that share the same DataType.
			  * Returns a node handle to the new node.  If operation fails, returns a handle to this node.
			  */
			virtual FNodeHandle ReplaceWithVersion(const FMetasoundFrontendVersionNumber& InNewVersion, TArray<FVertexNameAndType>* OutDisconnectedInputs, TArray<FVertexNameAndType>* OutDisconnectedOutputs) = 0;


			/** Returns an input with the given id.
			 *
			 * If the input does not exist, an invalid handle is returned.
			 */
			virtual FInputHandle GetInputWithID(FGuid InVertexID) = 0;

			/** Returns an input with the given name. 
			 *
			 * If the input does not exist, an invalid handle is returned.
			 */
			virtual FConstInputHandle GetInputWithID(FGuid InVertexID) const = 0;

			/** Returns an output with the given name. 
			 *
			 * If the output does not exist, an invalid handle is returned. 
			 */
			virtual FOutputHandle GetOutputWithID(FGuid InVertexID) = 0;

			/** Returns an output with the given name. 
			 *
			 * If the output does not exist, an invalid handle is returned. 
			 */
			virtual FConstOutputHandle GetOutputWithID(FGuid InVertexID) const = 0;

			virtual bool CanAddInput(const FVertexName& InVertexName) const = 0;
			virtual FInputHandle AddInput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) = 0;
			virtual bool RemoveInput(FGuid InVertexID) = 0;

			virtual bool CanAddOutput(const FVertexName& InVertexName) const = 0;
			virtual FInputHandle AddOutput(const FVertexName& InVertexName, const FMetasoundFrontendLiteral* InDefault) = 0;
			virtual bool RemoveOutput(FGuid InVertexID) = 0;

			// Returns an input's default literal if set, null if not.
			virtual const FMetasoundFrontendLiteral* GetInputLiteral(const FGuid& InVertexID) const = 0;

			// Sets an input's default literal
			virtual void SetInputLiteral(const FMetasoundFrontendVertexLiteral& InVertexLiteral) = 0;

			// Clears an input's default literal
			virtual bool ClearInputLiteral(FGuid InVertexID) = 0;

			virtual const FMetasoundFrontendClassMetadata& GetClassMetadata() const = 0;
			virtual const FMetasoundFrontendClassInterface& GetClassInterface() const = 0;

			/** Returns the node interface, which may be different than the class interface
			  * if the class supports dynamic input/output behavior (ex. Templates).
			  */
			virtual const FMetasoundFrontendNodeInterface& GetNodeInterface() const = 0;
			
#if WITH_EDITOR
			/** Returns associated node class data */
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const = 0;
			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const = 0;
			virtual const FMetasoundFrontendClassStyle& GetClassStyle() const = 0;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
			UE_DEPRECATED(5.6, "Moved to FMetaSoundFrontendDocumentBuilder")
			virtual bool DiffAgainstRegistryInterface(FClassInterfaceUpdates& OutInterfaceUpdates, bool bInUseHighestMinorVersion, bool bInForceRegenerateClassInterfaceOverride=true) const = 0;

			/** Returns whether the node is eligible for auto-updating (i.e.
			  * has undergone minor revision or the interface has changed, but
			  * no higher major revision is available). Provides interface updates
			  * to populate with any information regarding interface updates.
			  * Can return true if the interface has changed but only cosmetic
			  * differences (ex. DisplayName only used in editor) but no runtime
			  * behavior has been modified.
			  */
			UE_DEPRECATED(5.7, "Moved to FMetaSoundFrontendDocumentBuilder")
			virtual bool CanAutoUpdate(FClassInterfaceUpdates& OutInterfaceUpdates) const = 0;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
			/** Description of the given node. */
			virtual const FText& GetDescription() const = 0;
#endif // WITH_EDITOR

			/** If the node is also a graph, this returns a graph handle.
			 * If the node is not also a graph, it will return an invalid handle.
			 */
			virtual FGraphHandle AsGraph() = 0;

			/** If the node is also a graph, this returns a graph handle.
			 * If the node is not also a graph, it will return an invalid handle.
			 */
			virtual FConstGraphHandle AsGraph() const = 0;

			/** Returns the name of this node. */
			virtual const FVertexName& GetNodeName() const = 0;

#if WITH_EDITORONLY_DATA
			virtual const FMetasoundFrontendNodeStyle& GetNodeStyle() const = 0;
			virtual void SetNodeStyle(const FMetasoundFrontendNodeStyle& InStyle) = 0;
#endif // WITH_EDITORONLY_DATA

			virtual void SetNodeConfiguration(TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfig) = 0;
			virtual TConstStructView<FMetaSoundFrontendNodeConfiguration> GetNodeConfiguration() const = 0;

			/** Returns the ID associated with this node. */
			virtual FGuid GetID() const = 0;

			/** Returns the ID associated with the node class. */
			virtual FGuid GetClassID() const = 0;

			/** Returns the class ID of the metasound class which owns this node. */
			virtual FGuid GetOwningGraphClassID() const = 0;

			/** Returns the graph which owns this node. */
			virtual FGraphHandle GetOwningGraph() = 0;

			/** Returns the graph which owns this node. */
			virtual FConstGraphHandle GetOwningGraph() const = 0;
		};

		/* An IGraphController provides methods for querying and manipulating a Metasound graph. */
		class IGraphController : public TSharedFromThis<IGraphController>, public IDocumentAccessor
		{
			public:
			static UE_API FGraphHandle GetInvalidHandle();

			IGraphController() = default;
			virtual ~IGraphController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			/** Returns the ClassID associated with this graph. */
			virtual FGuid GetClassID() const = 0;

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			virtual const FMetasoundFrontendGraphClassPresetOptions& GetGraphPresetOptions() const;

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			virtual void SetGraphPresetOptions(const FMetasoundFrontendGraphClassPresetOptions& InPresetOptions);

			/** Return the metadata for the current graph. */
			virtual const FMetasoundFrontendClassMetadata& GetGraphMetadata() const = 0;

			/** Sets the metadata for the current graph. */
			virtual void SetGraphMetadata(const FMetasoundFrontendClassMetadata& InMetadata) = 0;

#if WITH_EDITOR
			// Returns graph style.
			virtual const FMetasoundFrontendGraphStyle& GetGraphStyle() const = 0;

			virtual const FMetasoundFrontendInterfaceStyle& GetInputStyle() const = 0;
			virtual const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const = 0;

			// Sets graph style.
			virtual void SetGraphStyle(FMetasoundFrontendGraphStyle Style) = 0;

			// Sets the input style for the graph.
			virtual void SetInputStyle(FMetasoundFrontendInterfaceStyle Style) = 0;

			// Sets the output style for the graph.
			virtual void SetOutputStyle(FMetasoundFrontendInterfaceStyle Style) = 0;

			/** Return the display name of the graph. */
			virtual FText GetDisplayName() const = 0;
#endif // WITH_EDITOR

			virtual TArray<FVertexName> GetInputVertexNames() const = 0;
			virtual TArray<FVertexName> GetOutputVertexNames() const = 0;

			/** Returns all nodes in the graph. */
			virtual TArray<FNodeHandle> GetNodes() = 0;

			/** Returns a node by NodeID. If the node does not exist, an invalid handle is returned. */
			virtual FConstNodeHandle GetNodeWithID(FGuid InNodeID) const = 0;

			/** Returns all nodes in the graph. */
			virtual TArray<FConstNodeHandle> GetConstNodes() const = 0;

			/** Returns a node by NodeID. If the node does not exist, an invalid handle is returned. */
			virtual FNodeHandle GetNodeWithID(FGuid InNodeID) = 0;

			/** Returns all output nodes in the graph. */
			virtual TArray<FNodeHandle> GetOutputNodes() = 0;

			/** Returns all output nodes in the graph. */
			virtual TArray<FConstNodeHandle> GetConstOutputNodes() const = 0;

			/** Returns all input nodes in the graph. */
			virtual TArray<FNodeHandle> GetInputNodes() = 0;

			/** Returns all input nodes in the graph. */
			virtual TArray<FConstNodeHandle> GetConstInputNodes() const = 0;

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			virtual const TSet<FName>& GetInputsInheritingDefault() const;

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			virtual bool SetInputInheritsDefault(FName InName, bool bDefaultIsInherited);

			UE_DEPRECATED(5.8, "Preset options are now serialized in the Preset MetaSound Template")
			virtual void SetInputsInheritingDefault(TSet<FName>&& InNames);

			/** Adds a new variable to the graph */
			virtual FVariableHandle AddVariable(const FName& InDataTypeName) = 0;

			/** Finds a variable by ID. 
			 *
			 * @param InVariableID - ID of existing variable.
			 */
			virtual FVariableHandle FindVariable(const FGuid& InVariableID) = 0;

			/** Finds a variable by ID. 
			 *
			 * @param InVariableID - ID of existing variable.
			 */
			virtual FConstVariableHandle FindVariable(const FGuid& InVariableID) const = 0;

			/** Finds a variable inspecting the nodes associated with the variable.
			 *
			 * @param InNodeID - ID of node associated with variable.
			 * @return If found, returns valid variable handle. An invalid handle otherwise. 
			 */
			virtual FVariableHandle FindVariableContainingNode(const FGuid& InNodeID) = 0;

			/** Finds a variable inspecting the nodes associated with the variable.
			 *
			 * @param InNodeID - ID of node associated with variable.
			 * @return If found, returns valid variable handle. An invalid handle otherwise. 
			 */
			virtual FConstVariableHandle FindVariableContainingNode(const FGuid& InNodeID) const = 0;

			UE_DEPRECATED(5.8, "Use the MetaSound Builder API")
			virtual bool RemoveVariable(const FGuid& InVariableID) { return false; }

			/* Returns an array of all variables associated with the graph. */
			virtual TArray<FVariableHandle> GetVariables() = 0;

			/* Returns an array of all variables associated with the graph. */
			virtual TArray<FConstVariableHandle> GetVariables() const = 0;

			/** Returns the variable mutator node. If none exist, one is created. 
			 *
			 * @param InVariableID - ID of existing variable.
			 */
			virtual FNodeHandle FindOrAddVariableMutatorNode(const FGuid& InVariableID) = 0;

			/** Creates and returns a variable accessor node. 
			 *
			 * @param InVariableID - ID of existing variable.
			 */
			virtual FNodeHandle AddVariableAccessorNode(const FGuid& InVariableID) = 0;

			/** Creates and returns a variable deferred accessor node. 
			 *
			 * @param InVariableID - ID of existing variable.
			 */
			virtual FNodeHandle AddVariableDeferredAccessorNode(const FGuid& InVariableID) = 0;


			/** Clears the graph, its associated interface, and synchronizes removed dependencies with the owning graph. */
			virtual void ClearGraph() = 0;

			/** Iterate over all input nodes with the given function. If ClassType is specified, only iterate over given type. */
			virtual void IterateNodes(TFunctionRef<void(FNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType = EMetasoundFrontendClassType::Invalid) = 0;

			/** Iterate over all nodes with the given function. If ClassType is specified, only iterate over given type. */
			virtual void IterateConstNodes(TFunctionRef<void(FConstNodeHandle)> InFunction, EMetasoundFrontendClassType InClassType = EMetasoundFrontendClassType::Invalid) const = 0;

			/** Returns true if an output vertex with the given Name exists.
			 *
			 * @param InName - Name of vertex.
			 * @param InTypeName - DataType Name of vertex.
			 * @return True if the vertex exists, false otherwise.
			 */
			virtual bool ContainsOutputVertex(const FVertexName& InName, const FName& InTypeName) const = 0;

			/** Returns true if an output vertex with the given Name exists.
			 *
			 * @param InName - Name of vertex.
			 * @param InTypeName - DataType Name of vertex.
			 * @return True if the vertex exists, false otherwise.
			 */
			virtual bool ContainsOutputVertexWithName(const FVertexName& InName) const = 0;

			/** Returns true if an input vertex with the given Name exists.
			 *
			 * @param InName - Name of vertex.
			 * @return True if the vertex exists, false otherwise.
			 */
			virtual bool ContainsInputVertex(const FVertexName& InName, const FName& InTypeName) const = 0;

			/** Returns true if an input vertex with the given Name exists.
			 *
			 * @param InName - Name of vertex.
			 * @return True if the vertex exists, false otherwise. 
			 */
			virtual bool ContainsInputVertexWithName(const FVertexName& InName) const = 0;

			/** Returns a handle to an existing output node for the given graph output name.
			 * If no output exists with the given name, an invalid node handle is returned. 
			 *
			 * @param InName - Name of graph output.. 
			 * @return The node handle for the output node. If the output does not exist, an invalid handle is returned.
			 */
			virtual FNodeHandle GetOutputNodeWithName(const FVertexName& InName) = 0;

			/** Returns a handle to an existing output node for the given graph output name.
			 * If no output exists with the given name, an invalid node handle is returned. 
			 *
			 * @param InName - Name of graph output.
			 * @return The node handle for the output node. If the node does not exist, an invalid handle is returned.
			 */
			virtual FConstNodeHandle GetOutputNodeWithName(const FVertexName& InName) const = 0;

			/** Returns a handle to an existing input node for the given graph input name.
			 * If no input exists with the given name, an invalid node handle is returned. 
			 *
			 * @param InName - Name of graph input. 
			 * @return The node handle for the input node. If the node does not exist, an invalid handle is returned.
			 */
			virtual FNodeHandle GetInputNodeWithName(const FVertexName & InName) = 0;

			/** Returns a handle to an existing input node for the given graph input name.
			 * If no input exists with the given name, an invalid node handle is returned. 
			 *
			 * @param InName - Name of graph input. 
			 * @return The node handle for the input node. If the node does not exist, an invalid handle is returned.
			 */
			virtual FConstNodeHandle GetInputNodeWithName(const FVertexName& InName) const = 0;

			virtual FConstClassInputAccessPtr FindClassInputWithName(const FVertexName& InName) const = 0;
			virtual FConstClassOutputAccessPtr FindClassOutputWithName(const FVertexName& InName) const = 0;

			/** Add a new input node using the input description. 
			 *
			 * @param InDescription - Description for input of graph.
			 * @return On success, a valid input node handle. On failure, an invalid node handle.
			 */
			virtual FNodeHandle AddInputVertex(const FMetasoundFrontendClassInput& InDescription) = 0;

			/** Remove the input with the given name. Returns true if successfully removed, false otherwise. */
			virtual bool RemoveInputVertex(const FVertexName& InputName) = 0;

			/** Add a new output node using the output description. 
			 *
			 * @param InDescription - Description for output of graph.
			 * @return On success, a valid output node handle. On failure, an invalid node handle.
			 */
			virtual FNodeHandle AddOutputVertex(const FMetasoundFrontendClassOutput& InDescription) = 0;
			virtual FNodeHandle AddOutputVertex(const FVertexName& InName, const FName InTypeName) = 0;

			/** Remove the output with the given name. Returns true if successfully removed, false otherwise. */
			virtual bool RemoveOutputVertex(const FVertexName& OutputName) = 0;

			/** Returns the preferred literal argument type for a given input.
			 * Returns ELiteralType::Invalid if the input couldn't be found, 
			 * or if the input doesn't support any kind of literals.
			 *
			 * @param InInputName - Name of graph input.
			 * @return The preferred literal argument type. 
			 */
			virtual ELiteralType GetPreferredLiteralTypeForInputVertex(const FVertexName& InInputName) const = 0;

			/** Return the UObject class corresponding an input. Meaningful for inputs whose preferred 
			 * literal type is UObject or UObjectArray.
			 *
			 * @param InInputName - Name of graph input.
			 *
			 * @return The UClass* for the literal argument input. nullptr on error or if UObject argument is not supported.
			 */
			virtual UClass* GetSupportedClassForInputVertex(const FVertexName& InInputName) = 0;

			virtual FGuid GetVertexIDForInputVertex(const FVertexName& InInputName) const = 0;
			virtual FGuid GetVertexIDForOutputVertex(const FVertexName& InOutputName) const = 0;

			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID) const = 0;

			virtual FMetasoundFrontendLiteral GetDefaultInput(const FGuid& InVertexID, TArrayView<const FGuid> InPageOrder) const = 0;

			// These can be used to set the default value for a given input on this graph.
			// @returns false if the input name couldn't be found, or if the literal type was incompatible with the Data Type of this input.

			UE_DEPRECATED(5.7, "Use version which provides page order")
			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral) = 0;

			virtual bool SetDefaultInput(const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, TArrayView<const FGuid> InPageOrder) = 0;

			/** Set the default value for the graph input.
			 *
			 * @param InInputName - Name of the graph input.
			 * @param InVertexID - Vertex to set to DataType default.
			 * @param InDataTypeName - Name of datatype to set to default.
			 *
			 * @return True on success. False if the input does not exist or if the literal type was incompatible with the input.
			 */
			virtual bool SetDefaultInputToDefaultLiteralOfType(const FGuid& InVertexID) = 0;

#if WITH_EDITOR
			/** Set the display name for the input with the given name. */
			virtual void SetInputDisplayName(const FVertexName& InName, const FText& InDisplayName) = 0;

			/** Set the display name for the output with the given name. */
			virtual void SetOutputDisplayName(const FVertexName& InName, const FText& InDisplayName) = 0;

			/** Get the description for the input with the given name. */
			virtual const FText& GetInputDescription(const FVertexName& InName) const = 0;

			/** Get the description for the output with the given name. */
			virtual const FText& GetOutputDescription(const FVertexName& InName) const = 0;

			/** Set the description for the input with the given name. */
			virtual void SetInputDescription(const FVertexName& InName, const FText& InDescription) = 0;

			/** Set the description for the output with the given name. */
			virtual void SetOutputDescription(const FVertexName& InName, const FText& InDescription) = 0;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
			/** Returns the sort order index for the input with the given name. Returns 0 if not found or unset. */
			virtual int32 GetSortOrderIndexForInput(const FVertexName& InName) const = 0;

			/** Returns the sort order index for the input with the given name. Returns 0 if not found or unset. */
			virtual int32 GetSortOrderIndexForOutput(const FVertexName& InName) const = 0;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
			/** Sets the sort order index for the input with the given name. No-ops if input not found. */
			virtual void SetSortOrderIndexForInput(const FVertexName& InName, int32 InSortOrderIndex) = 0;

			/** Sets the sort order index for the output with the given name. No-ops if output not found. */
			virtual void SetSortOrderIndexForOutput(const FVertexName& InName, int32 InSortOrderIndex) = 0;
#endif // WITH_EDITOR

			/**
			  * Updates the ChangeID for the class interface, which signals AutoUpdate to
			  * attempt to patch class references at runtime even if the graph class has not
			  * been versioned. TODO: Remove this once underlying architecture no longer requires it.
			  */
			virtual void UpdateInterfaceChangeID() = 0;

			UE_DEPRECATED(5.8, "Use Builder API instead")
			virtual bool ClearLiteralForInput(const FVertexName& InInputName, FGuid InVertexID) { return false;  }

			/** Add a new node to this graph from the node registry.
			 *
			 * @param InKey - Registry key for node.
			 * @param InNodeGuid - (Optional) Explicit guid for the new node. Must be unique within the graph.
			 * Only useful to specify explicitly when caller is managing or tracking the graph's guids (ex. replacing removed node).
			 * 
			 * @return Node handle for class. On error, an invalid handle is returned.
			 */
			virtual FNodeHandle AddNode(const FNodeRegistryKey& InKey, FGuid InNodeGuid = FGuid::NewGuid()) = 0;

			/** Add a new node to this graph.
			 *
			 * @param InClassMetadata - Info for node class.
			 * @param InNodeGuid - (Optional) Explicit guid for the new node. Must be unique within the graph.
			 * Only useful to specify explicitly when caller is managing or tracking the graph's guids (ex. replacing removed node).
			 *
			 * @return Node handle for class. On error, an invalid handle is returned.
			 */
			virtual FNodeHandle AddNode(const FMetasoundFrontendClassMetadata& InClassMetadata, FGuid InNodeGuid = FGuid::NewGuid()) = 0;

			/** Add a new node to this graph by duplicating the supplied node.
			 * The new node has the same interface and node class as the supplied node.
			 * This method will not duplicate node connections.
			 *
			 * @param InNodeController - Node to duplicate.
			 *
			 * @return Node handle for new node. On error, an invalid handle is returned. 
			 */
			virtual FNodeHandle AddDuplicateNode(const INodeController& InNodeController) = 0;

			/** Add a new template node to this graph, providing the defined interface as expected by the caller.
			 *
			 * @param InNodeTemplate - Node template to use to generate node
			 * @param InNodeGuid - (Optional) Explicit guid for the new node. Must be unique within the graph.
			 * Only useful to specify explicitly when caller is managing or tracking the graph's guids (ex. replacing removed node).
			 *
			 * @return Node handle for class. On error, an invalid handle is returned.
			 */
			virtual FNodeHandle AddTemplateNode(const INodeTemplate& InNodeTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeGuid = FGuid::NewGuid()) = 0;

			UE_DEPRECATED(5.5, "Use overload that provides INodeTemplate instead")
			virtual FNodeHandle AddTemplateNode(const FNodeRegistryKey& InKey, FMetasoundFrontendNodeInterface&& InNodeInterface, FGuid InNodeGuid = FGuid::NewGuid()) { return INodeController::GetInvalidHandle(); }

			/** Remove the node corresponding to this node handle.
			 *
			 * @return True on success, false on failure. 
			 */
			virtual bool RemoveNode(INodeController& InNode) = 0;

			/** Creates and inserts a new subgraph into this graph using the given metadata.
			 * By calling AsGraph() on the returned node handle, callers can modify
			 * the new subgraph.
			 *
			 * @param InInfo - Metadata for the subgraph.
			 *
			 * @return Handle to the subgraph node. On error, the handle is invalid.
			 */
			virtual FNodeHandle CreateEmptySubgraph(const FMetasoundFrontendClassMetadata& InInfo) = 0;

			/** Returns a handle to the document owning this graph. */
			virtual FDocumentHandle GetOwningDocument() = 0;

			/** Returns a handle to the document owning this graph. */
			virtual FConstDocumentHandle GetOwningDocument() const = 0;

			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithName(const FVertexName& InName) const = 0;
			virtual const FMetasoundFrontendClassInput* FindInputDescriptionWithVertexID(const FGuid& InVertexID) const = 0;
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithName(const FVertexName& InName) const = 0;
			virtual const FMetasoundFrontendClassOutput* FindOutputDescriptionWithVertexID(const FGuid& InVertexID) const = 0;
		};

		/* An IDocumentController provides methods for querying and manipulating a Metasound document. */
		class IDocumentController : public TSharedFromThis<IDocumentController>, public IDocumentAccessor
		{
		public:
			static UE_API FDocumentHandle GetInvalidHandle();

			/** Create a document from FMetasoundFrontendDocument description pointer. */
			static UE_API FDocumentHandle CreateDocumentHandle(FDocumentAccessPtr InDocument, bool bReloadBuilder = true);
			static UE_API FDocumentHandle CreateDocumentHandle(FMetasoundFrontendDocument& InDocument);

			/** Create a document from FMetasoundFrontendDocument description pointer. */
			static UE_API FConstDocumentHandle CreateDocumentHandle(FConstDocumentAccessPtr InDocument);
			static UE_API FConstDocumentHandle CreateDocumentHandle(const FMetasoundFrontendDocument& InDocument);

			IDocumentController() = default;
			virtual ~IDocumentController() = default;

			/** Returns true if the controller is in a valid state. */
			virtual bool IsValid() const = 0;

			// Iterates all dependent class definitions that exist on the document (do not necessarily correspond
			// to registered dependencies)
			virtual void IterateDependencies(TFunctionRef<void(FMetasoundFrontendClass&)> InFunction) = 0;
			virtual void IterateDependencies(TFunctionRef<void(const FMetasoundFrontendClass&)> InFunction) const = 0;

			// TODO: add info on environment variables.
			// TODO: consider find/add subgraph
			// TODO: perhaps functions returning access pointers could be removed from main interface and only exist in FDocumentController.
			/** Returns an array of all class dependencies for this document. */
			virtual const TArray<FMetasoundFrontendClass>& GetDependencies() const = 0;
			virtual const TArray<FMetasoundFrontendGraphClass>& GetSubgraphs() const = 0;
			virtual const FMetasoundFrontendGraphClass& GetRootGraphClass() const = 0;
			virtual void SetRootGraphClass(FMetasoundFrontendGraphClass&& InClass) = 0;

			virtual FConstClassAccessPtr FindDependencyWithID(FGuid InClassID) const = 0;
			virtual FConstGraphClassAccessPtr FindSubgraphWithID(FGuid InClassID) const = 0;
			virtual FConstClassAccessPtr FindClassWithID(FGuid InClassID) const = 0;

			virtual void SetMetadata(const FMetasoundFrontendDocumentMetadata& InMetadata) = 0;
			virtual const FMetasoundFrontendDocumentMetadata& GetMetadata() const = 0;
			virtual FMetasoundFrontendDocumentMetadata* GetMetadata() = 0;

			/** Returns an existing Metasound class description corresponding to 
			 * a dependency which matches the provided class information.
			 *
			 * @return A pointer to the found object, or nullptr if it could not be found.
			 */
			virtual FConstClassAccessPtr FindClass(const FNodeRegistryKey& InKey) const = 0; 

			/** Attempts to find an existing Metasound class description corresponding
			 * to a dependency which matches the provided class information. If the
			 * class is not found in the current dependencies, it is added to the 
			 * dependencies. If found and bInRefreshFromRegistry is set (optional)
			 * , copies the version found in the registry to the document.
			 *
			 * @return A pointer to the object, or nullptr on error.
			 */
			virtual FConstClassAccessPtr FindOrAddClass(const FNodeRegistryKey& InKey, bool bInRefreshFromRegistry = false) = 0;

			/** Returns an existing Metasound class description corresponding to 
			 * a dependency which matches the provided class information.
			 *
			 * @return A pointer to the found object, or nullptr if it could not be found.
			 */
			virtual FConstClassAccessPtr FindClass(const FMetasoundFrontendClassMetadata& InMetadata) const = 0;

			/** Attempts to find an existing Metasound class description corresponding
			 * to a dependency which matches the provided class information. If the
			 * class is not found in the current dependencies, it is added to the 
			 * dependencies.
			 *
			 * @return A pointer to the object, or nullptr on error.
			 */
			virtual FConstClassAccessPtr FindOrAddClass(const FMetasoundFrontendClassMetadata& InMetadata) = 0;

			/** Adds a duplicate subgraph to the document. This method creates a 
			 * copy of the graph and adds all the graph dependencies to this document.
			 * 
			 * @param InGraph - Graph to copy.
			 *
			 * @return Handle to new graph. On error, an invalid handle is returned.
			 */
			virtual FGraphHandle AddDuplicateSubgraph(const IGraphController& InGraph) = 0;

			virtual const TSet<FMetasoundFrontendVersion>& GetInterfaceVersions() const = 0;

			virtual void AddInterfaceVersion(const FMetasoundFrontendVersion& InVersion) = 0;
			virtual void RemoveInterfaceVersion(const FMetasoundFrontendVersion& InVersion) = 0;
			virtual void ClearInterfaceVersions() = 0;

			/** Removes all dependencies which are no longer referenced by any graphs within this document.
			  */
			virtual void RemoveUnreferencedDependencies() = 0;

			/** Synchronizes all dependency Metadata in document with that found in the registry. If not found
			  * in the registry, no action taken.  Returns array of pointers to classes that were modified.
			  */
			virtual TArray<FConstClassAccessPtr> SynchronizeDependencyMetadata() = 0;

			/** Returns an array of all subgraphs for this document. */
			virtual TArray<FGraphHandle> GetSubgraphHandles() = 0;

			/** Returns an array of all subgraphs for this document. */
			virtual TArray<FConstGraphHandle> GetSubgraphHandles() const = 0;

			/** Returns a graphs in the document with the given class ID.*/
			virtual FGraphHandle GetSubgraphWithClassID(FGuid InClassID) = 0;

			/** Returns a graphs in the document with the given class ID.*/
			virtual FConstGraphHandle GetSubgraphWithClassID(FGuid InClassID) const = 0;

			/** Returns the root graph of this document. */
			virtual FGraphHandle GetRootGraph() = 0;

			/** Returns the root graph of this document. */
			virtual FConstGraphHandle GetRootGraph() const = 0;

			/** Returns the document if set. */
			virtual FDocumentAccessPtr GetDocumentPtr() = 0;
			virtual const FDocumentAccessPtr GetDocumentPtr() const = 0;

			/** Exports the document to a json file at the provided path.
			 *
			 * @return True on success, false on failure. 
			 */
			virtual bool ExportToJSONAsset(const FString& InAbsolutePath) const = 0;

			/** Exports the document to a json formatted string. */
			virtual FString ExportToJSON() const = 0;
		};

		UE_DEPRECATED(5.5, "Reroute output recursion is now privately implemented in the reroute template node class. Use template registry to query reroute node input/output attributes")
		METASOUNDFRONTEND_API FConstOutputHandle FindReroutedOutput(FConstOutputHandle InOutputHandle);

		UE_DEPRECATED(5.5, "Reroute input recursion is now privately implemented in the reroute template node class. Use template registry to query reroute node input/output attributes")
		METASOUNDFRONTEND_API void FindReroutedInputs(FConstInputHandle InHandleToCheck, TArray<FConstInputHandle>& InOutInputHandles);

		UE_DEPRECATED(5.5, "Reroute input recursion is now privately implemented in the reroute template node class. Use template registry to query reroute node input/output attributes")
		METASOUNDFRONTEND_API void IterateReroutedInputs(FConstInputHandle InHandleToCheck, TFunctionRef<void(FConstInputHandle)> Func);
	} // namespace Frontend
} // namespace Metasound

#undef UE_API
