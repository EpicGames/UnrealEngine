// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "MetasoundEnum.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundLiteral.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	class IDataChannel;

	namespace Frontend
	{
		using IParameterAssignmentFunction = TUniqueFunction<void(const void*,void*)>;

		// Function signature for assigning a FLiteral to a FAnyDataReference 
		using FLiteralAssignmentFunction = void(*)(const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral, const FAnyDataReference& OutDataRef);

		/** FDataTypeRegsitryInfo contains runtime inspectable behavior of a registered
		 * MetaSound data type.
		 */
		struct FDataTypeRegistryInfo
		{
			// The name of the data type.
			FName DataTypeName;

			FText DataTypeDisplayText;

			// The preferred constructor argument type for creating instances of the data type.
			ELiteralType PreferredLiteralType = ELiteralType::Invalid;

			// Constructor argument support in TDataTypeLiteralFactory<TDataType>;
			bool bIsParsable = false;
			bool bIsDefaultParsable = false;
			bool bIsBoolParsable = false;
			bool bIsIntParsable = false;
			bool bIsFloatParsable = false;
			bool bIsStringParsable = false;
			bool bIsProxyParsable = false;
			bool bIsUniquePtrProxyParsable_DEPRECATED = false;
			bool bIsDefaultArrayParsable = false;
			bool bIsBoolArrayParsable = false;
			bool bIsIntArrayParsable = false;
			bool bIsFloatArrayParsable = false;
			bool bIsStringArrayParsable = false;
			bool bIsProxyArrayParsable = false;
			bool bIsUniquePtrProxyArrayParsable_DEPRECATED = false;

			// Is a TEnum wrapped enum
			bool bIsEnum = false;

			// Whether exact types are required
			bool bIsExplicit = false;

			// Determines whether the type can be used with send/receive transmitters
			bool bIsTransmittable = false;

			// Returns if DataType is a Variable type
			bool bIsVariable = false;

			// Returns if DataType can be used for constructor vertices.
			bool bIsConstructorType = false;

			// Returns if DataType represents an array type (ex. TArray<float>, TArray<int32>, etc.).
			bool bIsArrayType = false;

			// Returns if DataType supports array parsing and passing array of base type to constructor.
			bool bIsArrayParseable = false;

			// If provided in registration call, UClass this datatype was registered with.
			UClass* ProxyGeneratorClass = nullptr;

			// Whether the data type can be created from a UObject proxy. 
			const bool IsDataTypeProxyParsable() const
			{
				return bIsProxyParsable || bIsProxyArrayParsable;
			}

			// If this type is polymorphic.
			bool bIsPolymorphic = false;
			bool bIsAbstract = false;

			// If this type is polymorphic, what is its parent.
			FName ParentDataTypeName = {};
		};

		/** Interface for metadata of a registered MetaSound enum type. */
		struct IEnumDataTypeInterface
		{
			using FGenericInt32Entry = TEnumEntry<int32>;

			virtual ~IEnumDataTypeInterface() = default;

			virtual const TArray<FGenericInt32Entry>& GetAllEntries() const = 0;
			virtual FName GetNamespace() const = 0;
			virtual int32 GetDefaultValue() const = 0;

			template<typename Predicate>
			TOptional<FGenericInt32Entry> FindEntryBy(Predicate Pred) const
			{
				TArray<FGenericInt32Entry> Entries = GetAllEntries();
				if (FGenericInt32Entry* Found = Entries.FindByPredicate(Pred))
				{
					return *Found;
				}
				return {};
			}
			TOptional<FGenericInt32Entry> FindByValue(int32 InEnumValue) const
			{
				return FindEntryBy([InEnumValue](const FGenericInt32Entry& i) -> bool { return i.Value == InEnumValue; });
			}
			TOptional<FGenericInt32Entry> FindByName(FName InEnumName) const
			{
				return FindEntryBy([InEnumName](const FGenericInt32Entry& i) -> bool { return i.Name == InEnumName; });
			}
			TOptional<FName> ToName(int32 InEnumValue) const
			{
				if(TOptional<FGenericInt32Entry> Result = FindByValue(InEnumValue))
				{
					return Result->Name;
				}
				return {};
			}
			TOptional<int32> ToValue(FName InName) const
			{
				if (TOptional<FGenericInt32Entry> Result = FindByName(InName))
				{
					return Result->Value;
				}
				return {};
			}
		};

		struct IPolymorphicDataTypeInterface;
	
		/** Registry entry interface for a MetaSound data type. */
		class IDataTypeRegistryEntry
		{
		public:
			virtual ~IDataTypeRegistryEntry() = default;

			/** Return the FDataTypeRegistryInfo for the data type */
			virtual const FDataTypeRegistryInfo& GetDataTypeInfo() const = 0;

			/** Return the enum interface for the data type. If the data type does
			 * not support an enum interface, the returned pointer should be invalid.
			 */
			virtual TSharedPtr<const IEnumDataTypeInterface> GetEnumInterface() const = 0;

			UE_EXPERIMENTAL(5.7, "Metasound Polymorphic Datatypes are still in devleopement")
			virtual TSharedPtr<IPolymorphicDataTypeInterface> GetPolymorphicInterface() const = 0;

			/** Return an FMetasoundFrontendClass representing an input node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendInputClass() const = 0;

			/** Return an FNodeClassMetadata representing an input node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetInputClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an input node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendConstructorInputClass() const = 0;

			/** Return an FNodeClassMetadata representing an input node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetConstructorInputClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing a variable node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendLiteralClass() const = 0;

			/** Return an FMetasoundFrontendClass representing an output node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendOutputClass() const = 0;

			/** Return an FNodeClassMetadata representing an output node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetOutputClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an input node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendConstructorOutputClass() const = 0;

			/** Return an FNodeClassMetadata representing an input node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetConstructorOutputClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an init variable node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendVariableClass() const = 0;

			/** Return an FNodeClassMetadata representing an init variable node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an set variable node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendVariableMutatorClass() const = 0;

			/** Return an FNodeClassMetadata representing an set variable node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableMutatorClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an get variable node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendVariableAccessorClass() const = 0;

			/** Return an FNodeClassMetadata representing an get variable node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableAccessorClassMetadata() const = 0;

			/** Return an FMetasoundFrontendClass representing an get delayed variable node of the data type. */
			virtual const FMetasoundFrontendClass& GetFrontendVariableDeferredAccessorClass() const = 0;

			/** Return an FNodeClassMetadata representing an get delayed variable node of the data type. */
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableDeferredAccessorClassMetadata() const = 0;

			/** Create an input node */
			virtual TUniquePtr<INode> CreateInputNode(FNodeData InNodeData) const = 0;

			/** Create a constructor input node */
			virtual TUniquePtr<INode> CreateConstructorInputNode(FNodeData InNodeData) const = 0;

			/** Create an output node */
			virtual TUniquePtr<INode> CreateOutputNode(FNodeData InNodeData) const = 0;

			/** Create a constructor output node */
			virtual TUniquePtr<INode> CreateConstructorOutputNode(FNodeData InNodeData) const = 0;

			/** Create a literal node (deprecated — literal values should be set on the connected node directly). */
			UE_DEPRECATED(5.6, "Direct creation of literal nodes will no longer be supported")
			virtual TUniquePtr<INode> CreateLiteralNode(FLiteralNodeConstructorParams&&) const = 0;

			/** Create a receive node for this data type. */
			UE_DEPRECATED(5.6, "Direct creation of receive nodes will no longer be supported")
			virtual TUniquePtr<INode> CreateReceiveNode(const FNodeInitData&) const = 0;

			/* Get function that should be used when a parameter pack contains a setting this data type. */
			virtual const IParameterAssignmentFunction& GetRawAssignmentFunction() const = 0;

			/* Get a function that can be used to assign a FLiteral to a FAnyDataReference. */
			virtual FLiteralAssignmentFunction GetLiteralAssignmentFunction() const = 0;

			/** Create a init variable node for this data type.
			 *
			 *  @param InInitValue - A literal used to create the variable.
			 */
			virtual TUniquePtr<INode> CreateVariableNode(FLiteral InInitValue, FNodeData) const = 0;

			/** Create a set variable node for this data type. */
			virtual TUniquePtr<INode> CreateVariableMutatorNode(FNodeData) const = 0;

			/** Create a get variable node for this data type. */
			virtual TUniquePtr<INode> CreateVariableAccessorNode(FNodeData) const = 0;

			/** Create a get delayed variable node for this data type. */
			virtual TUniquePtr<INode> CreateVariableDeferredAccessorNode(FNodeData) const = 0;
			
			/** Create a data reference from a literal. */
			virtual TOptional<FAnyDataReference> CreateDataReference(EDataReferenceAccessType InAccessType, const FLiteral& InLiteral, const FOperatorSettings& InOperatorSettings) const = 0;

			/** Create a proxy from a UObject. If this data type does not support
			 * UObject proxies, return a nullptr. */
			virtual TSharedPtr<Audio::IProxyData> CreateProxy(UObject* InObject) const = 0;

			/** Create a data channel for transmission. If this data type does not
			 * support transmission, return a nullptr. */
			virtual TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FOperatorSettings&) const = 0;

			/** Clone this registry entry. */
			UE_DEPRECATED(5.6, "Cloning is not required for IDataTypeRegistryEntries")
			virtual TUniquePtr<IDataTypeRegistryEntry> Clone() const { return nullptr; }
		};

		class IDataTypeRegistry
		{
		public:
			static UE_API IDataTypeRegistry& Get();
			virtual ~IDataTypeRegistry() = default;

			/** Register a data type
			 * @param InEntry - TUniquePtr to data type registry entry.
			 *
			 * @return True on success, false on failure.
			 */
			virtual bool RegisterDataType(TUniquePtr<IDataTypeRegistryEntry>&& InEntry) = 0;

			/** Unregister a data type.
			 * @param InDataTypeName
			 *
			 * @return True on success, false on failure.
			 */
			virtual bool UnregisterDataType(const FName& InDataTypeName) = 0;

			/** Provides all names of registered DataTypes. */
			virtual void GetRegisteredDataTypeNames(TArray<FName>& OutNames) const = 0;

			virtual const IDataTypeRegistryEntry* FindDataTypeRegistryEntry(const FName& InDataTypeName) const = 0;

			/** Returns DataType info associated with the provided object. */
			virtual bool GetDataTypeInfo(const UObject* InObject, FDataTypeRegistryInfo& OutInfo) const = 0;

			/** Returns DataType info associated with the provided object class. */
			virtual bool GetDataTypeInfo(UClass* InClass, FDataTypeRegistryInfo& OutInfo) const = 0;

			/** Returns DataType info associated with the provided DataType name. */
			virtual bool GetDataTypeInfo(const FName& InDataType, FDataTypeRegistryInfo& OutInfo) const = 0;

			/** Iterates all registered data type info */
			virtual void IterateDataTypeInfo(TFunctionRef<void(const FDataTypeRegistryInfo&)> InFunction) const = 0;

			/** Returns whether or not a DataType is registered with the given name. */
			virtual bool IsRegistered(const FName& InDataType) const = 0;

			/** Return the enum interface for a data type. If the data type does not have an enum interface, returns a nullptr. */
			virtual TSharedPtr<const IEnumDataTypeInterface> GetEnumInterfaceForDataType(const FName& InDataType) const = 0;

			virtual ELiteralType GetDesiredLiteralType(const FName& InDataType) const = 0;

			virtual bool IsLiteralTypeSupported(const FName& InDataType, ELiteralType InLiteralType) const = 0;
			virtual bool IsLiteralTypeSupported(const FName& InDataType, EMetasoundFrontendLiteralType InLiteralType) const = 0;
			virtual bool IsUObjectProxyFactory(UObject* InObject) const = 0;
			virtual bool IsValidUObjectForDataType(const FName& InDataTypeName, const UObject* InUObject) const = 0;

			virtual UClass* GetUClassForDataType(const FName& InDataType) const = 0;

			virtual TSharedPtr<Audio::IProxyData> CreateProxyFromUObject(const FName& InDataType, UObject* InObject) const = 0;

			virtual FLiteral CreateDefaultLiteral(const FName& InDataType) const = 0;
			virtual FLiteral CreateLiteralFromUObject(const FName& InDataType, UObject* InObject) const = 0;
			virtual FLiteral CreateLiteralFromUObjectArray(const FName& InDataType, const TArray<UObject*>& InObjectArray) const = 0;

			/** Create a data reference of the data type given a literal. If not supported, the result will be not be set. */
			virtual TOptional<FAnyDataReference> CreateDataReference(const FName& InDataType, EDataReferenceAccessType InAccessType, const FLiteral& InLiteral, const FOperatorSettings& InOperatorSettings) const = 0;

			virtual TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FName& InDataType, const FOperatorSettings& InOperatorSettings) const = 0;

			/* Get function that should be used when a parameter pack contains a setting for the specified data type. */ 
			virtual const IParameterAssignmentFunction& GetRawAssignmentFunction(const FName& InDataType) const = 0;

			/* Get a function that can be used to assign a FLiteral to a FAnyDataReference. */
			virtual FLiteralAssignmentFunction GetLiteralAssignmentFunction(const FName& InDataType) const = 0;

			/** Return an FMetasoundFrontendClass representing an input node of the data type. */
			virtual bool GetFrontendInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an input node of the data type. */
			virtual bool GetFrontendConstructorInputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing a variable node of the data type. */
			virtual bool GetFrontendLiteralClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an output node of the data type. */
			virtual bool GetFrontendOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an output node of the data type. */
			virtual bool GetFrontendConstructorOutputClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an init variable node of the data type. */
			virtual bool GetFrontendVariableClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an set variable node of the data type. */
			virtual bool GetFrontendVariableMutatorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an get variable node of the data type. */
			virtual bool GetFrontendVariableAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Return an FMetasoundFrontendClass representing an get delayed variable node of the data type. */
			virtual bool GetFrontendVariableDeferredAccessorClass(const FName& InDataType, FMetasoundFrontendClass& OutClass) const = 0;

			/** Below are class metadata accessor for the special nodes related to
			 * a data type. */

			virtual TSharedPtr<const FNodeClassMetadata> GetInputClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetConstructorInputClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetOutputClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetConstructorOutputClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableMutatorClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableAccessorClassMetadata(const FName& InDataType) const = 0;
			virtual TSharedPtr<const FNodeClassMetadata> GetVariableDeferredAccessorClassMetadata(const FName& InDataType) const = 0;

			/** Below are node creation methods for creation special nodes related to 
			 * an individual data type. These nodes power core features of metasounds
			 * such as Inputs, Outputs, and Variables. 
			 */

			virtual TUniquePtr<INode> CreateInputNode(const FName& InInputType, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateConstructorInputNode(const FName& InInputType, FNodeData InNodeData) const = 0;

			UE_DEPRECATED(5.6, "Direct creation of literal nodes will no longer be supported")
			virtual TUniquePtr<INode> CreateLiteralNode(const FName& InLiteralType, FLiteralNodeConstructorParams&& InParams) const = 0;

			UE_DEPRECATED(5.6, "Direct creation of receive nodes will no longer be supported")
			virtual TUniquePtr<INode> CreateReceiveNode(const FName& InDataType, const FNodeInitData&) const = 0;

			virtual TUniquePtr<INode> CreateOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateConstructorOutputNode(const FName& InDataTypeName, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateVariableNode(const FName& InDataType, FLiteral InLiteral, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateVariableMutatorNode(const FName& InDataType, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateVariableAccessorNode(const FName& InDataType, FNodeData InNodeData) const = 0;
			virtual TUniquePtr<INode> CreateVariableDeferredAccessorNode(const FName& InDataType, FNodeData InNodeData) const = 0;
		};

		/**
		 * IPolymorphicDataTypeInterface.
		 * Experimental API. 
		 */	
		struct IPolymorphicDataTypeInterface
		{
			virtual ~IPolymorphicDataTypeInterface() = default;
			UE_EXPERIMENTAL(5.7, "Metasound Polymorphic Datatypes are still in devleopement")
			virtual bool IsA(const FName InDataTypeName, const IDataTypeRegistry& InRegistry = IDataTypeRegistry::Get() ) const = 0;
		};

		/**
		 * FPolymorphicDataTypeBase 
		 * Experimental API. 
		 */
		class FPolymorphicDataTypeBase : public IPolymorphicDataTypeInterface
		{
		public:
			UE_API FPolymorphicDataTypeBase(const FName InDataTypeName, const FName InParentName = NAME_None, const bool InbIsAbstract = false);
			virtual ~FPolymorphicDataTypeBase() override = default;

			UE_EXPERIMENTAL(5.7, "Metasound Polymorphic Datatypes are still in developement")			
			UE_API virtual bool IsA(const FName InDataTypeName, const IDataTypeRegistry& InRegistry) const override;
		private:
			void ResolveParent(const IDataTypeRegistry& InRegistry) const;
			FName DataTypeName = NAME_None;
			FName ParentTypeName = NAME_None;
			bool bIsAbstract = false;
			mutable TWeakPtr<const IPolymorphicDataTypeInterface> ParentDataType;
		};
		
		/** Create data references for all inputs on the FInputVertexInterfaceData 
		 * using the default literals which live on the FInputDataVertexes.
		 *
		 * If the vertex has access type EVertexAccessType::Reference, a writable
		 * data reference will be set, otherwise a value data reference will be set.
		 */
		METASOUNDFRONTEND_API void CreateDefaults(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData);

		/** Create data references for all inputs on the FInputVertexInterfaceData 
		 * which do not already have a data reference bound. Defaults are created
		 * using the default literals which live on the FInputDataVertexes.
		 *
		 * If the vertex has access type EVertexAccessType::Reference, a writable
		 * data reference will be set, otherwise a value data reference will be set.
		 */
		METASOUNDFRONTEND_API void CreateDefaultsIfNotBound(const FOperatorSettings& InOperatorSettings, FInputVertexInterfaceData& OutVertexData);
	}
}

#undef UE_API
