// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundDataFactory.h"

#include <type_traits>

namespace Metasound
{
	/** Enables or disables automatic registration of array types given a 
	 * MetaSound data type. By default this is true, and all data types will have
	 * an associated TArray<DataType> registered if it is supported. */
	template<typename ... Type>
	struct TEnableAutoArrayTypeRegistration
	{
		static constexpr bool Value = true;
	};

	/** Enables or disables automatic registration of auto conversion nodes given a 
	 * MetaSound data type. By default this is true, and all data types will have
	 * associated conversion nodes registered based upon the data types supported
	 * constructors and implicit conversions. */
	template<typename ... Type>
	struct TEnableAutoConverterNodeRegistration
	{
		static constexpr bool Value = true;
	};

	/** Enables or disables send and receive node registration for a given MetaSound
	 * data type. By default this is false because send/receive nodes are deprecated. */
	template<typename ... Type>
	struct TEnableTransmissionNodeRegistration
	{
		static constexpr bool Value = false;
	};

	template<typename ElementType>
	struct TEnableTransmissionNodeRegistration<TArray<ElementType>>
	{
		static constexpr bool Value = TEnableTransmissionNodeRegistration<ElementType>::Value;
	};

	/** Enables or disables using a data type in constructor vertices. 
	 * By default this is true and all data types that support copy assignment
	 * and copy construction. */
	template<typename ... Type>
	struct TEnableConstructorVertex
	{
		static constexpr bool Value = true;
	};

	// Helper utility to test if we a datatype is copyable
	template <typename TDataType>
	struct TIsCopyable
	{
	private:
		static constexpr bool bIsCopyConstructible = std::is_copy_constructible<TDataType>::value;
		static constexpr bool bIsCopyAssignable = std::is_copy_assignable<TDataType>::value;

	public:
		static constexpr bool Value = bIsCopyConstructible && bIsCopyAssignable;
	};

	// Helper to test if data type can be used on constructor pins.
	template<typename TDataType>
	struct TIsConstructorVertexSupported
	{
	private:
		static constexpr bool bIsParsable = TLiteralTraits<TDataType>::bIsParsableFromAnyLiteralType;
		static constexpr bool bIsCopyable = TIsCopyable<TDataType>::Value;
		static constexpr bool bIsEnabled = TEnableConstructorVertex<TDataType>::Value;
	public:
		static constexpr bool Value = bIsParsable && bIsCopyable && bIsEnabled;
	};

	// Helper utility to test if exact types are required for a datatype.
	template <typename TDataType>
	struct TIsExplicit
	{
		static constexpr bool Value = false;
	};

	// Helper utility to test if we can transmit a datatype between a send and a receive node.
	template <typename TDataType>
	struct TIsTransmittable
	{
		static constexpr bool Value = TIsCopyable<TDataType>::Value;
	};
}
