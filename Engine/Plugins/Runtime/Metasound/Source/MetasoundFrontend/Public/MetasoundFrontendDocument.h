// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DocumentTemplates/MetasoundFrontendDocumentTemplate.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Internationalization/Text.h"
#include "MetasoundAccessPtr.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include "Misc/EnumRange.h"
#include "Misc/Guid.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "Templates/Function.h"
#include "Templates/Invoke.h"
#include "Templates/TypeHash.h"

#if WITH_EDITORONLY_DATA
#include "Types/SlateVector2.h"
#endif // WITH_EDITORONLY_DATA

#include "MetasoundFrontendDocument.generated.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
struct FMetasoundFrontendClass;
struct FMetasoundFrontendClassInterface;
struct FMetaSoundFrontendDocumentBuilder;

enum class EMetasoundFrontendClassType : uint8;


namespace Metasound
{
	// Forward Declarations
	struct FLiteral;

	extern const FGuid METASOUNDFRONTEND_API FrontendInvalidID;

	namespace Frontend
	{
		extern int32 MetaSoundAutoUpdateNativeClassesOfEqualVersionCVar;

		constexpr FGuid DefaultPageID(0, 0, 0, 0);
		constexpr TCHAR DefaultPageName[] = TEXT("Default");

#if WITH_EDITORONLY_DATA
		extern const FText METASOUNDFRONTEND_API DefaultPageDisplayName;
#endif // WITH_EDITORONLY_DATA

		namespace DisplayStyle
		{
			namespace EdgeAnimation
			{
				extern const FLinearColor METASOUNDFRONTEND_API DefaultColor;
			} // namespace EdgeStyle

			namespace NodeLayout
			{
				extern const FVector2D METASOUNDFRONTEND_API DefaultOffsetX;
				extern const FVector2D METASOUNDFRONTEND_API DefaultOffsetY;
			} // namespace NodeLayout
		} // namespace DisplayStyle
	} // namespace Frontend
} // namespace Metasound


#if WITH_EDITORONLY_DATA
// Struct containing any modified data breadcrumbs to inform what the editor/view layer must synchronize or refresh.
class FMetasoundFrontendDocumentModifyContext
{
private:
	// Whether or not the owning asset's MetaSoundDocument has been modified. True by default to force refreshing views on loading/reloading asset.
	bool bDocumentModified = true;

	// Whether or not to force refresh all views. True by default to force refreshing views on loading/reloading asset.
	bool bForceRefreshViews = true;

	// Which Interfaces have been modified since the last editor graph synchronization
	TSet<FName> InterfacesModified;

	// Which MemberIDs have been modified since the last editor graph synchronization
	TSet<FGuid> MemberIDsModified;

	// Which NodeIDs have been modified since the last editor graph synchronization
	TSet<FGuid> NodeIDsModified;

public:
	UE_API void ClearDocumentModified();

	UE_API bool GetDocumentModified() const;
	UE_API bool GetForceRefreshViews() const;
	UE_API const TSet<FName>& GetInterfacesModified() const;
	UE_API const TSet<FGuid>& GetNodeIDsModified() const;
	UE_API const TSet<FGuid>& GetMemberIDsModified() const;

	UE_API void Reset();

	UE_API void SetDocumentModified();
	UE_API void SetForceRefreshViews();

	// Adds an interface name to the set of interfaces that have been modified since last context reset/construction
	UE_API void AddInterfaceModified(FName InInterfaceModified);

	// Performs union of provided interface set with the set of interfaces that have been modified since last context reset/construction
	UE_API void AddInterfacesModified(const TSet<FName>& InInterfacesModified);

	// Adds a MemberID to the set of MemberIDs that have been modified since last context reset/construction
	UE_API void AddMemberIDModified(const FGuid& InMemberNodeIDModified);

	// Performs union of provided MemberIDs set with the set of MemberIDs that have been modified since last context reset/construction
	UE_API void AddMemberIDsModified(const TSet<FGuid>& InMemberIDsModified);

	// Performs union of provided NodeID set with the set of NodeIDs that have been modified since last context reset/construction
	UE_API void AddNodeIDModified(const FGuid& InNodeIDModified);

	// Performs union of provided NodeID set with the set of NodeIDs that have been modified since last context reset/construction
	UE_API void AddNodeIDsModified(const TSet<FGuid>& InNodeIDsModified);
};
#endif // WITH_EDITORONLY_DATA


// Describes how a vertex accesses the data connected to it. 
UENUM()
enum class EMetasoundFrontendVertexAccessType
{
	Reference,	//< The vertex accesses data by reference.
	Value,		//< The vertex accesses data by value.

	Unset		//< The vertex access level is unset (ex. vertex on an unconnected reroute node).
				//< Not reflected as a graph core access type as core does not deal with reroutes
				//< or ambiguous accessor level (it is resolved during document pre-processing).
};

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMetasoundFrontendClassAccessFlags : uint16
{
	None = 0 UMETA(Hidden),

	// Class is marked as deprecated when referenced by
	// MetaSounds in the editor.
	Deprecated = 1 << 0,

	// If set, MetaSound can be referenced by other MetaSounds in either
	// editor or by builder Blueprint API.
	Referenceable = 1 << 1,

	Default = Referenceable UMETA(Hidden)
};

static_assert(static_cast<uint16>(EMetasoundFrontendClassAccessFlags::None) == static_cast<uint16>(Metasound::ENodeClassAccessFlags::None), "Enum values must match");
static_assert(static_cast<uint16>(EMetasoundFrontendClassAccessFlags::Deprecated) == static_cast<uint16>(Metasound::ENodeClassAccessFlags::Deprecated), "Enum values must match");
static_assert(static_cast<uint16>(EMetasoundFrontendClassAccessFlags::Referenceable) == static_cast<uint16>(Metasound::ENodeClassAccessFlags::Referenceable), "Enum values must match");
static_assert(static_cast<uint16>(EMetasoundFrontendClassAccessFlags::Default) == static_cast<uint16>(Metasound::ENodeClassAccessFlags::Default), "Enum values must match");

UE_API FString LexToString(const EMetasoundFrontendClassAccessFlags& InFlag);

ENUM_CLASS_FLAGS(EMetasoundFrontendClassAccessFlags)

ENUM_RANGE_BY_VALUES(EMetasoundFrontendClassAccessFlags,
	EMetasoundFrontendClassAccessFlags::Deprecated,
	EMetasoundFrontendClassAccessFlags::Referenceable)

UENUM(MinimalAPI)
enum class EMetasoundFrontendClassType : uint8
{
	// The MetaSound class is defined externally, in compiled code or in another document.
	External = 0,

	// The MetaSound class is a graph within the containing document.
	Graph,

	// The MetaSound class is an input into a graph in the containing document.
	Input,

	// The MetaSound class is an output from a graph in the containing document.
	Output,

	// The MetaSound class is an literal requiring a literal value to construct.
	Literal,

	// The MetaSound class is an variable requiring a literal value to construct.
	Variable,

	// The MetaSound class accesses variables.
	VariableDeferredAccessor,

	// The MetaSound class accesses variables.
	VariableAccessor,

	// The MetaSound class mutates variables.
	VariableMutator,

	// The MetaSound class is defined only by the Frontend, and associatively
	// performs a functional operation within the given document in a registration/cook step.
	Template,

	Invalid UMETA(Hidden)
};

UENUM()
enum class EMetaSoundFrontendGraphCommentMoveMode : uint8
{
	/** This comment box will move any fully contained nodes when it moves. */
	GroupMovement UMETA(DisplayName = "Group Movement"),

	/** This comment box has no effect on nodes contained inside it. */
	NoGroupMovement UMETA(DisplayName = "Comment")
};

/**
  * Migratory type to avoid adding dependency on Slate FDeprecateSlateVector2D, and by extension,
  * bring in unnecessary Engine dependencies therein.  At one point, this dependency was incorrectly
  * added leading to in-determinant serialization as either a double or a float vector. This type
  * exists to resolve that discrepancy properly. Considered soft deprecated and not to be used for runtime.
 */
USTRUCT()
struct FMetasoundCommentNodeIntVector : public FIntVector2
{
	GENERATED_BODY()

	FMetasoundCommentNodeIntVector() = default;
	UE_API FMetasoundCommentNodeIntVector(const FIntVector2& InValue);
	UE_API FMetasoundCommentNodeIntVector(const FVector2f& InValue);
	UE_API FMetasoundCommentNodeIntVector(const FVector2d& InValue);

#if WITH_EDITORONLY_DATA
	UE_API FMetasoundCommentNodeIntVector(const FDeprecateSlateVector2D& InValue);
#endif // WITH_EDITORONLY_DATA

	UE_API FMetasoundCommentNodeIntVector& operator=(const FVector2f& InValue);
	UE_API FMetasoundCommentNodeIntVector& operator=(const FVector2d& InValue);
	UE_API FMetasoundCommentNodeIntVector& operator=(const FIntVector2& InValue);

#if WITH_EDITORONLY_DATA
	UE_API FMetasoundCommentNodeIntVector& operator=(const FDeprecateSlateVector2D& InValue);
#endif // WITH_EDITORONLY_DATA

	UE_API bool Serialize(FStructuredArchive::FSlot Slot);
	UE_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FMetasoundCommentNodeIntVector>
	: TStructOpsTypeTraitsBase2<FMetasoundCommentNodeIntVector>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


USTRUCT()
struct FMetaSoundFrontendGraphComment
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FLinearColor Color = FLinearColor::Black;

	UPROPERTY()
	FString Comment;

	UPROPERTY()
	int32 Depth = 0;

	UPROPERTY()
	int32 FontSize = 0;

	UPROPERTY()
	FMetasoundCommentNodeIntVector Position = FIntVector2::ZeroValue;

	UPROPERTY()
	FMetasoundCommentNodeIntVector Size = FIntVector2::ZeroValue;

	UPROPERTY()
	EMetaSoundFrontendGraphCommentMoveMode MoveMode = EMetaSoundFrontendGraphCommentMoveMode::GroupMovement;

	UPROPERTY()
	uint8 bColorBubble : 1;
#endif // WITH_EDITORONLY_DATA
};

// General purpose version number for Metasound Frontend objects.
USTRUCT(BlueprintType)
struct FMetasoundFrontendVersionNumber
{
	GENERATED_BODY()

	static UE_API bool Parse(const FString& InString, FMetasoundFrontendVersionNumber& OutVersionNumber);

	// Major version number.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = General)
	int32 Major = 1;

	// Minor version number.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = General)
	int32 Minor = 0;

	static UE_API const FMetasoundFrontendVersionNumber& GetInvalid();
	UE_API bool IsValid() const;

	UE_API Audio::FParameterInterface::FVersion ToInterfaceVersion() const;

	friend bool operator==(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		return InLHS.Major == InRHS.Major && InLHS.Minor == InRHS.Minor;
	}

	friend bool operator!=(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		return InLHS.Major != InRHS.Major || InLHS.Minor != InRHS.Minor;
	}

	friend bool operator>(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		if (InLHS.Major > InRHS.Major)
		{
			return true;
		}

		if (InLHS.Major == InRHS.Major)
		{
			return InLHS.Minor > InRHS.Minor;
		}

		return false;
	}

	friend bool operator>=(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		return InLHS == InRHS || InLHS > InRHS;
	}

	friend bool operator<(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		if (InLHS.Major < InRHS.Major)
		{
			return true;
		}

		if (InLHS.Major == InRHS.Major)
		{
			return InLHS.Minor < InRHS.Minor;
		}

		return false;
	}

	friend bool operator<=(const FMetasoundFrontendVersionNumber& InLHS, const FMetasoundFrontendVersionNumber& InRHS)
	{
		return InLHS == InRHS || InLHS < InRHS;
	}

	UE_API FString ToString() const;

	friend inline uint32 GetTypeHash(const FMetasoundFrontendVersionNumber& InNumber)
	{
		return HashCombineFast(GetTypeHash(InNumber.Major), GetTypeHash(InNumber.Minor));
	}
};

// General purpose version info for Metasound Frontend objects.
USTRUCT(BlueprintType)
struct FMetasoundFrontendVersion
{
	GENERATED_BODY()

	// Name of version.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = CustomView)
	FName Name;

	// Version number.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = CustomView)
	FMetasoundFrontendVersionNumber Number;

	UE_API FString ToString() const;

	UE_API bool IsValid() const;

	static UE_API const FMetasoundFrontendVersion& GetInvalid();

	friend bool operator==(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		return InLHS.Name == InRHS.Name && InLHS.Number == InRHS.Number;
	}

	friend bool operator!=(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend bool operator>(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		if (InRHS.Name.FastLess(InLHS.Name))
		{
			return true;
		}

		if (InLHS.Name == InRHS.Name)
		{
			return InLHS.Number > InRHS.Number;
		}

		return false;
	}

	friend bool operator>=(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		return InLHS == InRHS || InLHS > InRHS;
	}

	friend bool operator<(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		if (InLHS.Name.FastLess(InRHS.Name))
		{
			return true;
		}

		if (InLHS.Name == InRHS.Name)
		{
			return InLHS.Number < InRHS.Number;
		}

		return false;
	}

	friend bool operator<=(const FMetasoundFrontendVersion& InLHS, const FMetasoundFrontendVersion& InRHS)
	{
		return InLHS == InRHS || InLHS < InRHS;
	}

	friend inline uint32 GetTypeHash(const FMetasoundFrontendVersion& InVersion)
	{
		return HashCombineFast(GetTypeHash(InVersion.Name), GetTypeHash(InVersion.Number));
	}
};

// An FMetasoundFrontendVertex provides a named connection point of a node.
USTRUCT() 
struct FMetasoundFrontendVertex
{
	GENERATED_BODY()

	// Name of the vertex. Unique amongst other vertices on the same interface.
	UPROPERTY(VisibleAnywhere, Category = CustomView)
	FName Name;

	// Data type name of the vertex.
	UPROPERTY(VisibleAnywhere, Category = Parameters)
	FName TypeName;

	// ID of vertex
	UPROPERTY()
	FGuid VertexID;

	// Returns true if vertices have equal name & type.
	static UE_API bool IsFunctionalEquivalent(const FMetasoundFrontendVertex& InLHS, const FMetasoundFrontendVertex& InRHS);

	friend METASOUNDFRONTEND_API bool operator==(const FMetasoundFrontendVertex& InLHS, const FMetasoundFrontendVertex& InRHS);
};

// Pair of guids used to address location of vertex within a FrontendDocument graph
USTRUCT()
struct FMetasoundFrontendVertexHandle
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FGuid NodeID;

	UPROPERTY()
	FGuid VertexID;

	// Returns whether or not the vertex handle is set (may or may not be
	// valid depending on what builder context it is referenced against)
	bool IsSet() const
	{
		return NodeID.IsValid() && VertexID.IsValid();
	}

	friend bool operator==(const FMetasoundFrontendVertexHandle& InLHS, const FMetasoundFrontendVertexHandle& InRHS)
	{
		return InLHS.NodeID == InRHS.NodeID && InLHS.VertexID == InRHS.VertexID;
	}

	friend bool operator!=(const FMetasoundFrontendVertexHandle& InLHS, const FMetasoundFrontendVertexHandle& InRHS)
	{
		return InLHS.NodeID != InRHS.NodeID || InLHS.VertexID != InRHS.VertexID;
	}

	friend inline uint32 GetTypeHash(const FMetasoundFrontendVertexHandle& InHandle)
	{
		return HashCombineFast(InHandle.NodeID.A, InHandle.VertexID.D);
	}
};

// Contains a default value for a single vertex ID
USTRUCT() 
struct FMetasoundFrontendVertexLiteral
{
	GENERATED_BODY()

	// ID of vertex.
	UPROPERTY(VisibleAnywhere, Category = Parameters)
	FGuid VertexID = Metasound::FrontendInvalidID;

	// Value to use when constructing input.
	UPROPERTY(EditAnywhere, Category = Parameters)
	FMetasoundFrontendLiteral Value;
};

// Contains graph data associated with a variable.
USTRUCT()
struct FMetasoundFrontendVariable
{
	GENERATED_BODY()

	// Name of the vertex. Unique amongst other vertices on the same interface.
	UPROPERTY(VisibleAnywhere, Category = CustomView)
	FName Name;

#if WITH_EDITORONLY_DATA
	// Variable display name
	UPROPERTY()
	FText DisplayName;

	// Variable description
	UPROPERTY()
	FText Description;

#endif // WITH_EDITORONLY_DATA

	// Variable data type name
	UPROPERTY()
	FName TypeName;

	// Literal used to initialize the variable.
	UPROPERTY()
	FMetasoundFrontendLiteral Literal;

	// Unique ID for the variable
	UPROPERTY()
	FGuid ID = Metasound::FrontendInvalidID;

	// Node ID of the associated VariableNode
	UPROPERTY()
	FGuid VariableNodeID = Metasound::FrontendInvalidID;

	// Node ID of the associated VariableMutatorNode
	UPROPERTY()
	FGuid MutatorNodeID = Metasound::FrontendInvalidID;

	// Node IDs of the associated VariableAccessorNodes
	UPROPERTY()
	TArray<FGuid> AccessorNodeIDs;

	// Node IDs of the associated VariableDeferredAccessorNodes
	UPROPERTY()
	TArray<FGuid> DeferredAccessorNodeIDs;
};


USTRUCT()
struct FMetasoundFrontendNodeInterface
{
	GENERATED_BODY()

	FMetasoundFrontendNodeInterface() = default;

	// Create a node interface which satisfies an existing class interface.
	UE_API FMetasoundFrontendNodeInterface(const FMetasoundFrontendClassInterface& InClassInterface);

	// Update the current node interface with the given class interface
	// Return true if interface update resulted in interface changes
	UE_INTERNAL
	UE_API bool Update(const FMetasoundFrontendClassInterface& InClassInterface);

	UE_INTERNAL
	UE_API bool Update(const FMetasoundFrontendClassInterface& InClassInterface, TFunctionRef<void(const FMetasoundFrontendVertex&)> OnPreRemoveInput, TFunctionRef<void(const FMetasoundFrontendVertex&)> OnPreRemoveOutput);

	// Input vertices to node.
	UPROPERTY()
	TArray<FMetasoundFrontendVertex> Inputs;

	// Output vertices to node.
	UPROPERTY()
	TArray<FMetasoundFrontendVertex> Outputs;

	// Environment variables of node.
	UPROPERTY()
	TArray<FMetasoundFrontendVertex> Environment;
};

/** 
* Struct for configuring a node. You can inherit from this 
* and include data passed to the operator
* and/or data used to determine an override of the node's interface.
* 
* In order for node configuration data to be editable in the details panel, 
* UProperties on your substruct should be marked with EditAnywhere.
* 
* Optional custom details customizations can be registered via 
* IMetasoundEditorModule::RegisterCustomNodeConfigurationDetailsCustomization
* 
* Example: 
USTRUCT()
struct FMetaSoundExperimentalExampleNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "1000"))
	uint32 NumInputs;

	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
}
*/
USTRUCT()
struct FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	virtual ~FMetaSoundFrontendNodeConfiguration() = default;

	/* Get the current interface for the class based upon the node extension */
	UE_API virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	/** Provide any data needed by IOperators instantiated from this node. */
	UE_EXPERIMENTAL(5.6, "Node operator data is still under development")
	UE_API virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const;
};


// DEPRECATED in Document Model v1.1
UENUM()
enum class EMetasoundFrontendNodeStyleDisplayVisibility : uint8
{
	Visible,
	Hidden
};

USTRUCT()
struct FMetasoundFrontendNodeStyleDisplay
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// DEPRECATED in Document Model v1.1: Visibility state of node
	UPROPERTY()
	EMetasoundFrontendNodeStyleDisplayVisibility Visibility = EMetasoundFrontendNodeStyleDisplayVisibility::Visible;

	// Map of visual node guid to 2D location. May have more than one if the node allows displaying in
	// more than one place on the graph (Only functionally relevant for nodes that cannot contain inputs.)
	UPROPERTY()
	TMap<FGuid, FVector2D> Locations;

	// Comment to display about the given instance's usage
	UPROPERTY()
	FString Comment;

	// Whether or not the comment is visible or not
	UPROPERTY()
	bool bCommentVisible = false;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendNodeStyle
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// Display style of a node
	UPROPERTY()
	FMetasoundFrontendNodeStyleDisplay Display;

	// Whether or not to display if
	// the node's version has been updated
	UPROPERTY(Transient)
	mutable bool bMessageNodeUpdated = false;

	UPROPERTY()
	bool bIsPrivate = false;
	
	//Whether or not Unconnected pins are hidden
	UPROPERTY()
	bool bUnconnectedPinsHidden = false;

#endif // WITH_EDITORONLY_DATA
};


// Represents a single connection from one point to another.
USTRUCT()
struct FMetasoundFrontendEdge
{
	GENERATED_BODY()

	// ID of source node.
	UPROPERTY()
	FGuid FromNodeID = Metasound::FrontendInvalidID;

	// ID of source point on source node.
	UPROPERTY()
	FGuid FromVertexID = Metasound::FrontendInvalidID;

	// ID of destination node.
	UPROPERTY()
	FGuid ToNodeID = Metasound::FrontendInvalidID;

	// ID of destination point on destination node.
	UPROPERTY()
	FGuid ToVertexID = Metasound::FrontendInvalidID;

	FMetasoundFrontendVertexHandle GetFromVertexHandle() const
	{
		return FMetasoundFrontendVertexHandle { FromNodeID, FromVertexID };
	}

	FMetasoundFrontendVertexHandle GetToVertexHandle() const
	{
		return FMetasoundFrontendVertexHandle { ToNodeID, ToVertexID };
	}

	friend bool operator==(const FMetasoundFrontendEdge& InLHS, const FMetasoundFrontendEdge& InRHS)
	{
		return InLHS.FromNodeID == InRHS.FromNodeID
			&& InLHS.FromVertexID == InRHS.FromVertexID
			&& InLHS.ToNodeID == InRHS.ToNodeID
			&& InLHS.ToVertexID == InRHS.ToVertexID;
	}

	friend bool operator!=(const FMetasoundFrontendEdge& InLHS, const FMetasoundFrontendEdge& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend inline uint32 GetTypeHash(const FMetasoundFrontendEdge& InEdge)
	{
		const int32 FromHash = HashCombineFast(InEdge.FromNodeID.A, InEdge.FromVertexID.B);
		const int32 ToHash = HashCombineFast(InEdge.ToNodeID.C, InEdge.ToVertexID.D);
		return HashCombineFast(FromHash, ToHash);
	}
};

USTRUCT()
struct FMetasoundFrontendEdgeStyleLiteralColorPair
{
	GENERATED_BODY()

	UPROPERTY()
	FMetasoundFrontendLiteral Value;

	UPROPERTY()
	FLinearColor Color = Metasound::Frontend::DisplayStyle::EdgeAnimation::DefaultColor;
};

// Styling for all edges associated with a given output (characterized by NodeID & Name)
USTRUCT()
struct FMetasoundFrontendEdgeStyle
{
	GENERATED_BODY()

	// Node ID for associated edge(s) that should use the given style data.
	UPROPERTY()
	FGuid NodeID;

	// Name of node's output to associate style information for its associated edge(s).
	UPROPERTY()
	FName OutputName;

	// Array of colors used to animate given output's associated edge(s). Interpolation
	// between values dependent on value used.
	UPROPERTY()
	TArray<FMetasoundFrontendEdgeStyleLiteralColorPair> LiteralColorPairs;
};

// Styling for a class
USTRUCT()
struct FMetasoundFrontendGraphStyle
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// Whether or not the graph is editable by a user
	UPROPERTY()
	bool bIsGraphEditable = true;

	// Styles for graph edges.
	UPROPERTY()
	TArray<FMetasoundFrontendEdgeStyle> EdgeStyles;

	// Map of comment id to comment data
	UPROPERTY()
	TMap<FGuid, FMetaSoundFrontendGraphComment> Comments;
#endif // WITH_EDITORONLY_DATA
};


// Metadata associated with a vertex.
USTRUCT()
struct FMetasoundFrontendVertexMetadata
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
private:
	// Display name for a vertex
	UPROPERTY(EditAnywhere, Category = Parameters, meta = (DisplayName = "Name"))
	FText DisplayName;

	// Display name for a vertex if vertex is natively defined
	// (must be transient to avoid localization desync on load)
	UPROPERTY(Transient)
	FText DisplayNameTransient;

	// Description of the vertex.
	UPROPERTY(EditAnywhere, Category = Parameters)
	FText Description;

	// Description of the vertex if vertex is natively defined
	// (must be transient to avoid localization desync on load)
	UPROPERTY(Transient)
	FText DescriptionTransient;

public:
	// Order index of vertex member when shown as a node.
	UPROPERTY()
	int32 SortOrderIndex = 0;

	// If true, vertex is shown for advanced display.
	UPROPERTY()
	bool bIsAdvancedDisplay = false;

private:
	// Whether or not the given metadata text should be serialized
	// or is procedurally maintained via auto-update & the referenced
	// registry class (to avoid localization text desync).  Should be
	// false for classes serialized as externally-defined dependencies
	// or interfaces.
	UPROPERTY()
	bool bSerializeText = true;

	FText& GetDescription()
	{
		return bSerializeText ? Description : DescriptionTransient;
	}

	FText& GetDisplayName()
	{
		return bSerializeText ? DisplayName : DisplayNameTransient;
	}

public:
	const FText& GetDescription() const
	{
		return bSerializeText ? Description : DescriptionTransient;
	}

	const FText& GetDisplayName() const
	{
		return bSerializeText ? DisplayName : DisplayNameTransient;
	}

	bool GetSerializeText() const
	{
		return bSerializeText;
	}

	void SetDescription(const FText& InText)
	{
		GetDescription() = InText;
	}

	void SetDisplayName(const FText& InText)
	{
		GetDisplayName() = InText;
	}

	void SetIsAdvancedDisplay(const bool InIsAdvancedDisplay)
	{		
		bIsAdvancedDisplay = InIsAdvancedDisplay;
	}

	void SetSerializeText(bool bInSerializeText)
	{
		if (bSerializeText)
		{
			if (!bInSerializeText)
			{
				DisplayNameTransient = DisplayName;
				DescriptionTransient = Description;

				DisplayName = { };
				Description  = { };
			}
		}
		else
		{
			if (bInSerializeText)
			{
				DisplayName = DisplayNameTransient;
				Description = DescriptionTransient;

				DisplayNameTransient = { };
				DescriptionTransient = { };
			}
		}
		
		bSerializeText = bInSerializeText;
	}
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendClassVertex : public FMetasoundFrontendVertex
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid NodeID = Metasound::FrontendInvalidID;

#if WITH_EDITORONLY_DATA
	// Metadata associated with vertex.
	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendVertexMetadata Metadata;

	const bool GetIsAdvancedDisplay() const { return Metadata.bIsAdvancedDisplay; };
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Reference;

	// Splits name into namespace & parameter name
	UE_API void SplitName(FName& OutNamespace, FName& OutParameterName) const;

	static UE_API bool IsFunctionalEquivalent(const FMetasoundFrontendClassVertex& InLHS, const FMetasoundFrontendClassVertex& InRHS);

	// Whether vertex access types are compatible when connecting from an output to an input 
	static UE_API bool CanConnectVertexAccessTypes(EMetasoundFrontendVertexAccessType InFromType, EMetasoundFrontendVertexAccessType InToType);
};


// Information regarding how to display a node class
USTRUCT()
struct FMetasoundFrontendClassStyleDisplay
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	FMetasoundFrontendClassStyleDisplay() = default;

	FMetasoundFrontendClassStyleDisplay(const Metasound::FNodeDisplayStyle& InDisplayStyle)
	:	ImageName(InDisplayStyle.ImageName)
	,	bShowName(InDisplayStyle.bShowName)
	,	bShowInputNames(InDisplayStyle.bShowInputNames)
	,	bShowOutputNames(InDisplayStyle.bShowOutputNames)
	,	bShowLiterals(InDisplayStyle.bShowLiterals)
	,	StyleSet(InDisplayStyle.StyleSet)
	,	Icon(InDisplayStyle.Icon)
	,	Color(InDisplayStyle.Color)
	{
	}

	UPROPERTY()
	FName ImageName;

	UPROPERTY()
	bool bShowName = true;

	UPROPERTY()
	bool bShowInputNames = true;

	UPROPERTY()
	bool bShowOutputNames = true;

	UPROPERTY()
	bool bShowLiterals = true;

	UPROPERTY()
	FName StyleSet;

	UPROPERTY()
	FName Icon;

	UPROPERTY()
	FName Color;
#endif // WITH_EDITORONLY_DATA
};


USTRUCT()
struct FMetasoundFrontendClassInputDefault
{
	GENERATED_BODY()

	FMetasoundFrontendClassInputDefault() = default;
	UE_API FMetasoundFrontendClassInputDefault(FMetasoundFrontendLiteral InLiteral);
	UE_API FMetasoundFrontendClassInputDefault(const FGuid& InPageID, FMetasoundFrontendLiteral InLiteral = { });
	UE_API FMetasoundFrontendClassInputDefault(const FAudioParameter& InParameter);

	static UE_API bool IsFunctionalEquivalent(const FMetasoundFrontendClassInputDefault& InLHS, const FMetasoundFrontendClassInputDefault& InRHS);
	friend bool operator==(const FMetasoundFrontendClassInputDefault& InLHS, const FMetasoundFrontendClassInputDefault& InRHS);

	UPROPERTY()
	FMetasoundFrontendLiteral Literal;

	UPROPERTY()
	FGuid PageID = Metasound::Frontend::DefaultPageID;
};

// Contains info for input vertex of a Metasound class.
USTRUCT() 
struct FMetasoundFrontendClassInput : public FMetasoundFrontendClassVertex
{
	GENERATED_BODY()

	FMetasoundFrontendClassInput() = default;
	UE_API FMetasoundFrontendClassInput(const FMetasoundFrontendClassVertex& InOther);
	UE_API FMetasoundFrontendClassInput(const Audio::FParameterInterface::FInput& InInput);

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecationMessage = "5.5 - Direct access will be revoked and page manipulation limited to public API in future builds. Field has been rolled into DefaultLiterals Array."))
	FMetasoundFrontendLiteral DefaultLiteral = FMetasoundFrontendLiteral::GetInvalid();
#endif // WITH_EDITORONLY_DATA

	static UE_API bool IsFunctionalEquivalent(const FMetasoundFrontendClassInput& InLHS, const FMetasoundFrontendClassInput& InRHS);

private:
	UPROPERTY(EditAnywhere, Category = Parameters)
	TArray<FMetasoundFrontendClassInputDefault> Defaults;

public:
	UE_API FMetasoundFrontendLiteral& AddDefault(const FGuid& InPageID);
	UE_API bool ContainsDefault(const FGuid& InPageID) const;
	UE_API const FMetasoundFrontendLiteral* FindConstDefault(const FGuid& InPageID) const;
	UE_API const FMetasoundFrontendLiteral& FindConstDefaultChecked(const FGuid& InPageID) const;
	UE_API FMetasoundFrontendLiteral* FindDefault(const FGuid& InPageID);
	UE_API FMetasoundFrontendLiteral& FindDefaultChecked(const FGuid& InPageID);
	UE_API const TArray<FMetasoundFrontendClassInputDefault>& GetDefaults() const;
	UE_API FMetasoundFrontendLiteral& InitDefault();
	UE_API void InitDefault(FMetasoundFrontendLiteral InitLiteral);
	UE_API void IterateDefaults(TFunctionRef<void(const FGuid&, FMetasoundFrontendLiteral&)> IterFunc);
	UE_API void IterateDefaults(TFunctionRef<void(const FGuid&, const FMetasoundFrontendLiteral&)> IterFunc) const;
	UE_API bool RemoveDefault(const FGuid& InPageID);
	UE_API void ResetDefaults(bool bInitializeDefaultPage = true);
	UE_API void SetDefaults(TArray<FMetasoundFrontendClassInputDefault> InputDefaults);
};

// Contains info for variable vertex of a Metasound class.
USTRUCT() 
struct FMetasoundFrontendClassVariable : public FMetasoundFrontendClassVertex
{
	GENERATED_BODY()

	FMetasoundFrontendClassVariable() = default;
	UE_API FMetasoundFrontendClassVariable(const FMetasoundFrontendClassVertex& InOther);

	// Default value for this variable.
	UPROPERTY(EditAnywhere, Category = Parameters)
	FMetasoundFrontendLiteral DefaultLiteral;
};

// Contains info for output vertex of a Metasound class.
USTRUCT()
struct FMetasoundFrontendClassOutput : public FMetasoundFrontendClassVertex
{
	GENERATED_BODY()

	FMetasoundFrontendClassOutput() = default;
	UE_API FMetasoundFrontendClassOutput(const FMetasoundFrontendClassVertex& InOther);
	UE_API FMetasoundFrontendClassOutput(const Audio::FParameterInterface::FOutput& Output);
};

USTRUCT()
struct FMetasoundFrontendClassEnvironmentVariable
{
	GENERATED_BODY()

	FMetasoundFrontendClassEnvironmentVariable() = default;
	UE_API FMetasoundFrontendClassEnvironmentVariable(const Audio::FParameterInterface::FEnvironmentVariable& InVariable);

	// Name of environment variable.
	UPROPERTY()
	FName Name;

	// Type of environment variable.
	UPROPERTY()
	FName TypeName;

	// True if the environment variable is needed in order to instantiate a node instance of the class.
	// TODO: Should be deprecated?
	UPROPERTY()
	bool bIsRequired = true;
};

// Style info of an interface.
USTRUCT()
struct FMetasoundFrontendInterfaceStyle
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// Default vertex sort order, where array index mirrors array interface index and value is display sort index.
	UPROPERTY()
	TArray<int32> DefaultSortOrder;

	// Map of member names with FText to be used as warnings if not hooked up
	UPROPERTY()
	TMap<FName, FText> RequiredMembers;

	UE_API void SortVertices(TArray<const FMetasoundFrontendVertex*>& OutVertices, TFunctionRef<FText(const FMetasoundFrontendVertex&)> InGetDisplayNamePredicate) const;

	template <typename HandleType, typename NamePredicateType>
	UE_DEPRECATED(5.7, "Use SortVertices and the builder API instead of this template (which is built primarily for use with the controller API, currently being deprecated)")
	void SortDefaults(TArray<HandleType>& OutHandles, NamePredicateType InGetDisplayNamePredicate) const
	{
		TMap<FGuid, int32> NodeIDToSortIndex;
		int32 HighestSortOrder = TNumericLimits<int32>::Min();
		for (int32 i = 0; i < OutHandles.Num(); ++i)
		{
			const FGuid& HandleID = OutHandles[i]->GetID();
			int32 SortIndex = 0;
			if (DefaultSortOrder.IsValidIndex(i))
			{
				SortIndex = DefaultSortOrder[i];
				HighestSortOrder = FMath::Max(SortIndex, HighestSortOrder);
			}
			else
			{
				SortIndex = ++HighestSortOrder;
			}
			NodeIDToSortIndex.Add(HandleID, SortIndex);
		}

		OutHandles.Sort([&NodeIDToSortIndex, &InGetDisplayNamePredicate](const HandleType& HandleA, const HandleType& HandleB) -> bool
		{
			const FGuid HandleAID = HandleA->GetID();
			const FGuid HandleBID = HandleB->GetID();
			const int32 AID = NodeIDToSortIndex[HandleAID];
			const int32 BID = NodeIDToSortIndex[HandleBID];

			// If IDs are equal, sort alphabetically using provided name predicate
			if (AID == BID)
			{
				return Invoke(InGetDisplayNamePredicate, HandleA).CompareTo(Invoke(InGetDisplayNamePredicate, HandleB)) < 0;
			}
			return AID < BID;
		});
	}
#endif // #if WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendClassInterface
{
	GENERATED_BODY()

private:
#if WITH_EDITORONLY_DATA

	// Style info for inputs.
	UPROPERTY()
	FMetasoundFrontendInterfaceStyle InputStyle;

	// Style info for outputs.
	UPROPERTY()
	FMetasoundFrontendInterfaceStyle OutputStyle;
#endif // WITH_EDITORONLY_DATA

public:

	// Generates class interface intended to be used as a registry descriptor from FNodeClassMetadata.
	// Does not initialize a change ID as it is not considered to be transactional.
	static UE_API FMetasoundFrontendClassInterface GenerateClassInterface(const Metasound::FVertexInterface& InVertexInterface);

	// Description of class inputs.
	UPROPERTY(VisibleAnywhere, Category = CustomView)
	TArray<FMetasoundFrontendClassInput> Inputs;

	// Description of class outputs.
	UPROPERTY(VisibleAnywhere, Category = CustomView)
	TArray<FMetasoundFrontendClassOutput> Outputs;

	// Description of class environment variables.
	UPROPERTY(VisibleAnywhere, Category = CustomView)
	TArray<FMetasoundFrontendClassEnvironmentVariable> Environment;

private:
	UPROPERTY(Transient)
	FGuid ChangeID;

public:
#if WITH_EDITORONLY_DATA
	const FMetasoundFrontendInterfaceStyle& GetInputStyle() const
	{
		return InputStyle;
	}

	void SetInputStyle(FMetasoundFrontendInterfaceStyle InInputStyle)
	{
		// A bit of a hack to only update the ChangeID if something in the sort
		// order has changed to avoid invalidating node widgets and editor graph
		// re-synchronization. This can cause major perf regression on graph edits.
		// Currently, RequiredMembers do not change as interfaces are registered
		// once so no need to check them.
		if (InInputStyle.DefaultSortOrder != InputStyle.DefaultSortOrder)
		{
			ChangeID = FGuid::NewGuid();
		}
		InputStyle = MoveTemp(InInputStyle);
	}

	const FMetasoundFrontendInterfaceStyle& GetOutputStyle() const
	{
		return OutputStyle;
	}

	void SetOutputStyle(FMetasoundFrontendInterfaceStyle InOutputStyle)
	{
		// A bit of a hack to only update the ChangeID if something in the sort
		// order has changed to avoid invalidating node widgets and editor graph
		// re-synchronization. This can cause major perf regression on graph edits.
		// Currently, RequiredMembers do not change as interfaces are registered
		// once so no need to check them.
		if (InOutputStyle.DefaultSortOrder != OutputStyle.DefaultSortOrder)
		{
			ChangeID = FGuid::NewGuid();
		}
		OutputStyle = MoveTemp(InOutputStyle);
	}

	void AddRequiredInputToStyle(const FName& InInputName, const FText& InRequiredText)
	{
		InputStyle.RequiredMembers.Add(InInputName, InRequiredText);
		ChangeID = FGuid::NewGuid();
	}

	void AddRequiredOutputToStyle(const FName& InOutputName, const FText& InRequiredText)
	{
		OutputStyle.RequiredMembers.Add(InOutputName, InRequiredText);
		ChangeID = FGuid::NewGuid();
	}

	bool IsMemberInputRequired(const FName& InInputName, FText& OutRequiredText)
	{
		if (FText* RequiredText = InputStyle.RequiredMembers.Find(InInputName))
		{
			OutRequiredText = *RequiredText;
			return true;
		}
		return false;
	}

	bool IsMemberOutputRequired(const FName& InOutputName, FText& OutRequiredText)
	{
		if (FText* RequiredText = OutputStyle.RequiredMembers.Find(InOutputName))
		{
			OutRequiredText = *RequiredText;
			return true;
		}
		return false;
	}

	void AddSortOrderToInputStyle(const int32 InSortOrder)
	{
		InputStyle.DefaultSortOrder.Add(InSortOrder);
		ChangeID = FGuid::NewGuid();
	}

	void AddSortOrderToOutputStyle(const int32 InSortOrder)
	{
		OutputStyle.DefaultSortOrder.Add(InSortOrder);
		ChangeID = FGuid::NewGuid();
	}
#endif // #if WITH_EDITORONLY_DATA


	const FGuid& GetChangeID() const
	{
		return ChangeID;
	}

	// TODO: This is unfortunately required to be manually managed and executed anytime the input/output/environment arrays
	// are mutated due to the design of the controller system obscuring away read/write permissions
	// when querying.  Need to add accessors and refactor so that this isn't as error prone and
	// remove manual execution at the call sites when mutating aforementioned UPROPERTIES.
	void UpdateChangeID()
	{
		ChangeID = FGuid::NewGuid();
	}

	// Required to allow caching registry data without modifying the ChangeID
	friend struct FMetasoundFrontendClass;
};

USTRUCT()
struct FMetasoundFrontendInterfaceVertexBinding
{
	GENERATED_BODY()

	UPROPERTY()
	FName OutputName;

	UPROPERTY()
	FName InputName;

	friend bool operator==(const FMetasoundFrontendInterfaceVertexBinding& InLHS, const FMetasoundFrontendInterfaceVertexBinding& InRHS)
	{
		return InLHS.OutputName == InRHS.OutputName && InLHS.InputName == InRHS.InputName;
	}

	friend bool operator!=(const FMetasoundFrontendInterfaceVertexBinding& InLHS, const FMetasoundFrontendInterfaceVertexBinding& InRHS)
	{
		return InLHS.OutputName != InRHS.OutputName || InLHS.InputName != InRHS.InputName;
	}

	FString ToString() const
	{
		return FString::Format(TEXT("{0}->{1}"), { OutputName.ToString(), InputName.ToString() });
	}

	friend inline uint32 GetTypeHash(const FMetasoundFrontendInterfaceVertexBinding& InBinding)
	{
		return HashCombineFast(GetTypeHash(InBinding.OutputName), GetTypeHash(InBinding.InputName));
	}
};


USTRUCT()
struct FMetasoundFrontendInterfaceBinding
{
	GENERATED_BODY()

	// Version of interface to bind from (the corresponding output vertices)
	UPROPERTY()
	FMetasoundFrontendVersion OutputInterfaceVersion;

	// Version of interface to bind to (the corresponding input vertices)
	UPROPERTY()
	FMetasoundFrontendVersion InputInterfaceVersion;

	// Value describing if interface binding priority is higher or lower than another interface
	// binding that may be shared between vertices attempting to be connected via binding functionality.
	UPROPERTY()
	int32 BindingPriority = 0;

	// Array of named pairs (output & input names) that describe what edges to create if binding functionality
	// is executed between two nodes.
	UPROPERTY()
	TArray<FMetasoundFrontendInterfaceVertexBinding> VertexBindings;
};


// Options used to restrict a corresponding UClass that interface may be applied to.
// If unspecified, interface is assumed to be applicable to any arbitrary UClass.
USTRUCT(BlueprintType)
struct FMetasoundFrontendInterfaceUClassOptions
{
	GENERATED_BODY()

	FMetasoundFrontendInterfaceUClassOptions() = default;
	UE_API FMetasoundFrontendInterfaceUClassOptions(const Audio::FParameterInterface::FClassOptions& InOptions);
	UE_API FMetasoundFrontendInterfaceUClassOptions(const FTopLevelAssetPath& InClassPath, bool bInIsModifiable = true, bool bInIsDefault = false);

	// Path to MetaSound class interface can be added to (ex. UMetaSoundSource or UMetaSound)
	UPROPERTY(BlueprintReadOnly, Category = Interface)
	FTopLevelAssetPath ClassPath;

	// True if user can add or remove the given class directly to or from the inherited interface UI, false if not.
	UPROPERTY()
	bool bIsModifiable = true;

	// True if interface should be added by default to newly created MetaSound assets, false if not.
	UPROPERTY()
	bool bIsDefault = false;
};


USTRUCT(BlueprintType)
struct FMetasoundFrontendInterfaceMetadata
{
	GENERATED_BODY()

	// Name and version number of the interface
	UPROPERTY(BlueprintReadOnly, Category = Interface)
	FMetasoundFrontendVersion Version;

	// If specified, options used to restrict a corresponding UClass that interface may be
	// applied to.  If unspecified, interface is assumed to be applicable to any arbitrary UClass.
	UPROPERTY(BlueprintReadOnly, Category = Interface)
	TArray<FMetasoundFrontendInterfaceUClassOptions> UClassOptions;

#if WITH_EDITORONLY_DATA
	// Developer-supplied description of the interface
	UPROPERTY(VisibleAnywhere, Category = Interface)
	FText Description;
#endif
};


// Definition of an interface that an FMetasoundFrontendClass adheres to in part or full.
USTRUCT()
struct FMetasoundFrontendInterface : public FMetasoundFrontendClassInterface
{
	GENERATED_BODY()

	FMetasoundFrontendInterface() = default;
	UE_API FMetasoundFrontendInterface(Audio::FParameterInterfacePtr InInterface);

	UPROPERTY()
	FMetasoundFrontendInterfaceMetadata Metadata;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "5.6 - Field never serialized but will be in the future, and moved to Metadata. Will be removed in subsequent release."))
	FMetasoundFrontendVersion Version;

	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "5.6 - Field never serialized but will be in the future, and moved to Metadata. Will be removed in subsequent release."))
	TArray<FMetasoundFrontendInterfaceUClassOptions> UClassOptions;

	UE_DEPRECATED(5.6, "Inlined where necessary. Use desired predicate look-up on UClassOptions from now-shared Metadata struct above.")
	UE_API const FMetasoundFrontendInterfaceUClassOptions* FindClassOptions(const FTopLevelAssetPath& InClassPath) const;
#endif // WITH_EDITORONLY_DATA
};


// Name of a Metasound class
USTRUCT(BlueprintType, meta = (DisplayName = "MetaSound Class Name"))
struct FMetasoundFrontendClassName
{
	GENERATED_BODY()

	FMetasoundFrontendClassName() = default;
	UE_API FMetasoundFrontendClassName(const FName& InNamespace, const FName& InName);
	UE_API FMetasoundFrontendClassName(const FName& InNamespace, const FName& InName, const FName& InVariant);
	UE_API FMetasoundFrontendClassName(const Metasound::FNodeClassName& InName);

	// Namespace of class.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	FName Namespace;

	// Name of class.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	FName Name;

	// Variant of class. The Variant is used to describe an equivalent class which performs the same operation but on differing types.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = General)
	FName Variant;

	// Returns a full name of the class.
	UE_API FName GetFullName() const;

	// Returns scoped name representing namespace and name. 
	UE_API FName GetScopedName() const;

	// Invalid form of class name (i.e. empty namespace, name, and variant)
	static UE_API const FMetasoundFrontendClassName InvalidClassName;

	// Whether or not this instance of a class name is a valid name.
	UE_API bool IsValid() const;

	// Returns NodeClassName version of full name
	UE_API Metasound::FNodeClassName ToNodeClassName() const;

	// Return string version of full name.
	UE_API FString ToString() const;

	// Return a string into an existing FNameBuilder
	UE_API void ToString(FNameBuilder& NameBuilder) const;

	// Parses string into class name.  For deserialization and debug use only.
	static UE_API bool Parse(const FString& InClassName, FMetasoundFrontendClassName& OutClassName);

	friend inline uint32 GetTypeHash(const FMetasoundFrontendClassName& ClassName)
	{
		const int32 NameHash = HashCombineFast(GetTypeHash(ClassName.Namespace), GetTypeHash(ClassName.Name));
		return HashCombineFast(NameHash, GetTypeHash(ClassName.Variant));
	}

	friend inline bool operator==(const FMetasoundFrontendClassName& InLHS, const FMetasoundFrontendClassName& InRHS)
	{
		return (InLHS.Namespace == InRHS.Namespace) && (InLHS.Name == InRHS.Name) && (InLHS.Variant == InRHS.Variant);
	}

	friend inline bool operator<(const FMetasoundFrontendClassName& InLHS, const FMetasoundFrontendClassName& InRHS)
	{
		if (InLHS.Namespace == InRHS.Namespace)
		{
			if (InLHS.Name == InRHS.Name)
			{
				return InLHS.Variant.FastLess(InRHS.Variant);
			}

			return InLHS.Name.FastLess(InRHS.Name);
		}

		return InLHS.Namespace.FastLess(InRHS.Namespace);
	}
};


USTRUCT()
struct FMetasoundFrontendClassMetadata
{
	GENERATED_BODY()

	UE_API FMetasoundFrontendClassMetadata();

	// Generates class metadata intended to be used as a registry descriptor from FNodeClassMetadata. Does not initialize a change ID as it is not considered to be transactional.
	static UE_API FMetasoundFrontendClassMetadata GenerateClassMetadata(const Metasound::FNodeClassMetadata& InNodeClassMetadata, EMetasoundFrontendClassType InType);

private:
	UPROPERTY(VisibleAnywhere, Category = Metasound)
	FMetasoundFrontendClassName ClassName;

	UPROPERTY(VisibleAnywhere, Category = Metasound)
	FMetasoundFrontendVersionNumber Version;

	UPROPERTY(VisibleAnywhere, Category = Metasound)
	EMetasoundFrontendClassType Type = EMetasoundFrontendClassType::Invalid;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Metasound)
	FText DisplayName;

	UPROPERTY(Transient)
	FText DisplayNameTransient;

	UPROPERTY(EditAnywhere, Category = Metasound)
	FText Description;

	UPROPERTY(Transient)
	FText DescriptionTransient;

	// TODO: Move to using a non-localized hint path.  Due to localization,
	// loading & the fact that class registration happens on demand (post serialization),
	// copying an FText to the referencing document can result in localization ids
	// mismatched to different text when attempting to gather text.
	UPROPERTY(Transient)
	FText PromptIfMissingTransient;

	UPROPERTY(EditAnywhere, Category = Metasound)
	FString Author;

	UPROPERTY(EditAnywhere, Category = Metasound)
	TArray<FText> Keywords;

	UPROPERTY(Transient)
	TArray<FText> KeywordsTransient;

	UPROPERTY(EditAnywhere, Category = Metasound)
	TArray<FText> CategoryHierarchy;

	UPROPERTY(Transient)
	TArray<FText> CategoryHierarchyTransient;

	// If true, this node is deprecated and should not be used in new MetaSounds.
	UPROPERTY(meta = (Deprecated = 5.7, DeprecationMessage = "Use Access Flags instead"))
	bool bIsDeprecated = false;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = Metasound, meta = (Bitmask, BitmaskEnum = "/Script/MetasoundFrontend.EMetasoundFrontendClassAccessFlags"))
	uint16 AccessFlags = static_cast<uint16>(EMetasoundFrontendClassAccessFlags::Referenceable);

#if WITH_EDITORONLY_DATA
	// If true, auto-update will manage (add and remove)
	// inputs/outputs associated with internally connected
	// nodes when the interface of the given node is auto-updated.
	UPROPERTY(meta = (Deprecated = 5.6, DeprecationMessage = "Boolean no longer observed (auto-update rules are managed by project settings now)"))
	bool bAutoUpdateManagesInterface = false;

	// Whether or not the given metadata text should be serialized
	// or is procedurally maintained via auto-update & the referenced
	// registry class (to avoid localization text desync).  Should be
	// false for classes serialized as externally-defined dependencies
	// or interfaces.
	UPROPERTY()
	bool bSerializeText = true;
#endif // WITH_EDITORONLY_DATA

	// ID used to identify if any of the above have been modified,
	// to determine if the parent class should be auto-updated.
	UPROPERTY(Transient)
	FGuid ChangeID;

public:
#if WITH_EDITOR
	static FName GetAccessFlagsPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, AccessFlags);
	}

	static FName GetAuthorPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, Author);
	}

	static FName GetCategoryHierarchyPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, CategoryHierarchy);
	}

	static FName GetDisplayNamePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, DisplayName);
	}

	static FName GetDescriptionPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, Description);
	}

	UE_DEPRECATED(5.7, "Deprecated in favor of access flags")
	static FName GetIsDeprecatedPropertyName()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, bIsDeprecated);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetKeywordsPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, Keywords);
	}

	static FName GetClassNamePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, ClassName);
	}

	static FName GetVersionPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FMetasoundFrontendClassMetadata, Version);
	}
#endif // WITH_EDITOR

	const FMetasoundFrontendClassName& GetClassName() const
	{
		return ClassName;
	}

	UE_API void SetClassName(const FMetasoundFrontendClassName& InClassName);

	UE_API EMetasoundFrontendClassType GetType() const;
	UE_API const FMetasoundFrontendVersionNumber& GetVersion() const;
	UE_API void AddAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags);
	UE_API void ClearAccessFlags();
	UE_API EMetasoundFrontendClassAccessFlags GetAccessFlags() const;
	UE_API void RemoveAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags);
	UE_API void SetAccessFlags(EMetasoundFrontendClassAccessFlags InAccessFlags);

#if WITH_EDITOR
	const FText& GetDisplayName() const
	{
		return bSerializeText ? DisplayName : DisplayNameTransient;
	}

	const FText& GetDescription() const
	{
		return bSerializeText ? Description : DescriptionTransient;
	}

	const FText& GetPromptIfMissing() const
	{
		return PromptIfMissingTransient;
	}

	const FString& GetAuthor() const
	{
		return Author;
	}

	const TArray<FText>& GetKeywords() const
	{
		return bSerializeText ? Keywords : KeywordsTransient;
	}

	const TArray<FText>& GetCategoryHierarchy() const
	{
		return bSerializeText ? CategoryHierarchy : CategoryHierarchyTransient;
	}

	UE_API void SetAuthor(const FString& InAuthor);
	UE_API void SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy);
	UE_API void SetDescription(const FText& InDescription);
	UE_API void SetDisplayName(const FText& InDisplayName);

	UE_DEPRECATED(5.7, "Use Add/Remove/Reset/Set/ClearAccessFlags instead")
	UE_API void SetIsDeprecated(bool bInIsDeprecated);

	UE_API void SetKeywords(const TArray<FText>& InKeywords);
	UE_API void SetPromptIfMissing(const FText& InPromptIfMissing);



	UE_API void SetSerializeText(bool bInSerializeText);
#endif // WITH_EDITOR

	UE_API void SetVersion(const FMetasoundFrontendVersionNumber& InVersion);

	const FGuid& GetChangeID() const
	{
		return ChangeID;
	}

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Use GetAccessFlags instead")
	bool GetIsDeprecated() const
	{
		return bIsDeprecated;
	}
#endif // WITH_EDITORONLY_DATA

	void SetType(const EMetasoundFrontendClassType InType)
	{
		Type = InType;
		// TODO: Type is modified while querying and swapped between
		// to be external, so don't modify the ChangeID in this case.
		// External/Internal should probably be a separate field.
		// ChangeID = FGuid::NewGuid();
	}

#if WITH_EDITORONLY_DATA
	// Deprecated field in favor of GraphClass PresetOptions
	bool GetAndClearAutoUpdateManagesInterface_Deprecated()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bool bToReturn = bAutoUpdateManagesInterface;
		bAutoUpdateManagesInterface = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return bToReturn;
	}
#endif // WITH_EDITORONLY_DATA
};


USTRUCT()
struct FMetasoundFrontendClassStyle
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	FMetasoundFrontendClassStyleDisplay Display;

	// Generates class style from core node class metadata.
	static FMetasoundFrontendClassStyle GenerateClassStyle(const Metasound::FNodeDisplayStyle& InNodeDisplayStyle);

	// Editor only ID that allows for pumping view to reflect changes to class.
	void UpdateChangeID() const
	{
		ChangeID = FGuid::NewGuid();
	}

	FGuid GetChangeID() const
	{
		return ChangeID;
	}

private:
	// TODO: Deprecate this change behavior in favor of using the builder API transaction counters
	UPROPERTY(Transient)
	mutable FGuid ChangeID;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendClass
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMetasoundFrontendClass() = default;
	virtual ~FMetasoundFrontendClass() = default;
	FMetasoundFrontendClass(const FMetasoundFrontendClass&) = default;
	FMetasoundFrontendClass(FMetasoundFrontendClass&&) = default;
	FMetasoundFrontendClass& operator=(const FMetasoundFrontendClass&) = default;
	FMetasoundFrontendClass& operator=(FMetasoundFrontendClass&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	FGuid ID = Metasound::FrontendInvalidID;

	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendClassMetadata Metadata;

	UE_DEPRECATED(5.6, "Use Get/Set Default Interface instead")
	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendClassInterface Interface;


	UE_API void SetDefaultInterface(const FMetasoundFrontendClassInterface& InInterface);
	UE_API FMetasoundFrontendClassInterface& GetDefaultInterface();
	UE_API const FMetasoundFrontendClassInterface& GetDefaultInterface() const;
	UE_API const FMetasoundFrontendClassInterface& GetInterfaceForNode(const FMetasoundFrontendNode& InNode) const;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FMetasoundFrontendClassStyle Style;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/*
	 * Caches transient style, class & vertex Metadata found in the registry
	 * on a passed (presumed) dependency.  Only modifies properties that are
	 * not necessary for serialization or core graph generation.
	 *
	 * @return - Whether class was found in the registry & data was cached successfully.
	 */
	static UE_API bool CacheGraphDependencyMetadataFromRegistry(FMetasoundFrontendClass& InOutDependency);
#endif // WITH_EDITOR
};

// An FMetasoundFrontendNode represents a single instance of a FMetasoundFrontendClass
USTRUCT()
struct FMetasoundFrontendNode
{
	GENERATED_BODY()

	FMetasoundFrontendNode() = default;

	UE_DEPRECATED(5.6, "Please use constructor which accepts a node extension.")
	UE_API FMetasoundFrontendNode(const FMetasoundFrontendClass& InClass);

	// Construct a node from a node class and an optional node extension 
	UE_API FMetasoundFrontendNode(const FMetasoundFrontendClass& InClass, TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InConfiguration);

private:
	// Unique ID of this node.
	UPROPERTY()
	FGuid ID = Metasound::FrontendInvalidID;

public:
	// ID of FMetasoundFrontendClass corresponding to this node.
	UPROPERTY()
	FGuid ClassID = Metasound::FrontendInvalidID;

	// Name of node instance.
	UPROPERTY()
	FName Name;

	// Interface of node instance.
	UPROPERTY()
	FMetasoundFrontendNodeInterface Interface;

	// Default values for node inputs.
	UPROPERTY()
	TArray<FMetasoundFrontendVertexLiteral> InputLiterals;

	// Instance of a configuration for this node.
	// This property is EditAnywhere with a false EditCondition 
	// so child properties (ex. node configuration) have exposed property handles for details customization code, 
	// but they should not be editable elsewhere in the editor (ex. property matrix)
	UPROPERTY(EditAnywhere, Category = CustomView, meta = (EditCondition = "false", HideProperty))
	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration;

	// An optional override to the default class interface.
	UPROPERTY()
	TInstancedStruct<FMetasoundFrontendClassInterface> ClassInterfaceOverride;

#if WITH_EDITORONLY_DATA
	// Style info related to a node.
	UPROPERTY()
	FMetasoundFrontendNodeStyle Style;
#endif // WITH_EDITORONLY_DATA

	const FGuid& GetID() const
	{
		return ID;
	}

	void UpdateID(const FGuid& InNewGuid)
	{
		ID = InNewGuid;
	}
};

// Preset options related to a parent graph class.  A graph class with bIsPreset set to true
// auto-updates to mirror the interface members (inputs & outputs) of the single, referenced
// node. It also connects all of these nodes' interface members on update to corresponding inputs
// & outputs, and inherits input defaults from the referenced node unless otherwise specified.
USTRUCT()
struct FMetasoundFrontendGraphClassPresetOptions
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.8 - Preset options are now serialized in the Preset MetaSound Template. See 'FMetaSoundFrontendPresetTemplate'"))
	bool bIsPreset = false;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.8 - Preset options are now serialized in the Preset MetaSound Template. See 'FMetaSoundFrontendPresetTemplate'"))
	TSet<FName> InputsInheritingDefault;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendGraph
{
	GENERATED_BODY()

	// Node contained in graph
	// This property is EditAnywhere with a false EditCondition 
	// so child properties (ex. node configuration) have exposed property handles for details customization code, 
	// but they should not be editable elsewhere in the editor (ex. property matrix)
	UPROPERTY(EditAnywhere, Category = CustomView, meta = (EditCondition = "false", HideProperty))
	TArray<FMetasoundFrontendNode> Nodes;

	// Connections between points on nodes.
	UPROPERTY()
	TArray<FMetasoundFrontendEdge> Edges;

	// Graph local variables.
	UPROPERTY()
	TArray<FMetasoundFrontendVariable> Variables;

#if WITH_EDITORONLY_DATA
	// Style of graph display.
	UPROPERTY()
	FMetasoundFrontendGraphStyle Style;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	FGuid PageID;
};

USTRUCT()
struct FMetasoundFrontendGraphClass : public FMetasoundFrontendClass
{
	GENERATED_BODY()

public:
	UE_API FMetasoundFrontendGraphClass();
	UE_API virtual ~FMetasoundFrontendGraphClass();

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecationMessage = "5.5 - GraphClasses now support multiple paged graphs. Use the provided page graph accessors"))
	FMetasoundFrontendGraph Graph;
#endif // WITH_EDITORONLY_DATA

private:
	// This property is EditAnywhere with a false EditCondition 
	// so child properties (ex. node configuration) have exposed property handles for details customization code, 
	// but they should not be editable elsewhere in the editor (ex. property matrix)
	UPROPERTY(EditAnywhere, Category = CustomView, meta = (EditCondition = "false", HideProperty))
	TArray<FMetasoundFrontendGraph> PagedGraphs;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecationMessage = "5.8 - PresetOptions is now characterized as configuration vertex metadata on MetaSound Document."))
	FMetasoundFrontendGraphClassPresetOptions PresetOptions;

	UE_API const FMetasoundFrontendGraph& AddGraphPage(const FGuid& InPageID, bool bDuplicateLastGraph = true, bool bSetAsBuildGraph = true);

	// Removes the page associated with the given PageID.  Returns true if removed, false if not.
	// If provided an "AdjacentPageID," sets the value at the given pointer to a page ID adjacent to
	// the removed page. If last page was removed, returns the default graph ID (which may or may not
	// exist).
	UE_API bool RemoveGraphPage(const FGuid& InPageID, FGuid* OutAdjacentPageID = nullptr);

	// Removes all graph pages except the default.  If bClearDefaultPage is true, clears the default graph page implementation.
	UE_API void ResetGraphPages(bool bClearDefaultGraph);
#endif // WITH_EDITORONLY_DATA

	UE_API bool ContainsGraphPage(const FGuid& InPageID) const;

	UE_API FMetasoundFrontendGraph& InitDefaultGraphPage();

	UE_API void IterateGraphPages(TFunctionRef<void(FMetasoundFrontendGraph&)> IterFunc);
	UE_API void IterateGraphPages(TFunctionRef<void(const FMetasoundFrontendGraph&)> IterFunc) const;

	UE_API FMetasoundFrontendGraph* FindGraph(const FGuid& InPageID);
	UE_API FMetasoundFrontendGraph& FindGraphChecked(const FGuid& InPageID);
	UE_API const FMetasoundFrontendGraph* FindConstGraph(const FGuid& InPageID) const;
	UE_API const FMetasoundFrontendGraph& FindConstGraphChecked(const FGuid& InPageID) const;
	const TArray<FMetasoundFrontendGraph>& GetConstGraphPages() const { return PagedGraphs; };
	UE_API FMetasoundFrontendGraph& GetDefaultGraph();
	UE_API const FMetasoundFrontendGraph& GetConstDefaultGraph() const;
	UE_API void ResetGraphs();

#if WITH_EDITORONLY_DATA
	struct IPropertyVersionTransform
	{
	public:
		virtual ~IPropertyVersionTransform() = default;

	protected:
		virtual bool Transform(FMetasoundFrontendGraphClass& OutClass) const = 0;

		// Allows for unsafe access to a document for property migration.
		static TArray<FMetasoundFrontendGraph>& GetPagesUnsafe(FMetasoundFrontendGraphClass& GraphClass);
	};
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, BlueprintType)
class UMetaSoundFrontendMemberMetadata : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid MemberID;

	// Enables migrating properties during asynchronous versioning safely. Favor using this
	// over versioning properties on 'PostLoad' to avoid race conditions.
	virtual bool MigrateProperties() { return false; }
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendDocumentMetadata
{
	GENERATED_BODY()

	UPROPERTY()
	FMetasoundFrontendVersion Version;

#if WITH_EDITORONLY_DATA
	// Actively being deprecated in favor of Document Builder Transaction Listener API.
	mutable FMetasoundFrontendDocumentModifyContext ModifyContext;

	// Map of MemberID to metadata used to constrain how literals can be manipulated
	// with the editor context. This can be used to implement things like numeric ranges,
	// hardware control parameters, etc.
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>> MemberMetadata;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMetasoundFrontendDocument
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "MaxVersion is now externally accessible via FVersioningManager::GetMaxDocumentVersion()")
	static UE_API FMetasoundFrontendVersionNumber GetMaxVersion();
#endif // WITH_EDITORONLY_DATA

	Metasound::Frontend::FAccessPoint AccessPoint;

	UE_API FMetasoundFrontendDocument();

	UPROPERTY(EditAnywhere, Category = Metadata)
	FMetasoundFrontendDocumentMetadata Metadata;

	UPROPERTY(VisibleAnywhere, Category = CustomView)
	TSet<FMetasoundFrontendVersion> Interfaces;

	UPROPERTY(EditAnywhere, Category = CustomView)
	FMetasoundFrontendGraphClass RootGraph;

	UPROPERTY()
	TArray<FMetasoundFrontendGraphClass> Subgraphs;

	UPROPERTY()
	TArray<FMetasoundFrontendClass> Dependencies;

	UPROPERTY(EditAnywhere, Category = CustomView)
	TInstancedStruct<FMetaSoundFrontendDocumentTemplate> Template;

	uint32 GetNextIdCounter() const
	{
		return IdCounter++;
	}

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - ArchetypeVersion has been migrated to InterfaceVersions array."))
	FMetasoundFrontendVersion ArchetypeVersion;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - InterfaceVersions has been migrated to Interfaces set."))
	TArray<FMetasoundFrontendVersion> InterfaceVersions;
#endif // WITH_EDITORONLY_DATA

	// Used for generating deterministic IDs per document. Serialized to avoid id collisions if deterministic IDs
	// are ever serialized (not ideal, but can occur in less common commandlet use cases such as resaving serialized
	// assets procedurally).
	UPROPERTY()
	mutable uint32 IdCounter = 1;

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Moved to internal implementation. See 'MigrateProperties'")
	bool RequiresInterfaceVersioning() const
	{
		return false;
	}

	UE_DEPRECATED(5.8, "Moved to internal implementation. See 'MigrateProperties'")
	void VersionInterfaces()
	{
	}

	// Migrates data from deprecated properties to new properties
	UE_INTERNAL bool MigrateProperties();
#endif // WITH_EDITORONLY_DATA
};

METASOUNDFRONTEND_API const TCHAR* LexToString(EMetasoundFrontendClassType InClassType);
METASOUNDFRONTEND_API const TCHAR* LexToString(EMetasoundFrontendVertexAccessType InVertexAccess);

namespace Metasound::Frontend
{
	// Convert access type between enums
	METASOUNDFRONTEND_API EMetasoundFrontendVertexAccessType CoreVertexAccessTypeToFrontendVertexAccessType(Metasound::EVertexAccessType InAccessType);

	// Convert access type between enums
	METASOUNDFRONTEND_API EVertexAccessType FrontendVertexAccessTypeToCoreVertexAccessType(EMetasoundFrontendVertexAccessType InAccessType);

	METASOUNDFRONTEND_API bool StringToClassType(const FString& InString, EMetasoundFrontendClassType& OutClassType);

	/** Signature of function called for each found literal. */
	using FForEachLiteralFunctionRef = TFunctionRef<void(const FName& InDataTypeName, const FMetasoundFrontendLiteral&)>;

	/** Execute the provided function for each literal on a FMetasoundFrontendDocument.*/
	UE_DEPRECATED(5.7, "Use version which requires a PageOrder")
	UE_API void ForEachLiteral(const FMetasoundFrontendDocument& InDoc, FForEachLiteralFunctionRef OnLiteral);

	/** Execute the provided function for each literal on a FMetasoundFrontendGraphClass.*/
	UE_DEPRECATED(5.7, "Use version which requires a PageOrder")
	UE_API void ForEachLiteral(const FMetasoundFrontendGraphClass& InGraphClass, FForEachLiteralFunctionRef OnLiteral);

	/** Execute the provided function for each literal on a FMetasoundFrontendClass.*/
	UE_DEPRECATED(5.7, "Use version which requires a PageOrder")
	UE_API void ForEachLiteral(const FMetasoundFrontendClass& InClass, FForEachLiteralFunctionRef OnLiteral);

	/** Execute the provided function for each literal on a FMetasoundFrontendNode.*/
	UE_DEPRECATED(5.7, "Use version which requires a PageOrder")
	UE_API void ForEachLiteral(const FMetasoundFrontendNode& InNode, FForEachLiteralFunctionRef OnLiteral);

	UE_DEPRECATED(5.7, "Use version which requires a PageOrder")
	UE_API void ForEachLiteral(const FMetasoundFrontendClassInterface& InClassInterface, FForEachLiteralFunctionRef OnLiteral);

	/** Execute the provided function for each literal on a FMetasoundFrontendDocument.*/
	UE_API void ForEachLiteral(const FMetasoundFrontendDocument& InDoc, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder);
	
	/** Execute the provided function for each literal on a FMetasoundFrontendGraphClass.*/
	UE_API void ForEachLiteral(const FMetasoundFrontendGraphClass& InGraphClass, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder);
	
	/** Execute the provided function for each literal on a FMetasoundFrontendClass.*/
	UE_API void ForEachLiteral(const FMetasoundFrontendClass& InClass, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder);

	/** Execute the provided function for each literal on a FMetasoundFrontendClassInterface.*/
	UE_API void ForEachLiteral(const FMetasoundFrontendClassInterface& InClassInterface, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder);

	/** Execute the provided function for each literal on a FMetasoundFrontendNode.*/
	UE_API void ForEachLiteral(const FMetasoundFrontendNode& InNode, FForEachLiteralFunctionRef OnLiteral, TArrayView<const FGuid> InPageOrder);

#if WITH_EDITORONLY_DATA
	// Provides list of interface members that have been added or removed
	// when querying if a node's class has been updated. Editor only as 
	// interface can only be updated from editor builds.
	struct FClassInterfaceUpdates
	{
		FClassInterfaceUpdates() = default;
		TArray<const FMetasoundFrontendClassInput*> AddedInputs;
		TArray<const FMetasoundFrontendClassOutput*> AddedOutputs;
		TArray<const FMetasoundFrontendClassInput*> RemovedInputs;
		TArray<const FMetasoundFrontendClassOutput*> RemovedOutputs;
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> AddedConfiguration;
		TConstStructView<FMetaSoundFrontendNodeConfiguration> RemovedConfiguration;
		TInstancedStruct<FMetasoundFrontendClassInterface> AddedClassInterfaceOverride;
		TConstStructView<FMetasoundFrontendClassInterface> RemovedClassInterfaceOverride;

		bool ContainsRemovedMembers() const
		{
			return !RemovedInputs.IsEmpty() || !RemovedOutputs.IsEmpty();
		}

		bool ContainsAddedMembers() const
		{
			return !AddedInputs.IsEmpty() || !AddedOutputs.IsEmpty();
		}

		bool ContainsUpdatedNodeConfig() const
		{
			return AddedConfiguration.IsValid() || RemovedConfiguration.IsValid();
		}

		bool ContainsUpdatedNodeClassInterfaceOverride() const
		{
			return AddedClassInterfaceOverride.IsValid() || RemovedClassInterfaceOverride.IsValid();
		}

		bool ContainsChanges() const
		{
			return ContainsRemovedMembers() || ContainsAddedMembers() || ContainsUpdatedNodeConfig() || ContainsUpdatedNodeClassInterfaceOverride();
		}

		// Cached copy of registry class potentially referenced by added members
		FMetasoundFrontendClass RegistryClass;
	};
#endif // WITH_EDITORONLY_DATA
} // namespace Metasound::Frontend

#undef UE_API
