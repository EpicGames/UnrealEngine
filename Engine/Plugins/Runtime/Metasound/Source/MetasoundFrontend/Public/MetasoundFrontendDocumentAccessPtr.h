// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundAccessPtr.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"

#define UE_API METASOUNDFRONTEND_API

/** Metasound Frontend Document Access Pointers provide convenience methods for
 * getting access pointers of FMetasoundFrontendDocument sub-elements. For instance,
 * to get an input vertex subobject on the root graph of the document, one can call:
 *
 * FDocumentAccessPtr DocumentAccessPtr = MakeAccessPtr<FDocumentAccessPtr>(AccessPoint, Document);
 *
 * FClassAccessPtr ClassAccessPtr = DocumentAccessPtr.GetRootGraph().GetNodeWithNodeID(NodeID).GetInputWithVertexID(VertexID);
 *
 */

namespace Metasound
{
	namespace Frontend
	{
		using FVertexAccessPtr = TAccessPtr<FMetasoundFrontendVertex>;
		using FConstVertexAccessPtr = TAccessPtr<const FMetasoundFrontendVertex>;
		using FClassInputAccessPtr = TAccessPtr<FMetasoundFrontendClassInput>;
		using FConstClassInputAccessPtr = TAccessPtr<const FMetasoundFrontendClassInput>;
		using FClassOutputAccessPtr = TAccessPtr<FMetasoundFrontendClassOutput>;
		using FConstClassOutputAccessPtr = TAccessPtr<const FMetasoundFrontendClassOutput>;
		using FVariableAccessPtr = TAccessPtr<FMetasoundFrontendVariable>;
		using FConstVariableAccessPtr = TAccessPtr<const FMetasoundFrontendVariable>;

		class FNodeAccessPtr : public TAccessPtr<FMetasoundFrontendNode>
		{
		public:
			using Super = TAccessPtr<FMetasoundFrontendNode>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FVertexAccessPtr GetInputWithName(const FVertexName& InName);
			UE_API FVertexAccessPtr GetInputWithVertexID(const FGuid& InID);
			UE_API FVertexAccessPtr GetOutputWithName(const FVertexName& InName);
			UE_API FVertexAccessPtr GetOutputWithVertexID(const FGuid& InID);

			UE_API FConstVertexAccessPtr GetInputWithName(const FVertexName& InName) const;
			UE_API FConstVertexAccessPtr GetInputWithVertexID(const FGuid& InID) const;
			UE_API FConstVertexAccessPtr GetOutputWithName(const FVertexName& InName) const;
			UE_API FConstVertexAccessPtr GetOutputWithVertexID(const FGuid& InID) const;
		};

		class FConstNodeAccessPtr : public TAccessPtr<const FMetasoundFrontendNode>
		{
		public:
			using Super = TAccessPtr<const FMetasoundFrontendNode>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FConstVertexAccessPtr GetInputWithName(const FVertexName& InName) const;
			UE_API FConstVertexAccessPtr GetInputWithVertexID(const FGuid& InID) const;
			UE_API FConstVertexAccessPtr GetOutputWithName(const FVertexName& InName) const;
			UE_API FConstVertexAccessPtr GetOutputWithVertexID(const FGuid& InID) const;
		};

		using FGraphAccessPtr = TAccessPtr<FMetasoundFrontendGraph>;
		using FConstGraphAccessPtr = TAccessPtr<const FMetasoundFrontendGraph>;

		class FClassAccessPtr : public TAccessPtr<FMetasoundFrontendClass>
		{
		public:
			using Super = TAccessPtr<FMetasoundFrontendClass>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FClassInputAccessPtr GetInputWithName(const FVertexName& InName);
			UE_API FClassOutputAccessPtr GetOutputWithName(const FVertexName& InName);
			UE_API FConstClassInputAccessPtr GetInputWithName(const FVertexName& InName) const;
			UE_API FConstClassOutputAccessPtr GetOutputWithName(const FVertexName& InName) const;
		};

		class FConstClassAccessPtr : public TAccessPtr<const FMetasoundFrontendClass>
		{
		public:
			using Super = TAccessPtr<const FMetasoundFrontendClass>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FConstClassInputAccessPtr GetInputWithName(const FVertexName& InName) const;
			UE_API FConstClassOutputAccessPtr GetOutputWithName(const FVertexName& InName) const;
		};

		class FGraphClassAccessPtr : public TAccessPtr<FMetasoundFrontendGraphClass>
		{
		public:
			using Super = TAccessPtr<FMetasoundFrontendGraphClass>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FClassInputAccessPtr GetInputWithName(const FVertexName& InName);
			UE_API FClassInputAccessPtr GetInputWithNodeID(const FGuid& InNodeID);
			UE_API FClassOutputAccessPtr GetOutputWithName(const FVertexName& InName);
			UE_API FClassOutputAccessPtr GetOutputWithNodeID(const FGuid& InNodeID);

			UE_API FVariableAccessPtr GetVariableWithID(const FGuid& InID);
			UE_API FNodeAccessPtr GetNodeWithNodeID(const FGuid& InNodeID);
			UE_API FGraphAccessPtr GetGraph();

			UE_API FConstClassInputAccessPtr GetInputWithName(const FVertexName& InName) const;
			UE_API FConstClassInputAccessPtr GetInputWithNodeID(const FGuid& InNodeID) const;
			UE_API FConstClassOutputAccessPtr GetOutputWithName(const FVertexName& InName) const;
			UE_API FConstClassOutputAccessPtr GetOutputWithNodeID(const FGuid& InNodeID) const;

			UE_API FConstVariableAccessPtr GetVariableWithID(const FGuid& InID) const;
			UE_API FConstNodeAccessPtr GetNodeWithNodeID(const FGuid& InNodeID) const;
			UE_API FConstGraphAccessPtr GetGraph() const;
		};

		class FConstGraphClassAccessPtr : public TAccessPtr<const FMetasoundFrontendGraphClass>
		{
			using Super = TAccessPtr<const FMetasoundFrontendGraphClass>;

			// Inherit constructors from base class
			using Super::Super;

			FConstClassInputAccessPtr GetInputWithName(const FVertexName& InName) const;
			FConstClassInputAccessPtr GetInputWithNodeID(const FGuid& InNodeID) const;
			FConstClassOutputAccessPtr GetOutputWithName(const FVertexName& InName) const;
			FConstClassOutputAccessPtr GetOutputWithNodeID(const FGuid& InNodeID) const;
			FConstVariableAccessPtr GetVariableWithID(const FGuid& InID) const;
			FConstNodeAccessPtr GetNodeWithNodeID(const FGuid& InNodeID) const;
			FConstGraphAccessPtr GetGraph() const;
		};

		class FDocumentAccessPtr : public TAccessPtr<FMetasoundFrontendDocument>
		{
		public:
			using Super = TAccessPtr<FMetasoundFrontendDocument>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FGraphClassAccessPtr GetRootGraph();
			UE_API FGraphClassAccessPtr GetSubgraphWithID(const FGuid& InID);
			UE_API FClassAccessPtr GetDependencyWithID(const FGuid& InID);
			UE_API FClassAccessPtr GetClassWithID(const FGuid& InID);
			UE_API FClassAccessPtr GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata);

			UE_DEPRECATED(5.8, "FNodeClassInfo is actively being deprecated, use overload with FNodeClassRegistryKey instead")
			UE_API FClassAccessPtr GetClassWithInfo(const FNodeClassInfo& InInfo);

			UE_API FClassAccessPtr GetClassWithRegistryKey(const FNodeRegistryKey& InKey);

			UE_API FConstGraphClassAccessPtr GetRootGraph() const;
			UE_API FConstGraphClassAccessPtr GetSubgraphWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetDependencyWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetClassWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata) const;

			UE_DEPRECATED(5.8, "FNodeClassInfo is actively being deprecated, use overload with FNodeClassRegistryKey instead")
			UE_API FConstClassAccessPtr GetClassWithInfo(const FNodeClassInfo& InInfo) const;

			UE_API FConstClassAccessPtr GetClassWithRegistryKey(const FNodeRegistryKey& InKey) const;
		};

		class FConstDocumentAccessPtr : public TAccessPtr<const FMetasoundFrontendDocument>
		{
		public:
			using Super = TAccessPtr<const FMetasoundFrontendDocument>;

			// Inherit constructors from base class
			using Super::Super;

			UE_API FConstGraphClassAccessPtr GetRootGraph() const;
			UE_API FConstGraphClassAccessPtr GetSubgraphWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetDependencyWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetClassWithID(const FGuid& InID) const;
			UE_API FConstClassAccessPtr GetClassWithMetadata(const FMetasoundFrontendClassMetadata& InMetadata) const;

			UE_DEPRECATED(5.8, "FNodeClassInfo is actively being deprecated, use overload with FNodeClassRegistryKey instead")
			UE_API FConstClassAccessPtr GetClassWithInfo(const FNodeClassInfo& InInfo) const;

			UE_API FConstClassAccessPtr GetClassWithRegistryKey(const FNodeRegistryKey& InKey) const;
		};
	}
}

#undef UE_API
