// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "DocumentTemplates/MetasoundFrontendDocumentVertexTemplate.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendDocumentCacheInterface.h"
#include "MetasoundDataReference.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundFrontendGraphLayer.h"
#include "MetasoundFrontendRegistries.h"
#include "MetasoundFrontendTransform.h"
#include "MetasoundLiteral.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/ScriptInterface.h"

#include "MetasoundFrontendDocumentBuilder.generated.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
struct FMetaSoundFrontendDocumentTemplate;
class FMetasoundAssetBase;


namespace Metasound::Frontend
{
	// Forward Declarations
	class INodeTemplate;
	class INodeUpdateTransform;
	struct FMetaSoundClassInfo;
	using FNodePredicateFunctionRef = TFunctionRef<bool(const FMetasoundFrontendNode&)>;
	using FConstClassAndNodeFunctionRef = TFunctionRef<void(const FMetasoundFrontendClass&, const FMetasoundFrontendNode&)>;
	using FConstClassAndNodeAndPageIDFunctionRef = TFunctionRef<void(const FMetasoundFrontendClass&, const FMetasoundFrontendNode&, const FGuid&)>;
	using FFinalizeNodeFunctionRef = TFunctionRef<void(FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&)>;
	using FVertexNameAndType = TTuple<FName, FName>;

	namespace NodeReplacement
	{
		struct FNodeInstanceReplacementData;
	}
	enum class EInvalidEdgeReason : uint8
	{
		None = 0,
		MismatchedAccessType,
		MismatchedDataType,
		MissingInput,
		MissingOutput,
		COUNT
	};

	METASOUNDFRONTEND_API FString LexToString(const EInvalidEdgeReason& InReason);

	struct FNamedEdge
	{
		const FGuid OutputNodeID;
		const FName OutputName;
		const FGuid InputNodeID;
		const FName InputName;

		friend bool operator==(const FNamedEdge& InLHS, const FNamedEdge& InRHS)
		{
			return InLHS.OutputNodeID == InRHS.OutputNodeID
				&& InLHS.OutputName == InRHS.OutputName
				&& InLHS.InputNodeID == InRHS.InputNodeID
				&& InLHS.InputName == InRHS.InputName;
		}

		friend bool operator!=(const FNamedEdge& InLHS, const FNamedEdge& InRHS)
		{
			return !(InLHS == InRHS);
		}

		friend inline uint32 GetTypeHash(const FNamedEdge& InBinding)
		{
			const int32 NameHash = HashCombineFast(GetTypeHash(InBinding.OutputName), GetTypeHash(InBinding.InputName));
			const int32 GuidHash = HashCombineFast(GetTypeHash(InBinding.OutputNodeID), GetTypeHash(InBinding.InputNodeID));
			return HashCombineFast(NameHash, GuidHash);
		}
	};

	struct FModifyInterfaceOptions
	{
		UE_API FModifyInterfaceOptions(const TArray<FMetasoundFrontendInterface>& InInterfacesToRemove, const TArray<FMetasoundFrontendInterface>& InInterfacesToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder =nullptr);
		UE_API FModifyInterfaceOptions(TArray<FMetasoundFrontendInterface>&& InInterfacesToRemove, TArray<FMetasoundFrontendInterface>&& InInterfacesToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder = nullptr);
		UE_API FModifyInterfaceOptions(const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToRemove, const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToAdd, const FMetaSoundFrontendDocumentBuilder* InReferencedBuilder = nullptr);

		TArray<FMetasoundFrontendInterface> InterfacesToRemove;
		TArray<FMetasoundFrontendInterface> InterfacesToAdd;

		// Function used to determine if an old of a removed interface
		// and new member of an added interface are considered equal and
		// to be swapped, retaining preexisting connections (and locations
		// if in editor and 'SetDefaultNodeLocations' option is set)
		TFunction<bool(FName, FName)> NamePairingFunction;

#if WITH_EDITORONLY_DATA
		bool bSetDefaultNodeLocations = true;
#endif // WITH_EDITORONLY_DATA

		// Optional builder to copy node ids from instead of regenerating
		const FMetaSoundFrontendDocumentBuilder* ReferencedBuilder = nullptr;
	};
} // namespace Metasound::Frontend


// Builder Document UObject, which is only used for registration purposes when attempting
// async registration whereby the original document is serialized and must not be mutated.
UCLASS(MinimalAPI)
class UMetaSoundBuilderDocument : public UObject, public IMetaSoundDocumentInterface
{
	GENERATED_BODY()

public:
	UE_API UMetaSoundBuilderDocument(const FObjectInitializer& ObjectInitializer);

	// Create and return a valid builder document which copies the provided interface's document & class
	static UE_API UMetaSoundBuilderDocument& Create(const IMetaSoundDocumentInterface& InDocToCopy);

	UE_API virtual bool ConformObjectToDocument() override;

	// Returns the document
	UE_API virtual const FMetasoundFrontendDocument& GetConstDocument() const override;

	// Returns the default access flags utilized when document is initialized.
	UE_API virtual EMetasoundFrontendClassAccessFlags GetDefaultAccessFlags() const final override;

	// Returns temp path of builder document
	UE_API virtual FTopLevelAssetPath GetAssetPathChecked() const override;

	// Returns the base class registered with the MetaSound UObject registry.
	UE_API virtual const UClass& GetBaseMetaSoundUClass() const final override;

	// Returns the builder class used to modify the given document.
	UE_API virtual const UClass& GetBuilderUClass() const final override;

	// Returns if the document is being actively built (always true as builder documents are always being actively built)
	UE_API virtual bool IsActivelyBuilding() const final override;

private:
	UE_API virtual FMetasoundFrontendDocument& GetDocument() override;

#if WITH_EDITORONLY_DATA
	// Convenience template function for attempting to get the underlying pointer of the set configuration.
	// If the provided configuration type is not that which the current document is set to or no configuration
	// is set, returns null.
	template<typename TTemplateType>
	TTemplateType* GetTemplateAs()
	{
		return GetDocument().Template.GetMutablePtr<TTemplateType>();
	}

	template<typename TTemplateType>
	UE_DEPRECATED(5.8, "Use GetTemplateAs() instead")
	TTemplateType* GetDocumentConfigurationAs() { return GetTemplateAs<TTemplateType>(); }
#endif // WITH_EDITORONLY_DATA

	UE_API virtual void OnBeginActiveBuilder() override;
	UE_API virtual void OnFinishActiveBuilder() override;

	UPROPERTY(Transient)
	FMetasoundFrontendDocument Document;

	UPROPERTY(Transient)
	TObjectPtr<const UClass> MetaSoundUClass = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<const UClass> BuilderUClass = nullptr;
};

// Builder used to support dynamically generating MetaSound documents at runtime. Builder contains caches that speed up
// common search and modification operations on a given document, which may result in slower performance on construction,
// but faster manipulation of its managed document.  The builder's managed copy of a document is expected to not be modified
// by any external system to avoid cache becoming stale.
USTRUCT()
struct FMetaSoundFrontendDocumentBuilder
{
	GENERATED_BODY()

public:
	// Default ctor should typically never be used directly as builder interface (and optionally delegates) should be specified on construction (Default exists only to make UObject reflection happy).
	UE_API FMetaSoundFrontendDocumentBuilder(TScriptInterface<IMetaSoundDocumentInterface> InDocumentInterface = { }, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> InDocumentDelegates = { }, bool bPrimeCache = false);
	UE_API virtual ~FMetaSoundFrontendDocumentBuilder();

	// Call when the builder will no longer modify the IMetaSoundDocumentInterface
	UE_API void FinishBuilding();

	UE_API void AddAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags);
	UE_API void ClearAccessFlags();
	UE_API EMetasoundFrontendClassAccessFlags GetAccessFlags() const;
	UE_API void RemoveAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags);
	UE_API void SetAccessFlags(EMetasoundFrontendClassAccessFlags AccessFlags);

	// Adds new dependency to MetaSound. Typically not necessary to call directly as dependencies are added automatically via registry when nodes are added, and can be removed when no longer referenced (see 'RemoveUnusedDependencies`).
	UE_API const FMetasoundFrontendClass* AddDependency(FMetasoundFrontendClass NewDependency);

	UE_API void AddEdge(FMetasoundFrontendEdge&& InNewEdge, const FGuid* InPageID = nullptr);
	UE_API bool AddNamedEdges(const TSet<Metasound::Frontend::FNamedEdge>& ConnectionsToMake, TArray<const FMetasoundFrontendEdge*>* OutEdgesCreated = nullptr, bool bReplaceExistingConnections = true, const FGuid* InPageID = nullptr);
	UE_API bool AddEdgesByNodeClassInterfaceBindings(const FGuid& InFromNodeID, const FGuid& InToNodeID, bool bReplaceExistingConnections = true, const FGuid* InPageID = nullptr);
	UE_API bool AddEdgesFromMatchingInterfaceNodeOutputsToGraphOutputs(const FGuid& InNodeID, TArray<const FMetasoundFrontendEdge*>& OutEdgesCreated, bool bReplaceExistingConnections = true, const FGuid* InPageID = nullptr);
	UE_API bool AddEdgesFromMatchingInterfaceNodeInputsToGraphInputs(const FGuid& InNodeID, TArray<const FMetasoundFrontendEdge*>& OutEdgesCreated, bool bReplaceExistingConnections = true, const FGuid* InPageID = nullptr);

	// Adds Graph Input to document, which in turn adds a referencing input node to ALL pages.  If valid PageID is provided, returns associated page's node pointer.
	// If none provided, returns node pointer to node for the builder's currently set build page ID (see 'GetBuildPageID').
	UE_API const FMetasoundFrontendNode* AddGraphInput(FMetasoundFrontendClassInput ClassInput, const FGuid* InPageID = nullptr);

	UE_API const FMetasoundFrontendVariable* AddGraphVariable(FName VariableName, FName DataType, const FMetasoundFrontendLiteral* Literal = nullptr, const FText* DisplayName = nullptr, const FText* Description = nullptr, const FGuid* InPageID = nullptr);
	UE_API const FMetasoundFrontendNode* AddGraphVariableNode(FName VariableName, EMetasoundFrontendClassType ClassType, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);
	UE_API const FMetasoundFrontendNode* AddGraphVariableMutatorNode(FName VariableName, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);
	UE_API const FMetasoundFrontendNode* AddGraphVariableAccessorNode(FName VariableName, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);
	UE_API const FMetasoundFrontendNode* AddGraphVariableDeferredAccessorNode(FName VariableName, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);

	// Adds node to document to the page associated with the given PageID.  If no valid PageID is provided, adds and returns node pointer to node for the builder's
	// currently set build page ID (see 'GetBuildPageID').
	UE_API const FMetasoundFrontendNode* AddGraphNode(const FMetasoundFrontendGraphClass& InClass, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);

	// Adds Graph Output to document, which in turn adds a referencing output node to ALL pages.  If valid PageID is provided, returns associated page's node pointer.
	// If none provided, returns node pointer to node for the builder's currently set build page ID (see 'GetBuildPageID').
	UE_API const FMetasoundFrontendNode* AddGraphOutput(FMetasoundFrontendClassOutput ClassOutput, const FGuid* InPageID = nullptr);

	// Adds an interface and associated inputs and outputs
	UE_API bool AddInterface(FName InterfaceName);

	// Internal AddInterface call for use in rebuild preset root graph
	// If bAddUserModifiableInterfaceOnly is true, only user modifiable interfaces can be added (see FMetasoundFrontendInterfaceUClassOptions)
	// Reference builder is used to copy node ids for added members so they aren't just regenerated with NewGuid
	UE_INTERNAL bool AddInterface(FName InterfaceName, bool bAddUserModifiableInterfaceOnly, const FMetaSoundFrontendDocumentBuilder* ReferenceBuilder);

	UE_API const FMetasoundFrontendNode* AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion = 1, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);

	// Add node with specific major and minor version. The public version of AddNodeByClassName adds the highest minor version for a given major version
	UE_INTERNAL const FMetasoundFrontendNode* AddNodeByClassName(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, int32 InMinorVersion, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);
	
	UE_API const FMetasoundFrontendNode* AddNodeByTemplate(const Metasound::Frontend::INodeTemplate& InTemplate, FNodeTemplateGenerateInterfaceParams Params, FGuid InNodeID = FGuid::NewGuid(), const FGuid* InPageID = nullptr);

#if WITH_EDITORONLY_DATA
	// Adds a graph page to the given builder's document
	UE_API const FMetasoundFrontendGraph& AddGraphPage(const FGuid& InPageID, bool bDuplicateLastGraph, bool bSetAsBuildGraph = true);

	// Applying node instance replacement data for node update transforms/autoupdate 
	UE_INTERNAL void ApplyNodeInstanceReplacementData(Metasound::Frontend::NodeReplacement::FNodeInstanceReplacementData&& InInstanceData,  const FGuid& InNodeID, const FGuid* InPageID = nullptr, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedInputs=nullptr, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedOutputs=nullptr);

	// Capture node replacement data for the existing node (connections, defaults, configuration, etc.)
	UE_INTERNAL Metasound::Frontend::NodeReplacement::FNodeInstanceReplacementData CaptureNodeInstanceReplacementData(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Caches transient dependency metadata & style (class & vertex) found in the
	// registry that is not necessary for serialization or core graph generation.
	UE_API void CacheRegistryMetadata() const;
#endif // WITH_EDITOR

	// Returns whether or not the given edge can be added, which requires that its input
	// is not already connected and the edge is valid (see function 'IsValidEdge').
	UE_API bool CanAddEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID = nullptr) const;

#if WITH_EDITORONLY_DATA
	// Returns whether the node is eligible for auto-updating. Eligibility reasons include
	// minor version revision, node update transform found,
	// or the interface has changed, but no higher major revision is available.
	// Can return true if the interface has changed but with only cosmetic
	// differences (ex. DisplayName only used in editor) but no runtime
	// behavior has been modified.
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_INTERNAL UE_API bool CanAutoUpdate(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
#endif // WITH_EDITORONLY_DATA

	// Clears document completely of all graph page data (nodes, edges, & member metadata), dependencies,
	// interfaces, member metadata, preset state, etc. Leaves ClassMetadata intact. Reloads the builder state,
	// so external delegates must be relinked if desired. NOTE: Template struct is not cleared and must be managed
	// by separate API call (see SetTemplate').
	UE_API void ClearDocument(TSharedRef<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates);

#if WITH_EDITORONLY_DATA
	UE_API bool ClearMemberMetadata(const FGuid& InMemberID);
#endif // WITH_EDITORONLY_DATA

	UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
	UE_API bool ConfigureDocument();

	UE_API bool ConformObjectToDocument();

	UE_API bool ContainsDependencyOfType(EMetasoundFrontendClassType ClassType) const;
	UE_API bool ContainsEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID = nullptr) const;
	UE_API bool ContainsNode(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	UE_DEPRECATED(5.8, "Use SetTemplate instead")
	UE_API bool ConvertFromPreset();

	UE_DEPRECATED(5.8, "Use SetTemplate instead")
	UE_API bool ConvertToPreset(const FMetasoundFrontendDocument& InReferencedDocument, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates = { });

	UE_DEPRECATED(5.8, "Use SetTemplate instead")
	UE_API bool ConvertToPreset(const FMetaSoundFrontendDocumentBuilder& InReferencedDocumentBuilder, TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> ModifyDelegates = { });
	

	static UE_API TOptional<Metasound::FAnyDataReference> CreateDataReference(const Metasound::FOperatorSettings& InOperatorSettings, FName DataType, const Metasound::FLiteral& InLiteral, Metasound::EDataReferenceAccessType AccessType);

	
#if WITH_EDITORONLY_DATA
	// Returns whether the node's interface is functionally the same as the node's class interface.
	UE_INTERNAL bool NodeInterfaceMatchesClassInterface(const FGuid& InNodeID, const FGuid& InPageID) const;
	// Fills out the provided ClassInterfaceUpdate struct with the differences between
	// the registry's version of the class interface and that of the node.
	// Returns whether or not the interface was found in the registry.
	UE_API bool DiffAgainstRegistryInterface(const FGuid& InNodeID, const FGuid& InPageID, bool bInUseHighestMinorVersion, Metasound::Frontend::FClassInterfaceUpdates& OutInterfaceUpdates, bool bInForceRegenerateClassInterfaceOverride = true) const;
#endif // WITH_EDITORONLY_DATA

	UE_API const FMetasoundFrontendClassInput* DuplicateGraphInput(FName ExistingName, FName NewName);
	UE_API const FMetasoundFrontendClassOutput* DuplicateGraphOutput(FName ExistingName, FName NewName);
	UE_API const FMetasoundFrontendVariable* DuplicateGraphVariable(FName ExistingName, FName NewName, const FGuid* InPageID = nullptr);

	TConstStructView<FMetasoundFrontendClassInterface> FindClassInterfaceOverride(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

#if WITH_EDITORONLY_DATA
	UE_API const FMetasoundFrontendEdgeStyle* FindConstEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID = nullptr) const;
	UE_API FMetasoundFrontendEdgeStyle* FindEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID = nullptr);
	UE_API FMetasoundFrontendEdgeStyle& FindOrAddEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID = nullptr);
	UE_API const FMetaSoundFrontendGraphComment* FindGraphComment(const FGuid& InCommentID, const FGuid* InPageID = nullptr) const;
	UE_API FMetaSoundFrontendGraphComment* FindGraphComment(const FGuid& InCommentID, const FGuid* InPageID = nullptr);
	UE_API FMetaSoundFrontendGraphComment& FindOrAddGraphComment(const FGuid& InCommentID, const FGuid* InPageID = nullptr);
	UE_API UMetaSoundFrontendMemberMetadata* FindMemberMetadata(const FGuid& InMemberID) const;
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.8, "Direct access to IInterfaceRegistryEntry is deprecated as it is not thread safe. Must now get copy of interface directly from the InterfaceRegistry.")
	static UE_API bool FindDeclaredInterfaces(const FMetasoundFrontendDocument& InDocument, TArray<const Metasound::Frontend::IInterfaceRegistryEntry*>& OutInterfaces);

	UE_DEPRECATED(5.8, "Direct access to IInterfaceRegistryEntry is deprecated as it is not thread safe. Must now get copy of interface directly from the InterfaceRegistry.")
	UE_API bool FindDeclaredInterfaces(TArray<const Metasound::Frontend::IInterfaceRegistryEntry*>& OutInterfaces) const;

	UE_API const FMetasoundFrontendClass* FindDependency(const FGuid& InClassID) const;
	UE_API const FMetasoundFrontendClass* FindDependency(const FMetasoundFrontendClassMetadata& InMetadata) const;
	UE_API const FMetasoundFrontendClass* FindDependency(const Metasound::Frontend::FNodeRegistryKey& InRegistryKey) const;

	UE_API TArray<const FMetasoundFrontendEdge*> FindEdges(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;

	UE_API const FMetasoundFrontendClassInput* FindGraphInput(FName InputName) const;
	UE_API const FMetasoundFrontendNode* FindGraphInputNode(FName InputName, const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendClassOutput* FindGraphOutput(FName OutputName) const;
	UE_API const FMetasoundFrontendNode* FindGraphOutputNode(FName OutputName, const FGuid* InPageID = nullptr) const;

	UE_API const FMetasoundFrontendVariable* FindGraphVariable(const FGuid& InVariableID, const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendVariable* FindGraphVariable(FName InVariableName, const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendVariable* FindGraphVariableByNodeID(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	UE_API bool FindInterfaceInputNodes(FName InterfaceName, TArray<const FMetasoundFrontendNode*>& OutInputs, const FGuid* InPageID = nullptr) const;
	UE_API bool FindInterfaceOutputNodes(FName InterfaceName, TArray<const FMetasoundFrontendNode*>& OutOutputs, const FGuid* InPageID = nullptr) const;

	// Accessor for the currently set build graph.
	UE_API const FMetasoundFrontendGraph& FindConstBuildGraphChecked() const;

	UE_API const FMetasoundFrontendNode* FindNode(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	UE_API const TConstStructView<FMetaSoundFrontendNodeConfiguration> FindNodeConfiguration(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	UE_EXPERIMENTAL(5.6, "Non const builder access to node configuration is experimental.")
	UE_API TInstancedStruct<FMetaSoundFrontendNodeConfiguration> FindNodeConfiguration(const FGuid& InNodeID, const FGuid* InPageID = nullptr);

	// Return the node's index in the document's specified paged graph's node array 
	UE_API const int32* FindNodeIndex(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	// Returns all inputs on a given node that are user modifiable
	UE_API TArray<const FMetasoundFrontendVertex*> FindUserModifiableNodeInputs(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	// Returns all outputs on a given node that are user modifiable
	UE_API TArray<const FMetasoundFrontendVertex*> FindUserModifiableNodeOutputs(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	UE_API const FMetasoundFrontendVertex* FindNodeInput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendVertex* FindNodeInput(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID = nullptr) const;

	// Returns class defaults associated with the given node input (as defined in the associated node's dependency)
	UE_API const TArray<FMetasoundFrontendClassInputDefault>* FindNodeClassInputDefaults(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID = nullptr) const;

	// Returns node input's vertex default if valid and assigned.
	UE_API const FMetasoundFrontendVertexLiteral* FindNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;

	// Returns node input's vertex default if valid and assigned.
	UE_API const FMetasoundFrontendVertexLiteral* FindNodeInputDefault(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID = nullptr) const;

	UE_API TArray<const FMetasoundFrontendVertex*> FindNodeInputs(const FGuid& InNodeID, FName TypeName = FName(), const FGuid* InPageID = nullptr) const;
	UE_API TArray<const FMetasoundFrontendVertex*> FindNodeInputsConnectedToNodeOutput(const FGuid& InOutputNodeID, const FGuid& InOutputVertexID, TArray<const FMetasoundFrontendNode*>* ConnectedInputNodes = nullptr, const FGuid* InPageID = nullptr) const;

	UE_API const FMetasoundFrontendVertex* FindNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendVertex* FindNodeOutput(const FGuid& InNodeID, FName InVertexName, const FGuid* InPageID = nullptr) const;
	UE_API TArray<const FMetasoundFrontendVertex*> FindNodeOutputs(const FGuid& InNodeID, FName TypeName = FName(), const FGuid* InPageID = nullptr) const;
	UE_API const FMetasoundFrontendVertex* FindNodeOutputConnectedToNodeInput(const FGuid& InInputNodeID, const FGuid& InInputVertexID, const FMetasoundFrontendNode** ConnectedOutputNode = nullptr, const FGuid* InPageID = nullptr) const;

	// Return the index of the given page in the document's PagedGraphs array. Returns
	// INDEX_NONE in document where the given graph with the provided PageID is not found.
	UE_API int32 FindGraphPageIndex(const FGuid& InPageID) const;

	UE_DEPRECATED(5.8, "Use 'FindGraphPageIndex' instead")
	int32 FindPageIndex(const FGuid& InPageID) const { return FindGraphPageIndex(InPageID); }

	UE_API const FMetasoundFrontendDocument& GetConstDocumentChecked() const;

	// Convenience function for attempting to get the underlying mutable
	// document template vertex metadata. If the provided template vertex type
	// is not that which the current document is set to or no document template is
	// set, returns null.
	template<typename TVertexMetadataType>
	UE_INTERNAL TVertexMetadataType* FindVertexMetadata(const FGuid& InNodeID) const
	{
		using namespace Metasound::Frontend;

		if (FDocumentVertexTemplate* Template = GetDocumentChecked().Template.GetMutablePtr<FDocumentVertexTemplate>())
		{
			return Template->FindVertexMetadata<TVertexMetadataType>(InNodeID);
		}

		return nullptr;
	}

	// Convenience function for attempting to get the underlying mutable
	// document template vertex metadata. If the provided template vertex type
	// is not that which the current document is set to or no document template is
	// set, returns null.
	template<typename TVertexMetadataType>
	UE_INTERNAL const TVertexMetadataType* FindConstVertexMetadata(const FGuid& InNodeID) const
	{
		using namespace Metasound::Frontend;

		if (const FDocumentVertexTemplate* Template = GetConstDocumentTemplate().GetPtr<FDocumentVertexTemplate>())
		{
			return Template->FindConstVertexMetadata<TVertexMetadataType>(InNodeID);
		}

		return nullptr;
	}

#if WITH_EDITOR
	// Returns mutable struct view of document template if set. Internal use only to avoid
	// writing against struct members in invalid or dangerous, non-editor contexts (use
	// 'SetTemplate' instead for most script or edit-time applications).
	UE_API TStructView<FMetaSoundFrontendDocumentTemplate> GetDocumentTemplate() const;

	// Soft deprecated. To be deprecated in subsequent change.
	UE_API TStructView<FMetaSoundFrontendDocumentTemplate> GetDocumentConfiguration() const;
#endif // WITH_EDITOR

	// Returns struct view of document template if set.
	UE_API TConstStructView<FMetaSoundFrontendDocumentTemplate> GetConstDocumentTemplate() const;

	UE_DEPRECATED(5.8, "Use GetConstDocumentTemplate() instead")
	TConstStructView<FMetaSoundFrontendDocumentTemplate> GetConstDocumentConfiguration() const { return GetConstDocumentTemplate(); }

	// Convenience template function for attempting to get the underlying const pointer of the set configuration.
	// If the provided configuration type is not that which the current document is set to or no configuration
	// is set, returns null.
	template<typename TTemplateType>
	const TTemplateType* GetConstTemplateAs() const
	{
		return GetConstDocumentTemplate().GetPtr<TTemplateType>();
	}

	template<typename TTemplateType>
	UE_DEPRECATED(5.8, "Use GetConstTemplateAs() instead")
	const TTemplateType* GetConstDocumentConfigurationAs() const { return GetConstTemplateAs<TTemplateType>(); }

	UE_API const IMetaSoundDocumentInterface& GetConstDocumentInterfaceChecked() const;
	UE_API const FString GetDebugName() const;

	// The graph ID used when requests are made to mutate specific paged graph topology (ex. adding or removing nodes or edges)
	UE_API const FGuid& GetBuildPageID() const;

#if WITH_EDITOR
	// Gets the editor-only style of a node with the given ID.
	UE_API const FMetasoundFrontendNodeStyle* GetNodeStyle(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;

	// Title used when displaying the given node. Returned text falls back in the following order:
	// 1. If input, output, or variable (member), returns member type name.
	// 2. If external or template node, uses:
		// a. Node's class display name if set
		// b. Asset name (if class is defined as an asset)
		// c. Class name (if class is defined in code)
	UE_API FText GetNodeTitle(const FGuid& InNodeID, const FGuid* InPageID = nullptr) const;
#endif // WITH_EDITOR

	template<typename TObjectType>
	TObjectType& CastDocumentObjectChecked() const
	{
		UObject* Owner = DocumentInterface.GetObject();
		return *CastChecked<TObjectType>(Owner);
	}

	// Generates and returns new class name for the given builder's document. Should be used with extreme caution
	// (i.e. on new assets, when migrating assets, or upon generation of transient MetaSounds), as using a persistent
	// builder registered with the DocumentBuilderRegistry may result in stale asset records keyed off of an undefined class
	// name.  In addition, this can potentially leave existing node references in an abandoned state to this class causing
	// asset validation errors.
	UE_API FMetasoundFrontendClassName GenerateNewClassName();

	UE_API FName GenerateUniqueInputName(const FString& InBaseName) const;
	UE_API FName GenerateUniqueOutputName(const FString& InBaseName) const;
	UE_API FName GenerateUniqueVariableName(const FString& InBaseName) const;

	UE_API Metasound::Frontend::FDocumentModifyDelegates& GetDocumentDelegates();

	// Path for document object provided at construction time. Cached on builder as a useful means of debugging
	// and enables weak reference removal from the builder registry should the object be mid-destruction.
	UE_API const FTopLevelAssetPath& GetHintPath() const;

	UE_API FMetasoundAssetBase& GetMetasoundAsset() const;

	UE_DEPRECATED(5.8, "Use Template API to access configuration as type FMetaSoundFrontendPresetTemplate and use the 'Parent' property directly.")
	UE_API FMetasoundAssetBase* GetReferencedPresetAsset() const;

	UE_API int32 GetTransactionCount() const;

	UE_API TArray<const FMetasoundFrontendNode*> GetGraphInputTemplateNodes(FName InInputName, const FGuid* InPageID = nullptr);

	UE_DEPRECATED(5.8, "Input inheritance data is now stored on MetaSound Preset template metadata.")
	UE_API const TSet<FName>* GetGraphInputsInheritingDefault() const;

	UE_API EMetasoundFrontendVertexAccessType GetNodeInputAccessType(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;

#if WITH_EDITORONLY_DATA
	// Returns friendly name of the given node input vertex. Unlike a straight look-up of the vertex metadata value, this
	// tries and determines the best display name based on the parent node's class, if no display name is given
	// converts the base FName to FText, etc.
	UE_API FText GetNodeInputDisplayName(const FGuid& InNodeID, const FName VertexName, const FGuid* InPageID = nullptr) const;

	// Returns friendly name of the given node output vertex. Unlike a straight look-up of the vertex metadata value, this
	// tries and determines the best display name based on the parent node's class, if no display name is given
	// converts the base FName to FText, etc.
	UE_API FText GetNodeOutputDisplayName(const FGuid& InNodeID, const FName VertexName, const FGuid* InPageID = nullptr) const;
#endif // WITH_EDITORONLY_DATA

	UE_API EMetasoundFrontendVertexAccessType GetNodeOutputAccessType(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;

#if WITH_EDITORONLY_DATA
	UE_API const bool GetIsAdvancedDisplay(const FName MemberName, const EMetasoundFrontendClassType Type) const;
#endif // WITH_EDITORONLY_DATA

	// Returns the default value set for the input with the given name on the given page.
	UE_API const FMetasoundFrontendLiteral* GetGraphInputDefault(FName InputName, const FGuid* InPageID = nullptr) const;

	// Returns the default value set for the variable with the given name on the given page.
	UE_API const FMetasoundFrontendLiteral* GetGraphVariableDefault(FName InputName, const FGuid* InPageID = nullptr) const;

#if WITH_EDITORONLY_DATA
	// If PageID set, returns whether or not the given page is editable.  If no page ID is set, returns whether graph with set build ID is editable.
	UE_API bool GetIsGraphEditable(const FGuid* InPageID = nullptr) const;
#endif // WITH_EDITORONLY_DATA

	// Initializes the builder's document, using the (optional) provided document template, (optional) class name, and (optionally) whether or not to reset the existing class version.
	UE_API void InitDocument(const FMetasoundFrontendDocument* InDocumentTemplate = nullptr, const FMetasoundFrontendClassName* InNewClassName = nullptr, bool bResetVersion = true);

	// Initializes GraphClass Metadata, optionally resetting the version back to 1.0 and/or creating a unique class name if a name is not provided.
	static UE_API void InitGraphClassMetadata(FMetasoundFrontendClassMetadata& InOutMetadata, bool bResetVersion = false, const FMetasoundFrontendClassName* NewClassName = nullptr);
	UE_API void InitGraphClassMetadata(bool bResetVersion, const FMetasoundFrontendClassName* NewClassName);

	UE_API void InitNodeLocations();

	// Whether or not the builder is actively configuring or re-configuring its document.
	UE_EXPERIMENTAL(5.8, "Configurator is in active development and subject to change")
	UE_API bool IsConfiguringDocument() const;

	UE_API bool IsDependencyReferenced(const FGuid& InClassID) const;

	// Determines if the builder is configured the given template type
	template<typename TTemplateType>
	const bool IsDocumentConfiguredBy() const
	{
		if (const UScriptStruct* Struct = GetConstDocumentChecked().Template.GetScriptStruct())
		{
			return Struct->IsChildOf(TTemplateType::StaticStruct());
		}

		return false;
	}

	UE_API bool IsNodeInputConnected(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;
	UE_API bool IsNodeInputConnectionUserModifiable(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;
	UE_API bool IsNodeOutputConnected(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;
	UE_API bool IsNodeOutputConnectionUserModifiable(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr) const;

	UE_API bool IsInterfaceDeclared(FName InInterfaceName) const;
	UE_API bool IsInterfaceDeclared(const FMetasoundFrontendVersion& InInterfaceVersion) const;
	UE_API bool IsPreset() const;

	// Returns whether or not builder is attached to a DocumentInterface and is valid to build or act on a document.
	UE_API bool IsValid() const;

	// Returns whether or not the given edge is valid (i.e. represents an input and output that equate in data and access types) or malformed.
	// Note that this does not return whether or not the given edge exists, but rather if it could be legally applied to the given edge vertices.
	UE_API Metasound::Frontend::EInvalidEdgeReason IsValidEdge(const FMetasoundFrontendEdge& InEdge, const FGuid* InPageID = nullptr) const;

	// Iterates all nodes for the given page (uses build page ID if not provided).
	UE_API void IterateNodes(Metasound::Frontend::FConstClassAndNodeFunctionRef Func, const FGuid* InPageID = nullptr) const;

	// Iterates nodes, filtered by the given predicate. If bIterateAllPages is true, nodes on all pages will be considered and Page ID is ignored.
	UE_API void IterateNodesByPredicate(Metasound::Frontend::FConstClassAndNodeAndPageIDFunctionRef Func, Metasound::Frontend::FNodePredicateFunctionRef PredicateFunc, const FGuid* InPageID = nullptr, bool bIterateAllPages = false) const;

	// Iterates nodes that are filtered by only subscribing to a class with the given type (asserts if provided invalid class type).
	UE_API void IterateNodesByClassType(Metasound::Frontend::FConstClassAndNodeFunctionRef Func, EMetasoundFrontendClassType ClassType, const FGuid* InPageID = nullptr) const;

	UE_API bool ModifyInterfaces(Metasound::Frontend::FModifyInterfaceOptions&& InOptions);

	// Removes all dependencies with the given ClassID. Removes any nodes (and corresponding edges) remaining in any MetaSound paged graphs.
	UE_API bool RemoveDependency(const FGuid& InClassID);

	// Removes all dependencies with the given Class Type, Name, & Version Number. Removes any nodes (and corresponding edges) remaining in any MetaSound paged graphs.
	UE_API bool RemoveDependency(EMetasoundFrontendClassType ClassType, const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InClassVersionNumber);

	UE_API bool RemoveEdge(const FMetasoundFrontendEdge& EdgeToRemove, const FGuid* InPageID = nullptr);

	// Removes all edges connected to an input or output vertex associated with the node of the given ID.
	UE_API bool RemoveEdges(const FGuid& InNodeID, const FGuid* InPageID = nullptr);

	UE_API bool RemoveEdgesByNodeClassInterfaceBindings(const FGuid& InOutputNodeID, const FGuid& InInputNodeID, const FGuid* InPageID = nullptr);
	UE_API bool RemoveEdgesFromNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr);
	UE_API bool RemoveEdgeToNodeInput(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr);

#if WITH_EDITORONLY_DATA
	UE_API bool RemoveEdgeStyle(const FGuid& InNodeID, FName OutputName, const FGuid* InPageID = nullptr);
	UE_API bool RemoveGraphComment(const FGuid& InCommentID, const FGuid* InPageID = nullptr);
#endif // WITH_EDITORONLY_DATA

	UE_API bool RemoveGraphInput(FName InputName, bool bRemoveTemplateInputNodes = true);
	UE_API bool RemoveGraphOutput(FName OutputName);

#if WITH_EDITORONLY_DATA
	UE_API bool RemoveGraphPage(const FGuid& InPageID);
#endif // WITH_EDITORONLY_DATA

	UE_API bool RemoveGraphVariable(FName VariableName, const FGuid* InPageID = nullptr);

	UE_API bool RemoveInterface(FName Name);
	UE_API bool RemoveNamedEdges(const TSet<Metasound::Frontend::FNamedEdge>& InNamedEdgesToRemove, TArray<FMetasoundFrontendEdge>* OutRemovedEdges = nullptr, const FGuid* InPageID = nullptr);
	
	// Removes node and connected edges from document. Does not remove dependency. Use RemoveUnusedDependencies
	UE_API bool RemoveNode(const FGuid& InNodeID, const FGuid* InPageID = nullptr);

#if WITH_EDITORONLY_DATA
	UE_API int32 RemoveNodeLocation(const FGuid& InNodeID, const FGuid* InLocationGuid = nullptr, const FGuid* InPageID = nullptr);
#endif // WITH_EDITOR

	UE_API void Reload(TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> Delegates = {}, bool bPrimeCache = false);

	UE_DEPRECATED(5.8, "Input inheritance data is now cached in MetaSound Preset template vertex metadata.  No need to call directly here. Use overload without 'bClearInheritsDefault' parameter.")
	UE_API bool RemoveGraphInputDefault(FName InputName, const FGuid& InPageID, bool bClearInheritsDefault);
	UE_API bool RemoveGraphInputDefault(FName InputName, const FGuid& InPageID);
	// Internal function to remove graph input default for cook. Same as RemoveGraphInputDefault but removing the default page input default is allowed.
	UE_INTERNAL UE_API bool StripGraphInputDefault(FName InputName, const FGuid& InPageID);

	UE_API bool RemoveNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FGuid* InPageID = nullptr);
	UE_API bool RemoveUnusedDependencies();

	// Replaces existing dependency with a new one. Removes all nodes using this class, and readds nodes with new class, 
	// trying to maintain existing connections (based on name/data type) and data. 
	// Optionally report disconnected inputs and outputs
	UE_API bool ReplaceDependency(const Metasound::Frontend::FNodeClassRegistryKey& OldClass, const Metasound::Frontend::FNodeClassRegistryKey& NewClass, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedInputs = nullptr, TArray<Metasound::Frontend::FVertexNameAndType>* OutDisconnectedOutputs = nullptr);

	UE_API bool ResetGraphInputDefault(FName InputName);
#if WITH_EDITORONLY_DATA
	// Removes all graph pages except the default.  If bClearDefaultPage is true, clears the default graph page implementation.
	UE_API void ResetGraphPages(bool bClearDefaultGraph);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API void SetAuthor(const FString& InAuthor);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	// Sets the builder's targeted paged graph ID to the given ID if it exists.
	// Returns true if the builder is already targeting the given ID or if it successfully
	// found a page implementation with the given ID and was able to switch to it, false if not.
	// Swapping the targeted build graph ID clears the local cache, so swapping frequently can
	// induce cache thrashing. BroadcastDelegate should always be true unless dealing with the controller
	// API (exposed as a mechanism for mutating via controllers while deprecating.  Option will be removed
	// in a future build).
	UE_API bool SetBuildPageID(const FGuid& InBuildPageID, bool bBroadcastDelegate = true);
#endif // WITH_EDITORONLY_DATA

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	// Sets the entire internal configuration with that provided.
	UE_API void SetTemplate(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> InConfig);

	UE_DEPRECATED(5.8, "Use SetTemplate() instead")
	void SetDocumentConfiguration(TInstancedStruct<FMetaSoundFrontendDocumentTemplate> InConfig) { SetTemplate(MoveTemp(InConfig)); }

	// Update the given document template inline using the internal struct view passed to a given function.
	UE_API void SetTemplateProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertyFunction);

	UE_DEPRECATED(5.8, "Use SetTemplateProperties() instead")
	void SetDocumentConfigurationProperties(TFunctionRef<void(TStructView<FMetaSoundFrontendDocumentTemplate>)> SetPropertyFunction) { SetTemplateProperties(SetPropertyFunction); }

	// If struct type of function coincides with that of the given builder's MetaSound configuration, sets
	// the given document template properties inline using the internal struct view passed to a given
	// function. Returns true if set function successfully executed, false if not.
	template <typename TConfig>
	bool SetTemplateProperties(TFunctionRef<void(TConfig&)> SetPropertyFunction)
	{
		bool bSucceeded = false;

		const UScriptStruct* ScriptStruct = GetConstDocumentTemplate().GetScriptStruct();
		if (ScriptStruct && ScriptStruct->IsChildOf(TConfig::StaticStruct()))
		{
			SetTemplateProperties([&SetPropertyFunction, &bSucceeded](TStructView<FMetaSoundFrontendDocumentTemplate> Config)
			{
				if (TConfig* ConfigPtr = Config.GetPtr<TConfig>())
				{
					SetPropertyFunction(*ConfigPtr);
					bSucceeded = true;
				}
			});
		}

		return bSucceeded;
	}

	template <typename TConfig>
	UE_DEPRECATED(5.8, "Use SetTemplateProperties() instead")
	bool SetDocumentConfigurationProperties(TFunctionRef<void(TConfig&)> SetPropertyFunction) { return SetTemplateProperties<TConfig>(SetPropertyFunction); }
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

#if WITH_EDITORONLY_DATA
	// If page ID is provided, updates the graph's editable flag to the given value. If no page ID is provided,
	// sets *all* paged graph fields to the given value.
	UE_API void SetIsGraphEditable(bool bIsEditable, const FGuid* PageID = nullptr) const;

	// Sets the given input`s IsAdvancedDisplay state. AdvancedDisplay pins are hidden in the node by default.
	// returns true if state was changed.
	UE_API bool SetGraphInputAdvancedDisplay(const FName InputName, const bool InAdvancedDisplay);
#endif // WITH_EDITORONLY_DATA

	// Sets the given graph input's access type. If connected to other nodes and access type is not compatible,
	// associated edges/connections are removed.  Returns true if either DataType was successfully set to new
	// value or if AccessType is already the given AccessType.
	UE_API bool SetGraphInputAccessType(FName InputName, EMetasoundFrontendVertexAccessType AccessType);

	// Sets the given graph input's data type. If connected to other nodes, associated edges/connections
	// are removed.  Returns true if either DataType was successfully set to new value or if DataType is
	// already the given DataType.
	UE_API bool SetGraphInputDataType(FName InputName, FName DataType);

	// Set the given graph input's default(s).
	// Change type can be optionally passed in so the on input changed delegate can report the context, since a set can be used to perform remove/reset operations.
	UE_API bool SetGraphInputDefault(FName InputName, FMetasoundFrontendLiteral InDefaultLiteral, const FGuid* InPageID = nullptr, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set);
	UE_API bool SetGraphInputDefaults(FName InputName, TArray<FMetasoundFrontendClassInputDefault> Defaults, EMetaSoundLiteralChangeType InChangeType = EMetaSoundLiteralChangeType::Set);

#if WITH_EDITORONLY_DATA
	// Sets the graph input's description. Returns true if found and set, false if not.
	UE_API bool SetGraphInputDescription(FName InputName, FText Description);

	// Sets the graph input's display name. Returns true if found and set, false if not.
	UE_API bool SetGraphInputDisplayName(FName InputName, FText DisplayName);
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.8, "Input inheritance information moved to preset document template")
	UE_API bool SetGraphInputInheritsDefault(FName InName, bool bInputInheritsDefault, bool bForceUpdate = false);

	UE_DEPRECATED(5.8, "Input inheritance information moved to preset document template")
	UE_API bool SetGraphInputsInheritingDefault(TSet<FName>&& InNames);

	// Sets a given graph input's name to a new name. Succeeds if the graph output exists and the new name is set (or is the same as the old name).
	UE_API bool SetGraphInputName(FName InputName, FName InName);

#if WITH_EDITORONLY_DATA
	// Sets the graph input's sort order index. Returns true if found and set, false if not.
	UE_API bool SetGraphInputSortOrderIndex(const FName InputName, const int32 InSortOrderIndex);

	// Sets the graph output's sort order index. Returns true if found and set, false if not.
	UE_API bool SetGraphOutputSortOrderIndex(const FName OutputName, const int32 InSortOrderIndex);

	// Sets the given output`s IsAdvancedDisplay state. AdvancedDisplay pins are hidden in the node by default.
	// returns true if state was changed.
	UE_API bool SetGraphOutputAdvancedDisplay(const FName OutputName, const bool InAdvancedDisplay);
#endif // WITH_EDITORONLY_DATA

	// Sets the given graph output's access type. If connected to other nodes and access type is not compatible,
	// associated edges/connections are removed.  Returns true if either DataType was successfully set to new
	// value or if AccessType is already the given AccessType.
	UE_API bool SetGraphOutputAccessType(FName OutputName, EMetasoundFrontendVertexAccessType AccessType);

	// Sets the given graph output's data type. If connected to other nodes, associated edges/connections
	// are removed.  Returns true if either DataType was successfully set to new value or if DataType is
	// already the given DataType.
	UE_API bool SetGraphOutputDataType(FName OutputName, FName DataType);

#if WITH_EDITORONLY_DATA
	// Sets the graph output's description. Returns true if found and set, false if not.
	UE_API bool SetGraphOutputDescription(FName OutputName, FText Description);

	// Sets the graph input's display name. Returns true if found and set, false if not.
	UE_API bool SetGraphOutputDisplayName(FName OutputName, FText DisplayName);
#endif // WITH_EDITORONLY_DATA

	// Sets a given graph output's name to a new name. Succeeds if the graph output exists and the new name is set (or is the same as the old name).
	UE_API bool SetGraphOutputName(FName InputName, FName InName);

	// Sets the given graph variable's default.
	UE_API bool SetGraphVariableDefault(FName VariableName, FMetasoundFrontendLiteral InDefaultLiteral, const FGuid* InPageID = nullptr);

#if WITH_EDITORONLY_DATA
	// Sets the given graph variable's description.
	UE_API bool SetGraphVariableDescription(FName VariableName, FText Description, const FGuid* InPageID = nullptr);

	// Sets the given graph variable's display name.
	UE_API bool SetGraphVariableDisplayName(FName VariableName, FText DisplayName, const FGuid* InPageID = nullptr);
#endif // WITH_EDITORONLY_DATA

	// Sets the given graph variable's description.
	UE_API bool SetGraphVariableName(FName VariableName, FName NewName, const FGuid* InPageID = nullptr);

#if WITH_EDITOR
	UE_API void SetDisplayName(const FText& InDisplayName);
	UE_API void SetDescription(const FText& InDescription);
	UE_API void SetKeywords(const TArray<FText>& InKeywords);
	UE_API void SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy);

	UE_DEPRECATED(5.7, "Use AccessFlags API calls with deprecation flag instead")
	UE_API void SetIsDeprecated(const bool bInIsDeprecated);
	UE_API void SetInputStyle(FMetasoundFrontendInterfaceStyle&& Style);
	UE_API void SetOutputStyle(FMetasoundFrontendInterfaceStyle&& Style);

	UE_API void SetMemberMetadata(UMetaSoundFrontendMemberMetadata& NewMetadata);

	// Sets the editor-only comment to the provided value.
	// Returns true if the node was found and the comment was updated, false if not.
	UE_API bool SetNodeComment(const FGuid& InNodeID, FString&& InNewComment, const FGuid* InPageID = nullptr);

	// Sets the editor-only comment visibility.
	// Returns true if the node was found and the visibility was set, false if not.
	UE_API bool SetNodeCommentVisible(const FGuid& InNodeID, bool bIsVisible, const FGuid* InPageID = nullptr);

	// Sets the editor-only node location of a node with the given ID to the provided location.
	// Returns true if the node was found and the location was updated, false if not.
	UE_API bool SetNodeLocation(const FGuid& InNodeID, const FVector2D& InLocation, const FGuid* InLocationGuid = nullptr, const FGuid* InPageID = nullptr);

	// Sets the editor-only Unconnected Pins Hidden for a node with the given ID.
	UE_API bool SetNodeUnconnectedPinsHidden(const FGuid& InNodeID, const bool bUnconnectedPinsHidden, const FGuid* InPageID = nullptr);
#endif // WITH_EDITOR
	
	// Sets the node configuration for the given node and updates the interface
	// returns true if the node configuration is set
	UE_API bool SetNodeConfiguration(const FGuid& InNodeID, TInstancedStruct<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration, const FGuid* InPageID = nullptr);

	UE_API bool SetNodeInputDefault(const FGuid& InNodeID, const FGuid& InVertexID, const FMetasoundFrontendLiteral& InLiteral, const FGuid* InPageID = nullptr);
	
	// Sets the document's version number.  Should only be called by document versioning.
	UE_API void SetVersionNumber(const FMetasoundFrontendVersionNumber& InDocumentVersionNumber);

	UE_API bool SwapGraphInput(const FMetasoundFrontendClassVertex& InExistingInputVertex, const FMetasoundFrontendClassVertex& NewInputVertex);
	UE_API bool SwapGraphOutput(const FMetasoundFrontendClassVertex& InExistingOutputVertex, const FMetasoundFrontendClassVertex& NewOutputVertex);

#if WITH_EDITOR
	// Synchronizes all dependency Metadata in document with that found in the registry. If not found
	// in the registry, no action taken. Returns false if any dependencies could not be checked for updates.
	// If provided array, pointers to classes that were modified will be added to it.
	UE_API bool SynchronizeDependencyMetadata(TArray<const FMetasoundFrontendClass*>* InOutModifiedClasses = nullptr);

	UE_API bool UpdateDependencyRegistryData(const TMap<Metasound::Frontend::FNodeRegistryKey, Metasound::Frontend::FNodeRegistryKey>& OldToNewClassKeys);
#endif // WITH_EDITOR

	// If a node contains a node configuration, update the node class interface and interface.
	// Returns true if node is found.
	UE_API bool UpdateNodeInterfaceFromConfiguration(const FGuid& InNodeID, const FGuid* InPageID = nullptr);

#if WITH_EDITORONLY_DATA
	// Transforms template nodes within the given builder's document, which can include swapping associated edges and/or
	// replacing nodes with other, registry-defined concrete node class instances. Returns true if any template nodes were processed.
	UE_API bool TransformTemplateNodes();

	// Struct enabling property migration of data that must be applied prior to versioning logic
	struct IPropertyVersionTransform
	{
	public:
		virtual ~IPropertyVersionTransform() = default;

	protected:
		virtual bool Transform(FMetaSoundFrontendDocumentBuilder& Builder) const = 0;

		// Allows for unsafe access to a document for property migration.
		static FMetasoundFrontendDocument& GetDocumentUnsafe(const FMetaSoundFrontendDocumentBuilder& Builder);
	};
#endif // WITH_EDITORONLY_DATA

	UE_API const FTopLevelAssetPath GetBuilderClassPath() const;

private:
	bool bInTemplateScope = false;

	using FFinalizeNodeFunctionRef = TFunctionRef<void(FMetasoundFrontendNode&, const Metasound::Frontend::FNodeRegistryKey&)>;

	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration(const FMetasoundFrontendClassMetadata& InClassMetadata) const;
	TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration(EMetasoundFrontendClassType InClassType, const Metasound::Frontend::FNodeRegistryKey& InClassKey) const;
	
	bool AddInterfaceInternal(FName InterfaceName, bool bAddUserModifiableInterfaceOnly = true, const FMetaSoundFrontendDocumentBuilder* ReferenceBuilder = nullptr);

	FMetasoundFrontendNode* AddNodeInternal(const FMetasoundFrontendClassMetadata& InClassMetadata, Metasound::Frontend::FFinalizeNodeFunctionRef FinalizeNode, const FGuid& InPageID, FGuid InNodeID = FGuid::NewGuid(), int32* NewNodeIndex = nullptr);
	FMetasoundFrontendNode* AddNodeInternal(const Metasound::Frontend::FNodeRegistryKey& InClassKey, FFinalizeNodeFunctionRef FinalizeNode, const FGuid& InPageID, FGuid InNodeID = FGuid::NewGuid(), int32* NewNodeIndex = nullptr);

	const FMetasoundFrontendNode* AddNodeByClassInternal(FMetasoundFrontendClass&& InClass, FGuid InNodeID, const FGuid* InPageID);

	void BeginBuilding(TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> Delegates = {}, bool bPrimeCache = false);

	// Conforms GraphOutput node's ClassID, Access & Data Type with the GraphOutput.
	// Creates and removes dependencies as necessary within the document dependency array. Does *NOT*
	// modify edge data (i.e. if the DataType is changed on the given node and it has corresponding
	// edges, edges may then be invalid due to access type/DataType incompatibility).
	bool ConformGraphInputNodeToClass(const FMetasoundFrontendClassInput& GraphInput);

	// Conforms GraphOutput node's ClassID, Access & Data Type with the GraphOutput.
	// Creates and removes dependencies as necessary within the document dependency array. Does *NOT*
	// modify edge data (i.e. if the DataType is changed on the given node and it has corresponding
	// edges, edges may then be invalid due to access type/DataType incompatibility).
	bool ConformGraphOutputNodeToClass(const FMetasoundFrontendClassOutput& GraphOutput);

	FMetasoundFrontendGraph& FindBuildGraphChecked() const;

	FMetasoundFrontendVariable* FindGraphVariableInternal(FName InVariableName, const FGuid* InPageID = nullptr);

	bool FindNodeClassInterfaces(const FGuid& InNodeID, TSet<FMetasoundFrontendVersion>& OutInterfaces, const FGuid& InPageID) const;

	// Returns pointer to editable node.  Modifying data on the node directly can be inherently dangerous if associated node cache
	// data is not updated to reflect any changes to the given node (ex. changing NodeID, interface, etc.)
	FMetasoundFrontendNode* FindNodeInternal(const FGuid& InNodeID, const FGuid* InPageID = nullptr);

	const FMetasoundFrontendNode* FindHeadNodeInVariableStack(FName VariableName, const FGuid* InPageID = nullptr) const;
	const FMetasoundFrontendNode* FindTailNodeInVariableStack(FName VariableName, const FGuid* InPageID = nullptr) const;

	void IterateNodesConnectedWithVertex(const FMetasoundFrontendVertexHandle& Vertex, TFunctionRef<void(const FMetasoundFrontendEdge&, FMetasoundFrontendNode&)> NodeIndexIterFunc, const FGuid& InPageID);

	UE_API FMetasoundFrontendDocument& GetDocumentChecked() const;
	UE_API IMetaSoundDocumentInterface& GetDocumentInterfaceChecked() const;

	// In place dependency replacement of a dependency with a new dependency at the same index with the same ID
	// For internal use only as a bit of a backdoor for updating multiple nodes of dependency that requires an in place update
	// so subsequent updates use the new dependency but without having to Remove and Add the old dependency and all its nodes at once
	// Returns new dependency
	UE_INTERNAL const FMetasoundFrontendClass* ReplaceDependencyInternal(const FMetasoundFrontendClass& OldDependency, FMetasoundFrontendClass NewDependency);
	void RemoveSwapDependencyInternal(int32 Index);
	
	bool RemoveGraphInputDefaultInternal(FName InputName, const FGuid& InPageID);

	bool SpliceVariableNodeFromStack(const FGuid& InNodeID, const FGuid& InPageID);
	bool UnlinkVariableNode(const FGuid& InNodeID, const FGuid& InPageID);

	struct FNodeConfigurationUpdateData
	{
		TConstStructView<FMetaSoundFrontendNodeConfiguration> ExistingConfig;
		TConstStructView<FMetasoundFrontendClassInterface> ExistingClassInterfaceOverride;
		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> RegisteredConfig;
		TInstancedStruct<FMetasoundFrontendClassInterface> RegeneratedClassInterfaceOverride;
		bool bDidUpdateClassInterfaceOverride = false;
	};

	bool ShouldReplaceExistingNodeConfig(TConstStructView<FMetaSoundFrontendNodeConfiguration> InRegisteredNodeConfig, TConstStructView<FMetaSoundFrontendNodeConfiguration> InExistingConfig) const;
	const FMetasoundFrontendClassInterface& GetApplicableRegistryInterface(const FMetasoundFrontendClass& InRegisteredClass, const FNodeConfigurationUpdateData& InNodeConfigurationUpdates) const;
	void FindNodeConfigurationUpdates(const FGuid& InNodeID, const FGuid& InPageID, const FMetasoundFrontendClass& InRegisteredClass, FNodeConfigurationUpdateData& OutNodeConfigUpdates, bool bInForceRegenerateClassInterfaceOverride) const;

	UPROPERTY(Transient)
	TScriptInterface<IMetaSoundDocumentInterface> DocumentInterface;

	// Page ID to apply build transaction to if no optional PageID is provided in explicit function call.
	// (Also used to support back compat for Controller API until mutable controllers are adequately deprecated).
	UPROPERTY(Transient)
	FGuid BuildPageID;

	TSharedPtr<Metasound::Frontend::IDocumentCache> DocumentCache;
	TSharedPtr<Metasound::Frontend::FDocumentModifyDelegates> DocumentDelegates;

	FTopLevelAssetPath HintPath;
};
#undef UE_API
