// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDocumentInterface.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOperatorSettings.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/TopLevelAssetPath.h"


// Forward Declarations
struct FMetasoundFrontendClassInputDefault;
struct FMetasoundFrontendClassInput;
struct FMetaSoundFrontendDocumentBuilder;
struct FMetasoundFrontendGraphClass;
struct FTopLevelAssetPath;

#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
	class IDocumentBuilderRegistry
	{
	public:
		virtual ~IDocumentBuilderRegistry() = default;

#if WITH_EDITORONLY_DATA
		// Given the provided builder, removes paged data within the associated document for a cooked build.
		// This function removes graphs and input defaults which are not to ever be used by a given cook
		// platform, allowing users to optimize away data and scale the amount of memory required for
		// initial load of input UObjects and graph topology, which can also positively effect runtime
		// performance as well, etc. Returns true if builder modified the document, false if not.
		UE_DEPRECATED(5.7, "Use StripUnusedPages(...) instead.")
		virtual bool CookPages(FName CookPlatformName, FMetaSoundFrontendDocumentBuilder& Builder) const = 0;
#endif // WITH_EDITORONLY_DATA

		virtual FMetaSoundFrontendDocumentBuilder* FindBuilder(TScriptInterface<IMetaSoundDocumentInterface> MetaSound) const = 0;
		virtual FMetaSoundFrontendDocumentBuilder* FindBuilder(const FMetasoundFrontendClassName& ClassName, const FTopLevelAssetPath& AssetPath) const = 0;
		virtual TArray<FMetaSoundFrontendDocumentBuilder*> FindBuilders(const FMetasoundFrontendClassName& ClassName) const = 0;
		virtual FMetaSoundFrontendDocumentBuilder* FindOutermostBuilder(const UObject& InSubObject) const = 0;

#if WITH_EDITORONLY_DATA
		// Find the existing builder for the given MetaSound, or optionally begin building by attaching a new builder.  Only available
		// in builds with editor only data as building serialized assets (which may have template nodes, cooked builds do not) is only
		// supported when editor data is loaded. Creating transient builders can simply be done by passing a new MetaSound asset to
		// a FMetaSoundFrontendDocumentBuilder constructor, or this registry's implementation may supply its own create call for tracking
		// and reuse purposes.
		virtual FMetaSoundFrontendDocumentBuilder& FindOrBeginBuilding(TScriptInterface<IMetaSoundDocumentInterface> MetaSound) = 0;
#endif // WITH_EDITORONLY_DATA

		// Removes builder from registry, clearing any cached builder state. (Optionally) forces unregistration from the Frontend Node Class Registry
		// (If the builder has outstanding transactions, unregistration from the Node Class Registry will occur regardless).
		virtual bool FinishBuilding(const FMetasoundFrontendClassName& InClassName, bool bForceUnregisterNodeClass = false) const = 0;
		virtual bool FinishBuilding(const FMetasoundFrontendClassName& InClassName, const FTopLevelAssetPath& InAssetPath, bool bForceUnregisterNodeClass = false) const = 0;

		// Reloads the given builder, maintaining all modify delegate subscriptions. Returns true if builder was found and reloaded,
		// false if not found.
		virtual bool ReloadBuilder(const FMetasoundFrontendClassName& InClassName) const = 0;

		UE_DEPRECATED(5.7, "This is no longer supported. FindPreferredPage(...)")
		virtual FGuid ResolveTargetPageID(const FMetasoundFrontendGraphClass& InGraphClass) const = 0;

		UE_DEPRECATED(5.7, "This is no longer supported. FindPreferredPage(...)")
		virtual FGuid ResolveTargetPageID(const FMetasoundFrontendClassInput& InClassInput) const = 0;

		UE_DEPRECATED(5.7, "This is no longer supported. FindPreferredPage(...)")
		virtual FGuid ResolveTargetPageID(const TArray<FMetasoundFrontendClassInputDefault>& Defaults) const = 0;

		UE_DEPRECATED(5.7, "This is only used to support existing deprecated functionality")
		virtual TArrayView<const FGuid> GetPageOrder() const = 0;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.7, "This is only used to support existing deprecated functionality")
		virtual TArray<FGuid> GetCookedTargetPages(FName InPlatformName) const = 0;

		UE_DEPRECATED(5.7, "This is only used to support existing deprecated functionality")
		virtual TArray<FGuid> GetCookedPageOrder(FName InPlatformName) const = 0;
#endif // WITH_EDITORONLY_DATA

		static UE_API class IDocumentBuilderRegistry* Get();
		static UE_API class IDocumentBuilderRegistry& GetChecked();
		static UE_API void Deinitialize();
		static UE_API void Initialize(TUniquePtr<IDocumentBuilderRegistry>&& InBuilderRegistry);
	};
} // namespace Metasound::Frontend

#undef UE_API
