// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "MetasoundFrontendDocumentModifyDelegates.h"
#include "MetasoundFrontendRegistryKey.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"


// Forward Declarations
struct FMetasoundFrontendClass;
struct FMetasoundFrontendClassMetadata;
struct FMetasoundFrontendDocument;
struct FMetaSoundFrontendDocumentBuilder;
struct FMetasoundFrontendEdge;
struct FMetasoundFrontendGraph;
struct FMetasoundFrontendNode;
struct FMetasoundFrontendVertex;


namespace Metasound::Frontend
{
	/** Interface for querying cached document nodes. */
	class IDocumentGraphNodeCache : public TSharedFromThis<IDocumentGraphNodeCache>
	{
	public:
		virtual ~IDocumentGraphNodeCache() = default;

		virtual bool ContainsNode(const FGuid& InNodeID) const = 0;
		virtual bool ContainsNodesOfClassID(const FGuid& InClassID) const = 0;

		virtual const int32* FindNodeIndex(const FGuid& InNodeID) const = 0;
		virtual TArray<const FMetasoundFrontendNode*> FindNodesOfClassID(const FGuid& InClassID) const = 0;

		virtual const FMetasoundFrontendNode* FindNode(const FGuid& InNodeID) const = 0;

		virtual TArray<const FMetasoundFrontendVertex*> FindNodeInputs(const FGuid& InNodeID, FName TypeName = FName()) const = 0;
		virtual TArray<const FMetasoundFrontendVertex*> FindNodeOutputs(const FGuid& InNodeID, FName TypeName = FName()) const = 0;

		// Returns corresponding input vertex if it exists
		virtual const FMetasoundFrontendVertex* FindInputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		// Returns corresponding input vertex if it exists
		virtual const FMetasoundFrontendVertex* FindInputVertex(const FGuid& InNodeID, FName InVertexName) const = 0;

		// Returns corresponding output vertex if it exists
		virtual const FMetasoundFrontendVertex* FindOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		// Returns corresponding output vertex if it exists
		virtual const FMetasoundFrontendVertex* FindOutputVertex(const FGuid& InNodeID, FName InVertexName) const = 0;

		// Recursively finds rerouted input vertices if they exist. Returns corresponding nodes to vertices if array provided. Sets boolean determining if connected to reroute if provided (optional).
		virtual TArray<const FMetasoundFrontendVertex*> FindReroutedInputVertices(const FGuid& InNodeID, const FGuid& InVertexID, TArray<const FMetasoundFrontendNode*>* ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const = 0;

		// Recursively finds rerouted input vertices if they exist. Returns corresponding nodes to vertices if array provided. Sets boolean determining if connected to reroute if provided (optional).
		virtual TArray<const FMetasoundFrontendVertex*> FindReroutedInputVertices(const FGuid& InNodeID, FName InVertexName, TArray<const FMetasoundFrontendNode*>* ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const = 0;

		// Recursively finds rerouted output vertex if it exist. Returns corresponding nodes to vertices if pointer provided. Sets boolean determining if connected to reroute if provided (optional).
		virtual const FMetasoundFrontendVertex* FindReroutedOutputVertex(const FGuid& InNodeID, const FGuid& InVertexID, const FMetasoundFrontendNode** ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const = 0;

		// Recursively finds rerouted output vertex if it exist. Returns corresponding node to vertices if pointer provided. Sets boolean determining if connected to reroute if provided (optional).
		virtual const FMetasoundFrontendVertex* FindReroutedOutputVertex(const FGuid& InNodeID, FName InVertexName, const FMetasoundFrontendNode** ConnectedNodes = nullptr, bool* bOutIsRerouted = nullptr) const = 0;
	};

	/** Interface for querying cached document edges. */
	class IDocumentGraphEdgeCache : public TSharedFromThis<IDocumentGraphEdgeCache>
	{
	public:
		virtual ~IDocumentGraphEdgeCache() = default;

		virtual bool ContainsEdge(const FMetasoundFrontendEdge& InEdge) const = 0;

		virtual bool IsNodeInputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;
		virtual bool IsNodeOutputConnected(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		virtual TArray<const FMetasoundFrontendEdge*> FindEdges(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;
		virtual const int32* FindEdgeIndexToNodeInput(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		virtual const TArrayView<const int32> FindEdgeIndicesFromNodeOutput(const FGuid& InNodeID, const FGuid& InVertexID) const = 0;
	};

	/** Interface for querying cached document interface. */
	class IDocumentGraphInterfaceCache : public TSharedFromThis<IDocumentGraphInterfaceCache>
	{
	public:
		virtual ~IDocumentGraphInterfaceCache() = default;

		virtual const FMetasoundFrontendClassInput* FindInput(FName InputName) const = 0;
		virtual const int32* FindInputIndex(FName InputName) const = 0;
		virtual const FMetasoundFrontendClassOutput* FindOutput(FName OutputName) const = 0;
		virtual const int32* FindOutputIndex(FName OutputName) const = 0;
	};

	/** Interface for querying cached document dependencies. */
	class IDocumentCache : public TSharedFromThis<IDocumentCache>
	{
	public:
		virtual ~IDocumentCache() = default;

		virtual bool ContainsDependency(const FNodeRegistryKey& InClassKey) const = 0;
		virtual bool ContainsDependencyOfType(EMetasoundFrontendClassType ClassType) const = 0;
		virtual const FMetasoundFrontendClass* FindDependency(const Metasound::Frontend::FNodeRegistryKey& InClassKey) const = 0;
		virtual const FMetasoundFrontendClass* FindDependency(const FGuid& InClassID) const = 0;
		virtual const int32* FindDependencyIndex(const Metasound::Frontend::FNodeRegistryKey& InClassKey) const = 0;
		virtual const int32* FindDependencyIndex(const FGuid& InClassID) const = 0;

		virtual const FMetasoundFrontendDocument& GetDocument() const = 0;

		UE_DEPRECATED(5.5, "Use overload providing pageID")
		virtual const IDocumentGraphNodeCache& GetNodeCache() const { return GetNodeCache(::Metasound::Frontend::DefaultPageID); }

		// Returns NodeCache associated with the given PageID
		virtual const IDocumentGraphNodeCache& GetNodeCache(const FGuid& InPageID) const = 0;

		UE_DEPRECATED(5.5, "Use overload providing pageID")
		virtual const IDocumentGraphEdgeCache& GetEdgeCache() const { return GetEdgeCache(::Metasound::Frontend::DefaultPageID); }

		// Returns EdgeCache associated with the given PageID
		virtual const IDocumentGraphEdgeCache& GetEdgeCache(const FGuid& InPageID) const = 0;

		// Returns InterfaceCache for the given document
		virtual const IDocumentGraphInterfaceCache& GetInterfaceCache() const = 0;

#if WITH_EDITORONLY_DATA
		// Sets the default BuildPageID for the given document
		virtual void SetBuildPageID(const FGuid& InPageID) = 0;
#endif // WITH_EDITORONLY_DATA
	};

} // namespace Metasound::Frontend
