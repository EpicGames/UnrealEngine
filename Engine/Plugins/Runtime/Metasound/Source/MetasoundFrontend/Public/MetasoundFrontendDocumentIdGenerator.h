// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MetasoundFrontendDocument.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Guid.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	namespace Frontend
	{
		extern METASOUNDFRONTEND_API int32 MetaSoundEnableCookDeterministicIDGeneration;

		/*** 
		 * For generating IDs using a given document. 
		 * USAGE:
		 *
		 * If you want everything within the calling scope to be deterministic, use
		 * the scope determinism lock like you would a `FScopeLock`
		 *
		 * {
		 * 		constexpr bool bIsDeterministic = true;
		 * 		FDocumentIDGenerator::FScopeDeterminism DeterminismScope(bIsDeterministic);
		 *
		 *      // Anything called in this scope will use a deterministic ID generator.
		 * 		// Once the `DeterminismScope` variable is destroyed, it will return to
		 * 		// whatever the prior behavior was
		 * 		MetaSoundAsset->UpdateOrWhatever();
		 * }
		 *
		 * void FMetaSoundAsset::UpdateOrWhatever()
		 * {
		 *      // Anytime you call IDGen::Create*ID(..), it will obey whatever the outside scope set it to be.
		 *      AddNewNodeOrWhatever(...)
		 * }
		 *
		 * void FMetaSoundAsset::AddNodeOrWhatever(...)
		 * {
		 * 		FDocumentIDGenerator& IDGen = FDocumentIDGenerator::Get();
		 *
		 * 		FGuid NewNodeID = IDGen::CreateNewNodeID();
		 * 		...
		 * }
		 *
		 */
		class FDocumentIDGenerator
		{
		public:
			class FScopeDeterminism final
			{
			public:
				UE_API FScopeDeterminism(bool bInIsDeterministic);
				UE_API bool GetDeterminism() const;
				UE_API ~FScopeDeterminism();

			private:
				bool bOriginalValue = false;
			};

			static UE_API FDocumentIDGenerator& Get();

			UE_API FGuid CreateNodeID(const FMetasoundFrontendDocument& InDocument) const;
			UE_API FGuid CreateVertexID(const FMetasoundFrontendDocument& InDocument) const;
			UE_API FGuid CreateClassID(const FMetasoundFrontendDocument& InDocument) const;

			UE_API FGuid CreateIDFromDocument(const FMetasoundFrontendDocument& InDocument) const;

		private:
			FDocumentIDGenerator() = default;

			friend class FScopeDeterminism;
			UE_API void SetDeterminism(bool bInIsDeterministic);
			UE_API bool GetDeterminism() const;

			bool bIsDeterministic = false;
		};

		// For generating IDs that are derived from a given class vertex.  Unlike
		// document ID generation, the class ID generator's results are not unique
		// upon each request and therefore can deterministically generate the same
		// ID for the same provided vertex.
		class FClassIDGenerator
		{
		public: 
			static FClassIDGenerator& Get();

			FGuid CreateInputID(const FMetasoundFrontendClassInput& Input) const;
			FGuid CreateInputID(const Audio::FParameterInterface::FInput& Input) const;
			FGuid CreateOutputID(const FMetasoundFrontendClassOutput& Output) const;
			FGuid CreateOutputID(const Audio::FParameterInterface::FOutput& Output) const;

			FGuid CreateNamespacedIDFromString(const FGuid& NamespaceGuid, const FString& StringToHash) const;
		};

		METASOUNDFRONTEND_API FGuid CreateLocallyUniqueId();
	}
}

#undef UE_API
