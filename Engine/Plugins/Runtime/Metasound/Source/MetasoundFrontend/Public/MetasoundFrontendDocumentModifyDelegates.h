// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/DelegateCombinations.h"
#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"

#include "MetasoundFrontendDocumentModifyDelegates.generated.h"

#define UE_API METASOUNDFRONTEND_API


// TODO: Move these to namespace
DECLARE_MULTICAST_DELEGATE(FOnMetaSoundFrontendDocumentMutateMetadata);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetaSoundFrontendDocumentMutateArray, int32 /* Index */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetaSoundFrontendDocumentMutateInterfaceArray, const FMetasoundFrontendInterface& /* Interface */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundFrontendDocumentRemoveSwappingArray, int32 /* Index */, int32 /* LastIndex */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundFrontendDocumentRenameClass, const int32 /* Index */, const FMetasoundFrontendClassName& /* NewName */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray, int32 /* NodeIndex */, int32 /* VertexIndex */, int32 /* LiteralIndex */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMetaSoundFrontendDocumentRenameVertex, FName /* OldName */, FName /* NewName */);

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetaSoundFrontendDocumentMutateMember, FGuid /* Member ID*/);
#endif // WITH_EDITOR

UENUM(BlueprintType)
enum class EMetaSoundLiteralChangeType : uint8
{
	// Initial, unset type
	Invalid = 0,
		
	// Default is set 
	Set,
		
	// Default is removed for this page
	Remove,
		
	// Default is reset to default value 
	Reset
};

namespace Metasound::Frontend
{
	struct FDocumentMutatePageArgs
	{
		FGuid PageID;
	};

	enum class EDocumentTemplateChangeType : uint8
	{
		// Initial, unset value.
		Invalid = 0,

		// Config property or properties were changed
		Property,

		// Config struct has been completely replaced
		// (Could now be unset/cleared)
		Struct,

		// Transaction has been re-applied (redo) or rolled back (undo)
		UndoRedo,
	};
	
	struct FDocumentArrayPagedInputArgs
	{
		int32 Index = INDEX_NONE;
		FGuid PageID;
		EMetaSoundLiteralChangeType ChangeType = EMetaSoundLiteralChangeType::Invalid;
	};
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMetaSoundFrontendDocumentMutateInput, const FDocumentArrayPagedInputArgs& /* Args */);

	struct FDocumentTemplateChangedArgs
	{
		EDocumentTemplateChangeType Type = EDocumentTemplateChangeType::Invalid;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDocumentTemplateChanged, const FDocumentTemplateChangedArgs& /* Args */);

	UE_DEPRECATED(5.8, "Use FOnDocumentTemplateChanged instead")
	typedef FOnDocumentTemplateChanged FOnDocumentConfigurationChanged;

	UE_DEPRECATED(5.8, "Use EDocumentTemplateChangeType instead")
	typedef EDocumentTemplateChangeType EDocumentConfigChangeType;

	UE_DEPRECATED(5.8, "Use FDocumentTemplateChangedArgs instead")
	typedef FDocumentTemplateChangedArgs FDocumentConfigChangedArgs;

	// Soft deprecated in 5.8 in favor of FOnDocumentTemplateChanged.  Once delegate is removed, it can be officially deprecated.
	struct FDocumentPresetStateChangedArgs {};

	// Soft deprecated in 5.8 as usage is no longer valid (migrated to OnDocumentTemplateChanged)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDocumentPresetStateChanged, const FDocumentPresetStateChangedArgs& /* Args */);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDocumentPageAdded, const FDocumentMutatePageArgs& /* Args */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDocumentRemovingPage, const FDocumentMutatePageArgs& /* Args */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDocumentPageSet, const FDocumentMutatePageArgs& /* Args */);

	struct FPageModifyDelegates
	{
		FOnDocumentPageAdded OnPageAdded;
		FOnDocumentRemovingPage OnRemovingPage;
		FOnDocumentPageSet OnPageSet;
	};

	struct FInterfaceModifyDelegates
	{
		FOnMetaSoundFrontendDocumentMutateInterfaceArray OnInterfaceAdded;
		FOnMetaSoundFrontendDocumentMutateInterfaceArray OnRemovingInterface;

		FOnMetaSoundFrontendDocumentMutateArray OnInputAdded;
		FOnMetaSoundFrontendDocumentMutateArray OnRemovingInput;
		FOnMetaSoundFrontendDocumentRenameVertex OnInputNameChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputDataTypeChanged;
		FOnMetaSoundFrontendDocumentMutateInput OnInputDefaultChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputIsConstructorPinChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputInheritsDefaultChanged;

#if WITH_EDITOR
		FOnMetaSoundFrontendDocumentMutateArray OnInputDisplayNameChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputDescriptionChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputSortOrderIndexChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnInputIsAdvancedDisplayChanged;
#endif // WITH_EDITOR

		FOnMetaSoundFrontendDocumentMutateArray OnOutputAdded;
		FOnMetaSoundFrontendDocumentMutateArray OnRemovingOutput;
		FOnMetaSoundFrontendDocumentRenameVertex OnOutputNameChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnOutputDataTypeChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnOutputIsConstructorPinChanged;
#if WITH_EDITOR
		FOnMetaSoundFrontendDocumentMutateArray OnOutputDisplayNameChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnOutputDescriptionChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnOutputSortOrderIndexChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnOutputIsAdvancedDisplayChanged;

		FOnMetaSoundFrontendDocumentMutateMember OnMemberMetadataSet;
		FOnMetaSoundFrontendDocumentMutateMember OnRemovingMemberMetadata;
#endif // WITH_EDITOR
	};

	struct FNodeModifyDelegates
	{
		FOnMetaSoundFrontendDocumentMutateArray OnNodeAdded;
		FOnMetaSoundFrontendDocumentMutateArray OnNodeConfigurationUpdated;
		FOnMetaSoundFrontendDocumentRemoveSwappingArray OnRemoveSwappingNode;

		FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray OnNodeInputLiteralSet;
		FOnMetaSoundFrontendDocumentMutateNodeInputLiteralArray OnRemovingNodeInputLiteral;
	};

	struct FEdgeModifyDelegates
	{
		FOnMetaSoundFrontendDocumentMutateArray OnEdgeAdded;
		FOnMetaSoundFrontendDocumentRemoveSwappingArray OnRemoveSwappingEdge;
	};

	struct FGraphModifyDelegates
	{
		FEdgeModifyDelegates EdgeDelegates;
		FNodeModifyDelegates NodeDelegates;
	};

	struct FDocumentModifyDelegates : TSharedFromThis<FDocumentModifyDelegates>
	{
		UE_API FDocumentModifyDelegates();
		UE_API FDocumentModifyDelegates(const FMetasoundFrontendDocument& Document);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		~FDocumentModifyDelegates() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FOnMetaSoundFrontendDocumentMutateMetadata OnDocumentMetadataChanged;
		FOnMetaSoundFrontendDocumentMutateArray OnDependencyAdded;
		FOnMetaSoundFrontendDocumentRemoveSwappingArray OnRemoveSwappingDependency;
		FOnMetaSoundFrontendDocumentRenameClass OnRenamingDependencyClass;

		// Soft deprecated in 5.8: "Use OnDocumentTemplateChanged instead"
		FOnDocumentPresetStateChanged OnPresetStateChanged;

		FOnDocumentTemplateChanged OnDocumentTemplateChanged;

		FPageModifyDelegates PageDelegates;
		FInterfaceModifyDelegates InterfaceDelegates;

	private:
		TSortedMap<FGuid, FGraphModifyDelegates> GraphDelegates;

	public:
		UE_API void AddPageDelegates(const FGuid& InPageID);
		UE_API void RemovePageDelegates(const FGuid& InPageID, bool bBroadcastNotify = true);

		UE_DEPRECATED(5.7, "Use FindGraphDelegatesChecked instead")
		UE_API FNodeModifyDelegates& FindNodeDelegatesChecked(const FGuid& InPageID);

		UE_DEPRECATED(5.7, "Use FindGraphDelegatesChecked instead")
		UE_API FEdgeModifyDelegates& FindEdgeDelegatesChecked(const FGuid& InPageID);

		UE_DEPRECATED(5.7, "Use IterateGraphDelegates instead")
		UE_API void IterateGraphEdgeDelegates(TFunctionRef<void(FEdgeModifyDelegates&)> Func);

		UE_DEPRECATED(5.7, "Use IterateGraphDelegates instead")
		UE_API void IterateGraphNodeDelegates(TFunctionRef<void(FNodeModifyDelegates&)> Func);

		UE_API FGraphModifyDelegates& FindGraphDelegatesChecked(const FGuid& InPageID);

		UE_API void IterateGraphDelegates(TFunctionRef<void(FGraphModifyDelegates&)> Func);
	};

	class IDocumentBuilderTransactionListener : public TSharedFromThis<IDocumentBuilderTransactionListener>
	{
	public:
		virtual ~IDocumentBuilderTransactionListener() = default;

		// Called when the builder is reloaded, at which point the document cache and delegates are refreshed
		virtual void OnBuilderReloaded(FDocumentModifyDelegates& OutDelegates) = 0;
	};
} // namespace Metasound::Frontend

#undef UE_API
