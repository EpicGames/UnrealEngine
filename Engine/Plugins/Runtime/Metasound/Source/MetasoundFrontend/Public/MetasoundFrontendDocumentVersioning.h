// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/CoreMiscDefines.h"

#if WITH_EDITORONLY_DATA
#include <atomic>

#include "MetasoundFrontendDocument.h"
#include "Misc/CoreMiscDefines.h"
#include "HAL/CriticalSection.h"
#include "UObject/GarbageCollection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"


// Forward Declarations
class FMetasoundAssetBase;
class IMetaSoundDocumentInterface;
struct FMetaSoundFrontendDocumentBuilder;

#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
	// Asynchronously manages versioning of provided MetaSounds asset data.
	// Only required for serialized assets and passed data for managing
	// immediately upon loading in builds with editor-only data enabled.
	class FVersioningManager
	{
	public:
		FVersioningManager() = default;
		~FVersioningManager();

		UE_API static FVersioningManager& Get();
		UE_API static void Initialize();
		UE_API static void Shutdown();

		// Current, max document version
		UE_API static FMetasoundFrontendVersionNumber GetMaxDocumentVersion();

		// Version where page data was migrated from a singleton graph to
		// multiple paged graphs.  Some legacy behavior requires access to
		// this version to initialize a builder prior to versioning document
		// data.
		UE_INTERNAL static FMetasoundFrontendVersionNumber GetPageMigrationVersion();

		// Feature to check if an ID is changed when Document Metadata is updated.
		// Is actively being deprecated in favor of the Builder API where transaction
		// ids are centrally tracked.
		UE_INTERNAL static bool ChangeIDComparisonEnabledInAutoUpdate();

		// Versions the given MetaSound asynchronously. Ensures provided MetaSound is a serialized asset.
		UE_API void VersionAssetAsync(UObject& InMetaSound, bool bIsDeterministic);

		// Versions document interfaces on the given builder's object. Primarily exposed
		// for testing purposes from other modules. Waits for all other active async versioning
		// requests for safety (is and should not be called from versioning async task).
		UE_API bool VersionInterfaces(FMetaSoundFrontendDocumentBuilder& DocBuilder) const;

		// Waits until the given MetaSound is finished asynchronously versioning.
		UE_API void WaitUntilVersioningComplete(const UObject& InMetaSound) const;

	private:
		// Versions Frontend Document. Called asynchronously while versioning asset.
		bool VersionDocument(FMetaSoundFrontendDocumentBuilder& DocBuilder) const;

		// Interface versioning implementation (does not guard against thread access
		// or impose any timing restrictions as the public API call does).
		bool VersionInterfacesInternal(FMetaSoundFrontendDocumentBuilder& DocBuilder) const;

		// Waits until the given MetaSound's references finish asynchronously versioning.
		void WaitUntilVersioningReferencesComplete(const UObject& InMetaSound) const;

		using FUniqueId = uint32;

		mutable FCriticalSection ActiveVersionCritSection;

		struct FTaskData
		{
			TStrongObjectPtr<UObject> AssetPtr;
			UE::Tasks::FTask Task;
		};
		TMap<FUniqueId, FTaskData> ActiveVersionTaskData;
	};
} // namespace Metasound::Frontend
#endif // WITH_EDITORONLY_DATA
#undef UE_API
