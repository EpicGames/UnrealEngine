// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "MetasoundFrontendGraphBuilder.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundGraph.h"
#include "MetasoundNodeConstructorParams.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	/** FFrontendGraph is a utility graph for use in the frontend. It can own nodes
	 * that live within the graph and provides query interfaces for finding nodes
	 * by dependency ID or input/output index. 
	 */
	class FFrontendGraph : public FGraph
	{
		public:
			UE_DEPRECATED(5.7, "Use constructor that provides a top-level asset path instead")
			UE_API FFrontendGraph(const FString& InInstanceName, const FGuid& InInstanceID);

			/** FFrontendGraph constructor.
			 *
			 * @parma InAssetPath - Path to this graph's asset.
			 * @parma InInstanceID - ID of this graph.
			 */
			UE_API FFrontendGraph(const FTopLevelAssetPath& InAssetPath, const FGuid& InInstanceID);

			virtual ~FFrontendGraph() = default;

			/** Add an input node to this graph.
			 *
			 * @param InNodeID - The NodeID related to the parent FMetasoundFrontendClass.
			 * @param InIndex - The positional index for the input.
			 * @param InVertexName - The key for the graph input vertex.
			 * @param InNode - A shared pointer to an input node. 
			 */
			UE_DEPRECATED(5.6, "Use AddInputNode(...) version which does not take an InIndex")
			UE_API void AddInputNode(FGuid InNodeID, int32 InIndex, const FVertexName& InVertexName, TSharedPtr<const INode> InNode);

			// Unhide AddInputNode function on base class
			using FGraph::AddInputNode;

			/** Add an output node to this graph.
			 *
			 * @param InNodeID - The NodeID related to the parent FMetasoundFrontendClass.
			 * @param InIndex - The positional index for the output.
			 * @param InVertexName - The key for the graph output vertex.
			 * @param InNode - A shared pointer to an output node. 
			 */
			UE_DEPRECATED(5.6, "Use AddOutputNode(...) version which does not take an InIndex")
			UE_API void AddOutputNode(FGuid InNodeID, int32 InIndex, const FVertexName& InVertexName, TSharedPtr<const INode> InNode);

			// Unhide AddOutputNode function on base class
			using FGraph::AddOutputNode;

			/** Retrieve node by input index.
			 *
			 * @param InIndex - The index of the requested input.
			 *
			 * @return Pointer to the Node if it is stored on this graph. nullptr otherwise. 
			 */
			UE_DEPRECATED(5.6, "Retrieving input nodes nodes by index will no longer supported on this object")
			UE_API const INode* FindInputNode(int32 InIndex) const;

			/** Retrieve node by output index.
			 *
			 * @param InIndex - The index of the requested output.
			 *
			 * @return Pointer to the Node if it is stored on this graph. nullptr otherwise. 
			 */
			UE_DEPRECATED(5.6, "Retrieving output nodes nodes by index will no longer supported on this object")
			UE_API const INode* FindOutputNode(int32 InIndex) const;

		private:

			TMap<int32, const INode*> InputNodes;
			TMap<int32, const INode*> OutputNodes;
	};

	using FFrontendGraphBuilder UE_DEPRECATED(5.6, "FFrontendGraphBuilder is deprecated. Please use Frontend::FGraphBuilder instead.") = Frontend::FGraphBuilder;

}

#undef UE_API
