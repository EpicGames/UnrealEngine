// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentIdGenerator.h"
#include "MetasoundGraph.h"
#include "MetasoundNodeConstructorParams.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	// Forward declare
	class FFrontendGraph;
}

namespace Metasound::Frontend
{
	// Forward declare
	class IDataTypeRegistry;

	/** FGraphBuilder builds a FFrontendGraph from a FMetasoundDocument
	 * or FMetasoundFrontendClass.
	 */
	class FGraphBuilder
	{
	public:

		struct FCreateNodeParams
		{
#if WITH_EDITORONLY_DATA
			UE_DEPRECATED(5.7, "Deprecated in favor of AssetPath")
			const FString DebugAssetName = { };
#endif // WITH_EDITORONLY_DATA

			/** The Frontend representation of the node being created. */
			const FMetasoundFrontendNode& FrontendNode;

			/** The Frontend Node Class representing the node. */
			const FMetasoundFrontendClass& FrontendNodeClass;

			/** The Frontend Graph which contains the Frontend Node being created. */
			const FMetasoundFrontendGraph& OwningFrontendGraph;

			/** The Frontend Graph Class which contains the OwningFrontendGraph. */
			const FMetasoundFrontendGraphClass& OwningFrontendGraphClass;

			/** Optional ProxyDataCache. This is needed for creating nodes when being
			 * executed off of the game thread. */
			const Frontend::FProxyDataCache* ProxyDataCache = nullptr;

			/** Optional DataTypeRegistry. Providing a reusable data type registry
			 * can avoid repeated lookups to gather the registry. */
			const Frontend::IDataTypeRegistry* DataTypeRegistry = nullptr;

			/** Optional subgraph map. Subgraphs are not fully supported in MetaSound
			 * so this is kept optional to avoid creation of unused TMap<>s. */
			const TMap<FGuid, TSharedPtr<const IGraph>>* Subgraphs = nullptr;

			/** An array of preferred pages where most preferred pages occur before
			 * less preferred. */
			TArrayView<const FGuid> PageOrder = { };

			/** AssetPath to graph of which the node is being added to. */
			const FTopLevelAssetPath& OwningAssetPath;
		};

		/** Create a INode from a document representation of the node. 
		 *
		 * Note: This does not connect the node or place it in any graph. 
		 */
		static UE_API TUniquePtr<INode> CreateNode(const FCreateNodeParams& InParams);

		/* Create a FFrontendGraph from a FMetasoundFrontendDocument.*/
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(const FMetasoundFrontendDocument& InDocument, const FTopLevelAssetPath& InAssetPath, TArrayView<const FGuid> InPageOrder = { });

		/* Create a FFrontendGraph from a FMetasoundFrontendDocument retrieving proxies from a FProxyDataCache.*/
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendDocument& InDocument,
			const Frontend::FProxyDataCache& InProxies,
			const FTopLevelAssetPath& InAssetPath,
			const FGuid InGraphId = Frontend::CreateLocallyUniqueId(),
			TArrayView<const FGuid> InPageOrder = { });

		/* Create a FFrontendGraph from a FMetasoundFrontendDocument and subobjects.*/
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendGraphClass& InGraph,
			const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
			const TArray<FMetasoundFrontendClass>& InDependencies,
			const FTopLevelAssetPath& InAssetPath,
			TArrayView<const FGuid> InPageOrder = { });

		/* Create a FFrontendGraph from a FMetasoundFrontendDocument and subobjects, retrieving proxies from a FProxyDataCache.*/
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendGraphClass& InGraph,
			const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
			const TArray<FMetasoundFrontendClass>& InDependencies,
			const Frontend::FProxyDataCache& InProxyDataCache,
			const FTopLevelAssetPath& InAssetPath,
			const FGuid InGraphId = Frontend::CreateLocallyUniqueId(), 
			TArrayView<const FGuid> InPageOrder = { });

		// DEPRECATED API BELOW
		UE_DEPRECATED(5.7, "Use overload providing TopLevelAssetPath instead of DebugAssetName string")
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(const FMetasoundFrontendDocument& InDocument, const FString& InDebugAssetName, TArrayView<const FGuid> InPageOrder = { });

		UE_DEPRECATED(5.7, "Use overload providing TopLevelAssetPath instead of DebugAssetName string")
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendDocument& InDocument,
			const Frontend::FProxyDataCache& InProxies,
			const FString& InDebugAssetName,
			const FGuid InGraphId = Frontend::CreateLocallyUniqueId(),
			TArrayView<const FGuid> InPageOrder = { });

		UE_DEPRECATED(5.7, "Use overload providing TopLevelAssetPath instead of DebugAssetName string")
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendGraphClass& InGraph,
			const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
			const TArray<FMetasoundFrontendClass>& InDependencies,
			const FString& InDebugAssetName,
			TArrayView<const FGuid> InPageOrder = { });

		UE_DEPRECATED(5.7, "Use overload providing TopLevelAssetPath instead of DebugAssetName string")
		static UE_API TUniquePtr<FFrontendGraph> CreateGraph(
			const FMetasoundFrontendGraphClass& InGraph,
			const TArray<FMetasoundFrontendGraphClass>& InSubgraphs,
			const TArray<FMetasoundFrontendClass>& InDependencies,
			const Frontend::FProxyDataCache& InProxyDataCache,
			const FString& InDebugAssetName,
			const FGuid InGraphId = Frontend::CreateLocallyUniqueId(),
			TArrayView<const FGuid> InPageOrder = { });
	};
}

#undef UE_API
