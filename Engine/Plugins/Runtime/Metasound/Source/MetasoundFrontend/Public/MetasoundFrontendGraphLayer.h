// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "MetasoundFrontendDocument.h"
#include "UObject/NoExportTypes.h"


#include "MetasoundFrontendGraphLayer.generated.h"


#define UE_API METASOUNDFRONTEND_API

USTRUCT()
struct FMetasoundFrontendGraphLayer
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FMetasoundFrontendClass> Dependencies;

	UPROPERTY()
	FMetasoundFrontendGraph Graph;

	UPROPERTY()
	TArray<FMetasoundFrontendClassInput> Inputs;

	UPROPERTY()
	TArray<FMetasoundFrontendClassOutput> Outputs;
};

#undef UE_API
