// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	namespace Frontend
	{
		// Forward declare.
		class IInputController;
		class IOutputController;
		class INodeController;

		class FGraphLinter
		{
		public:
			using FDepthFirstVisitFunction = TFunctionRef<TSet<FGuid> (const INodeController&)>;

			/** Returns true if connecting thing input and output controllers will cause
			 * a loop in the graph. Returns false otherwise. 
			 */
			static UE_API bool DoesConnectionCauseLoop(const IInputController& InInputController, const IOutputController& InOutputController);

			/** Returns true if the FromNode can reach the ToNode by traversing the graph 
			 * in the forward direction. */
			static UE_API bool IsReachableDownstream(const INodeController& InFromNode, const INodeController& InToNode);

			/** Returns true if the FromNode can reach the ToNode by traversing the graph backwards
			 * (aka by traversing the transpose graph).
			 */
			static UE_API bool IsReachableUpstream(const INodeController& InFromNode, const INodeController& InToNode);

			/** Visits nodes using depth first traversals. */
			static UE_API void DepthFirstTraversal(const INodeController& Node, FDepthFirstVisitFunction Visit);
		};
	}
}

#undef UE_API
