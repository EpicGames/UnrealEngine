// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Transform.h"
#include "AudioParameterControllerInterface.h"
#include "Containers/ContainersFwd.h"
#include "MetasoundLiteral.h"

#include <type_traits>

#include "MetasoundFrontendLiteral.generated.h"

#define UE_API METASOUNDFRONTEND_API

// The type of a given literal for an input value.
//
// The EMetasoundFrontendLiteralType's are matched to Metasound::ELiteralType`s 
// by giving them the same value. This supports easy conversion from one type to
// another.
UENUM(BlueprintType)
enum class EMetasoundFrontendLiteralType : uint8
{

	None = static_cast<uint8>(Metasound::ELiteralType::None), //< A value of None expresses that an object being constructed with a literal should be default constructed.
	Boolean = static_cast<uint8>(Metasound::ELiteralType::Boolean),
	Integer = static_cast<uint8>(Metasound::ELiteralType::Integer),
	Float = static_cast<uint8>(Metasound::ELiteralType::Float),
	String = static_cast<uint8>(Metasound::ELiteralType::String),
	UObject = static_cast<uint8>(Metasound::ELiteralType::UObjectProxy),

	NoneArray = static_cast<uint8>(Metasound::ELiteralType::NoneArray), //< A NoneArray expresses the number of objects to be default constructed.
	BooleanArray = static_cast<uint8>(Metasound::ELiteralType::BooleanArray),
	IntegerArray = static_cast<uint8>(Metasound::ELiteralType::IntegerArray),
	FloatArray = static_cast<uint8>(Metasound::ELiteralType::FloatArray),
	StringArray = static_cast<uint8>(Metasound::ELiteralType::StringArray),
	UObjectArray = static_cast<uint8>(Metasound::ELiteralType::UObjectProxyArray),

	Invalid UMETA(Hidden)
};

static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::None) == static_cast<uint8>(EAudioParameterType::None), "Type 'None' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::Boolean) == static_cast<uint8>(EAudioParameterType::Boolean), "Type 'Boolean' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::Integer) == static_cast<uint8>(EAudioParameterType::Integer), "Type 'Integer' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::Float) == static_cast<uint8>(EAudioParameterType::Float), "Type 'Float' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::String) == static_cast<uint8>(EAudioParameterType::String), "Type 'String' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::UObject) == static_cast<uint8>(EAudioParameterType::Object), "Type 'UObjectProxy' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::NoneArray) == static_cast<uint8>(EAudioParameterType::NoneArray), "Type 'NoneArray' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::BooleanArray) == static_cast<uint8>(EAudioParameterType::BooleanArray), "Type 'BooleanArray' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::IntegerArray) == static_cast<uint8>(EAudioParameterType::IntegerArray), "Type 'IntegerArray' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::FloatArray) == static_cast<uint8>(EAudioParameterType::FloatArray), "Type 'FloatArray' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::StringArray) == static_cast<uint8>(EAudioParameterType::StringArray), "Type 'StringArray' value must match");
static_assert(static_cast<uint8>(EMetasoundFrontendLiteralType::UObjectArray) == static_cast<uint8>(EAudioParameterType::ObjectArray), "Type 'UObjectProxyArray' value must match");

// Check that the static_cast<>s above are using the correct type.
static_assert(std::is_same<uint8, std::underlying_type<EMetasoundFrontendLiteralType>::type>::value, "Update type in static_cast<TYPE> from Metasound::ELiteralType to EMetasoundFrontendLiteralType in EMetasoundFrontendLiteralType declaration.");
static_assert(std::is_same<std::underlying_type<Metasound::ELiteralType>::type, std::underlying_type<EMetasoundFrontendLiteralType>::type>::value, "EMetasoundFrontendLiteralType and Metasound::ELiteralType must have matching underlying types to support conversion.");

// Forward Declare
namespace Metasound
{
	namespace Frontend
	{
		class IDataTypeRegistry;
		class FProxyDataCache;
	}
}

// Represents the serialized version of variant literal types. 
USTRUCT(BlueprintType, meta = (DisplayName = "MetaSound Literal"))
struct FMetasoundFrontendLiteral
{
	GENERATED_BODY()

	struct FDefault {};
	struct FDefaultArray { int32 Num = 0; };

	FMetasoundFrontendLiteral() = default;

	UE_API FMetasoundFrontendLiteral(const Metasound::FLiteral& InLiteral);

	UE_API FMetasoundFrontendLiteral(const FAudioParameter& InParameter);

private:
	// The set type of this literal.
	UPROPERTY()
	EMetasoundFrontendLiteralType Type = EMetasoundFrontendLiteralType::None;

	UPROPERTY()
	int32 AsNumDefault = 0;

	UPROPERTY()
	TArray<bool> AsBoolean;

	UPROPERTY()
	TArray<int32> AsInteger;

	UPROPERTY()
	TArray<float> AsFloat;

	UPROPERTY()
	TArray<FString> AsString;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> AsUObject;

public:
	// Returns true if the stored Type is an array type.
	UE_API bool IsArray() const;

	// Returns whether the other literal is value equivalent
	UE_API bool IsEqual(const FMetasoundFrontendLiteral& InOther) const;

	// Returns true if the literal is in a valid state (Type != EMetasoundFrontendLiteralType::Invalid)
	UE_API bool IsValid() const;

	UE_API bool TryGet(UObject*& OutValue) const;
	UE_API bool TryGet(TArray<UObject*>& OutValue) const;

	UE_API bool TryGet(bool& OutValue) const;
	UE_API bool TryGet(TArray<bool>& OutValue) const;
	UE_API bool TryGet(int32& OutValue) const;
	UE_API bool TryGet(TArray<int32>& OutValue) const;
	UE_API bool TryGet(float& OutValue) const;
	UE_API bool TryGet(TArray<float>& OutValue) const;
	UE_API bool TryGet(FString& OutValue) const;
	UE_API bool TryGet(TArray<FString>& OutValue) const;

	// Sets the literal to the given type and value to default;
	UE_API void SetType(EMetasoundFrontendLiteralType InType);

	UE_API void Set(FDefault InValue);
	UE_API void Set(FDefaultArray InValue);
	UE_API void Set(bool InValue);
	UE_API void Set(TArray<bool> InValue);
	UE_API void Set(int32 InValue);
	UE_API void Set(TArray<int32> InValue);
	UE_API void Set(float InValue);
	UE_API void Set(TArray<float> InValue);
	UE_API void Set(FString InValue);
	UE_API void Set(TArray<FString> InValue);
	UE_API void Set(UObject* InValue);
	UE_API void Set(TArray<UObject*> InValue);

	UE_API void SetFromLiteral(const Metasound::FLiteral& InLiteral);

	UE_API EMetasoundFrontendLiteralType GetType() const;
	
	// Get the number of array elements if this literal is an array type, otherwise return 0
	UE_API int32 GetArrayNum() const;

	// Return a Metasound::FLiteral representation of this object. 
	//
	// @param InMetaSoundDataType - The name of the MetaSound data type
	// @param InDataTypeRegistry - A pointer to an existing data type registry. If null, the data type registry will be retrieved within this function.
	// @param InProxyDataCache - A pointer to an existing proxy data cache. If not null, UObject proxies will be retrieved from the cache. If null, UObject proxies will be created in this function.
	//
	// @returns An FLiteral. If the data type couldn't be found, or if the literal type was incompatible with the data type, then an invalid FLiteral is returned.
	UE_API Metasound::FLiteral ToLiteral(const FName& InMetaSoundDataType, const Metasound::Frontend::IDataTypeRegistry* InDataTypeRegistry=nullptr, const Metasound::Frontend::FProxyDataCache* InProxyDataCache=nullptr) const;

	// Return a Metasound::FLiteral representation of this object, excluding UObject proxies.
	UE_API Metasound::FLiteral ToLiteralNoProxy() const;

	// Convert the value to a string for printing. 
	UE_API FString ToString() const;

	// Remove any stored data and set to an invalid state.
	UE_API void Clear();

	static UE_API FMetasoundFrontendLiteral GetInvalid();

	friend bool operator==(const FMetasoundFrontendLiteral& A, const FMetasoundFrontendLiteral& B)
	{
		return A.IsEqual(B);
	}

private:
	// Remove all values.
	UE_API void Empty();
};

namespace Metasound::Frontend
{
	// Convenience function to convert Metasound::ELiteralType to EMetasoundFrontendLiteralType.
	UE_API EMetasoundFrontendLiteralType GetMetasoundFrontendLiteralType(ELiteralType InLiteralType);

	// Convenience function to convert EMetasoundFrontendLiteralType to Metasound::ELiteralType.
	UE_API ELiteralType GetMetasoundLiteralType(EMetasoundFrontendLiteralType InLiteralType);

	template<typename ... ArgTypes>
	FMetasoundFrontendLiteral MakeFrontendLiteral(ArgTypes&& ... InArgs)
	{
		FMetasoundFrontendLiteral Value;
		if constexpr (sizeof...(InArgs) == 0)
		{
			Value.Set(FMetasoundFrontendLiteral::FDefault{ });
		}
		else if constexpr (sizeof...(InArgs) == 1)
		{
			Value.Set(Forward<ArgTypes>(InArgs)...);
		}
		else
		{
			static_assert(sizeof...(InArgs) <= 1, "MakeFrontendLiteral: Invalid num arguments");
		}

		return Value;
	}

	template <typename ObjectType = UObject>
	FMetasoundFrontendLiteral MakeFrontendLiteral(TConstArrayView<TObjectPtr<ObjectType>> ObjectPtrs)
	{
		TArray<UObject*> Objects;
		auto ObjPtrToPtr = [](const TObjectPtr<ObjectType> Obj) { return Obj.Get(); };
		Algo::Transform(ObjectPtrs, Objects, ObjPtrToPtr);

		FMetasoundFrontendLiteral Value;
		Value.Set(MoveTemp(Objects));
		return Value;
	}
}

#undef UE_API
