// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PreprocessorHelpers.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API

// These macros are required in order to create module local registration actions.
#ifndef METASOUND_ENABLE_PER_MODULE_REGISTRATION
#if defined(METASOUND_MODULE) && defined(METASOUND_PLUGIN)
#define METASOUND_ENABLE_PER_MODULE_REGISTRATION 1
#else
#define METASOUND_ENABLE_PER_MODULE_REGISTRATION 0
#endif // if defined(METASOUND_MODULE) && defined(METASOUND_PLUGIN)
#endif // ifndef METASOUND_ENABLE_PER_MODULE_REGISTRATION

/** MetaSound Frontend Registration Actions
 *
 * MetaSound Frontend Registration Actions are used to register and unregister
 * MetaSound nodes and data types at the module level. 
 *
 * Usage:
 *
 * Node Developers can utilize these actions by implementing the following:
 * - Add private definitions to the module's Build.cs file for METASOUND_PLUGIN and METASOUND_MODULE
 *   	PrivateDefinitions.Add("METASOUND_PLUGIN=MyPlugin")
 *   	PrivateDefinitions.Add("METASOUND_MODULE=MyModule")
 *   These definitions must be private to the module. 
 *
 * - Implement a module registration group using the provided macros.
 *   In MyModule.cpp
 *   	METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
 *
 * - Use METASOUND_REGISTER_NODE(...) and METASOUND_REGISTER_NODE_AND_CONFIGURATION(...) 
 *   in the cpp files to setup registration actions for their nodes.
 *   
 * - Trigger registration in their module's StartupModule and ShutdownModule calls
 *   void FMyModule::StartupModule()
 *   {
 * 		METASOUND_REGISTER_ITEMS_IN_MODULE
 *   }
 *   void FMyModule::ShutdownModule()
 *   {
 * 		METASOUND_UNREGISTER_ITEMS_IN_MODULE
 *   }
 *
 *
 * Implementation:
 *
 * This system uses creates a linked list of FRegistrationActions that is constructed
 * during static initialization. Each module contains it's own linked list of FRegistrationActions
 * and thus only performs registration and unregistration actions associated with 
 * the module. This allows registration actions to be declared locally within the
 * cpp file containing the node or data type, while maintaining the association 
 * with the owning plugin and module.
 *
 * Each module's linked list contains a unique class name and does not export it's 
 * API. This is in order to protect against modules accidentally adding their registration
 * actions to the wrong linked list. The unique name for the module's list is created
 * by combining the METASOUND_PLUGIN and METASOUND_MODULE private macro definitions.
 * If those definitions are missing, a global linked list serves as a backup. 
 *
 * Thread safety for this system is provided by two mechanisms.
 * - Initialization of the registration actions linked list is thread safe by the
 *   fact that static initialization in C++ is single threaded. The list structure
 *   (e.g. Head & Next pointers) should not be modified outside of static initialization.
 *
 * - RegisterAll and UnregisterAll should only be called during StartupModule and 
 *   ShutdownModule because module startup and shutdown are also run single threaded. 
 */

namespace Metasound::Frontend
{
	// FModuleInfo captures information about the module which registers an item.
	struct FModuleInfo
	{
#if WITH_EDITORONLY_DATA
		FLazyName PluginName;
		FLazyName ModuleName;
#endif
	};

	namespace RegistrationPrivate
	{
		// Function signature of a registration function
		typedef bool (*FRegistrationFunction)(const FModuleInfo&);
		
		// Registration actions build a linked list during static initialization.
		struct FRegistrationAction
		{
			UE_API FRegistrationAction(FRegistrationAction*& InOutHead, FRegistrationFunction InRegister, FRegistrationFunction InUnregister);

#if WITH_EDITORONLY_DATA
		// Include debug name info in editoronly_data builds. 
		template<size_t N>
		FRegistrationAction(FRegistrationAction*& InOutHead, FRegistrationFunction InRegister, FRegistrationFunction InUnregister, const TCHAR(&NameLiteral)[N])
		: FRegistrationAction(InOutHead, InRegister, InUnregister)
		{
			Name = NameLiteral;
		}
#endif

		FRegistrationAction* Next = nullptr;
		FRegistrationFunction Register = nullptr;
		FRegistrationFunction Unregister = nullptr;

#if WITH_EDITORONLY_DATA
		const TCHAR* Name = nullptr; // Warning: Expected to be only used with string literals which provide static storage duration. 
#endif
	};

		// FRegistrationListBase implements common functionality used by all registration
		// list classes
		class FRegistrationListBase
		{
		public:
			// Calls "Register" all FRegistrationActions in the linked list beginning with the provided Head. 
			UE_API static void RegisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head);
			// Calls "Unregister" all FRegistrationActions in the linked list beginning with the provided Head. 
			UE_API static void UnregisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head);
		};

		// FGlobalRegistrationList is a fallback for when the module has not defined
		// the proper macros to create it's own registration list. This class should not
		// be used because it cannot appropriately execute unregister actions for a 
		// single plugin or module unloading.
		class FGlobalRegistrationList 
		{
		public: 
			UE_API static FModuleInfo ModuleInfo;

			/* If you've hit this deprecation warning it is because you are using
			 * a backwards compatible MetaSound node registration code path for your 
			 * module. In order to better support loading and unloading of MetaSound, 
			 * perform the following updates. This will remove this deprecation 
			 * warning.
			 *
			 * - Add private definitions to the module's Build.cs file for METASOUND_PLUGIN 
			 *   and METASOUND_MODULE
			 *   	PrivateDefinitions.Add("METASOUND_PLUGIN=MyPlugin")
			 *   	PrivateDefinitions.Add("METASOUND_MODULE=MyModule")
			 *   These definitions must be private to the module. 
			 *
			 * - Implement a module registration group using the provided macros.
			 *   In MyModule.cpp
			 *   	METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
			 *
			 * - Trigger registration in your module's StartupModule and ShutdownModule calls
			 *
			 *   void FMyModule::StartupModule()
			 *   {
			 * 		METASOUND_REGISTER_ITEMS_IN_MODULE
			 *   }
			 *   void FMyModule::ShutdownModule()
			 *   {
			 * 		METASOUND_UNREGISTER_ITEMS_IN_MODULE
			 *   }
			 *
			 */
#if METASOUND_ENABLE_PER_MODULE_REGISTRATION 
			UE_DEPRECATED(5.7, "Macro based registration has been updated. See MetasoundFrontendModuleRegistrationMacros.h for a guide on defining module level registration.")
			UE_API static FRegistrationAction*& GetHeadAction();
#else
			UE_API static FRegistrationAction*& GetHeadAction();
#endif

			// Used internally to do incremental registration on a global list. 
			static FRegistrationAction*& GetRegisterTailAction();

			UE_API static void RegisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head);
			UE_API static void UnregisterAll(const FModuleInfo& InOwningModuleInfo, FRegistrationAction* Head);
		};
	} // namespace RegistrationPrivate
} // namespace Metasound::Frontend



#if METASOUND_ENABLE_PER_MODULE_REGISTRATION

// By default, always unregister on module shutdown. The only time we do not unregister
// is when things cannot be associated with a module. This currently only happens
// when using the deprecated global registration list. 
#define METASOUND_UNREGISTER_ON_MODULE_SHUTDOWN 1

// Module registration list class name
#define METASOUND_MODULE_REGISTRATION_LIST_CLASS UE_JOIN(F, METASOUND_PLUGIN, _, METASOUND_MODULE, _Registration)

// Declare a module registration list. This needs to be visible wherever METASOUND_REGISTER_NODE
// is called. Generally this can be a header file within the module's Private directory. 
#define METASOUND_DEFINE_MODULE_REGISTRATION_LIST \
namespace Metasound::Frontend::RegistrationPrivate \
{\
	/* Methods in this class are purposely not exported because they should only be accessed within the module. */ \
	class METASOUND_MODULE_REGISTRATION_LIST_CLASS : public FRegistrationListBase \
	{\
		static_assert(sizeof(UE_STRINGIZE(METASOUND_PLUGIN)) > 1, "\"METASOUND_PLUGIN\" macro definition cannot be empty"); \
		static_assert(sizeof(UE_STRINGIZE(METASOUND_MODULE)) > 1, "\"METASOUND_MODULE\" macro definition cannot be empty"); \
		\
	public: \
		static const Metasound::Frontend::FModuleInfo ModuleInfo; \
		static FRegistrationAction*& GetHeadAction(); \
	};\
}


// Only allow module registration lists to be defined once.
#ifndef METASOUND_MODULE_REGISTRATION_LIST_DEFINED
#define METASOUND_MODULE_REGISTRATION_LIST_DEFINED
METASOUND_DEFINE_MODULE_REGISTRATION_LIST
#endif

// ModuleRegistrationInfo only exists WITH_EDITORONLY_DATA
#if WITH_EDITORONLY_DATA
#define METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST_INFO \
	const FModuleInfo METASOUND_MODULE_REGISTRATION_LIST_CLASS::ModuleInfo\
	{\
		UE_STRINGIZE(METASOUND_PLUGIN), \
		UE_STRINGIZE(METASOUND_MODULE) \
	};

#else
#define METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST_INFO const FModuleInfo METASOUND_MODULE_REGISTRATION_LIST_CLASS::ModuleInfo = {};
#endif // if WITH_EDITORONLY_DATA

// Implement a module registration list. This should be placed within a CPP file.
#define METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST \
namespace Metasound::Frontend::RegistrationPrivate \
{ \
	METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST_INFO \
	FRegistrationAction*& METASOUND_MODULE_REGISTRATION_LIST_CLASS::GetHeadAction() \
	{ \
		static FRegistrationAction* Head = nullptr;  \
		return Head; \
	} \
}

#else // if METASOUND_ENABLE_PER_MODULE_REGISTRATION

// If METASOUND_MODULE and METASOUND_PLUGIN are not both defined, we rely on the
// global registration list class as a fallback for ensuring the registration 
// actions still execute.
#define METASOUND_MODULE_REGISTRATION_LIST_CLASS FGlobalRegistrationList

// Disable unregistering on module shutdown because the actions are not associated with
// a module specific registration list. If this were enabled, one module would incidentally
// run unregister actions from another module.
#define METASOUND_UNREGISTER_ON_MODULE_SHUTDOWN 0

// The module registration lists cannot be defined or implemented. The global 
// registration list is used instead.
#define METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST

#endif

// The registration list class including global namespace access
#define METASOUND_MODULE_REGISTRATION_LIST_CLASS_NS ::Metasound::Frontend::RegistrationPrivate::METASOUND_MODULE_REGISTRATION_LIST_CLASS

// Retrieve the head action from the registration list class
#define METASOUND_GET_HEAD_REGISTRATION_ACTION METASOUND_MODULE_REGISTRATION_LIST_CLASS_NS::GetHeadAction()

// Retrieve the module info from the registration list class
#define METASOUND_GET_MODULE_LIST_INFO METASOUND_MODULE_REGISTRATION_LIST_CLASS_NS::ModuleInfo

// Register all nodes in the registration list class
#define METASOUND_REGISTER_ITEMS_IN_MODULE METASOUND_MODULE_REGISTRATION_LIST_CLASS_NS::RegisterAll(METASOUND_GET_MODULE_LIST_INFO, METASOUND_GET_HEAD_REGISTRATION_ACTION);

// Only run unregister actions if it is safe to do s.
#if METASOUND_UNREGISTER_ON_MODULE_SHUTDOWN
#define METASOUND_UNREGISTER_ITEMS_IN_MODULE METASOUND_MODULE_REGISTRATION_LIST_CLASS_NS::UnregisterAll(METASOUND_GET_MODULE_LIST_INFO, METASOUND_GET_HEAD_REGISTRATION_ACTION);
#else // if METASOUND_UNREGISTER_ON_MODULE_SHUTDOWN
#define METASOUND_UNREGISTER_ITEMS_IN_MODULE 
#endif

// Implement a registration action by declaring statically initialized FRegistrationAction
#if WITH_EDITORONLY_DATA

// With editoronly data includes additional debug logging info
#define METASOUND_IMPLEMENT_REGISTRATION_ACTION(UniqueName, RegisterFunction, UnregisterFunction) static ::Metasound::Frontend::RegistrationPrivate::FRegistrationAction UE_JOIN(RegistrationAction,UniqueName)(METASOUND_GET_HEAD_REGISTRATION_ACTION, RegisterFunction, UnregisterFunction, TEXT(UE_STRINGIZE(UniqueName))); 

#else

#define METASOUND_IMPLEMENT_REGISTRATION_ACTION(UniqueName, RegisterFunction, UnregisterFunction) static ::Metasound::Frontend::RegistrationPrivate::FRegistrationAction UE_JOIN(RegistrationAction,UniqueName)(METASOUND_GET_HEAD_REGISTRATION_ACTION, RegisterFunction, UnregisterFunction); 
#endif

#undef UE_API
