// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeMigration.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeConstructorParams.h"
#include "MetasoundVertex.h"
#include "Templates/Function.h"

#define UE_API METASOUNDFRONTEND_API

#ifndef UE_METASOUND_PURE_VIRTUAL_CREATE_FRONTEND_NODE_EXTENSION
#define UE_METASOUND_PURE_VIRTUAL_CREATE_FRONTEND_NODE_EXTENSION (0)
#endif

// Forward declare
class UObject;

namespace Metasound
{
	// Forward Declarations
	namespace Frontend
	{
		using FIterateMetasoundFrontendClassFunction = TFunctionRef<void(const FMetasoundFrontendClass&)>;
		class FBaseNodeUpdateTransform;
	}

	using FIterateMetasoundFrontendClassFunction UE_DEPRECATED(5.6, "Use TIterateMetasoundFrontendClassFunction defined in Metasound::Frontend namespace") = Frontend::FIterateMetasoundFrontendClassFunction;

	class FGraph;
} // namespace Metasound

namespace Metasound::Frontend
{
	/** INodeClassRegistryEntry declares the interface for a node registry entry.
		* Each node class in the registry must satisfy this interface. 
		*/
	class INodeClassRegistryEntry
	{
	public:
		virtual ~INodeClassRegistryEntry() = default;

		UE_DEPRECATED(5.8, "NodeClassInfo is actively being deprecated. Use FNodeClassRegistryKey instead (which can be generated from the frontend class)")
		virtual const FNodeClassInfo& GetClassInfo() const { return FNodeClassInfo::GetInvalid(); }

		/** Create a node given FNodeData. */
		virtual TUniquePtr<INode> CreateNode(FNodeData) const = 0;

		/** Return a FMetasoundFrontendClass which describes the node. */
		virtual const FMetasoundFrontendClass& GetFrontendClass() const = 0;

		/** Clone this registry entry. */
		UE_DEPRECATED(5.6, "Node Class Registration Entries do not need to be cloned.")
		virtual TUniquePtr<INodeClassRegistryEntry> Clone() const { return nullptr; }

		/** Returns set of implemented interface versions.
			*
			* Returns nullptr if node class implementation does not support interface implementation.
			*/
		virtual const TSet<FMetasoundFrontendVersion>* GetImplementedInterfaces() const = 0;

		virtual FVertexInterface GetDefaultVertexInterface() const = 0;

		/** Return the FClassInterface for this node class, which includes
		 * sub-interface and variant schema information. */
		virtual const FClassInterface& GetClassInterface() const = 0;

		UE_DEPRECATED(5.6, "Node class registry no longer tracks nature of implementation (i.e. if was registered "
			"from an asset or c++. Use the MetaSoundAssetManager to determine if the class has been defined within an asset(s).")
		virtual bool IsNative() const { return false; }

		/** Optionally create the node extension associated with the node. */
#if UE_METASOUND_PURE_VIRTUAL_CREATE_FRONTEND_NODE_EXTENSION
		virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const = 0;
#else
		UE_API virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const;
#endif
		/** Return whether this entry's node configuration is compatible with the given node configuration. */
		virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const = 0;

#if WITH_EDITORONLY_DATA
		/** Name of the plugin registering this node. */
		UE_API virtual FName GetPluginName() const;
		
		/** Name of the module registering this node. */
		UE_API virtual FName GetModuleName() const;
#endif
	};

	using INodeRegistryEntry UE_DEPRECATED(5.6, "Use INodeClassRegistryEntry instead") = INodeClassRegistryEntry;

	struct FConverterNodeClassRegistryKey
	{
		// The datatype one would like to convert from.
		FName FromDataType;

		// The datatype one would like to convert to.
		FName ToDataType;

		inline bool operator==(const FConverterNodeClassRegistryKey& Other) const
		{
			return FromDataType == Other.FromDataType && ToDataType == Other.ToDataType;
		}

		friend uint32 GetTypeHash(const FConverterNodeClassRegistryKey& InKey)
		{
			return HashCombine(GetTypeHash(InKey.FromDataType), GetTypeHash(InKey.ToDataType));
		}
	};
	using FConverterNodeRegistryKey = FConverterNodeClassRegistryKey;

	struct FConverterNodeClassInfo
	{
		// If this node has multiple input pins, we use this to designate which pin should be used.
		FVertexName PreferredConverterInputPin;

		// If this node has multiple output pins, we use this to designate which pin should be used.
		FVertexName PreferredConverterOutputPin;

		// The key for this node in the node registry.
		FNodeClassRegistryKey NodeKey;

		inline bool operator==(const FConverterNodeClassInfo& Other) const
		{
			return NodeKey == Other.NodeKey;
		}
	};
	using FConverterNodeInfo = FConverterNodeClassInfo;

	struct FConverterNodeClassRegistryValue
	{
		// A list of nodes that can perform a conversion between the two datatypes described in the FConverterNodeClassRegistryKey for this map element.
		TArray<FConverterNodeClassInfo> PotentialConverterNodes;
	};
	using FConverterNodeClassRegistryValue = FConverterNodeClassRegistryValue;

	using FRegistryTransactionID = int32;

	class FNodeClassRegistryTransaction 
	{
	public:
		using FTimeType = uint64;

		/** Describes the type of transaction. */
		enum class ETransactionType : uint8
		{
			NodeRegistration,     //< Something was added to the registry.
			NodeUnregistration,  //< Something was removed from the registry.
			NodeMigrationRegistration, //< Something migrated.
			NodeMigrationUnregistration, //< Module doing migration is unloaded.
			Invalid
		};

		static FString LexToString(ETransactionType InTransactionType)
		{
			using namespace Metasound;

			switch (InTransactionType)
			{
				case ETransactionType::NodeRegistration:
				{
					return TEXT("Node Registration");
				}

				case ETransactionType::NodeUnregistration:
				{
					return TEXT("Node Unregistration");
				}

				case ETransactionType::NodeMigrationRegistration:
				{
					return TEXT("Node Migration Registration");
				}

				case ETransactionType::NodeMigrationUnregistration:
				{
					return TEXT("Node Migration Unregistration");
				}

				case ETransactionType::Invalid:
				{
					return TEXT("Invalid");
				}

				default:
				{
					checkNoEntry();
					return TEXT("");
				}
			}
		}

		UE_DEPRECATED(5.8, "Use constructor providing RegistryKey instead as NodeClassInfo is actively being deprecated")
		UE_API FNodeClassRegistryTransaction(ETransactionType InType, const FNodeClassInfo& InNodeClassInfo, FTimeType InTimestamp);

		UE_API FNodeClassRegistryTransaction(ETransactionType InType, FNodeClassRegistryKey InRegistryKey, FTimeType InTimestamp);

		UE_API ETransactionType GetTransactionType() const;

		UE_DEPRECATED(5.8, "NodeClassInfo is actively being deprecated. Use NodeRegistryKey instead (and SearchEngine to enquire about node class data via newer FMetaSoundClassInfo struct)")
		UE_API const FNodeClassInfo& GetNodeClassInfo() const;

		UE_API FNodeClassRegistryKey GetNodeRegistryKey() const;
		UE_API FTimeType GetTimestamp() const;

	private:
		ETransactionType Type;
		FNodeClassRegistryKey RegistryKey;
		FTimeType Timestamp;
	};

	using FNodeRegistryTransaction UE_DEPRECATED(5.8, "FNodeRegistryTransaction is now FNodeClassRegistryTransaction") = FNodeClassRegistryTransaction;

	namespace NodeClassRegistryKey
	{
		// Returns true if the class metadata represent the same entry in the node registry.
		METASOUNDFRONTEND_API bool IsEqual(const FMetasoundFrontendClassMetadata& InLHS, const FMetasoundFrontendClassMetadata& InRHS);

		// Returns true if the class info and class metadata represent the same entry in the node registry.
		METASOUNDFRONTEND_API bool IsEqual(const FNodeClassInfo& InLHS, const FMetasoundFrontendClassMetadata& InRHS);
	}
	namespace NodeRegistryKey = NodeClassRegistryKey;

	/**
	 * Singleton registry for all types and nodes.
	 */
	class INodeClassRegistry
	{

	public:
		// The MetaSound frontend does not rely on the Engine module and therefore
		// does not have the ability to provide GC protection. This interface allows
		// external modules to provide GC protection so that async tasks can safely
		// use UObjects
		class IObjectReferencer
		{
		public:
			virtual ~IObjectReferencer() = default;

			// Called when an object should be referenced.
			virtual void AddObject(UObject* InObject) = 0;

			// Called when an object no longer needs to be referenced. 
			virtual void RemoveObject(UObject* InObject) = 0;
		};


		static UE_API INodeClassRegistry* Get();
		static UE_API INodeClassRegistry& GetChecked();

		UE_DEPRECATED(5.6, "This function is no longer supported")
		static UE_API void ShutdownMetasoundFrontend();


		static UE_API bool GetFrontendClassFromRegistered(const FNodeClassRegistryKey& InKey, FMetasoundFrontendClass& OutClass);

		static UE_API bool GetInputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey);
		static UE_API bool GetVariableNodeRegistryKeyForDataType(const FName& InDataTypeName, FNodeClassRegistryKey& OutKey);
		static UE_API bool GetOutputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey);

	protected:
		INodeClassRegistry() = default;
	public:
		virtual ~INodeClassRegistry() = default;

		INodeClassRegistry(const INodeClassRegistry&) = delete;
		INodeClassRegistry& operator=(const INodeClassRegistry&) = delete;

		// Enqueue and init command for registering a node or data type.
		// The command queue will be processed on module init or when calling `RegisterPendingNodes()`
		UE_DEPRECATED(5.7, "Enqueuing init commands is no longer support. See MetasoundFrontendModuleRegistrationMacros.h for information on queuing up registeration actions.")
		virtual bool EnqueueInitCommand(TUniqueFunction<void()>&& InFunc) = 0;

		virtual void SetObjectReferencer(TUniquePtr<IObjectReferencer> InReferencer) = 0;

		// This is called on module startup. This invokes any registration commands enqueued by our registration macros.
		UE_DEPRECATED(5.7, "Replace with METASOUND_REGISTER_ITEMS_IN_MODULE. See MetasoundFrontendModuleRegistrationMacros.h for more info on triggering registration of nodes and data types.")
		virtual void RegisterPendingNodes() = 0;

		// Wait for async graph registration to complete for a specific graph
		virtual void WaitForAsyncGraphRegistration(const FGraphRegistryKey& InRegistryKey) const = 0;

		// Retrieve a registered graph. 
		//
		// If the graph is registered asynchronously, this will wait until the registration task has completed.
		virtual TSharedPtr<const Metasound::FGraph> GetGraph(const FGraphRegistryKey& InRegistryKey) const = 0;

		/** Register an external node with the frontend.
		 *
		 * @param InCreateNode - Function for creating node from FNodeInitData.
		 * @param InCreateDescription - Function for creating a FMetasoundFrontendClass.
		 *
		 * @return A node registration key. If the registration failed, then the registry 
		 *         key will be invalid.
		 */
		virtual FNodeClassRegistryKey RegisterNode(TUniquePtr<INodeClassRegistryEntry>&& InEntry) = 0;

		/** Unregister an external node from the frontend.
		 *
		 * @param InKey - The registration key for the node.
		 *
		 * @return True on success, false on failure.
		 */
		virtual bool UnregisterNode(const FNodeClassRegistryKey& InKey) = 0;

#if WITH_EDITORONLY_DATA
		/** Register a node migration from one plugin and/or module to another. */
		virtual bool RegisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo) = 0;
		
		/** Unregister a node migration from one plugin and/or module to another. This should be used during module shutdown. */
		virtual bool UnregisterNodeMigration(const FNodeMigrationInfo& InMigrationInfo) = 0;

		/** Name of the plugin registering this node. */
		virtual FName GetOwningPluginName(const FNodeClassRegistryKey& InKey) const = 0;
		
		/** Name of the module registering this node. */
		virtual FName GetOwningModuleName(const FNodeClassRegistryKey& InKey) const = 0;
#endif

		/** Returns true if the provided registry key corresponds to a valid registered node. */
		virtual bool IsNodeRegistered(const FNodeClassRegistryKey& InKey) const = 0;

		/** Returns true if the provided registry key (node key and asset path) corresponds to a valid registered graph. */
		virtual bool IsGraphRegistered(const FGraphRegistryKey& InKey) const = 0;

		UE_DEPRECATED(5.6, "Node class registry no longer tracks donor asset state/implementation at the interface "
			"level. Use the MetaSoundAssetManager to determine if the class has been defined within an asset(s).")
		virtual bool IsNodeNative(const FNodeClassRegistryKey& InKey) const { return false; }

		// Iterates class types in registry.  If InClassType is set to a valid class type (optional), only iterates classes of the given type
		UE_DEPRECATED(5.8, "This will be used internally only due to thread safety issues. Use Metasound::Frontend::ISearchEngine instead.")
		virtual void IterateRegistry(FIterateMetasoundFrontendClassFunction InIterFunc, EMetasoundFrontendClassType InClassType = EMetasoundFrontendClassType::Invalid) const = 0;

		// Query for MetaSound Frontend document objects.

		// Get the default vertex interface for the node class entry for the given key, 
		// returning true on success, false otherwise 
		virtual bool FindDefaultVertexInterface(const FNodeClassRegistryKey& InKey, FVertexInterface& OutVertexInterface) const = 0;

		/** Get the FClassInterface (with sub-interface/variant schema) for the given key.
		 * Returns true on success, false if the key is not found. */
		virtual bool FindClassInterface(const FNodeClassRegistryKey& InKey, FClassInterface& OutClassInterface) const = 0;

		virtual bool FindFrontendClassFromRegistered(const FNodeClassRegistryKey& InKey, FMetasoundFrontendClass& OutClass) const = 0;

		/** Returns true if the provided node configuration is compatible (same derived class) with the registered configuration for the node represented by the registry key. */
		virtual bool IsCompatibleNodeConfiguration(const FNodeClassRegistryKey& InKey, TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const = 0;

		/** Return the node extension associated with the node. If there is no extension associated with the node, the returned TInstancedStruct will be invalid. */
		virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration(const FNodeClassRegistryKey& InKey) const = 0;

		virtual bool FindImplementedInterfacesFromRegistered(const FNodeClassRegistryKey& InKey, TSet<FMetasoundFrontendVersion>& OutInterfaceVersions) const = 0;
		
		virtual bool FindInputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey) = 0;
		virtual bool FindVariableNodeRegistryKeyForDataType(const FName& InDataTypeName, FNodeClassRegistryKey& OutKey) = 0;
		virtual bool FindOutputNodeRegistryKeyForDataType(const FName& InDataTypeName, const EMetasoundFrontendVertexAccessType InAccessType, FNodeClassRegistryKey& OutKey) = 0;

		/** Create a MetaSound Node with the given registration key and Node Data 
		 * 
		 * Returns an invalid TUniquePtr<> if the key is not in the registry. 
		 */
		virtual TUniquePtr<Metasound::INode> CreateNode(const FNodeClassRegistryKey& InKey, Metasound::FNodeData) const = 0;

		/** Create a MetaSound Node with the given registration key and Init Data.  
		 * The node will be created with a default interface.
		 * 
		 * Returns an invalid TUniquePtr<> if the key is not in the registry. 
		 */
		/** Registers a node as a converter node */
		virtual bool RegisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FConverterNodeClassInfo& InNodeInfo) = 0;
		
		/** Unregisters a node as a converter node */
		virtual bool UnregisterConversionNode(const FConverterNodeClassRegistryKey& InConversionKey, const FNodeClassRegistryKey& InNodeKey) = 0;

		// Returns a list of possible nodes to use to convert from FromDataType to ToDataType.
		// Returns an empty array if none are available.
		virtual TArray<FConverterNodeClassInfo> GetPossibleConverterNodes(const FName& FromDataType, const FName& ToDataType) = 0;

#if WITH_EDITORONLY_DATA
		// Node Update Transforms
		// Register a node update transform associated with a registered node class and major version.
		// Autoapplied node update transforms will be applied after all minor version updates for the node's major version have been applied during autoupdate 
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		UE_EXPERIMENTAL(5.7, "Node update transforms are experimental")
		virtual void RegisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion, const TSharedRef<const FBaseNodeUpdateTransform>& InTransform) = 0;
	
		// Unregister the custom node update transform for this class name and version
		UE_EXPERIMENTAL(5.7, "Node update transforms are experimental")
		virtual bool UnregisterCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion) = 0;

		UE_EXPERIMENTAL(5.7, "Node update transforms are experimental")
		virtual const TSharedPtr<const FBaseNodeUpdateTransform> FindCustomNodeUpdateTransform(const FMetasoundFrontendClassName& ClassName, int32 MajorVersion) const = 0;
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
#endif // WITH_EDITORONLY_DATA
	};
} // namespace Metasound::Frontend

#undef UE_API
