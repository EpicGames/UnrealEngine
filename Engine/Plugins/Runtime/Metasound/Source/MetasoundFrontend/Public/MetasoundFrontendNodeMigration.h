// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"
#include "MetasoundNodeInterface.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
#if WITH_EDITORONLY_DATA
	struct FNodeMigrationInfo
	{
		// UE Version when migration occurred (e.g. "5.6")
		FName UEVersion;

		// ClassName of node which was migrated
		FNodeClassName ClassName;

		// Major version of node when migration happened
		int32 MajorVersion = -1;

		// Minor version of node when migration happened
		int32 MinorVersion = -1;

		// Prior plugin and module where node was registered. 
		FName FromPlugin;
		FName FromModule;

		// New plugin and module where node is registered. 
		FName ToPlugin;
		FName ToModule;

		UE_API FString ToString() const;
		friend bool operator==(const FNodeMigrationInfo& InLHS, const FNodeMigrationInfo& InRHS);
	};
#endif // if WITH_EDITORONLY_DATA

} // namespace Metasound::Frontend
#undef UE_API
