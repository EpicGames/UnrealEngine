// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundFrontendTemplateNodeConfiguration.h"
#include "MetasoundVertex.h"
#include "Templates/UniquePtr.h"
#include "UObject/NoExportTypes.h"

#include "MetasoundFrontendNodeTemplateRegistry.generated.h"

#define UE_API METASOUNDFRONTEND_API

// Forward Declarations
class IMetaSoundDocumentInterface;
struct FMetaSoundFrontendDocumentBuilder;

namespace Metasound
{
	namespace DynamicGraph
	{
		class FDynamicOperatorTransactor;
	} // DynamicGraph
} // namespace Metasound


USTRUCT()
struct FNodeTemplateGenerateInterfaceParams
{
	GENERATED_BODY()

	// TODO: Currently just DataType FName. Subsequent change will replace this with a VertexHandle
	// and will add builder reference to once builder supports template nodes and controllers
	// are no longer used to add template nodes from editor code.
	UPROPERTY()
	TArray<FName> InputsToConnect;
	
	UPROPERTY()
	TArray<FName> OutputsToConnect;
};

namespace Metasound::Frontend
{
	class INodeTransform;

#if WITH_EDITOR
	class INodeController;
	class IInputController;
	class IOutputController;

	using FConstNodeHandle = TSharedRef<const INodeController>;

	struct FTemplateNodeTransactionContext
	{
		// Builder executing the transaction
		const FMetaSoundFrontendDocumentBuilder& Builder;

		// PageID of node being acted on
		const FGuid& PageID;

		// Node being transacted on
		const FMetasoundFrontendNode* Node = nullptr;
	};

	struct FTemplateNodeEdgeTransactionContext
	{
		// Builder executing the transaction
		const FMetaSoundFrontendDocumentBuilder& Builder;

		// PageID of node being acted on
		const FGuid& PageID;

		// From node being transacted on
		const FMetasoundFrontendNode* FromNode = nullptr;

		// Node Output being transacted on
		const FMetasoundFrontendVertex* FromOutput = nullptr;

		// To Node being transacted on
		const FMetasoundFrontendNode* ToNode = nullptr;

		// Node Input being transacted on
		const FMetasoundFrontendVertex* ToInput = nullptr;
	};

	// Removal requires re-instating an input default, which must be selected based
	// on page context.  Selection of page context is engine-level behavior.  Thus,
	// a function ref is provided for default selection should the node template be
	// providing a graph topology reduction that requires look-up of a separate node's
	// input on intercepting the given edge removal transaction.
	struct FTemplateNodeEdgeRemoveTransactionContext : public FTemplateNodeEdgeTransactionContext
	{
		using FResolveActiveInputDefaultFunctionRef = TFunctionRef<const FMetasoundFrontendLiteral*(
			const FGuid& /* NodeID */,
			const FName /* VertexName */
			)>;

		// Reference to function used to determine preferred page input default for a given
		// node's vertex. If default is set directly on paged node, uses that value.
		// Otherwise, uses the default provided by the node's class interface.
		const FResolveActiveInputDefaultFunctionRef ResolveActiveNodeInputDefault;
	};
#endif // WITH_EDITOR

	class INodeTemplateTransform
	{
	public:
		virtual ~INodeTemplateTransform() = default;

		/** Return true if the builder was modified, false otherwise. */
		virtual bool Transform(const FGuid& InPageID, const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const = 0;
	};

	/**
		* Base interface for a node template, which acts in place of frontend node class and respective instance(s).
		* Instances are preprocessed, allowing for custom graph manipulation prior to generating a respective runtime
		* graph operator representation.
		*/
	class INodeTemplate
	{
	public:
		virtual FMetasoundFrontendNodeInterface GenerateNodeInterface(FNodeTemplateGenerateInterfaceParams InParams) const = 0;

		virtual ~INodeTemplate() = default;

		virtual const TArray<FMetasoundFrontendClassInputDefault>* FindNodeClassInputDefaults(
			const FMetaSoundFrontendDocumentBuilder& InBuilder,
			const FGuid& InPageID,
			const FGuid& InNodeID,
			FName VertexName) const = 0;

		// Returns note template class name.
		virtual const FMetasoundFrontendClassName& GetClassName() const = 0;

		// Create a node configuration for the node template.
		virtual TInstancedStruct<FMetaSoundFrontendTemplateNodeConfiguration> CreateFrontendTemplateNodeConfiguration() const = 0;

#if WITH_EDITOR
		virtual FText GetNodeDisplayName(const IMetaSoundDocumentInterface& Interface, const FGuid& InPageID, const FGuid& InNodeID) const = 0;

		virtual FText GetInputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName InputName) const = 0;

		virtual FText GetOutputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName OutputName) const = 0;

		static UE_API FText ResolveMemberDisplayName(FName VertexName, FText DisplayName, bool bIncludeNamespace);
#endif // WITH_EDITOR

		// Generates node transform that is used to preprocess nodes.
		virtual TUniquePtr<INodeTemplateTransform> GenerateNodeTransform() const = 0;

		// Returns the class definition for the given node class template.
		virtual const FMetasoundFrontendClass& GetFrontendClass() const = 0;

		// Returns access type of the given input within the provided builder's document
		virtual EMetasoundFrontendVertexAccessType GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		// Returns access type of the given output within the provided builder's document
		virtual EMetasoundFrontendVertexAccessType GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const = 0;

		// Returns note template class version.
		virtual const FMetasoundFrontendVersionNumber& GetVersionNumber() const = 0;

#if WITH_EDITOR
		// Returns whether or not the given node template has the necessary
		// required connections to be preprocessed (editor only).
		virtual bool HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage = nullptr) const = 0;
#endif // WITH_EDITOR

		// Returns whether template can dynamically assign a node's input access type (as opposed to it being assigned on the class input definition)
		virtual bool IsInputAccessTypeDynamic() const = 0;

		// Whether or not input connections are user modifiable
		virtual bool IsInputConnectionUserModifiable() const = 0;

		// Returns whether template can dynamically assign a node's output's access type (as opposed to it being assigned on the class output definition)
		virtual bool IsOutputAccessTypeDynamic() const = 0;

		// Whether or not output connections are user modifiable
		virtual bool IsOutputConnectionUserModifiable() const = 0;

		// Given the provided node interface, returns whether or not it conforms to an expected format
		// that can be successfully manipulated by a generated node template transform.
		virtual bool IsValidNodeInterface(const FMetasoundFrontendNodeInterface& InNodeInterface) const = 0;

#if WITH_EDITOR
		// Provided a Frontend transaction context, apply adding edge transaction to the given dynamic core transactor.
		// Returns whether or not Frontend operation was handled and applied to the given transactor.
		virtual bool OnEdgeAdded(const FTemplateNodeEdgeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const = 0;

		// Provided a Frontend transaction context, apply adding node transaction to the given dynamic core transactor.
		// Returns whether or not Frontend operation was handled and applied to the given transactor.
		virtual bool OnNodeAdded(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const = 0;

		// Provided a Frontend transaction context, apply updating node config transaction to the given dynamic core transactor.
		// Returns whether or not Frontend operation was handled and applied to the given transactor.
		virtual bool OnNodeConfigurationUpdated(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const = 0;

		// Provided a Frontend transaction context, apply removing edge transaction to the given dynamic core transactor.
		// Returns whether or not Frontend operation was handled and applied to the given transactor.
		virtual bool OnRemoveSwappingEdge(const FTemplateNodeEdgeRemoveTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const = 0;

		// Provided a Frontend transaction context, apply removing node transaction to the given dynamic core transactor.
		// Returns whether or not Frontend operation was handled and applied to the given transactor.
		virtual bool OnRemoveSwappingNode(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const = 0;
#endif // WITH_EDITOR
	};

	class INodeTemplateRegistry
	{
	public:
		// Returns singleton template registry.
		static UE_API INodeTemplateRegistry& Get();

		virtual ~INodeTemplateRegistry() = default;

		// Find a template with the given key. Returns null if entry not found with given key.
		virtual const INodeTemplate* FindTemplate(const FNodeRegistryKey& InKey) const = 0;

		// Find a template with the given class name with the highest version. Returns null if entry not found with given name.
		virtual const INodeTemplate* FindTemplate(const FMetasoundFrontendClassName& InClassName) const = 0;
	};

	class FNodeTemplateBase : public INodeTemplate
	{
	public:
		UE_API virtual const TArray<FMetasoundFrontendClassInputDefault>* FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const override;
		UE_API virtual TInstancedStruct<FMetaSoundFrontendTemplateNodeConfiguration> CreateFrontendTemplateNodeConfiguration() const override;

#if WITH_EDITOR
		UE_API virtual FText GetNodeDisplayName(const IMetaSoundDocumentInterface& Interface, const FGuid& InPageID, const FGuid& InNodeID) const override;
		UE_API virtual FText GetInputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName InputName) const override;
		UE_API virtual FText GetOutputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName OutputName) const override;
		UE_API virtual bool HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage = nullptr) const override;

		UE_API virtual bool OnEdgeAdded(const FTemplateNodeEdgeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnNodeAdded(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnNodeConfigurationUpdated(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnRemoveSwappingEdge(const FTemplateNodeEdgeRemoveTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnRemoveSwappingNode(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
#endif // WITH_EDITOR
	};

	// Register/UnregisterNodeTemplate are limited to internal implementation to avoid document corruption. Node Config is
	// likely the better option for most class interface "switching" applications (templates are reserved for cook-only
	// applications/optimizations with more advanced node instance interface manipulation).
	void RegisterNodeTemplate(TUniquePtr<INodeTemplate>&& InTemplate);
	void UnregisterNodeTemplate(const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InTemplateVersion);
} // namespace Metasound::Frontend

#undef UE_API
