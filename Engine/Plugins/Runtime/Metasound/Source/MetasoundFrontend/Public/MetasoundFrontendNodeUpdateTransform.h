// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundFrontendRegistryKey.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API

struct FMetaSoundFrontendDocumentBuilder;
namespace Metasound::Frontend
{
	using FVertexNameAndType = TTuple<FName, FName>;
	namespace NodeReplacement
	{
		struct UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FInputConnectionInfo
		{
			// Handle of connected output vertex
			FMetasoundFrontendVertexHandle ConnectedOutput;
			// Name and data type of node input vertex
			FName Name;
			FName DataType;
			FMetasoundFrontendLiteral DefaultValue;
			bool bLiteralSet = false;
		};
        
		struct UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FOutputConnectionInfo
		{
			// Handles of connected input vertices
			TArray<FMetasoundFrontendVertexHandle> ConnectedInputs;
			// Name and data type of node output vertex
			FName Name;
			FName DataType;
		};
        
		// Map of input name/type to input info
		using FInputConnections = TMap<FVertexNameAndType, FInputConnectionInfo>;
		// Map of output name/type to connected inputs
		using FOutputConnections = TMap<FVertexNameAndType, FOutputConnectionInfo>;
		
		struct UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FNodeInstanceReplacementData
		{
#if WITH_EDITOR
			FMetasoundFrontendNodeStyle Style;
#endif // WITH_EDITOR
        
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Configuration;
			TInstancedStruct<FMetasoundFrontendClassInterface> ClassInterfaceOverride;
        		
			FInputConnections InputConnections;
			FOutputConnections OutputConnections;
		};
	}
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS

	// Cached data for performing a node update (ex. old node connections that need to be reapplied after replacing it) 
	struct UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FNodeUpdateTransformData 
	{
		NodeReplacement::FNodeInstanceReplacementData NodeReplacementData;
	};
	
	class UE_EXPERIMENTAL(5.7, "Node update transforms are experimental") INodeUpdateTransform
	{
#if WITH_EDITORONLY_DATA
	public:
		virtual ~INodeUpdateTransform() = default;

		// The node class registry key of the new node that the old node will be transformed into
		// Node update transforms are many to 1 or 1 to 1 node replacements
		virtual FNodeClassRegistryKey GetNewNodeClassKey() const = 0;

		// Whether the transform should be automatically applied to node class keys registered with this transform during autoupdate
		virtual bool ShouldAutoApply() const = 0;

		// Perform the internal node replacement update on a given node
		// Not exposed directly to users, see subclasses for the data that users can customize
		UE_INTERNAL virtual void Update(FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid InNodeID, const FGuid* InPageID = nullptr) const = 0;
#endif // if WITH_EDITORONLY_DATA
	};

	// Base update transform that replaces a node with another node of the given target node class key,
	// maintaining connections and data if possible. Inherit from this to make a custom node update transform. 
	class UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FBaseNodeUpdateTransform : public INodeUpdateTransform
	{
	public:
#if WITH_EDITORONLY_DATA
		UE_API FBaseNodeUpdateTransform(const FNodeClassRegistryKey& InNewNodeClassKey);

		UE_API virtual FNodeClassRegistryKey GetNewNodeClassKey() const override;
		
		UE_API virtual bool ShouldAutoApply() const override;
		
		UE_INTERNAL UE_API virtual void Update(FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid InNodeID, const FGuid* InPageID) const final override;

		// Data (ex. connections, defaults, node configuration) to be reapplied to the new node after it has been updated
		// This represents the data we want applied to the new node (ex. new node configuration), which is not necessarily just what we currently have (though in simplest case those are the same)
		UE_API virtual FNodeUpdateTransformData CreateNodeUpdateTransformData(const FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid& InNodeID, const FGuid* InPageID) const;

	private:
		FNodeClassRegistryKey NewNodeClassKey;
#endif // if WITH_EDITORONLY_DATA
	};

	// An example node update transform for maintaining a node's specified default values,
	// when the next version has a new default value and would otherwise stomp currently unoverwritten values.
	// If a user has overwritten the value, it will stay as that overwritten value.
	// Ex. A filter bandwidth default is changed in v2.0, so use this transform to keep old content at the v1.0 default
	// if the default was not set on the individual nodes, while new content will use the new v2.0 default
	class UE_EXPERIMENTAL(5.8, "Node update transforms are experimental") FMaintainDefaultsNodeUpdateTransform : public FBaseNodeUpdateTransform
	{
	public:
#if WITH_EDITORONLY_DATA
		UE_API FMaintainDefaultsNodeUpdateTransform(const FNodeClassRegistryKey& InTargetNodeClassKey);

		UE_API virtual FNodeUpdateTransformData CreateNodeUpdateTransformData(const FMetaSoundFrontendDocumentBuilder& InOutBuilder, const FGuid& InNodeID, const FGuid* InPageID) const override;
		UE_API virtual bool ShouldAutoApply() const override;
#endif // if WITH_EDITORONLY_DATA
	};
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
} // namespace Metasound::Frontend
#undef UE_API
