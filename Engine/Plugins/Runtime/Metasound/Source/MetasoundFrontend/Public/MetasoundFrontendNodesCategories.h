// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Metasound
{
	namespace NodeCategories
	{
		extern const FText METASOUNDFRONTEND_API Conversions;
		extern const FText METASOUNDFRONTEND_API EnumConversions;
		extern const FText METASOUNDFRONTEND_API Functions;
		extern const FText METASOUNDFRONTEND_API Graphs;
		extern const FText METASOUNDFRONTEND_API Inputs;
		extern const FText METASOUNDFRONTEND_API Outputs;
		extern const FText METASOUNDFRONTEND_API Variables;
		extern const FText METASOUNDFRONTEND_API Favorites;
		extern const FText METASOUNDFRONTEND_API Recents;
	} // namespace NodeCategories
} // namespace Metasound
