// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"

#define UE_API METASOUNDFRONTEND_API

// Forward declare
struct FGuid;
struct FMetaSoundFrontendDocumentBuilder;

namespace Metasound::Frontend
{
#if WITH_EDITORONLY_DATA
	/** Remove page data which will not be use given the supplied target pages and page order.
	 *
	 * @return true if the builder removed any pages. 
	 */
	UE_API bool StripUnusedPages(FMetaSoundFrontendDocumentBuilder& InBuilder, TArrayView<const FGuid> InTargetPages, TArrayView<const FGuid> InPageOrder);
#endif // WITH_EDITORONLY_DATA

	/** Return a pointer to the most preferred element for a given page order. 
	 *
	 * PageIDs with lower indices in InPageOrder are preferred over those with
	 * higher indices. If none of the elements have a PageID which exists in the 
	 * InPageOrder, then a pointer to the first element is returned. If the InElements array is
	 * empty, then nullptr is returned. 
	 */
	template<typename ElementType>
	const ElementType* FindPreferredPage(const TArray<ElementType>& InElements, TArrayView<const FGuid> InPageOrder)
	{
		const ElementType* Target = nullptr;
		// Handle common case of only a single page existing
		if (InElements.Num() == 1)
		{
			Target = &InElements[0];
		}
		else if (InElements.Num() > 0)
		{
			// Handle multiple pages. InPageOrder represents the sorted page resolution
			// in order of preference. We find the most preferred page.
			int32 BestPos = InPageOrder.Num();
			for (const ElementType& Element : InElements)
			{
				int32 Pos = InPageOrder.Find(Element.PageID);
				if ((Pos != INDEX_NONE) && (Pos < BestPos))
				{
					BestPos = Pos;
					Target = &Element;
					if (Pos == 0)
					{
						// The best page has been found. 
						break;
					}
				}
			}

			// Fallback to any available page if none were found
			if (nullptr == Target)
			{
				Target = &InElements[0];
			}
		}

		return Target;
	}
}

#undef UE_API 
