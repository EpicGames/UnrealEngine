// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "Math/NumericLimits.h"
#include "MetasoundAssetKey.h"
#include "MetasoundAssetManagerTransaction.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendRegistries.h"
#include "Misc/TVariant.h"

#include "MetasoundFrontendQuery.generated.h"

#define UE_API METASOUNDFRONTEND_API


/** MetaSound Frontend Query
 *
 * MetaSound Frontend Query provides a way to systematically organize and update
 * streaming data associated with the MetaSound Frontend. It is a streaming MapReduce 
 * framework for querying streams of data (https://en.wikipedia.org/wiki/MapReduce) 
 *
 * While it does not support the computational parallelism commonly found in MapReduce 
 * frameworks, it does offer:
 * 	- An encapsulated and reusable set of methods for manipulating streamed data.
 *  - Support for incremental updates (a.k.a. streamed data).
 * 	- An indexed output for efficient lookup. 
 *
 * Data within MetaSound Frontend Query is organized similarly to a NoSQL database
 * (https://en.wikipedia.org/wiki/NoSQL). Each object (FFrontendQueryEntry) is 
 * assigned a unique ID. Keys (FFrontendQueryKey) are associated with sets of entries
 * (FFrontendQueryPartition) and allow partitions to be retrieved efficiently. 
 * Each partition holds a set of entries which is determined by the steps in the 
 * query (FFrontendQuery). A FFrontendQueryKey or FFrontendQueryValue represent
 * one of multiple types by using a TVariant<>.
 *
 * A query contains a sequence of steps that get executed on streaming data. The 
 * various types of steps reflect common operations performed in MapReduce and 
 * NoSQL database queries. 
 *
 * Step Types
 * 	Stream: 	Produce a stream of FFrontendQueryValues.
 * 	Map:		Map a FFrontendQueryEntry to a partition associated with a FFrontendQueryKey.
 * 	Reduce: 	Apply an incremental summarization of a FFrontendQueryPartition.
 * 	Transform:	Alter a FFrontendQueryValue.
 * 	Filter:		Remove FFrontendQueryValues with a test function.
 * 	Score:		Calculate a score for a FFrontendQueryValue.
 * 	Sort:		Sort a FFrontendQueryPartition.
 * 	Limit:		Limit the size of a FFrontendQueryPartition.
 */


// Forward Declarations
class FAssetRegistryTagsContext;
class IMetaSoundDocumentInterface;
struct FAssetData;


// Condensed set of class vertex data that is serialized to editor-only asset
// tag data, allowing editor scripts and code to query MetaSounds without loading
// them in entirety.
USTRUCT(BlueprintType)
struct FMetaSoundClassVertexInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	FName Name;

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	FName TypeName;

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	EMetasoundFrontendVertexAccessType AccessType = EMetasoundFrontendVertexAccessType::Unset;
};

// Condensed set of class metadata that is serialized to editor-only asset
// tag data, allowing editor scripts and code to search and display MetaSounds
// without loading in asset selection contexts without loading them in entirety.
USTRUCT(BlueprintType)
struct FMetaSoundClassSearchInfo
{
	GENERATED_BODY()

	FMetaSoundClassSearchInfo() = default;
	UE_API FMetaSoundClassSearchInfo(const FMetasoundFrontendClassMetadata& InClassMetadata);

	// Human readable DisplayName of Class (optional, overrides the
	// package name in the editor if specified by MetaSound Asset Author).
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	FText ClassDisplayName;

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	FText ClassDescription;

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FText> Hierarchy;

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FText> Keywords;
};

// Condensed set of vertex data that is serialized to editor-only asset tag data,
// allowing editor scripts and code to query MetaSounds without loading them
// in entirety.
USTRUCT(BlueprintType)
struct FMetaSoundClassVertexCollectionInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FMetaSoundClassVertexInfo> ClassVertexInfo;
};

// Condensed set of class data that is serialized to editor-only asset tag data,
// allowing editor scripts and code to query MetaSounds without loading them
// in entirety.
USTRUCT(BlueprintType, DisplayName = "MetaSound Class Interface Info")
struct FMetaSoundClassInterfaceInfo
{
	GENERATED_BODY()

	FMetaSoundClassInterfaceInfo() = default;
	UE_API FMetaSoundClassInterfaceInfo(const IMetaSoundDocumentInterface& InDocInterface);
	UE_API FMetaSoundClassInterfaceInfo(const FMetasoundFrontendClass& InClass);
	UE_API FMetaSoundClassInterfaceInfo(const FAssetData& InAssetData, bool& bOutIsValid);

	UE_API void ExportToContext(FAssetRegistryTagsContext& OutContext) const;

	// Interfaces metadata associated with interfaces defined by this class.
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FMetasoundFrontendInterfaceMetadata> DefinedInterfaces;

	// Editor-only search info
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	FMetaSoundClassSearchInfo SearchInfo;

	// Collection of identifiable input vertex data cached in query for fast access & serializability
	// (ex. in asset tags)
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FMetaSoundClassVertexInfo> Inputs;

	// Collection of identifiable output vertex data cached in query for fast access & serializability
	// (ex. in asset tags)
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FMetaSoundClassVertexInfo> Outputs;

	// Interfaces metadata associated with a given class definition.
	UPROPERTY(BlueprintReadOnly, Category = ClassInfo)
	TArray<FMetasoundFrontendVersion> InheritedInterfaces;
};

namespace Metasound
{
	namespace Frontend
	{
		// Class query info accessible in the Search Engine.  Represents class info for a given class
		// that may or may not be loaded and/or registered (ex. in editor, class info may be supplied by
		// asset registry tags or class info could be provided by a cloud service, enabling class browsing
		// without a local asset required to be loaded or registered with the node class registry).
		struct FMetaSoundClassInfo
		{
			UE_API FMetaSoundClassInfo(EMetasoundFrontendClassType InClassType = EMetasoundFrontendClassType::Invalid);
			UE_API FMetaSoundClassInfo(const FMetasoundFrontendClass& InClass);
			UE_API FMetaSoundClassInfo(const FAssetData& InAssetData);

			virtual ~FMetaSoundClassInfo() = default;

			// Exports class query info as tag data to the given RegistryContext
			UE_API virtual void ExportToContext(FAssetRegistryTagsContext& OutContext) const;

			// If asset is loaded, retrieves version number from loaded data. Otherwise, parses just the
			// tag data necessary to get the given asset's version number. Does not attempt to load asset,
			// and will fail if data is not found and asset isn't loaded.
			static UE_API bool TryGetClassVersion(const FAssetData& InAssetData, FMetasoundFrontendVersionNumber& OutKey);

		protected:
			UE_API virtual void InitFromDocument(const IMetaSoundDocumentInterface& InDocInterface);

		public:
			// ClassName of class
			FMetasoundFrontendClassName ClassName;

			EMetasoundFrontendClassType ClassType = EMetasoundFrontendClassType::Invalid;

			// Version of class
			FMetasoundFrontendVersionNumber Version;

#if WITH_EDITORONLY_DATA
			FMetaSoundClassInterfaceInfo InterfaceInfo;

			UE_API bool InheritsInterface(FName InterfaceName, const  FMetasoundFrontendVersionNumber& VersionNumber = { 0, 0 }) const;
#endif // WITH_EDITORONLY_DATA

			FNodeClassRegistryKey ToRegistryKey() const
			{
				return FNodeClassRegistryKey(ClassType, ClassName, Version);
			}

			EMetasoundFrontendClassAccessFlags AccessFlags;

			// If true, class info is valid and accurately reflects
			// that from what was provided on construction. If false,
			// info was constructed from AssetData that failed to
			// provide all expected tags or tag values (indicating tag
			// data is out-of-date). Default constructed info is considered
			// invalid as well.
			uint8 bIsValid : 1;
		};
	} // namespace Frontend


	/** FFrontendQueryKey allows entries to be partitioned by their key. A key
	 * can be created by default constructor, int32, FString or FName.
	 */
	struct FFrontendQueryKey
	{
		UE_API FFrontendQueryKey();
		UE_API explicit FFrontendQueryKey(int32 InKey);
		UE_API explicit FFrontendQueryKey(const FString& InKey);
		UE_API explicit FFrontendQueryKey(const FName& InKey);

		FFrontendQueryKey(const FFrontendQueryKey&) = default;
		FFrontendQueryKey& operator=(const FFrontendQueryKey&) = default;
		FFrontendQueryKey(FFrontendQueryKey&&) = default;
		FFrontendQueryKey& operator=(FFrontendQueryKey&&) = default;

		friend bool operator==(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS);
		friend bool operator!=(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS);
		friend bool operator<(const FFrontendQueryKey& InLHS, const FFrontendQueryKey& InRHS);
		friend uint32 GetTypeHash(const FFrontendQueryKey& InKey);
		
		UE_API bool IsNull() const;

	private:
		struct FNull{};

		using FKeyType = TVariant<FNull, int32, FString, FName>;
		FKeyType Key;
		uint32 Hash;
	};

	/** A FFrontendQueryValue contains data of interest. */
	using FFrontendQueryValue = TVariant
	<
		FMetasoundFrontendVersion,
		Frontend::FAssetManagerTransaction,
		Frontend::FInterfaceRegistryTransaction,
		Frontend::FMetaSoundClassInfo,
		Frontend::FNodeClassRegistryTransaction
	>;

	/** FFrontendQueryEntry represents one value in the query. It contains an ID,
	 * value and score.  */
	struct FFrontendQueryEntry
	{
		using FValue = FFrontendQueryValue;

		FGuid ID;
		FValue Value;
		float Score = 0.f;

		friend uint32 GetTypeHash(const FFrontendQueryEntry& InEntry);
		friend bool operator==(const FFrontendQueryEntry& InLHS, const FFrontendQueryEntry& InRHS);
	};

	/** A FFrontendQueryPartition represents a set of entries associated with a 
	 * single FFrontendQueryKey. */
	
	using FFrontendQueryPartition = TArray<FFrontendQueryEntry, TInlineAllocator<1>>;

	/** A FFrontendQuerySelection holds a map of keys to partitions. */
	using FFrontendQuerySelection = TSortedMap<FFrontendQueryKey, FFrontendQueryPartition>;

	/** Interface for an individual step in a query */
	class IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryStep() = default;
	};

	/** Interface for a query step which streams new entries. */
	class IFrontendQueryStreamStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryStreamStep() = default;
			virtual void Stream(TArray<FFrontendQueryValue>& OutEntries) = 0;
	};

	/** Interface for a query step which transforms an entry's value. */
	class IFrontendQueryTransformStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryTransformStep() = default;
			virtual void Transform(FFrontendQueryEntry::FValue& InValue) const = 0;
	};

	/** Interface for a query step which maps entries to keys. */
	class IFrontendQueryMapStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryMapStep() = default;
			virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const = 0;
	};

	/** Interface for a query step which maps entries to multiple keys. */
	class IFrontendQueryMultiMapStep : public IFrontendQueryStep
	{
	public:
		virtual ~IFrontendQueryMultiMapStep() = default;
		virtual TArray<FFrontendQueryKey> Map(const FFrontendQueryEntry& InEntry) const = 0;
	};

	/** Interface for a query step which reduces entries with the same key. */
	class IFrontendQueryReduceStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryReduceStep() = default;
			virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const = 0;
	};

	/** Interface for a query step which filters entries. */
	class IFrontendQueryFilterStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryFilterStep() = default;
			virtual bool Filter(const FFrontendQueryEntry& InEntry) const = 0;
	};

	/** Interface for a query step which scores entries. */
	class IFrontendQueryScoreStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryScoreStep() = default;
			virtual float Score(const FFrontendQueryEntry& InEntry) const = 0;
	};

	/** Interface for a query step which sorts entries. */
	class IFrontendQuerySortStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQuerySortStep() = default;
			virtual bool Sort(const FFrontendQueryEntry& InEntryLHS, const FFrontendQueryEntry& InEntryRHS) const = 0;
	};
	
	class IFrontendQueryLimitStep : public IFrontendQueryStep
	{
		public:
			virtual ~IFrontendQueryLimitStep() = default;
			virtual int32 Limit() const = 0;
	};

	/** FFrontendQueryStep wraps all the support IFrontenQueryStep interfaces 
	 * and supplies unified `ExecuteStep(...)` member function.
	 */
	class FFrontendQueryStep
	{
		FFrontendQueryStep() = delete;

	public:
		// Represents an incremental update to the existing data.
		struct FIncremental
		{
			// Keys that are affected by this incremental update.
			TSet<FFrontendQueryKey> ActiveKeys;
			// The selection being manipulated in the incremental update.
			FFrontendQuerySelection ActiveSelection;

			// Keys that contain active removals.
			TSet<FFrontendQueryKey> ActiveRemovalKeys;
			// Selection containing entries to remove during a merge.
			FFrontendQuerySelection ActiveRemovalSelection;
		};

		/* Interface for executing a step in the query. */
		struct IStepExecuter
		{
			virtual ~IStepExecuter() = default;

			// Merge new result with the existing result from this step.
			virtual void Merge(FIncremental& InOutIncremental, FFrontendQuerySelection& InOutSelection) const = 0;

			// Execute step. 
			virtual void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const = 0;

			// Returns true if a steps result is conditioned on the composition of a partition.
			//
			// Most steps are only dependent upon individual entries, but some
			// (Reduce, Limit, Sort) are specifically dependent upon the composition
			// of the Partition. They require special handling during incremental
			// updates.
			virtual bool IsDependentOnPartitionComposition() const = 0;

			// Return true if the step can be used to process downstream removals.
			virtual bool CanProcessRemovals() const = 0;

			// Return true if the step can produce new entries. This information
			// is used to early-out on queries with no new entries. 
			virtual bool CanProduceEntries() const = 0;
		};

		using FStreamFunction = TUniqueFunction<void (TArray<FFrontendQueryValue>&)>;
		using FTransformFunction = TFunction<void (FFrontendQueryEntry::FValue&)>;
		using FMapFunction = TFunction<FFrontendQueryKey (const FFrontendQueryEntry&)>;
		using FMultiMapFunction = TFunction<TArray<FFrontendQueryKey>(const FFrontendQueryEntry&)>;
		using FReduceFunction = TFunction<void (const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries)>;
		using FFilterFunction = TFunction<bool (const FFrontendQueryEntry&)>;
		using FScoreFunction = TFunction<float (const FFrontendQueryEntry&)>;
		using FSortFunction = TFunction<bool (const FFrontendQueryEntry& InEntryLHS, const FFrontendQueryEntry& InEntryRHS)>;
		using FLimitFunction = TFunction<int32 ()>;

		/** Create query step using TFunction or lambda. */
		UE_API FFrontendQueryStep(FStreamFunction&& InFunc);
		UE_API FFrontendQueryStep(FTransformFunction&& InFunc);
		UE_API FFrontendQueryStep(FMapFunction&& InFunc);
		UE_API FFrontendQueryStep(FMultiMapFunction&& InFunc);
		UE_API FFrontendQueryStep(FReduceFunction&& InFunc);
		UE_API FFrontendQueryStep(FFilterFunction&& InFilt);
		UE_API FFrontendQueryStep(FScoreFunction&& InScore);
		UE_API FFrontendQueryStep(FSortFunction&& InSort);
		UE_API FFrontendQueryStep(FLimitFunction&& InLimit);

		/** Create a query step using a IFrontedQueryStep */
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryStreamStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryTransformStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryMapStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryMultiMapStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryReduceStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryFilterStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryScoreStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQuerySortStep>&& InStep);
		UE_API FFrontendQueryStep(TUniquePtr<IFrontendQueryLimitStep>&& InStep);


		// Merge an incremental result with the prior result from this step.
		UE_API void Merge(FIncremental& InIncremental, FFrontendQuerySelection& InOutSelection) const;

		// Execute step. Assume not other prior results exist.
		UE_API void Execute(TSet<FFrontendQueryKey>& InOutUpdatedKeys, FFrontendQuerySelection& InOutResult) const;

		// Returns true if a steps result is conditioned on the composition of a partition.
		//
		// Most steps are only dependent upon individual entries, but some
		// (Reduce, Limit, Sort) are specifically dependent upon the composition
		// of the Partition. They require special handling during incremental
		// updates.
		UE_API bool IsDependentOnPartitionComposition() const;

		// Return true if the step can be used to process downstream removals.
		UE_API bool CanProcessRemovals() const;

		// Return true if the step can produce new entries. This information
		// is used to early-out on queries with no new entries. 
		UE_API bool CanProduceEntries() const;

	private:
		TUniquePtr<IStepExecuter> StepExecuter;
	};

	/** FFrontendQuery contains a set of query steps which produce a FFrontendQuerySelectionView */
	class FFrontendQuery
	{
	public:

		using FStreamFunction = FFrontendQueryStep::FStreamFunction;
		using FTransformFunction = FFrontendQueryStep::FTransformFunction;
		using FMapFunction = FFrontendQueryStep::FMapFunction;
		using FReduceFunction = FFrontendQueryStep::FReduceFunction;
		using FFilterFunction = FFrontendQueryStep::FFilterFunction;
		using FScoreFunction = FFrontendQueryStep::FScoreFunction;
		using FSortFunction = FFrontendQueryStep::FSortFunction;
		using FLimitFunction = FFrontendQueryStep::FLimitFunction;

		UE_API FFrontendQuery();
		FFrontendQuery(FFrontendQuery&&) = default;
		FFrontendQuery& operator=(FFrontendQuery&&) = default;

		FFrontendQuery(const FFrontendQuery&) = delete;
		FFrontendQuery& operator=(const FFrontendQuery&) = delete;


		/** Add a step to the query. */
		template<typename StepType, typename... ArgTypes>
		FFrontendQuery& AddStep(ArgTypes&&... Args)
		{
			return AddStep(MakeUnique<FFrontendQueryStep>(MakeUnique<StepType>(Forward<ArgTypes>(Args)...)));
		}

		template<typename FuncType>
		FFrontendQuery& AddFunctionStep(FuncType&& InFunc)
		{
			return AddStep(MakeUnique<FFrontendQueryStep>(MoveTemp(InFunc)));
		}

		UE_API FFrontendQuery& AddStreamLambdaStep(FStreamFunction&& InFunc);
		UE_API FFrontendQuery& AddTransformLambdaStep(FTransformFunction&& InFunc);
		UE_API FFrontendQuery& AddMapLambdaStep(FMapFunction&& InFunc);
		UE_API FFrontendQuery& AddReduceLambdaStep(FReduceFunction&& InFunc);
		UE_API FFrontendQuery& AddFilterLambdaStep(FFilterFunction&& InFunc);
		UE_API FFrontendQuery& AddScoreLambdaStep(FScoreFunction&& InFunc);
		UE_API FFrontendQuery& AddSortLambdaStep(FSortFunction&& InFunc);
		UE_API FFrontendQuery& AddLimitLambdaStep(FLimitFunction&& InFunc);

		/** Add a step to the query. */
		UE_API FFrontendQuery& AddStep(TUniquePtr<FFrontendQueryStep>&& InStep);

		/** Calls all steps in the query and returns the selection. */
		UE_API const FFrontendQuerySelection& Update(TSet<FFrontendQueryKey>& OutUpdatedKeys);
		UE_API const FFrontendQuerySelection& Update();

		/** Returns the current result. */
		UE_API const FFrontendQuerySelection& GetSelection() const;

	private:
		using FIncremental = FFrontendQueryStep::FIncremental;

		UE_API void UpdateInternal(TSet<FFrontendQueryKey>& OutUpdatedKeys);
		UE_API void MergeInternal(FFrontendQueryStep& Step, FIncremental& InOutIncremental, FFrontendQuerySelection& InOutMergedSelection);

		TSharedRef<FFrontendQuerySelection, ESPMode::ThreadSafe> Result;

		struct FStepInfo
		{
			TUniquePtr<FFrontendQueryStep> Step;
			FFrontendQuerySelection OutputCache;
			bool bMergeAndCacheOutput = false;
			bool bProcessRemovals = false;
		};

		UE_API void AppendPartitions(const TSet<FFrontendQueryKey>& InKeysToAppend, const FFrontendQuerySelection& InSelection, TSet<FFrontendQueryKey>& OutKeysModified, FFrontendQuerySelection& OutSelection) const;

		TArray<FStepInfo> Steps;
		int32 FinalEntryProducingStepIndex = INDEX_NONE;
	};
}

#undef UE_API
