// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendQuery.h"
#include "MetasoundFrontendRegistries.h"
#include "Misc/Guid.h"
#include "Templates/PimplPtr.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	// Forward declare private implementation
	class FAssetRegistrationEventsPimpl;
	class FNodeClassRegistrationEventsPimpl;

	class FMapToNull : public IFrontendQueryMapStep
	{
	public:
		virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override
		{
			return FFrontendQueryKey();
		}
	};

	/** Streams node classes that have been newly registered or unregistered since last call to Stream()
	 */
	class FNodeClassRegistrationEvents : public IFrontendQueryStreamStep
	{
	public:
		UE_API FNodeClassRegistrationEvents();
		UE_API virtual void Stream(TArray<FFrontendQueryValue>& OutValues) override;

	private:
		TPimplPtr<FNodeClassRegistrationEventsPimpl> Pimpl;
	};

	/** Streams assets that have been newly registered or unregistered since last call to Stream()
	 */
	class FAssetRegistrationEvents : public IFrontendQueryStreamStep
	{
	public:
		UE_API FAssetRegistrationEvents();
		UE_API virtual void Stream(TArray<FFrontendQueryValue>& OutValues) override;

	private:
		TPimplPtr<FAssetRegistrationEventsPimpl> Pimpl;
	};

	/** Partitions asset registration events by their node registration keys. */
	class FMapAssetRegistrationEventsToNodeRegistryKeys : public IFrontendQueryMapStep
	{
	public:
		UE_API virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
	};

	/** Partitions class registration events by their node registration keys. */
	class FMapClassRegistrationEventsToNodeRegistryKeys : public IFrontendQueryMapStep
	{
	public:
		UE_API virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
	};

	/** Reduces asset registration events mapped to the same key by inspecting their add/remove state in
	 * order to determine their final state. If an item has been added more than it has been removed,
	 * then it is added to the output. Otherwise, it is omitted. */
	class FReduceAssetRegistrationEventsToCurrentStatus : public IFrontendQueryReduceStep
	{
	public:
		UE_API virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override;
	private:
		using FTimeType = Frontend::FAssetManagerTransaction::FTimeType;
		static FTimeType GetTransactionTimestamp(const FFrontendQueryEntry& InEntry);
		static bool IsValidTransactionOfType(Frontend::FAssetManagerTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry);
	};

	/** Reduces registration events mapped to the same key by inspecting their add/remove state in
	 * order to determine their final state. If an item has been added more than it has been removed,
	 * then it is added to the output. Otherwise, it is omitted. */
	class FReduceRegistrationEventsToCurrentStatus: public IFrontendQueryReduceStep
	{
	public:
		UE_API virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override;
	private:
		using FTimeType = Frontend::FNodeClassRegistryTransaction::FTimeType;
		static FTimeType GetTransactionTimestamp(const FFrontendQueryEntry& InEntry);
		static bool IsValidTransactionOfType(Frontend::FNodeClassRegistryTransaction::ETransactionType InType, const FFrontendQueryEntry* InEntry);
	};

	/** Transforms an asset registration event into a FMetaSoundClassInfo. */
	class FTransformAssetRegistrationEventsToClassInfo : public IFrontendQueryTransformStep
	{
	public:
		UE_API virtual void Transform(FFrontendQueryEntry::FValue& InValue) const override;
	};

	/** Transforms a node class registration event into a FMetaSoundClassInfo. */
	class FTransformNodeClassRegistrationEventsToClassInfo : public IFrontendQueryTransformStep
	{
	public:
		UE_API virtual void Transform(FFrontendQueryEntry::FValue& InValue) const override;
	};

	class FFilterClassesByInputVertexDataType : public IFrontendQueryFilterStep
	{
	public: 
		template<typename DataType>
		FFilterClassesByInputVertexDataType()
		:	FFilterClassesByInputVertexDataType(GetMetasoundDataTypeName<DataType>())
		{
		}

		UE_API FFilterClassesByInputVertexDataType(const FName& InTypeName);

		UE_API virtual bool Filter(const FFrontendQueryEntry& InEntry) const override;

	private:
		FName InputVertexTypeName;
	};

	class FFilterClassesByOutputVertexDataType : public IFrontendQueryFilterStep
	{
	public: 
		template<typename DataType>
		FFilterClassesByOutputVertexDataType()
		:	FFilterClassesByOutputVertexDataType(GetMetasoundDataTypeName<DataType>())
		{
		}

		UE_API FFilterClassesByOutputVertexDataType(const FName& InTypeName);

		UE_API virtual bool Filter(const FFrontendQueryEntry& InEntry) const override;

	private:
		FName OutputVertexTypeName;
	};

	class FMapClassesToClassName : public IFrontendQueryMapStep
	{
	public: 
		UE_API virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;

	};

	class FFilterClassesByClassID : public IFrontendQueryFilterStep
	{
	public:
		UE_API FFilterClassesByClassID(const FGuid InClassID);

		UE_API virtual bool Filter(const FFrontendQueryEntry& InEntry) const override;

	private:
		FGuid ClassID;
	};

	class FMapToFullClassName : public IFrontendQueryMapStep
	{
	public:
		UE_API virtual FFrontendQueryKey Map(const FFrontendQueryEntry& InEntry) const override;
	};

	class FReduceClassesToHighestVersion : public IFrontendQueryReduceStep
	{
	public:
		UE_API virtual void Reduce(const FFrontendQueryKey& InKey, FFrontendQueryPartition& InOutEntries) const override;
	};

	class FSortClassesByVersion : public IFrontendQuerySortStep
	{
	public:
		UE_API virtual bool Sort(const FFrontendQueryEntry& InEntryLHS, const FFrontendQueryEntry& InEntryRHS) const override;
	};
}

#undef UE_API
