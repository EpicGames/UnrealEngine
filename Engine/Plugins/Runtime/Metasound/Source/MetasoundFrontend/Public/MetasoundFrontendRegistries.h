// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "MetasoundFrontendRegistryContainer.h"

// TODO: Repurpose or deprecate this file
