// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendNodeClassRegistry.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "IAudioProxyInitializer.h"
#include "MetasoundDataReference.h"
#include "MetasoundEnum.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundLiteral.h"
#include "MetasoundNodeConstructorParams.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundFrontendRegistryKey.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"
#include "Templates/Function.h"
#endif

// Forward Declarations
class IMetaSoundDocumentInterface;
template <typename InInterfaceType>
class TScriptInterface;

using FMetasoundFrontendRegistryContainer = Metasound::Frontend::INodeClassRegistry;

