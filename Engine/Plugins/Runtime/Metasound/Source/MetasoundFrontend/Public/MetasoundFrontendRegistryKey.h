// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendDocument.h"
#include "MetasoundNodeInterface.h"
#include "Misc/CoreDefines.h"
#include "UObject/NoExportTypes.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_API METASOUNDFRONTEND_API


struct FMetaSoundAssetKey;

namespace Metasound::Frontend
{
	/** FNodeClassInfo contains a minimal set of information needed to find and query node classes. 
	  * THIS CLASS IS ACTIVELY BEING DEPRECATED. USE FNodeClassRegistryKey instead.
	  */
	struct FNodeClassInfo
	{
		// ClassName of the given class
		FMetasoundFrontendClassName ClassName;

		// The type of this node class
		EMetasoundFrontendClassType Type = EMetasoundFrontendClassType::Invalid;

		UE_DEPRECATED(5.6, "Data now characterized in asset class data")
		FGuid AssetClassID;

		UE_DEPRECATED(5.6, "Data now characterized in asset class data")
		FTopLevelAssetPath AssetPath;

		// Version of the registered class
		FMetasoundFrontendVersionNumber Version;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.6, "Data now characterized in asset class data ('Inputs')")
		TSet<FName> InputTypes;

		UE_DEPRECATED(5.6, "Type data no longer required by class info ('Outputs')")
		TSet<FName> OutputTypes;

		UE_DEPRECATED(5.6, "Preset data no longer required by class info")
		bool bIsPreset;
#endif // WITH_EDITORONLY_DATA

		UE_API FNodeClassInfo();

		// Constructor used to generate NodeClassInfo from a class' Metadata.
		// (Does not cache AssetPath and thus may not support loading asset
		// should the class originate from one).
		UE_API FNodeClassInfo(const FMetasoundFrontendClassMetadata& InMetadata);

		// Constructor used to generate NodeClassInfo from a graph class
		UE_API FNodeClassInfo(const FMetasoundFrontendGraphClass& InClass);

		UE_DEPRECATED(5.6, "Asset data no longer contained on NodeClassInfo, see AssetClass Info")
		UE_API FNodeClassInfo(const FMetasoundFrontendGraphClass& InClass, const FTopLevelAssetPath& InAssetPath);

		UE_API FNodeClassInfo(const FNodeClassInfo& Other);
		UE_API FNodeClassInfo(FNodeClassInfo&& Other);

		UE_API FNodeClassInfo& operator=(const FNodeClassInfo& Other);
		UE_API FNodeClassInfo& operator=(FNodeClassInfo&& Other);

		// Added to aid in deprecation of this class in favor of FNodeClassRegistryKey
		static const FNodeClassInfo& GetInvalid()
		{
			static const FNodeClassInfo Invalid;
			return Invalid;
		}
	};

	struct FNodeClassRegistryKey
	{
		EMetasoundFrontendClassType Type = EMetasoundFrontendClassType::Invalid;
		FMetasoundFrontendClassName ClassName;
		FMetasoundFrontendVersionNumber Version;

		FNodeClassRegistryKey() = default;
		UE_API FNodeClassRegistryKey(EMetasoundFrontendClassType InType, const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion, int32 InMinorVersion);
		UE_API FNodeClassRegistryKey(EMetasoundFrontendClassType InType, const FMetasoundFrontendClassName& InClassName, const FMetasoundFrontendVersionNumber& InVersion);
		UE_API FNodeClassRegistryKey(const FNodeClassMetadata& InNodeMetadata);
		UE_API FNodeClassRegistryKey(const FMetasoundFrontendClassMetadata& InNodeMetadata);
		UE_API FNodeClassRegistryKey(const FMetasoundFrontendGraphClass& InGraphClass);

		UE_DEPRECATED(5.8, "FNodeClassInfo is actively being deprecated, use overload with NodeClassRegistryKey or otherwise")
		UE_API FNodeClassRegistryKey(const FNodeClassInfo& InClassInfo);

		UE_API FNodeClassRegistryKey(const FMetaSoundAssetKey& AssetKey);

		inline friend bool operator==(const FNodeClassRegistryKey& InLHS, const FNodeClassRegistryKey& InRHS)
		{
			return (InLHS.Type == InRHS.Type) && (InLHS.ClassName == InRHS.ClassName) && (InLHS.Version == InRHS.Version);
		}

		inline friend bool operator<(const FNodeClassRegistryKey& InLHS, const FNodeClassRegistryKey& InRHS)
		{
			if (static_cast<uint8>(InLHS.Type) == static_cast<uint8>(InRHS.Type))
			{
				if (InLHS.ClassName == InRHS.ClassName)
				{
					return InLHS.Version < InRHS.Version;
				}

				return InLHS.ClassName < InRHS.ClassName;
			}

			return static_cast<uint8>(InLHS.Type) < static_cast<uint8>(InRHS.Type);
		}

		friend inline uint32 GetTypeHash(const FNodeClassRegistryKey& InKey)
		{
			const int32 Hash = HashCombineFast(static_cast<uint32>(InKey.Type), GetTypeHash(InKey.ClassName));
			return HashCombineFast(Hash, GetTypeHash(InKey.Version));
		}

		// Returns invalid (default constructed) key
		static UE_API const FNodeClassRegistryKey& GetInvalid();

		// Returns whether or not instance is valid or not
		UE_API bool IsValid() const;

		// Resets the key back to an invalid default state
		UE_API void Reset();

		// Returns string representation of key
		UE_API FString ToString() const;

		// Convenience function to convert to a string representation of the given key with a scope header (primarily for tracing).
		UE_API FString ToString(const FString& InScopeHeader) const;

		// Parses string representation of key into registry key.  For debug and deserialization use only.
		// Returns true if parsed successfully.
		static UE_API bool Parse(const FString& InKeyString, FNodeClassRegistryKey& OutKey);
	};
	using FNodeRegistryKey = FNodeClassRegistryKey;

	struct FGraphClassRegistryKey
	{
		FNodeClassRegistryKey NodeKey;
		FTopLevelAssetPath AssetPath;
		uint32 ObjectID = (uint32)INDEX_NONE;

		UE_API FString ToString() const;

		// Convenience function to convert to a string representation of the given key with a scope header (primarily for tracing).
		UE_API FString ToString(const FString& InScopeHeader) const;

		UE_API bool IsValid() const;

		inline friend bool operator==(const FGraphClassRegistryKey& InLHS, const FGraphClassRegistryKey& InRHS)
		{
			return (InLHS.NodeKey == InRHS.NodeKey) && (InLHS.AssetPath == InRHS.AssetPath) && (InLHS.ObjectID == InRHS.ObjectID);
		}

		friend inline uint32 GetTypeHash(const FGraphClassRegistryKey& InKey)
		{
			const int32 Hash = HashCombineFast(::GetTypeHash(InKey.ObjectID), HashCombineFast(GetTypeHash(InKey.NodeKey), GetTypeHash(InKey.AssetPath)));
			return Hash;
		}
	};
	using FGraphRegistryKey = FGraphClassRegistryKey;
} // namespace Metasound::Frontend

#undef UE_API
