// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundNodeInterface.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound::Frontend
{
	// Forward Declarations
	struct FMetaSoundClassInfo;

	/** Interface for frontend search engine. A frontend search engine provides
		* a simple interface for common frontend queries. It also serves as an
		* opportunity to cache queries to reduce CPU load. 
		*/
	class ISearchEngine
	{
		public:
			enum class EResultVersion : uint8
			{
				All,
				Highest
			};

			enum class ESortByVersion : uint8
			{
				Yes,
				No
			};

			/** Return an instance of a search engine. */
			static UE_API ISearchEngine& Get();

			virtual ~ISearchEngine() = default;

			/** Updates internal state to speed up queries. */
			virtual void Prime() = 0;

			/** Find the class info with the given ClassName & Major Version. Returns false if not found, true if found. */
			virtual bool FindRegisteredClass(const FMetasoundFrontendClassName& InName, int32 InMajorVersion, FMetaSoundClassInfo& OutClass) = 0;

			UE_DEPRECATED(5.8, "Use FindRegisteredClass.  If a full class definition is required, use the resulting class info to look-up the class directly from the INodeClassRegistry.")
			virtual bool FindClassWithHighestMinorVersion(const FMetasoundFrontendClassName& InName, int32 InMajorVersion, FMetasoundFrontendClass& OutClass);

			/** Finds all registered interfaces with the given name. */
			virtual TArray<FMetasoundFrontendVersion> FindAllRegisteredInterfacesWithName(FName InInterfaceName) = 0;

			/** Finds the registered interface version with the highest version of the given name. Returns true if found, false if not. */
			virtual bool FindHighestInterfaceVersion(FName InInterfaceName, FMetasoundFrontendVersion& OutVersion) = 0;

			/** Finds the registered interface with the highest version of the given name. Returns true if found, false if not. */
			virtual bool FindInterfaceWithHighestVersion(FName InInterfaceName, FMetasoundFrontendInterface& OutInterface) = 0;

			/** Returns all interfaces that are to be added to a given document when it is initialized on an object with the given class**/
			virtual TArray<FMetasoundFrontendVersion> FindUClassDefaultInterfaceVersions(const FTopLevelAssetPath& InUClassPath) = 0;

#if WITH_EDITORONLY_DATA
			/** Find all class infos including those not registered but available via the asset manager for selection.
				* Options - If set to 'All', include all versions (i.e. deprecated classes and versions of classes that are not the highest major version).
				* bIncludeUnloadedAssets - If true, include class info for assets not loaded/registered.
				*/
			virtual TArray<FMetaSoundClassInfo> FindAllClasses(EResultVersion Options, bool bIncludeUnloadedAssets = false) = 0;

			UE_DEPRECATED(5.8, "Use FindAllClasses overload that returns ClassInfo instead")
			virtual TArray<FMetasoundFrontendClass> FindAllClasses(bool bInIncludeAllVersions);

			/** Find all class info with the given ClassName.
				* (Optional) Sort matches based on version.
				*/
			virtual TArray<FMetaSoundClassInfo> FindClassesWithName(const FMetasoundFrontendClassName& InName, ESortByVersion SortByVersion = ESortByVersion::Yes) = 0;

			/** Find all classes with the given ClassName.
				* (Optional) Sort matches based on version.
				*/
			virtual TArray<FMetasoundFrontendClass> FindClassesWithName(const FMetasoundFrontendClassName& InName, bool bInSortByVersion) = 0;


			/** Find the highest version of class info with the given ClassName. Returns false if not found, true if found. */
			virtual bool FindClassWithHighestVersion(const FMetasoundFrontendClassName& InName, FMetaSoundClassInfo& OutInfo) = 0;

			/** (To be deprecated) Find the highest version of a class with the given ClassName. Returns false if not found, true if found. */
			virtual bool FindClassWithHighestVersion(const FMetasoundFrontendClassName& InName, FMetasoundFrontendClass& OutClass);

			/** Returns array with all registered interface versions. Optionally, include interface versions that are not the highest version. */
			virtual TArray<FMetasoundFrontendVersion> FindAllInterfaceVersions(bool bInIncludeAllVersions = false) = 0;

			UE_DEPRECATED(5.8, "Use FindAllInterfaceVersions instead, and refine results further if required using IInterfaceRegistry")
			virtual TArray<FMetasoundFrontendInterface> FindAllInterfaces(bool bInIncludeAllVersions = false) { return { }; }
#endif // WITH_EDITORONLY_DATA

		protected:
			ISearchEngine() = default;
	};
} // namespace Metasound::Frontend

#undef UE_API
