// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"

#include "MetasoundFrontendTemplateNodeConfiguration.generated.h"

// Default node configuration used by the Node Template system (see IMetaSoundNodeTemplateBase
// & FNodeTemplateBase). Distinct from FMetaSoundFrontendDocumentTemplate, which configures the
// MetaSound document as a whole. Also distinct from FMetaSoundFrontendNodeConfiguration, which
// stores per-node interface swap data (node configuration, a separate feature).
USTRUCT()
struct FMetaSoundFrontendTemplateNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()
};

