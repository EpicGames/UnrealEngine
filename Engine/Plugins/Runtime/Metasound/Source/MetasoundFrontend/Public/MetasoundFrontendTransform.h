// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "Interfaces/MetasoundFrontendInterfaceRegistry.h"
#include "MetasoundAssetBase.h"
#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundLog.h"
#include "MetasoundVertex.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;
struct FMetaSoundFrontendPresetTemplate;


#define	METASOUND_VERSIONING_LOG(Verbosity, Format, ...) if (DocumentTransform::bVersioningLoggingEnabled) { UE_LOG(LogMetaSound, Verbosity, Format, ##__VA_ARGS__); }

namespace Metasound::Frontend
{
	namespace DocumentTransform
	{
		extern bool bVersioningLoggingEnabled;

		#if WITH_EDITOR
		using FGetNodeDisplayNameProjection = TFunction<FText(const FNodeHandle&)>;
		using FGetNodeDisplayNameProjectionRef = TFunctionRef<FText(const FNodeHandle&)>;

		UE_API bool GetVersioningLoggingEnabled();
		UE_API void SetVersioningLoggingEnabled(bool bIsEnabled);
		UE_API void RegisterNodeDisplayNameProjection(FGetNodeDisplayNameProjection&& InNameProjection);
		UE_API FGetNodeDisplayNameProjectionRef GetNodeDisplayNameProjection();
#endif // WITH_EDITOR
	}

	class IBuilderTransform
	{
	public:
		virtual ~IBuilderTransform() = default;

		virtual bool Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilder) const = 0;
	};

	class IBuilderVersionTransform : public IBuilderTransform
	{
	public:
		virtual ~IBuilderVersionTransform() = default;

		// Returns version update transform applies
		virtual const FMetasoundFrontendVersion& GetVersion() const = 0;
	};

	/** Interface for transforms applied to documents. */
	class IDocumentTransform
	{
	public:
		virtual ~IDocumentTransform() = default;

		/** Return true if InDocument was modified, false otherwise. */
		virtual bool Transform(FDocumentHandle InDocument) const = 0;

		/** Return true if InDocument was modified, false otherwise.
			* This function is soft deprecated.  It is not pure virtual
			* to grandfather in old transform implementation. Old transforms
			* should be deprecated and rewritten to use the Controller-less
			* API in the interest of better performance and simplicity.
			*/
		UE_API virtual bool Transform(FMetasoundFrontendDocument& InOutDocument) const;
	};

	class UE_DEPRECATED(5.8, "Use IBuilderTransform or similar instead") IGraphTransform
	{
	public:
		virtual ~IGraphTransform() = default;

		virtual bool Transform(FMetasoundFrontendGraph& InOutGraph) const = 0;
	};

	/** Interface for transforming a node. */
	class UE_DEPRECATED(5.8, "Use IBuilderTransform or similar instead") INodeTransform
	{
	public:
		virtual ~INodeTransform() = default;

		UE_API virtual bool Transform(const FGuid& InNodeID, FMetaSoundFrontendDocumentBuilder& OutBuilder) const;
	};

	/** Adds or swaps document members (inputs, outputs) and removing any document members where necessary and adding those missing. */
	class FModifyRootGraphInterfaces : public IDocumentTransform
	{
	public:
		UE_API FModifyRootGraphInterfaces(const TArray<FMetasoundFrontendInterface>& InInterfacesToRemove, const TArray<FMetasoundFrontendInterface>& InInterfacesToAdd);
		UE_API FModifyRootGraphInterfaces(const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToRemove, const TArray<FMetasoundFrontendVersion>& InInterfaceVersionsToAdd);

#if WITH_EDITOR
		// Whether or not to propagate node locations to new members. Setting to false
		// results in members not having a default physical location in the editor graph.
		UE_API void SetDefaultNodeLocations(bool bInSetDefaultNodeLocations);
#endif // WITH_EDITOR

		// Override function used to match removed members with added members, allowing
		// transform to preserve connections made between removed interface members & new interface members
		// that may be related but not be named the same.
		UE_API void SetNamePairingFunction(const TFunction<bool(FName, FName)>& InNamePairingFunction);

		UE_API virtual bool Transform(FDocumentHandle InDocument) const override;
		UE_API virtual bool Transform(FMetasoundFrontendDocument& InOutDocument) const override;

	private:
		bool AddMissingVertices(FGraphHandle GraphHandle) const;
		void Init(const TFunction<bool(FName, FName)>* InNamePairingFunction = nullptr);
		bool SwapPairedVertices(FGraphHandle GraphHandle) const;
		bool RemoveUnsupportedVertices(FGraphHandle GraphHandle) const;
		bool UpdateInterfacesInternal(FDocumentHandle DocumentHandle) const;

#if WITH_EDITORONLY_DATA
		void UpdateAddedVertexNodePositions(FGraphHandle GraphHandle) const;

		bool bSetDefaultNodeLocations = true;
#endif // WITH_EDITORONLY_DATA

		TArray<FMetasoundFrontendInterface> InterfacesToRemove;
		TArray<FMetasoundFrontendInterface> InterfacesToAdd;

		using FVertexPair = TTuple<FMetasoundFrontendClassVertex, FMetasoundFrontendClassVertex>;
		TArray<FVertexPair> PairedInputs;
		TArray<FVertexPair> PairedOutputs;

		struct FInputData
		{
			FMetasoundFrontendClassInput Input;
			const FMetasoundFrontendInterface* InputInterface = nullptr;
		};

		struct FOutputData
		{
			FMetasoundFrontendClassOutput Output;
			const FMetasoundFrontendInterface* OutputInterface = nullptr;
		};

		TArray<FInputData> InputsToAdd;
		TArray<FMetasoundFrontendClassInput> InputsToRemove;
		TArray<FOutputData> OutputsToAdd;
		TArray<FMetasoundFrontendClassOutput> OutputsToRemove;

	};

	class FUpdateRootGraphInterface : public IDocumentTransform
	{
	public:
		FUpdateRootGraphInterface(const FMetasoundFrontendVersion& InInterfaceVersion, const FString& InOwningAssetName=FString(TEXT("Unknown")))
		{
		}

		UE_DEPRECATED(5.5, "RootGraph update is now handled privately by internal MetaSound asset management")
		virtual bool Transform(FDocumentHandle InDocument) const override { return false; }
	};

	/** Completely rebuilds the graph connecting a preset's inputs to the reference (parent)
		* document's root graph. It maintains previously set input values entered upon 
		* the preset's wrapping graph. */
	class FRebuildPresetRootGraph : public IDocumentTransform
	{
	public:
		/** Create transform.
			* @param InReferenceDocument - The document containing the wrapped MetaSound graph.
			*/
		UE_DEPRECATED(5.7, "Use the constructor which takes a FMetaSoundFrontendDocumentBuilder")
		FRebuildPresetRootGraph(FConstDocumentHandle InReferencedDocument)
			: ParentBuilder(nullptr)
		{
		}

		FRebuildPresetRootGraph(const FMetaSoundFrontendDocumentBuilder& InParentDocumentBuilder)
			: ParentBuilder(&InParentDocumentBuilder)
		{
		}

		UE_DEPRECATED(5.7, "Use Transform which takes in a document builder.")
		UE_API FRebuildPresetRootGraph(const FMetasoundFrontendDocument& InReferencedDocument);

		UE_DEPRECATED(5.7, "Use Transform which takes in a document builder.")
		UE_API virtual bool Transform(FDocumentHandle InDocument) const override;

		UE_DEPRECATED(5.7, "Use Transform which takes in a document builder.")
		UE_API virtual bool Transform(FMetasoundFrontendDocument& InOutDocument) const override;

		UE_API bool Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const;

	private:
		// Get the class inputs needed for this preset. Input literals set on
		// the preset graph will be used if they are set as overriding
		// the default from the parent graph.
		TArray<FMetasoundFrontendClassInput> GenerateRequiredClassInputs(FMetaSoundFrontendDocumentBuilder& InDocumentToTransformBuilder) const;

		// Get the class Outputs needed for this preset.
		TArray<FMetasoundFrontendClassOutput> GenerateRequiredClassOutputs(FMetaSoundFrontendDocumentBuilder& InDocumentToTransformBuilder) const;
		
		// Add inputs to parent graph and connect to wrapped graph node.
		void AddAndConnectInputs(const FMetaSoundFrontendPresetTemplate& PresetTemplate, const TArray<FMetasoundFrontendClassInput>& InClassInputs, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InReferencedNodeID) const;

		// Add outputs to parent graph and connect to wrapped graph node.
		void AddAndConnectOutputs(const TArray<FMetasoundFrontendClassOutput>& InClassOutputs, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InReferencedNodeID) const;

#if WITH_EDITORONLY_DATA
		using FMemberIDToMetadataMap = TMap<FGuid, TObjectPtr<UMetaSoundFrontendMemberMetadata>>;
		void AddMemberMetadata(const FMemberIDToMetadataMap& InCachedMemberMetadata, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const;
		FMemberIDToMetadataMap CacheMemberMetadata(const FMetaSoundFrontendPresetTemplate& PresetTemplate, FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform) const;
#endif // WITH_EDITORONLY_DATA

		const FMetaSoundFrontendDocumentBuilder* ParentBuilder = nullptr;
	};

#if WITH_EDITORONLY_DATA
	/** Automatically updates all nodes and respective dependencies in graph where
		* newer versions exist in the loaded MetaSound Class Node Registry.
		*/
	class FAutoUpdateRootGraph 
	{
	public:
		/** Construct an AutoUpdate transform
			*
			* @param InDebugAssetPath - Asset path used for debug logs on warnings and errors.
			* @param bInLogWarningOnDroppedConnections - If true, warnings will be logged if a node update results in a dropped connection.
			*/
		FAutoUpdateRootGraph(FString&& InDebugAssetPath, bool bInLogWarningOnDroppedConnection)
			: DebugAssetPath(MoveTemp(InDebugAssetPath))
			, bLogWarningOnDroppedConnection(bInLogWarningOnDroppedConnection)
		{
		}

		UE_DEPRECATED(5.7, "Use Transform which takes in a document builder.")
		UE_API bool Transform(FDocumentHandle InDocument);
		
		UE_API bool Transform(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform);
		UE_API static bool CanAutoUpdate(const FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InNodeID, const FGuid& InPageID);
	private:
		// Helper function to go through external nodes and update dependencies
		bool UpdateExternalDependencies(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform);

		static bool HasInterfaceUpdate(const FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FGuid& InNodeID, const FGuid& InPageID);
		static bool HasAutoAppliedCustomUpdateTransform(const FMetasoundFrontendClassName& InClassName, int32 InMajorVersion);
		
		// Process various updates. Returns true if an update was applied.
		bool ProcessMinorVersionUpdate(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FNodeClassRegistryKey& InCurrentNodeKey, const FNodeClassRegistryKey& InHighestRegistryVersionKey, const FGuid& InNodeID, const FGuid& InPageID, FString* InOutAutoUpdateReasonString=nullptr);
		bool ProcessMajorVersionUpdate(FMetaSoundFrontendDocumentBuilder& InOutBuilderToTransform, const FNodeClassRegistryKey& InCurrentNodeRegistryKey, const FGuid& InNodeID, const FGuid& InPageID, FNodeClassRegistryKey& OutNewNodeRegistryKey);
		
		const FString DebugAssetPath;
		bool bLogWarningOnDroppedConnection;
	};
#endif // WITH_EDITORONLY_DATA

	/** Sets the document's graph class, optionally updating the namespace and variant. */
	class FRenameRootGraphClass : public IDocumentTransform
	{
		const FMetasoundFrontendClassName NewClassName;

	public:
		UE_DEPRECATED(5.5, "Use FMetasoundFrontendDocumentBuilder::GenerateNewClassName instead")
		static bool Generate(FDocumentHandle InDocument, const FGuid& InGuid, const FName Namespace = { }, const FName Variant = { })
		{
			return false;
		}

		UE_DEPRECATED(5.5, "Use FMetasoundFrontendDocumentBuilder::GenerateNewClassName instead")
		static bool Generate(FMetasoundFrontendDocument& InDocument, const FGuid& InGuid, const FName Namespace = { }, const FName Variant = { })
		{
			return false;
		}

		UE_DEPRECATED(5.5, "Use FMetasoundFrontendDocumentBuilder::GenerateNewClassName instead")
		FRenameRootGraphClass(const FMetasoundFrontendClassName InClassName)
			: NewClassName(InClassName)
		{
		}

		UE_DEPRECATED(5.5, "Use FMetasoundFrontendDocumentBuilder::GenerateNewClassName instead")
		UE_API bool Transform(FDocumentHandle InDocument) const override;

		UE_DEPRECATED(5.5, "Use FMetasoundFrontendDocumentBuilder::GenerateNewClassName instead")
		UE_API bool Transform(FMetasoundFrontendDocument& InOutDocument) const override;
	};
} // Metasound::Frontend

#undef UE_API
