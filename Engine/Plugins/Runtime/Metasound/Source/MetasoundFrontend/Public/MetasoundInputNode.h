// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundBasicNode.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundBuildError.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontendDataTypeTraits.h"
#include "MetasoundNodeConstructorParams.h"
#include "MetasoundLiteral.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	namespace MetasoundInputNodePrivate
	{
		class FNonExecutableInputOperatorBase : public IOperator
		{	
		public:
			UE_API virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			UE_API virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			UE_API virtual IOperator::FExecuteFunction GetExecuteFunction() override;
			UE_API virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override;
			UE_API virtual IOperator::FResetFunction GetResetFunction() override;

		protected:
			UE_API FNonExecutableInputOperatorBase(const FVertexName& InVertexName, FAnyDataReference&& InDataRef);

			FVertexName VertexName;
			FAnyDataReference DataRef;
		};


		class FNonExecutableInputPassThroughOperator : public FNonExecutableInputOperatorBase
		{
		public:
			template<typename DataType>
			FNonExecutableInputPassThroughOperator(const FVertexName& InVertexName, const TDataReadReference<DataType>& InDataRef)
			: FNonExecutableInputOperatorBase(InVertexName, FAnyDataReference{InDataRef})
			{
			}

			template<typename DataType>
			FNonExecutableInputPassThroughOperator(const FVertexName& InVertexName, const TDataWriteReference<DataType>& InDataRef)
			: FNonExecutableInputPassThroughOperator(InVertexName, TDataReadReference<DataType>(InDataRef))
			{
			}
		};


		/** TInputValueOperator provides an input for value references. */
		template<typename DataType>
		class TInputValueOperator : public FNonExecutableInputOperatorBase
		{
		public:
			/** Construct an TInputValueOperator with the name of the vertex and the 
			 * value reference associated with input. 
			 */
			explicit TInputValueOperator(const FName& InVertexName, const TDataValueReference<DataType>& InValueRef)
			: FNonExecutableInputOperatorBase(InVertexName, FAnyDataReference{InValueRef})
			{
			}

			TInputValueOperator(const FVertexName& InVertexName, const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			: FNonExecutableInputOperatorBase(InVertexName, FAnyDataReference{TDataValueReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral)})
			{
			}

			TInputValueOperator(const FVertexName& InVertexName, const FOperatorSettings& InSettings, const FInputVertexInterfaceData& InInterfaceData)
			: FNonExecutableInputOperatorBase(InVertexName, FAnyDataReference{InInterfaceData.GetOrCreateDefaultDataValueReference<DataType>(InVertexName, InSettings)})
			{
			}
		};

		template<typename DataType>
		class TPostExecutableInputOperator : public IOperator
		{
			static_assert(TPostExecutableDataType<DataType>::bIsPostExecutable, "TPostExecutableInputOperator should only be used with post executable data types");

		public:
			using FDataWriteReference = TDataWriteReference<DataType>;

			TPostExecutableInputOperator(const FVertexName& InDataReferenceName, TDataWriteReference<DataType> InValue)
				: DataReferenceName(InDataReferenceName)
				, DataRef(InValue)
			{
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindVertex(DataReferenceName, DataRef);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex<DataType>(DataReferenceName, DataRef);
			}

			virtual FExecuteFunction GetExecuteFunction() override
			{
				return nullptr;
			}

			virtual FPostExecuteFunction GetPostExecuteFunction() override
			{
				// This condition is checked at runtime as its possible dynamic graphs may reassign ownership
				// of underlying data to operate on in post execute. In this case, the expectation is that the
				// data reference is now owned by another provider/operator.
				if (DataRef.GetAccessType() == EDataReferenceAccessType::Write)
				{
					return &PostExecute;
				}
				else
				{
					return nullptr;
				}
			}

			virtual FResetFunction GetResetFunction() override
			{
				// This condition is checked at runtime as its possible dynamic graphs may reassign ownership
				// of underlying data to operate on in post execute. In this case, the expectation is that the
				// data reference is now owned by another provider/operator.
				if (DataRef.GetAccessType() == EDataReferenceAccessType::Write)
				{
					return &NoOpReset;
				}
				else
				{
					return nullptr;
				}
			}

		protected:
			static void NoOpReset(IOperator* InOperator, const IOperator::FResetParams& InParams)
			{
				// All post executable nodes must have a reset.  This is a special
				// case of a non-owning node performing post execute on a data type
				// owned by an external system.
			}

			static void PostExecute(IOperator* InOperator)
			{
				using FPostExecutableInputOperator = TPostExecutableInputOperator<DataType>;

				FPostExecutableInputOperator* DerivedOperator = static_cast<FPostExecutableInputOperator*>(InOperator);
				check(nullptr != DerivedOperator);

				DataType* Value = DerivedOperator->DataRef.template GetWritableValue<DataType>();
				if (ensure(Value != nullptr))
				{
					TPostExecutableDataType<DataType>::PostExecute(*Value);
				}
			}

			FVertexName DataReferenceName;
			FAnyDataReference DataRef;
		};

		// To reset the state of a PostExecutable input operator, we need to reset
		// the data to it's original state. In order to do that, the FLiteral is
		// stored on the operator so that it can be used to reinitialize the data 
		// when the operator reset.  
		template<typename DataType>
		class TResetablePostExecutableInputOperator : public TPostExecutableInputOperator<DataType>
		{
		public:
			using FDataWriteReference = TDataWriteReference<DataType>;
			using FDataWriteReferenceFactory = TDataWriteReferenceLiteralFactory<DataType>;
			using TPostExecutableInputOperator<DataType>::DataRef;

			TResetablePostExecutableInputOperator(const FVertexName& InDataReferenceName, const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			: TPostExecutableInputOperator<DataType>(InDataReferenceName, FDataWriteReferenceFactory::CreateExplicitArgs(InSettings, InLiteral))
			, Literal(InLiteral)
			{
			}

			TResetablePostExecutableInputOperator(const FVertexName& InDataReferenceName, const FOperatorSettings& InSettings, const FInputVertexInterfaceData& InData)
			: TPostExecutableInputOperator<DataType>(InDataReferenceName, InData.GetOrCreateDefaultDataWriteReference<DataType>(InDataReferenceName, InSettings))
			, Literal(InData.GetVertex(InDataReferenceName).GetDefaultLiteral())
			{
				checkf(!InData.IsVertexBound(InDataReferenceName), TEXT("Vertex %s should not be bound when using TResetablePostExecutableInputOperator"), *InDataReferenceName.ToString());
			}

			virtual IOperator::FResetFunction GetResetFunction() override
			{
				if (DataRef.GetAccessType() == EDataReferenceAccessType::Write)
				{
					return &Reset;
				}
				else
				{
					// If DataRef is not writable, reference is assumed to be reset by another owning operator.
					return nullptr;
				}
			}

		private:

			static void Reset(IOperator* InOperator, const IOperator::FResetParams& InParams)
			{
				using FResetablePostExecutableInputOperator = TResetablePostExecutableInputOperator<DataType>;

				FResetablePostExecutableInputOperator* Operator = static_cast<FResetablePostExecutableInputOperator*>(InOperator);
				check(nullptr != Operator);

				DataType* Value = Operator->DataRef.template GetWritableValue<DataType>();
				if (ensure(Value != nullptr))
				{
					*Value = TDataTypeLiteralFactory<DataType>::CreateExplicitArgs(InParams.OperatorSettings, Operator->Literal);
				}
			}

			FLiteral Literal;
		};

		/** Non owning input operator that may need execution. */
		template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
		using TNonOwningInputOperator = std::conditional_t<
			TPostExecutableDataType<DataType>::bIsPostExecutable,
			TPostExecutableInputOperator<DataType>, // Use this input operator if the data type is not owned by the input node but needs post execution.
			MetasoundInputNodePrivate::FNonExecutableInputPassThroughOperator // Use this input operator if the data type is not owned by the input node and is not executable, nor post executable.
		>;
	}

	/** Owning input operator that may need execution. */
	template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
	using TInputOperator = std::conditional_t<
		VertexAccess == EVertexAccessType::Value || !TPostExecutableDataType<DataType>::bIsPostExecutable,
		MetasoundInputNodePrivate::TInputValueOperator<DataType>, // Use this input operator if the data type is owned by the input node and is not executable, nor post executable.
		MetasoundInputNodePrivate::TResetablePostExecutableInputOperator<DataType> // Use this input operator if the data type is owned by the input node and is post executable.
	>;

	/** Choose pass through operator based upon data type and access type */
	template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
	using TPassThroughOperator = std::conditional_t<
		VertexAccess == EVertexAccessType::Value,
		MetasoundInputNodePrivate::TInputValueOperator<DataType>,
		MetasoundInputNodePrivate::FNonExecutableInputPassThroughOperator
	>;

	namespace MetasoundInputNodePrivate
	{
		// Factory for creating input operators. 
		template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
		class TInputNodeOperatorFactory : public IOperatorFactory
		{
			static constexpr bool bIsReferenceVertexAccess = VertexAccess == EVertexAccessType::Reference;
			static constexpr bool bIsValueVertexAccess = VertexAccess == EVertexAccessType::Value;

			static_assert(bIsValueVertexAccess || bIsReferenceVertexAccess, "Unsupported EVertexAccessType");

			// Choose which data reference type is created based on template parameters
			using FPassThroughDataReference = std::conditional_t<bIsReferenceVertexAccess, TDataReadReference<DataType>, TDataValueReference<DataType>>;

			// Return correct data reference type based on vertex access type for pass through scenario.
			FPassThroughDataReference CreatePassThroughDataReference(const FAnyDataReference& InRef)
			{
				if constexpr (bIsReferenceVertexAccess)
				{
					return InRef.GetDataReadReference<DataType>();
				}
				else if constexpr (bIsValueVertexAccess)
				{
					return InRef.GetDataValueReference<DataType>();
				}
				else
				{
					static_assert("Unsupported EVertexAccessType");
				}
			}

		public:
			explicit TInputNodeOperatorFactory(const FVertexName& InVertexName)
			: VertexName(InVertexName)
			{
			}

			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				using namespace MetasoundInputNodePrivate;

				if (const FAnyDataReference* Ref = InParams.InputData.FindDataReference(VertexName))
				{
					if constexpr (bIsReferenceVertexAccess)
					{
						if (EDataReferenceAccessType::Write == Ref->GetAccessType())
						{
							return MakeUnique<TNonOwningInputOperator<DataType, VertexAccess>>(VertexName, Ref->GetDataWriteReference<DataType>());
						}
					}
					// Pass through input value
					return MakeUnique<TPassThroughOperator<DataType, VertexAccess>>(VertexName, CreatePassThroughDataReference(*Ref));
				}
				else
				{
					// Owned input value
					return MakeUnique<TInputOperator<DataType, VertexAccess>>(VertexName, InParams.OperatorSettings, InParams.InputData);
				}
			}

		private:
			FVertexName VertexName;
		};
	}

	/** FInputNode represents an input to a metasound graph. */
	class FInputNode : public FBasicNode
	{
		static FLazyName ConstructorVariant;
		// Use Variant names to differentiate between normal input nodes and constructor 
		// input nodes.
		static FName GetVariantName(EVertexAccessType InVertexAccess);

		static FVertexInterface CreateVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess, const FLiteral& InLiteral);

	protected:

		static UE_API FVertexInterface CreateDefaultVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess, const FLiteral* InDefaultLiteral=nullptr);

		UE_API explicit FInputNode(FOperatorFactorySharedRef InFactory, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata);

	public:
		static UE_API FText GetInputDescription();
		UE_DEPRECATED(5.6, "Use CreateNodeClassMetadata(...) instead")
		static UE_API FNodeClassMetadata GetNodeMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess);
		static UE_API FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccess);

		/* Construct a TInputNode using the TInputOperatorLiteralFactory<> and moving
		 * InParam to the TInputOperatorLiteralFactory constructor.*/
		UE_API explicit FInputNode(FInputNodeConstructorParams&& InParams, const FName& InDataTypeName, EVertexAccessType InVertexAccess, FOperatorFactorySharedRef InFactory);


		UE_DEPRECATED(5.6, "You can find the vertex name by inspecting the FVertexInterface.")
		UE_API const FVertexName& GetVertexName() const;

		UE_API virtual TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override;

	private:
		FOperatorFactorySharedRef Factory;
	};


	/** TInputNode represents an input to a metasound graph. */
	template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
	class TInputNode : public FInputNode
	{
		static constexpr bool bIsConstructorInput = VertexAccess == EVertexAccessType::Value;
		static constexpr bool bIsSupportedConstructorInput = TIsConstructorVertexSupported<DataType>::Value && bIsConstructorInput;
		static constexpr bool bIsReferenceInput = VertexAccess == EVertexAccessType::Reference;
		static constexpr bool bIsSupportedReferenceInput = TLiteralTraits<DataType>::bIsParsableFromAnyLiteralType && bIsReferenceInput;

		static constexpr bool bIsSupportedInput = bIsSupportedConstructorInput || bIsSupportedReferenceInput;

	public:
		// If true, this node can be instantiated by the Frontend.
		static constexpr bool bCanRegister = bIsSupportedInput;

		UE_DEPRECATED(5.6, "Use CreateNodeClassMetadata(...) instead")
		static FNodeClassMetadata GetNodeInfo(const FVertexName& InVertexName)
		{
			return FInputNode::CreateNodeClassMetadata(InVertexName, GetMetasoundDataTypeName<DataType>(), VertexAccess);
		}

		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InVertexName)
		{
			return FInputNode::CreateNodeClassMetadata(InVertexName, GetMetasoundDataTypeName<DataType>(), VertexAccess);
		}

		/* Construct a TInputNode using the TInputOperatorLiteralFactory<> and moving
		 * InParam to the TInputOperatorLiteralFactory constructor.*/
		explicit TInputNode(FInputNodeConstructorParams&& InParams)
		:	FInputNode(MoveTemp(InParams), GetMetasoundDataTypeName<DataType>(), VertexAccess, MakeShared<MetasoundInputNodePrivate::TInputNodeOperatorFactory<DataType, VertexAccess>>(InParams.VertexName))
		{
		}

		explicit TInputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		: FInputNode(MakeShared<MetasoundInputNodePrivate::TInputNodeOperatorFactory<DataType, VertexAccess>>(InVertexName), MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		{
		}
	};
} // namespace Metasound

#undef UE_API
