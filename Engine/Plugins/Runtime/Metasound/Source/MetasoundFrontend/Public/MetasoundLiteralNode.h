// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundInputNode.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundVertex.h"

#define UE_API METASOUNDFRONTEND_API

#define LOCTEXT_NAMESPACE "MetasoundLiteralNode"

namespace Metasound
{
	namespace LiteralNodeNames
	{
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, OutputValue)
	}

	template<typename DataType>
	class UE_DEPRECATED(5.3, "The TLiteralOperator will be removed.") TLiteralOperator : public FNoOpOperator
	{
		static_assert(!TExecutableDataType<DataType>::bIsExecutable, "TLiteralOperator is only suitable for non-executable data types");

	public:
		TLiteralOperator(TDataValueReference<DataType> InValue)
		: Value (InValue)
		{
		}

		virtual ~TLiteralOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace LiteralNodeNames;
			InVertexData.BindValueVertex(METASOUND_GET_PARAM_NAME(OutputValue), Value);
		}

	private:
		TDataValueReference<DataType> Value;
	};

	/** TExecutableLiteralOperator is used for executable types. The majority of literals
	 * are constant values, but executable data types cannot be constant values.
	 * 
	 * Note: this is supporting for FTrigger literals which are deprecated.
	 */
	template<typename DataType>
	class UE_DEPRECATED(5.3, "The TExecutableLiteralOperator will be removed.") TExecutableLiteralOperator : public TExecutableOperator<TExecutableLiteralOperator<DataType>>
	{
		static_assert(TExecutableDataType<DataType>::bIsExecutable, "TExecutableLiteralOperator is only suitable for executable data types");

	public:
		using FDataWriteReference = TDataWriteReference<DataType>;

		TExecutableLiteralOperator(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			// Executable DataTypes require a copy of the output to operate on whereas non-executable
			// types do not. Avoid copy by assigning to reference for non-executable types.
			: Literal(InLiteral)
			, InputValue(TDataWriteReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral))
			, OutputValue(TDataWriteReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral))
		{
		}

		virtual ~TExecutableLiteralOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
		{
			using namespace LiteralNodeNames;
			InVertexData.BindValueVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		virtual FDataReferenceCollection GetInputs() const override
		{
			FDataReferenceCollection Inputs;
			return Inputs;
		}

		virtual FDataReferenceCollection GetOutputs() const override
		{
			using namespace LiteralNodeNames;
			FDataReferenceCollection Outputs;
			Outputs.AddDataReadReference<DataType>(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
			return Outputs;
		}

		void Execute()
		{
			TExecutableDataType<DataType>::Execute(*InputValue, *OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			*OutputValue = *InputValue = TDataTypeLiteralFactory<DataType>::CreateExplicitArgs(InParams.OperatorSettings, Literal);
		}

	private:
		FLiteral Literal;
		FDataWriteReference InputValue;
		FDataWriteReference OutputValue;
	};


	/** TLiteralOperatorLiteralFactory creates an input by passing it a literal. */
	template<typename DataType>
	class TLiteralOperatorLiteralFactory : public IOperatorFactory
	{
	public:

		using FDataWriteReference = TDataWriteReference<DataType>;

		TLiteralOperatorLiteralFactory(FLiteral&& InInitParam)
			: InitParam(MoveTemp(InInitParam))
		{
		}

		virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;

	private:

		class FLiteralOperator : public TInputOperator<DataType>
		{
		public:
			using TInputOperator<DataType>::TInputOperator;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				// Do not bind inputs as there are no inputs exposed from the literal node. 
				// This overrides the implementation already in the TInputOperator<>
			}
		};

		FLiteral InitParam;
	};

	/** TLiteralNode represents a variable within a metasound graph. */
	class FLiteralNode : public FNode
	{
	protected:
		static UE_API FVertexInterface CreateVertexInterface(const FName& InDataTypeName, EVertexAccessType InAccessType);

	public:

		static UE_API FNodeClassMetadata GetNodeMetadata(const FName& InDataTypeName, EVertexAccessType InAccessType);

		UE_API FLiteralNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FName& InDataTypeName, EVertexAccessType InAccessType, FOperatorFactorySharedRef InFactory);

		UE_API virtual const FVertexInterface& GetVertexInterface() const override;

		UE_API virtual TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override;

	private:
		FVertexInterface Interface;
		FOperatorFactorySharedRef Factory;
	};

	/** TLiteralNode represents a variable within a metasound graph. */
	template<typename DataType>
	class TLiteralNode : public FLiteralNode
	{
		// Executable data types handle Triggers which need to be advanced every
		// block.
		static constexpr bool bIsExecutableDataType = TExecutableDataType<DataType>::bIsExecutable;
		static constexpr bool bIsPostExecutableDataType = TPostExecutableDataType<DataType>::bIsPostExecutable;

		// If the data type is executable, we treat it as a reference since it is non-const
		static constexpr EVertexAccessType AccessType = (bIsExecutableDataType || bIsPostExecutableDataType) ? EVertexAccessType::Reference : EVertexAccessType::Value;

	public:

		UE_DEPRECATED(5.3, "Use the default interface on the node class metadata")
		static FVertexInterface DeclareVertexInterface()
		{
			return CreateVertexInterface(GetMetasoundDataTypeName<DataType>(), AccessType);
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			return GetNodeMetadata(GetMetasoundDataTypeName<DataType>(), AccessType);
		}

		/* Construct a TLiteralNode using the TLiteralOperatorLiteralFactory<> and moving
		 * InParam to the TLiteralOperatorLiteralFactory constructor.*/
		explicit TLiteralNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, FLiteral&& InParam)
		:	FLiteralNode(InInstanceName, InInstanceID, GetMetasoundDataTypeName<DataType>(), AccessType, MakeOperatorFactoryRef<TLiteralOperatorLiteralFactory<DataType>>(MoveTemp(InParam)))
		{
		}

	};

	template<typename DataType>
	TUniquePtr<IOperator> TLiteralOperatorLiteralFactory<DataType>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace LiteralNodeNames;
		return MakeUnique<FLiteralOperator>(METASOUND_GET_PARAM_NAME(OutputValue), InParams.OperatorSettings, InitParam);
	}

} // namespace Metasound
#undef LOCTEXT_NAMESPACE // MetasoundLiteralNode

#undef UE_API
