// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundLiteral.h"
#include "MetasoundVertex.h"


namespace Metasound
{
	// Base implementation for NodeConstructorCallbacks
	struct FDefaultNamedVertexNodeConstructorParams
	{
		// the instance name and name of the specific connection that should be used.
		FVertexName NodeName;
		FGuid InstanceID;
		FVertexName VertexName;
	};

	struct FDefaultNamedVertexWithLiteralNodeConstructorParams
	{
		// the instance name and name of the specific connection that should be used.
		FVertexName NodeName;
		FGuid InstanceID;
		FVertexName VertexName;
		FLiteral InitParam = FLiteral::CreateInvalid();
	};

	struct FDefaultLiteralNodeConstructorParams
	{
		FVertexName NodeName;
		FGuid InstanceID;
		FLiteral Literal = FLiteral::CreateInvalid();
	};

	struct FInputNodeConstructorParams
	{
		// the instance name and name of the specific connection that should be used.
		FVertexName NodeName;
		FGuid InstanceID;
		FVertexName VertexName;
		FLiteral InitParam = FLiteral::CreateInvalid();

		// If defined this is the polymorphic type name to use.
		FName PolymorphicTypeName;

		UE_DEPRECATED(5.3, "Transmission is no longer enabled on input nodes. In order set values on a MetaSound input, access them through the bound FInputVertexInterfaceData.")
		bool bEnableTransmission = false;
	};

	using FOutputNodeConstructorParams = FDefaultNamedVertexNodeConstructorParams;
	using FLiteralNodeConstructorParams = FDefaultLiteralNodeConstructorParams;
	using FVariableNodeConstructorParams = FDefaultLiteralNodeConstructorParams;
} // namespace Metasound
