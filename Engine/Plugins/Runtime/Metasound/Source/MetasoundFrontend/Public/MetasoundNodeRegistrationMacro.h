// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundBasicNode.h"
#include "MetasoundFrontend.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundLog.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"

#include "Traits/MetasoundNodeConstructorTraits.h"
#include "Traits/MetasoundNodeStaticMemberTraits.h"

#define UE_API METASOUNDFRONTEND_API

// In UE 5.6, registered node are expected to support the constructor signature Constructor(FNodeData, TSharedRef<const FNodeClassMetadata>)
// Because there are many existing nodes, it may take time to update them. For convenience, the deprecations related to this change are
// configurable via a preprocessor macro so that the deprecation warnings do not drown out other compiler errors and warnings.
#ifndef UE_METASOUND_DISABLE_5_6_NODE_REGISTRATION_DEPRECATION_WARNINGS
#define UE_METASOUND_DISABLE_5_6_NODE_REGISTRATION_DEPRECATION_WARNINGS (0)
#endif

namespace Metasound::Frontend
{

	namespace NodeRegistrationPrivate
	{
		// Utilize base class to reduce template bloat in TNodeRegistryEntry
		class FNodeRegistryEntryBase : public INodeClassRegistryEntry
		{
		public:
			UE_API FNodeRegistryEntryBase(const Metasound::FNodeClassMetadata& InMetadata, const FModuleInfo& InOwningModuleInfo);

			virtual ~FNodeRegistryEntryBase() = default;

			UE_API virtual const FMetasoundFrontendClass& GetFrontendClass() const override;

			UE_API virtual const TSet<FMetasoundFrontendVersion>* GetImplementedInterfaces() const override;

			UE_API virtual FVertexInterface GetDefaultVertexInterface() const override;

			UE_API virtual const FClassInterface& GetClassInterface() const override;

			UE_API virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const override;
			UE_API virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override;

#if WITH_EDITORONLY_DATA
			UE_API virtual FName GetPluginName() const override;

			UE_API virtual FName GetModuleName() const override;
#endif

		protected:

			UE_API TSharedRef<const FNodeClassMetadata> GetNodeClassMetadata() const;

		private:
			TSharedRef<FNodeClassMetadata> ClassMetadata;
			FMetasoundFrontendClass FrontendClass;

#if WITH_EDITORONLY_DATA
			FName PluginName;
			FName ModuleName;
#endif
		};

		template<typename TNodeType>
		class TNodeRegistryEntryBase : public FNodeRegistryEntryBase
		{
		public:
			// Expose FNodeRegistryEntryBase constructors. 
			using FNodeRegistryEntryBase::FNodeRegistryEntryBase;

			virtual TUniquePtr<INode> CreateNode(FNodeData InNodeData) const override
			{
				if constexpr(std::is_constructible_v<TNodeType, ::Metasound::FNodeData, TSharedRef<const FNodeClassMetadata>>)
				{
					// Prefer construction of nodes using (FNodeData, TShareRef<const FNodeClassMetadata>)
					return MakeUnique<TNodeType>(MoveTemp(InNodeData), GetNodeClassMetadata());
				}
				else if constexpr(std::is_constructible_v<TNodeType, ::Metasound::FNodeData>)
				{
					// Some node classes have FNodeClassMetadata declared as static members on
					// the node class and do not need a separate TSharedRef<const FNodeClassMetadata>. 
					return MakeUnique<TNodeType>(MoveTemp(InNodeData));
				}
				else
				{
					checkNoEntry();
					return nullptr;
				}
			}
		};

		// A node registry entry which also provides a node extension.
		template<typename NodeType, typename ConfigurationType>
		class TNodeRegistryEntry : public TNodeRegistryEntryBase<NodeType>
		{
			static_assert(std::is_base_of_v<FMetaSoundFrontendNodeConfiguration, ConfigurationType>, "Configurations must inherit from FMetaSoundFrontendNodeConfiguration");
		public:
			TNodeRegistryEntry(const Metasound::FNodeClassMetadata& InMetadata, const FModuleInfo& InOwningModuleInfo)
				: TNodeRegistryEntryBase<NodeType>(InMetadata, InOwningModuleInfo)
			{
				const Metasound::FClassInterface& ClassInterface = this->GetNodeClassMetadata()->DefaultInterface;
				if (ClassInterface.ContainsSubInterfaces() || ClassInterface.ContainsVariants())
				{
					constexpr bool bDerivedFromBaseConfig = std::is_base_of_v<FMetaSoundDynamicInterfaceNodeConfiguration, ConfigurationType>;
					ensureMsgf(bDerivedFromBaseConfig,
						TEXT("Node with sub-interfaces/variants must use a config deriving from FMetaSoundDynamicInterfaceNodeConfiguration"));
				}
			}

			virtual TInstancedStruct<FMetaSoundFrontendNodeConfiguration> CreateFrontendNodeConfiguration() const override
			{
				return TInstancedStruct<FMetaSoundFrontendNodeConfiguration>::Make<ConfigurationType>();
			}

			virtual bool IsCompatibleNodeConfiguration(TConstStructView<FMetaSoundFrontendNodeConfiguration> InNodeConfiguration) const override 
			{
				return InNodeConfiguration.GetScriptStruct() == ConfigurationType::StaticStruct();
			}
		};

		// A partial template specialization for scenario where no node extension is provided.
		template<typename NodeType>
		class TNodeRegistryEntry<NodeType, void> : public TNodeRegistryEntryBase<NodeType>
		{
		public:
			using TNodeRegistryEntryBase<NodeType>::TNodeRegistryEntryBase;
		};

		UE_API bool RegisterNodeInternal(TUniquePtr<INodeClassRegistryEntry>&& InEntry);
	} // namespace NodeRegistrationPrivate

	template<typename TNodeType, typename ConfigurationType=void>
	bool RegisterNode(const FNodeClassMetadata& InMetadata, const FModuleInfo& InOwningModuleInfo)
	{
		using namespace NodeRegistrationPrivate;
		if constexpr (std::is_void_v<ConfigurationType>)
		{
			// Auto-supply FMetaSoundDynamicInterfaceNodeConfiguration for nodes with
			// sub-interfaces or variants that were registered without an
			// explicit configuration type (i.e. via METASOUND_REGISTER_NODE).
			const Metasound::FClassInterface& ClassInterface = InMetadata.DefaultInterface;
			if (ClassInterface.ContainsSubInterfaces() || ClassInterface.ContainsVariants())
			{
				return NodeRegistrationPrivate::RegisterNodeInternal(MakeUnique<TNodeRegistryEntry<TNodeType, FMetaSoundDynamicInterfaceNodeConfiguration>>(InMetadata, InOwningModuleInfo));
			}
			return NodeRegistrationPrivate::RegisterNodeInternal(MakeUnique<TNodeRegistryEntry<TNodeType, void>>(InMetadata, InOwningModuleInfo));
		}
		else
		{
			return NodeRegistrationPrivate::RegisterNodeInternal(MakeUnique<TNodeRegistryEntry<TNodeType, ConfigurationType>>(InMetadata, InOwningModuleInfo));
		}
	}
 

	template <typename TNodeType, typename ConfigurationType=void>
	UE_DEPRECATED(5.7, "Use RegisterNode(...) which provides FModuleInfo")
	bool RegisterNode(const FNodeClassMetadata& InMetadata)
	{
		return RegisterNode<TNodeType, ConfigurationType>(InMetadata, FModuleInfo{});
	} 

	template <typename TNodeType, typename ConfigurationType=void>
	bool RegisterNode(const FModuleInfo& InOwningModuleInfo)
	{
		static_assert(::Metasound::TIsNodeConstructorSupported<TNodeType>::Value, "In order to be registered as a MetaSound node, the node needs to implement the following public constructor: Constructor(Metasound::FNodeData InNodeData) or Construct(Metasound::FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)");

		using namespace NodeRegistrationPrivate;
		if constexpr(TIsCreateNodeClassMetadataDeclared<TNodeType>::Value)
		{
			return RegisterNode<TNodeType, ConfigurationType>(TNodeType::CreateNodeClassMetadata(), InOwningModuleInfo);
		}
		else
		{
			static_assert(sizeof(TNodeType) == 0, "Node class must implement 'static FNodeClassMetadata CreateNodeClassMetadata()'");
			return false;
		}
	}

	template <typename TNodeType, typename ConfigurationType=void>
	UE_DEPRECATED(5.7, "Use RegisterNode(...) which provides FModuleInfo")
	bool RegisterNode()
	{
		return RegisterNode<TNodeType, ConfigurationType>(FModuleInfo{});
	}

	/** Unregister a node using a registry key. */
	UE_API bool UnregisterNode(const FNodeClassRegistryKey& InRegistryKey, const FModuleInfo& InOwningModuleInfo);
	
	/** Unregister a node using FNodeClassMetadata. */
	UE_API bool UnregisterNode(const FNodeClassMetadata& InMetadata, const FModuleInfo& InOwningModuleInfo);

	/** Unregister a node by node template param. */
	template<typename TNodeType>
	bool UnregisterNode(const FModuleInfo& InOwningModuleInfo)
	{
		using namespace NodeRegistrationPrivate;
		if constexpr(TIsCreateNodeClassMetadataDeclared<TNodeType>::Value)
		{
			return UnregisterNode(FNodeClassRegistryKey{TNodeType::CreateNodeClassMetadata()}, InOwningModuleInfo);
		}
		else
		{
			static_assert(sizeof(TNodeType) == 0, "Node class must implement 'static FNodeClassMetadata CreateNodeClassMetadata()'");
			return false;
		}
	}

	template<typename TNodeType>
	UE_DEPRECATED(5.7, "Use UnregisterNode(...) which provides FModuleInfo")
	bool UnregisterNode()
	{
		return UnregisterNode<TNodeType>(FModuleInfo{});
	}
}

namespace Metasound
{
	template<typename T, typename ConfigurationType=void>
	UE_DEPRECATED(5.6, "Use Frontend::RegisterNode()")
	bool RegisterNodeWithFrontend()
	{
		return Frontend::RegisterNode<T, ConfigurationType>();
	}

	template<typename T, typename ConfigurationType=void>
	UE_DEPRECATED(5.6, "Use Frontend::RegisterNode(const Metasound::FNodeClassMetadata&)")
	bool RegisterNodeWithFrontend(const Metasound::FNodeClassMetadata& InMetadata)
	{
		return Frontend::RegisterNode<T, ConfigurationType>(InMetadata);
	}
}

#define METASOUND_REGISTER_NODE_AND_CONFIGURATION(NodeClass, ConfigurationClass) \
	 static_assert(std::is_base_of<::Metasound::INodeBase, NodeClass>::value, "To be registered as a  Metasound Node," #NodeClass "need to be a derived class from Metasound::INodeBase, Metasound::INode, or Metasound::FNode."); \
	 static_assert(::Metasound::TIsNodeConstructorSupported<NodeClass>::Value, "In order to be registered as a Metasound Node, " #NodeClass " needs to implement the following public constructor: " #NodeClass "(Metasound::FNodeData InNodeData);"); \
	 METASOUND_IMPLEMENT_REGISTRATION_ACTION(NodeClass, (::Metasound::Frontend::RegisterNode<NodeClass, ConfigurationClass>), (::Metasound::Frontend::UnregisterNode<NodeClass>));


#define METASOUND_REGISTER_NODE(NodeClass) METASOUND_REGISTER_NODE_AND_CONFIGURATION(NodeClass, void)

/*
Macros to help define various FText node fields.
*/
#if WITH_EDITOR
#define METASOUND_LOCTEXT(KEY, NAME_TEXT) LOCTEXT(KEY, NAME_TEXT)
#define METASOUND_LOCTEXT_FORMAT(KEY, NAME_TEXT, ...) FText::Format(LOCTEXT(KEY, NAME_TEXT), __VA_ARGS__)
#else 
#define METASOUND_LOCTEXT(KEY, NAME_TEXT) FText{}
#define METASOUND_LOCTEXT_FORMAT(KEY, NAME_TEXT, ...) FText{}
#endif // WITH_EDITOR

#undef UE_API
