// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "MetasoundBasicNode.h"
#include "MetasoundBuildError.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontendDataTypeTraits.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundNode.h"
#include "MetasoundVertex.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDFRONTEND_API

#define LOCTEXT_NAMESPACE "MetasoundGraphCore"

namespace Metasound
{
	namespace OutputNodePrivate
	{
		class FOutputOperatorFactory;
	}

	/** FMissingOutputNodeInputReferenceError
	 *
	 * Caused by Output not being able to generate an IOperator instance due to
	 * the type requiring an input reference (i.e. it is not default constructable).
	 */
	class FMissingOutputNodeInputReferenceError : public FBuildErrorBase
	{
	public:
		FMissingOutputNodeInputReferenceError(const INode& InNode, const FText& InDataType)
			: FBuildErrorBase(
				"MetasoundMissingOutputDataReferenceForTypeError",
				METASOUND_LOCTEXT_FORMAT("MissingOutputNodeInputReferenceForTypeError", "Missing required output node input reference for type {0}.", InDataType))
		{
			AddNode(InNode);
		}

		FMissingOutputNodeInputReferenceError(const INode& InNode)
			: FBuildErrorBase(
				"MetasoundMissingOutputDataReferenceError",
				METASOUND_LOCTEXT("MissingOutputNodeInputReferenceError", "Missing required output node input reference."))
		{
			AddNode(InNode);
		}

		virtual ~FMissingOutputNodeInputReferenceError() = default;
	};

	class FOutputNode : public FBasicNode
	{
	public:
		UE_API FOutputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata);

		static UE_API FName GetVariantName(EVertexAccessType InVertexAccessType);
		static UE_API FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType);
		static UE_API FVertexInterface CreateVertexInterface(const FVertexName& InVertexName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType);

		UE_API virtual TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override;

	protected:
		UE_API FOutputNode(const FVertexName& InVertexName, const FGuid& InInstanceID, const FVertexName& InInstanceName, const FName& InDataTypeName, EVertexAccessType InVertexAccessType);

	private:
		TSharedRef<OutputNodePrivate::FOutputOperatorFactory, ESPMode::ThreadSafe> Factory;
	};

	/** Output nodes are used to expose graph data to external entities. */
	template<typename DataType, EVertexAccessType VertexAccess=EVertexAccessType::Reference>
	class TOutputNode : public FOutputNode
	{
	public:
		TOutputNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FVertexName& InVertexName)
		:	FOutputNode(InVertexName, InInstanceID, InInstanceName, GetMetasoundDataTypeName<DataType>(), VertexAccess)
		{
		}

		TOutputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		:	FOutputNode(InVertexName, MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		{
		}
	};
}
#undef LOCTEXT_NAMESPACE // MetasoundOutputNode

#undef UE_API
