// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	class IOutputStorage
	{
	public:
		virtual ~IOutputStorage() = default;

		virtual FName GetDataTypeName() const = 0;

		virtual TUniquePtr<IOutputStorage> Clone() const = 0;
	};
	
	template<typename DataType>
	class TOutputStorage final : public IOutputStorage
	{
	public:
		explicit TOutputStorage(const DataType& InData)
			: Data(InData)
		{}
		
		virtual FName GetDataTypeName() const override
		{
			static FName TypeName = GetMetasoundDataTypeName<DataType>();
			return TypeName;
		}

		virtual TUniquePtr<IOutputStorage> Clone() const override
		{
			return MakeUnique<TOutputStorage<DataType>>(Data);
		}
		
		void Set(const DataType& Value)
		{
			Data = Value;
		}

		void Set(DataType&& Value)
		{
			Data = MoveTemp(Value);
		}

		const DataType& Get() const
		{
			return Data;
		}

	private:
		DataType Data;
	};
}
