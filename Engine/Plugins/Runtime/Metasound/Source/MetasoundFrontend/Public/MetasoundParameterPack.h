// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/AlignmentTemplates.h"

#include "MetasoundFrontendLiteral.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundRouter.h"
#include "MetasoundFrontendDocument.h"

#include "MetasoundParameterPack.generated.h"

#define UE_API METASOUNDFRONTEND_API

class UMetaSoundSource;

// A structure that encapsulates a "parameter tag" in the raw 'bag-o-bytes'
// that will be the parameter storage. It lets us walk through memory to 
// find a parameter with a specific name and specific type. 
namespace MetasoundParameterPackPrivate
{
	struct FMetasoundParameterPackItemBase
	{
		FMetasoundParameterPackItemBase(const FName& InName, const FName& InTypeName, uint16 InNextOffset, uint16 InValueOffset)
			: Name(InName)
			, TypeName(InTypeName)
			, NextHeaderOffset(InNextOffset)
			, ValueOffset(InValueOffset)
		{}
		FMetasoundParameterPackItemBase(const FMetasoundParameterPackItemBase& Other) = default;
		void* GetPayload()
		{
			uint8* ptr = reinterpret_cast<uint8*>(this);
			return ptr + ValueOffset;
		}
		FName  Name;
		FName  TypeName;
		uint16 NextHeaderOffset;
		uint16 ValueOffset;
	};

	template<bool, typename T, int32 MaxNumItems = 0>
	struct TRootValueTypeHelper { using Type = T; };

	template<typename T, int32 MaxNumItems>
	struct TRootValueTypeHelper<true, T, MaxNumItems> { using Type = Metasound::TParamPackFixedArray<typename T::ElementType, MaxNumItems>; };

	template<typename T, int32 MaxNumItems = 0>
	struct TRootValueType : TRootValueTypeHelper<TIsTArray_V<T>, T, MaxNumItems> {};

	// A template that lets us stick an arbitrary data type into the raw 
	// 'bag-o-bytes' that is the parameter storage. Uses the base class 
	// above.
	template <typename T, int32 MaxNumItems = 0>
	struct FMetasoundParameterPackItem : public FMetasoundParameterPackItemBase
	{
		using ThisType = FMetasoundParameterPackItem<T,MaxNumItems>;
		FMetasoundParameterPackItem(const FName& InName, const FName& InTypeName, const T& InValue)
			: FMetasoundParameterPackItemBase(InName, InTypeName, SizeOfParam(InValue), offsetof(FMetasoundParameterPackItem<ThisType>, Value))
			, Value(InValue)
		{
			if constexpr (TIsTArray_V<T>)
			{
				check(InValue.Num() <= MaxNumItems);
			}
		}
		FMetasoundParameterPackItem(const FMetasoundParameterPackItem& Other) = default;
		typename TRootValueType<T, MaxNumItems>::Type Value;
	
		static uint16 SizeOfParam(const T& InValue)
		{
			return AlignArbitrary<uint16>((uint16)sizeof(ThisType), alignof(std::max_align_t));
		}
	};
}

// This next class owns the 'bag-o-bytes' that holds all of the parameters. 
struct FMetasoundParameterPackStorage
{
	TArray<uint8> Storage;	
	int32         StorageBytesInUse = 0;

	FMetasoundParameterPackStorage() = default;
	FMetasoundParameterPackStorage(const FMetasoundParameterPackStorage& Other)
		: Storage(Other.Storage)
		, StorageBytesInUse(Other.StorageBytesInUse)
	{}
	FMetasoundParameterPackStorage(FMetasoundParameterPackStorage&& Other)
		: Storage(MoveTemp(Other.Storage))
		, StorageBytesInUse(Other.StorageBytesInUse)
	{
		Other.StorageBytesInUse = 0;
	}

	struct ParameterIterator
	{
		uint8* CurrentNode;
		uint8* EndAddress;

		ParameterIterator(uint8* Start, uint8* End)
			: CurrentNode(Start)
			, EndAddress(End)
		{}

		ParameterIterator& operator++()
		{
			using namespace MetasoundParameterPackPrivate;

			if (!CurrentNode)
			{
				return *this; 
			}
			MetasoundParameterPackPrivate::FMetasoundParameterPackItemBase * ParamAddress = reinterpret_cast<FMetasoundParameterPackItemBase*>(CurrentNode);
			CurrentNode += ParamAddress->NextHeaderOffset;
			if (CurrentNode >= EndAddress)
			{
				CurrentNode = nullptr;
			}
			return *this;
		}

		ParameterIterator operator++(int)
		{
			ParameterIterator ReturnValue = *this;
			++*this;
			return ReturnValue;
		}

		bool operator==(const ParameterIterator& Other) const
		{
			return CurrentNode == Other.CurrentNode;
		}

		bool operator!=(const ParameterIterator& Other) const
		{
			return CurrentNode != Other.CurrentNode;
		}

		MetasoundParameterPackPrivate::FMetasoundParameterPackItemBase* operator->()
		{
			using namespace MetasoundParameterPackPrivate;
			return reinterpret_cast<FMetasoundParameterPackItemBase*>(CurrentNode);
		}

		MetasoundParameterPackPrivate::FMetasoundParameterPackItemBase* GetParameterBase()
		{
			using namespace MetasoundParameterPackPrivate;
			return reinterpret_cast<FMetasoundParameterPackItemBase*>(CurrentNode);
		}

	};

	ParameterIterator begin()
	{
		if (Storage.IsEmpty())
		{
			return ParameterIterator(nullptr, nullptr);
		}
		return ParameterIterator(&Storage[0], &Storage[0] + StorageBytesInUse);
	}

	ParameterIterator end()
	{
		return ParameterIterator(nullptr, nullptr);
	}

	template<typename T, int32 MaxNumItems = 0>
	MetasoundParameterPackPrivate::FMetasoundParameterPackItem<T, MaxNumItems>* FindParameter(const FName& ParamName, const FName& InTypeName)
	{
		using namespace MetasoundParameterPackPrivate;
		if (Storage.IsEmpty())
		{
			return nullptr;
		}
		FMetasoundParameterPackItemBase* ParamWalker = reinterpret_cast<FMetasoundParameterPackItemBase*>(&Storage[0]);
		int32 StorageOffset = 0;
		while (ParamWalker)
		{
			if (ParamWalker->Name == ParamName)
			{
				if (ParamWalker->TypeName == InTypeName)
				{
					return static_cast<FMetasoundParameterPackItem<T, MaxNumItems>*>(ParamWalker);
				}
				return nullptr;
			}
			StorageOffset += ParamWalker->NextHeaderOffset;
			if (StorageOffset < StorageBytesInUse)
			{
				ParamWalker = reinterpret_cast<FMetasoundParameterPackItemBase*>(&Storage[StorageOffset]);
			}
			else
			{
				break;
			}
		}
		return nullptr;
	}

	// non-class version
	template<typename T>
	std::enable_if_t<!std::is_class_v<T>,T*> AddParameter(const FName& Name, const FName& InTypeName, const T& InValue)
	{
		using namespace MetasoundParameterPackPrivate;
		int32 NextParamLocation = AlignArbitrary<int32>(StorageBytesInUse, alignof(std::max_align_t));
		int32 TotalStorageNeeded = NextParamLocation + FMetasoundParameterPackItem<T>::SizeOfParam(InValue);
		if (TotalStorageNeeded > Storage.Num())
		{
			Storage.AddUninitialized(TotalStorageNeeded - Storage.Num());
		}
		FMetasoundParameterPackItem<T>* Destination = new (reinterpret_cast<FMetasoundParameterPackItem<T>*>(&Storage[NextParamLocation])) FMetasoundParameterPackItem<T>(Name, InTypeName, InValue);
		StorageBytesInUse = NextParamLocation + Destination->NextHeaderOffset;
		return &Destination->Value;
	}

	// class version
	template<typename T>
	std::enable_if_t<std::is_class_v<T> && !TIsTArray_V<T>, T*> AddParameter(const FName& Name, const FName& InTypeName, const T& InValue)
	{
		using namespace MetasoundParameterPackPrivate;
		int32 NextParamLocation = AlignArbitrary<int32>(StorageBytesInUse, alignof(std::max_align_t));
		int32 TotalStorageNeeded = NextParamLocation + FMetasoundParameterPackItem<T>::SizeOfParam(InValue);
		if (TotalStorageNeeded > Storage.Num())
		{
			Storage.AddUninitialized(TotalStorageNeeded - Storage.Num());
		}
		FMetasoundParameterPackItem<T>* Destination = new (reinterpret_cast<FMetasoundParameterPackItem<T>*>(&Storage[NextParamLocation])) FMetasoundParameterPackItem<T>(Name, InTypeName, InValue);
		StorageBytesInUse = NextParamLocation + Destination->NextHeaderOffset;
		return &Destination->Value;
	}

	// array version
	template<typename T, int32 MaxNumItems>
	std::enable_if_t<std::is_class_v<T> && TIsTArray_V<T>, void> AddArrayParameter(const FName& Name, const FName& InTypeName, const T& InValue)
	{
		using namespace MetasoundParameterPackPrivate;
		check(InValue.Num() <= MaxNumItems);
		int32 NextParamLocation = AlignArbitrary<int32>(StorageBytesInUse, alignof(std::max_align_t));
		int32 TotalStorageNeeded = NextParamLocation + FMetasoundParameterPackItem<T, MaxNumItems>::SizeOfParam(InValue);
		if (TotalStorageNeeded > Storage.Num())
		{
			Storage.AddUninitialized(TotalStorageNeeded - Storage.Num());
		}
		FMetasoundParameterPackItem<T, MaxNumItems>* Destination = new (reinterpret_cast<FMetasoundParameterPackItem<T>*>(&Storage[NextParamLocation])) FMetasoundParameterPackItem<T, MaxNumItems>(Name, InTypeName, InValue);
		StorageBytesInUse = NextParamLocation + Destination->NextHeaderOffset;
	}
};

using FSharedMetasoundParameterStoragePtr = TSharedPtr<FMetasoundParameterPackStorage, ESPMode::ThreadSafe>;

UENUM(BlueprintType)
enum class ESetParamResult : uint8
{
	Succeeded,
	Failed
};

// Here is the UObject BlueprintType that can be used in c++ and blueprint code. It holds a FMetasoundParamStorage 
// instance and can pass it along to the audio system's SetObjectParameter function via an AudioProxy.
UCLASS(MinimalAPI, BlueprintType,meta = (DisplayName = "MetaSoundParameterPack"))
class UMetasoundParameterPack : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack")
	static UE_API UMetasoundParameterPack* MakeMetasoundParameterPack();

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API ESetParamResult SetBool(FName ParameterName, bool InValue, bool OnlyIfExists = true);
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API ESetParamResult SetInt(FName ParameterName, int32 InValue, bool OnlyIfExists = true);
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API ESetParamResult SetFloat(FName ParameterName, float InValue, bool OnlyIfExists = true);
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API ESetParamResult SetString(FName ParameterName, const FString& InValue, bool OnlyIfExists = true);
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API ESetParamResult SetTrigger(FName ParameterName, bool OnlyIfExists = true);

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	UE_API bool GetBool(FName ParameterName, ESetParamResult& Result) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	UE_API int32 GetInt(FName ParameterName, ESetParamResult& Result) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	UE_API float GetFloat(FName ParameterName, ESetParamResult& Result) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	UE_API FString GetString(FName ParameterName, ESetParamResult& Result) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = "Result"))
	UE_API bool GetTrigger(FName ParameterName, ESetParamResult& Result) const;

	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API bool HasBool(FName ParameterName) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API bool HasInt(FName ParameterName) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API bool HasFloat(FName ParameterName) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API bool HasString(FName ParameterName) const;
	UFUNCTION(BlueprintCallable, Category = "MetaSoundParameterPack", meta = (ExpandEnumAsExecs = ReturnValue))
	UE_API bool HasTrigger(FName ParameterName) const;

	UE_API void AddBoolParameter(FName Name, bool InValue);
	UE_API void AddIntParameter(FName Name, int32 InValue);
	UE_API void AddFloatParameter(FName Name, float InValue);
	UE_API void AddStringParameter(FName Name, const FString& InValue);
	UE_API void AddTriggerParameter(FName Name, bool InValue = true);

	// YIKES! BEWARE: If the returned pointers are only valid until another parameter is added!
	UE_API bool*    GetBoolParameterPtr(FName Name) const;
	UE_API int32*   GetIntParameterPtr(FName Name) const;
	UE_API float*   GetFloatParameterPtr(FName Name) const;
	UE_API FString* GetStringParameterPtr(FName Name) const;
	UE_API bool*    GetTriggerParameterPtr(FName Name) const;

	template<typename T>
	ESetParamResult SetParameter(const FName& Name, const FName& InTypeName, const T& InValue, bool OnlyIfExists = true)
	{
		typename MetasoundParameterPackPrivate::TRootValueType<T>::Type* TheParam = GetParameterPtr<T>(Name, InTypeName);
		if (TheParam)
		{
			*TheParam = InValue;
			return ESetParamResult::Succeeded;
		}
		else if (!OnlyIfExists)
		{
			// TODO Add test to see if we are allowed to create it!
			AddParameter<T>(Name, InTypeName, InValue);
			return ESetParamResult::Succeeded;
		}
		return ESetParamResult::Failed;
	}

	template<typename T, int32 MaxNumItems>
	ESetParamResult SetArrayParameter(const FName& Name, const FName& InTypeName, const T& InValue, bool OnlyIfExists = true)
	{
		static_assert(TIsTArray_V<T>);
		if (!ensureMsgf(InValue.Num() <= MaxNumItems, TEXT("Too many items in source array \"%s\"! Max is %d."), *Name.ToString(), MaxNumItems))
		{
			return ESetParamResult::Failed;
		}
		typename MetasoundParameterPackPrivate::TRootValueType<T, MaxNumItems>::Type* TheParam = GetParameterPtr<T,MaxNumItems>(Name, InTypeName);
		if (TheParam)
		{
			*TheParam = InValue;
			return ESetParamResult::Succeeded;
		}
		else if (!OnlyIfExists)
		{
			// TODO Add test to see if we are allowed to create it!
			AddArrayParameter<T, MaxNumItems>(Name, InTypeName, InValue);
			return ESetParamResult::Succeeded;
		}
		return ESetParamResult::Failed;
	}

	template<typename T, int32 MaxNumItems=0>
	bool HasParameter(const FName& Name, const FName& InTypeName) const
	{
		if (!ParameterStorage.IsValid())
		{
			return false;
		}
		return ParameterStorage->FindParameter<T,MaxNumItems>(Name, InTypeName) != nullptr;
	}

	template<typename T>
	void AddParameter(const FName& Name, const FName& InTypeName, const T& InValue)
	{
		if (!ParameterStorage.IsValid())
		{
			ParameterStorage = MakeShared<FMetasoundParameterPackStorage>();
		}
		ParameterStorage->AddParameter(Name, InTypeName, InValue);
	}

	template<typename T, int32 MaxNumItems>
	void AddArrayParameter(const FName& Name, const FName& InTypeName, const T& InValue)
	{
		if (!ParameterStorage.IsValid())
		{
			ParameterStorage = MakeShared<FMetasoundParameterPackStorage>();
		}
		ParameterStorage->AddArrayParameter<T,MaxNumItems>(Name, InTypeName, InValue);
	}

	template<typename T>
	T GetParameter(const FName& Name, const FName& InTypeName, ESetParamResult& Result) const
	{
		T* TheParameter = GetParameterPtr<T>(Name, InTypeName);
		if (!TheParameter)
		{
			Result = ESetParamResult::Failed;
			return T();
		}
		Result = ESetParamResult::Succeeded;
		return *TheParameter;
	}

	template<typename T, int32 MaxNumItems>
	void GetParameter(const FName& Name, const FName& InTypeName, TArray<typename T::ElementType>& ResultArray, ESetParamResult& Result) const
	{
		typename MetasoundParameterPackPrivate::TRootValueType<T, MaxNumItems>::Type* TheParameter = GetParameterPtr<T,MaxNumItems>(Name, InTypeName);
		if (!TheParameter)
		{
			Result = ESetParamResult::Failed;
			return;
		}
		TheParameter->CopyToArray(ResultArray);
		Result = ESetParamResult::Succeeded;
	}

	UE_API TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	
	// A couple of utilities for use by MetasoundAssetBase and MetasoundGenerator to set
	// up the routing for parameter packs
	UE_DEPRECATED(5.3, "FSendAddress are no longer used to communicate with MetaSound instances.")
	static UE_API Metasound::FSendAddress CreateSendAddressFromEnvironment(const Metasound::FMetasoundEnvironment& InEnvironment);
	static UE_API FMetasoundFrontendClassInput GetClassInput();

	UE_API FSharedMetasoundParameterStoragePtr GetParameterStorage() const;
	UE_API FSharedMetasoundParameterStoragePtr GetCopyOfParameterStorage() const;

private:
	template<typename T, int MaxNumItems = 0>
	typename MetasoundParameterPackPrivate::TRootValueType<T, MaxNumItems>::Type* GetParameterPtr(const FName& Name, const FName& InTypeName) const
	{
		if (!ParameterStorage.IsValid())
		{
			return nullptr;
		}
		auto TheParameter = ParameterStorage->FindParameter<T, MaxNumItems>(Name, InTypeName);
		if (!TheParameter)
		{
			return nullptr;
		}
		return &TheParameter->Value;
	}

	FSharedMetasoundParameterStoragePtr ParameterStorage;
};

// Here is the proxy that UMetasoundParameterPack creates when asked for a proxy by the audio system
class FMetasoundParameterPackProxy : public Audio::TProxyData<FMetasoundParameterPackProxy>
{
public:
	IMPL_AUDIOPROXY_CLASS(FMetasoundParameterPackProxy);

	explicit FMetasoundParameterPackProxy(FSharedMetasoundParameterStoragePtr& Data)
		: ParameterStoragePtr(Data)
	{}

	FSharedMetasoundParameterStoragePtr GetParamStorage()
	{
		return ParameterStoragePtr;
	}

private:
	FSharedMetasoundParameterStoragePtr ParameterStoragePtr;
};

// And finally... A type we can register with Metasound that holds a TSharedPtr to an FMetasoundParamStorage instance.
// It can be created by various systems in Metasound given a proxy.
class FMetasoundParameterStorageWrapper
{
	FSharedMetasoundParameterStoragePtr ParameterStoragePtr;

public:

	FMetasoundParameterStorageWrapper() = default;
	FMetasoundParameterStorageWrapper(const FMetasoundParameterStorageWrapper&) = default;
	FMetasoundParameterStorageWrapper& operator=(const FMetasoundParameterStorageWrapper& Other) = default;
	FMetasoundParameterStorageWrapper(const TSharedPtr<Audio::IProxyData>& InInitData)
	{
		if (InInitData.IsValid())
		{
			if (InInitData->CheckTypeCast<FMetasoundParameterPackProxy>())
			{
				// should we be getting handed a SharedPtr here?
				FMetasoundParameterPackProxy& AsParameterPack = InInitData->GetAs<FMetasoundParameterPackProxy>();
				ParameterStoragePtr = AsParameterPack.GetParamStorage();
			}
		}
	}

	FSharedMetasoundParameterStoragePtr Get() const
	{
		return ParameterStoragePtr;
	}

	bool IsPackValid() const
	{
		return ParameterStoragePtr.IsValid();
	}

	const FSharedMetasoundParameterStoragePtr& GetPackProxy() const
	{
		return ParameterStoragePtr;
	}

	const FMetasoundParameterPackStorage* operator->() const
	{
		return ParameterStoragePtr.Get();
	}

	FMetasoundParameterPackStorage* operator->()
	{
		return ParameterStoragePtr.Get();
	}

	friend inline uint32 GetTypeHash(const FMetasoundParameterStorageWrapper& InParameterStorageWrapper)
	{
		if (InParameterStorageWrapper.IsPackValid())
		{
			return GetTypeHash(InParameterStorageWrapper->Storage);
		}
		return INDEX_NONE;
	}
};

DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMetasoundParameterStorageWrapper, METASOUNDFRONTEND_API, FMetasoundParameterStorageWrapperTypeInfo, FMetasoundParameterStorageWrapperReadRef, FMetasoundParameterStorageWrapperWriteRef)

#undef UE_API
