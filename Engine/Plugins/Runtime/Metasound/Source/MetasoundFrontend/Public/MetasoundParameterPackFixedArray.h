// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/TypeCompatibleBytes.h"

namespace Metasound
{

	template<typename ElementType, int32 NumInlineElements>
	class TParamPackFixedArray
	{
	public:
		TParamPackFixedArray(const TArray<ElementType>& Other)
		{
			*this = Other;
		}

		TParamPackFixedArray<ElementType, NumInlineElements>& operator=(const TArray<ElementType>& Other)
		{
			int32 NumToCopy = FMath::Min(Other.Num(), NumInlineElements);
			for (int32 i = 0; i < NumToCopy; ++i)
			{
				GetData()[i] = Other[i];
			}
			NumValidElements = NumToCopy;
			return *this;
		}

		void CopyToArray(TArray<ElementType>& Dest) const
		{
			Dest.Empty();
			for (int32 i = 0; i < NumValidElements; ++i)
			{
				Dest.Add(GetData()[i]);
			}
		}

		inline const ElementType* GetData() const
		{
			return (ElementType*)InlineData;
		}

		inline ElementType* GetData()
		{
			return (ElementType*)InlineData;
		}

	public:
		int32 NumValidElements = 0;
		using FByteBucketType = TTypeCompatibleBytes<ElementType>;
		FByteBucketType InlineData[NumInlineElements];
	};
}
