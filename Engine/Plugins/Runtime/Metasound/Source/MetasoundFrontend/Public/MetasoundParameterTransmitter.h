// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameterControllerInterface.h"
#include "Containers/SpscQueue.h"
#include "IAudioParameterTransmitter.h"
#include "MetasoundDataReference.h"
#include "MetasoundFrontendLiteral.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundPrimitives.h"
#include "MetasoundRouter.h"

#define UE_API METASOUNDFRONTEND_API

struct FMetasoundFrontendLiteral;

namespace Metasound
{
	extern METASOUNDFRONTEND_API int32 MetaSoundParameterEnableWarningOnIgnoredParameterCVar;

	class FMetasoundGenerator;

	namespace Frontend
	{
		METASOUNDFRONTEND_API FLiteral ConvertParameterToLiteral(FAudioParameter&& InValue);
		METASOUNDFRONTEND_API FLiteral ConvertParameterToLiteral(const FAudioParameter& InValue);
		METASOUNDFRONTEND_API FName ConvertParameterToDataType(EAudioParameterType InParameterType);
	}


	/** FMetaSoundParameterTransmitter provides a communication interface for 
	 * sending values to a MetaSound instance. It relies on the send/receive transmission
	 * system to ferry data from the transmitter to the MetaSound instance. Data will
	 * be safely ushered across thread boundaries in scenarios where the instance
	 * transmitter and metasound instance live on different threads. 
	 */
	class FMetaSoundParameterTransmitter : public Audio::FParameterTransmitterBase
	{
		FMetaSoundParameterTransmitter(const FMetaSoundParameterTransmitter&) = delete;
		FMetaSoundParameterTransmitter& operator=(const FMetaSoundParameterTransmitter&) = delete;

	public:
		/** FSendInfo describes the MetaSounds input parameters as well as the 
		 * necessary information to route data to the instances inputs. 
		 */
		struct FSendInfo
		{
			/** Global address of instance input. */
			FSendAddress Address;

			/** Name of parameter on MetaSound instance. */
			FName ParameterName;

			/** Type name of parameter on MetaSound instance. */
			FName TypeName;
		};
		
		
		struct FParameter
		{
			FName Name;
			FLiteral Value;
		};

		/** Initialization parameters for a FMetaSoundParameterTransmitter. */
		struct FInitParams
		{
			/** FOperatorSettings must match the operator settings of the MetaSound 
			 * instance to ensure proper operation. */
			FOperatorSettings OperatorSettings;

			/** ID of the MetaSound instance.  */
			uint64 InstanceID;

			/** Available input parameters on MetaSound instance. */
			TArray<FName> ValidParameterNames;
			
			/** Available input parameters on MetaSound instance. */
			TArray<FSendInfo> Infos;

			/** Name of MetaSound used to log parameter related errors. */
			FName DebugMetaSoundName;

			/** Default Audio Parameters set when transmitter is initialized. */
			TArray<FAudioParameter> DefaultParams;

			/** Shared queue used to communicate with running MetaSound instance. */
			TSharedPtr<TSpscQueue<FParameter>> DataChannel;

			FInitParams(const FOperatorSettings& InSettings, uint64 InInstanceID, TArray<FAudioParameter>&& InDefaultParams, TArray<FName>&& InValidParameterNames, TSharedPtr<TSpscQueue<FParameter>> InDataChannel)
			: OperatorSettings(InSettings)
			, InstanceID(InInstanceID)
			, ValidParameterNames(MoveTemp(InValidParameterNames))
			, DefaultParams(MoveTemp(InDefaultParams))
			, DataChannel(InDataChannel)
			{
			}
		};

		/** Creates a unique send address using the given MetaSound environment. */
		UE_DEPRECATED(5.3, "This function is no longer used. Send addresses are no longer used when communicating with MetaSound inputs")
		static UE_API FSendAddress CreateSendAddressFromEnvironment(const FMetasoundEnvironment& InEnvironment, const FVertexName& InVertexName, const FName& InTypeName);
		
		/** Creates a unique send address using the given InstanceID. */
		UE_DEPRECATED(5.3, "This function is no longer used. Send addresses are no longer used when communicating with MetaSound inputs")
		static UE_API FSendAddress CreateSendAddressFromInstanceID(uint64 InInstanceID, const FVertexName& InVertexName, const FName& InTypeName);

		UE_API FMetaSoundParameterTransmitter(FMetaSoundParameterTransmitter::FInitParams&& InInitParams);
		virtual ~FMetaSoundParameterTransmitter() = default;

		UE_API void AddAvailableParameter(FName InName);
		UE_API void RemoveAvailableParameter(FName InName);

		/** Sets parameters using array of AudioParameter structs
		 *
		 * @param InParameter - Parameter to set.
		 */
		UE_API virtual bool SetParameters(TArray<FAudioParameter>&& InParameters) override;

		/** Set a parameter using a literal.
		 *
		 * @param InParameterName - Name of MetaSound instance parameter.
		 * @param InValue - Literal value used to construct parameter value. 
		 *
		 * @return true on success, false on failure. 
		 */
		UE_API bool SetParameterWithLiteral(FName InParameterName, const FLiteral& InValue);

	private:
		/** Set parameters when virtualized
		 * 
		 *  @return true if all non trigger parameters are set on the parameter base
		 *	(trigger parameters are removed on virtualized sounds 
		 *	and parameters are not actually set on the MetaSound until realization)
		 */
		UE_API bool SetVirtualizedParameters(TArray<FAudioParameter>&& InParameters);

		uint64 InstanceID;
		FName DebugMetaSoundName;
		TArray<FName> AvailableParameterNames;
		TSharedPtr<TSpscQueue<FParameter>> DataChannel;
	};
}

#undef UE_API
