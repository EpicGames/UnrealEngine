// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundDataReferenceMacro.h"

namespace Metasound
{
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(bool, METASOUNDFRONTEND_API, FBoolTypeInfo, FBoolReadRef, FBoolWriteRef);
	
	DECLARE_METASOUND_DATA_REFERENCE_TYPES(int32, METASOUNDFRONTEND_API, FInt32TypeInfo, FInt32ReadRef, FInt32WriteRef);

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(float, METASOUNDFRONTEND_API, FFloatTypeInfo, FFloatReadRef, FFloatWriteRef);

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FString, METASOUNDFRONTEND_API, FStringTypeInfo, FStringReadRef, FStringWriteRef);
}

