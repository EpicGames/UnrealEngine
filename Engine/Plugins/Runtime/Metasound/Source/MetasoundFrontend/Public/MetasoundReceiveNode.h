// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundBuilderInterface.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "MetasoundRouter.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "MetasoundFrontend"


namespace Metasound
{
	namespace ReceiveNodeInfo
	{
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, AddressInput)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, DefaultDataInput)
		DECLARE_METASOUND_PARAM(METASOUNDFRONTEND_API, Output)

		extern METASOUNDFRONTEND_API const FDataVertexMetadata AddressInputMetadata;
		extern METASOUNDFRONTEND_API const FDataVertexMetadata DefaultDataInputMetadata;
		extern METASOUNDFRONTEND_API const FDataVertexMetadata OutputMetadata;

		METASOUNDFRONTEND_API FNodeClassName GetClassNameForDataType(const FName& InDataTypeName);
		METASOUNDFRONTEND_API int32 GetCurrentMajorVersion();
		METASOUNDFRONTEND_API int32 GetCurrentMinorVersion();
	};

	namespace ReceiveNodePrivate
	{
		UE_INTERNAL
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateNodeClassRegistryKey(const FName& InDataTypeName);

		template<typename TDataType>
		class TReceiverOperator : public TExecutableOperator<TReceiverOperator<TDataType>>
		{
			TReceiverOperator() = delete;

		public:
			static FVertexInterface DeclareVertexInterface()
			{
				using namespace ReceiveNodeInfo;
				return FVertexInterface(
					FInputVertexInterface(
						TInputDataVertex<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), AddressInputMetadata),
						TInputDataVertex<TDataType>(METASOUND_GET_PARAM_NAME(DefaultDataInput), DefaultDataInputMetadata)
					),
					FOutputVertexInterface(
						TOutputDataVertex<TDataType>(METASOUND_GET_PARAM_NAME(Output), OutputMetadata)
					)
				);
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto InitNodeInfo = []() -> FNodeClassMetadata
				{
					FNodeClassMetadata Info;
					Info.ClassName = ReceiveNodeInfo::GetClassNameForDataType(GetMetasoundDataTypeName<TDataType>());
					Info.MajorVersion = ReceiveNodeInfo::GetCurrentMajorVersion();
					Info.MinorVersion = ReceiveNodeInfo::GetCurrentMinorVersion();
					Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_ReceiveNodeDisplayNameFormat", "Receive {0}", GetMetasoundDataTypeDisplayText<TDataType>());
					Info.Description = METASOUND_LOCTEXT("Metasound_ReceiveNodeDescription", "Receives data from a send node with the same name.");
					Info.Author = PluginAuthor;
					Info.PromptIfMissing = PluginNodeMissingPrompt;
					Info.DefaultInterface = DeclareVertexInterface();
					Info.CategoryHierarchy = { METASOUND_LOCTEXT("Metasound_TransmissionNodeCategory", "Transmission") };
					Info.Keywords = { };

					// Then send & receive nodes do not work as expected, particularly
					// around multiple-consumer scenarios. Deprecate them to avoid
					// metasound assets from relying on send & receive nodes.
					EnumAddFlags(Info.AccessFlags, ENodeClassAccessFlags::Deprecated);

					return Info;
				};

				static const FNodeClassMetadata Info = InitNodeInfo();

				return Info;
			}

			TReceiverOperator(TDataReadReference<TDataType> InInitDataRef, TDataWriteReference<TDataType> InOutDataRef, TDataReadReference<FSendAddress> InSendAddress, const FOperatorSettings& InOperatorSettings)
				: bHasNotReceivedData(true)
				, DefaultData(InInitDataRef)
				, OutputData(InOutDataRef)
				, SendAddress(InSendAddress)
				, CachedSendAddress(*InSendAddress)
				, CachedReceiverParams({InOperatorSettings})
				, Receiver(nullptr)
			{
				Receiver = CreateNewReceiver();
			}

			virtual ~TReceiverOperator() 
			{
				ResetReceiverAndCleanupChannel();
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				using namespace ReceiveNodeInfo; 
				InOutVertexData.BindReadVertex<TDataType>(METASOUND_GET_PARAM_NAME(DefaultDataInput), DefaultData);
				InOutVertexData.BindReadVertex<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), SendAddress);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				using namespace ReceiveNodeInfo;
				InOutVertexData.BindReadVertex<TDataType>(METASOUND_GET_PARAM_NAME(Output), OutputData);
			}

			void Execute()
			{
				if (*SendAddress != CachedSendAddress)
				{
					ResetReceiverAndCleanupChannel();
					CachedSendAddress = *SendAddress;

					Receiver = CreateNewReceiver();
				}

				bool bHasNewData = false;
				if (ensure(Receiver.IsValid()))
				{
					bHasNewData = Receiver->CanPop();
					if (bHasNewData)
					{
						Receiver->Pop(*OutputData);
						bHasNotReceivedData = false;
					}
				}

				if (bHasNotReceivedData)
				{
					if constexpr (std::is_copy_assignable<TDataType>::value)
					{
						*OutputData = *DefaultData;
					}
					bHasNewData = true;
				}

				if (TExecutableDataType<TDataType>::bIsExecutable)
				{
					TExecutableDataType<TDataType>::ExecuteInline(*OutputData, bHasNewData);
				}
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				if constexpr (std::is_copy_assignable<TDataType>::value)
				{
					*OutputData = *DefaultData;
				}
				bHasNotReceivedData = true;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				using namespace ReceiveNodeInfo; 

				TDataReadReference<TDataType> DefaultReadRef = TDataReadReferenceFactory<TDataType>::CreateAny(InParams.OperatorSettings);

				if (InParams.InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(DefaultDataInput)))
				{
					DefaultReadRef = InParams.InputData.GetDataReadReference<TDataType>(METASOUND_GET_PARAM_NAME(DefaultDataInput));
				}

				return MakeUnique<TReceiverOperator>(
					DefaultReadRef,
					TDataWriteReferenceFactory<TDataType>::CreateAny(InParams.OperatorSettings, *DefaultReadRef),
					InParams.InputData.GetOrCreateDefaultDataReadReference<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), InParams.OperatorSettings),
					InParams.OperatorSettings
					);
			}

		private:

			FSendAddress GetSendAddressWithDataType(const FSendAddress& InAddress) const 
			{
				// The data type of a send address is inferred by the underlying
				// data type of this node. A full send address, including the data type,
				// cannot be constructed from a literal. 
				return FSendAddress{ InAddress.GetChannelName(), GetMetasoundDataTypeName<TDataType>(), InAddress.GetInstanceID() };
			}

			TReceiverPtr<TDataType> CreateNewReceiver() const
			{
				if (ensure(SendAddress->GetDataType().IsNone() || (GetMetasoundDataTypeName<TDataType>() == SendAddress->GetDataType())))
				{
					return FDataTransmissionCenter::Get().RegisterNewReceiver<TDataType>(GetSendAddressWithDataType(*SendAddress), CachedReceiverParams);
				}
				return TReceiverPtr<TDataType>(nullptr);
			}
			
			void ResetReceiverAndCleanupChannel()
			{
				Receiver.Reset();
				FDataTransmissionCenter::Get().UnregisterDataChannelIfUnconnected(GetSendAddressWithDataType(CachedSendAddress));
			}

			bool bHasNotReceivedData;

			TDataReadReference<TDataType> DefaultData;
			TDataWriteReference<TDataType> OutputData;
			TDataReadReference<FSendAddress> SendAddress;

			FSendAddress CachedSendAddress;
			FReceiverInitParams CachedReceiverParams;

			TReceiverPtr<TDataType> Receiver;
		};
	}

	template<typename TDataType>
	class TReceiveNode : public TNodeFacade<ReceiveNodePrivate::TReceiverOperator<TDataType>>
	{
	public:
		using FOperator = ReceiveNodePrivate::TReceiverOperator<TDataType>;
		using FSuper = TNodeFacade<FOperator>;
		using FSuper::FSuper;

		static FVertexInterface DeclareVertexInterface()
		{
			return FOperator::DeclareVertexInterface();
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			return FOperator::GetNodeInfo();
		}
	};
}
#undef LOCTEXT_NAMESPACE
