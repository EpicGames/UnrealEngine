// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/Dsp.h"
#include "DSP/MultithreadedPatching.h"
#include "HAL/Platform.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundLiteral.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundTrigger.h"
#include "Misc/Guid.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"

#include <atomic>
#include <type_traits>

#define UE_API METASOUNDFRONTEND_API

/**
 * METASOUND TRANSMISSION SYSTEM
 * This allows for transmission of arbitrary data types using transmission addresses.
 *
 * Typical use case:
 * TSenderPtr<float> FloatSender FDataTransmissionCenter::Get().RegisterNewSender<float>(ExampleAddress, InitSettings);
 * TReceiverPtr<float> FloatReceiver FDataTransmissionCenter::Get().RegisterNewReceiver<float>(ExampleAddress, InitSettings);
 *
 * //...
 * FloatSender->Push(4.5f);
 *
 * //... elsewhere on a different thread:
 * float ReceivedFloat = FloatReceiver->Pop();
 *
 * In general, some performance considerations for this system:
 * 1) Senders and receivers are meant to have long lifecycles, and requesting them from the FDataTransmissionCenter may be expensive if done every tick.
 * 2) Most non-audio data types cannot be mixed, so multiple senders to the same address will cause only one of the senders to be effective.
 * 3) while the overall system is thread safe, individual TSender and TReceiver objects should only be used by a single thread.
 */

namespace Metasound
{
	//Forward declare
	class FTransmissionAddress;
	uint32 GetTypeHash(const Metasound::FTransmissionAddress&);

	/** A FTranmissionAddress acts as a identifier for a data channel shared by 
	 * one or more senders and or receivers. 
	 *
	 * Different addressing schemes are supported by deriving a system specific 
	 * address type from a FTransmissionAddress. This allows different systems 
	 * to construct and store address information in their desired format while
	 * all utilizing the same system for transmission.
	 */
	class FTransmissionAddress
	{
	public:
		FTransmissionAddress() = default;
		virtual ~FTransmissionAddress() = default;

		/** Return the type of address. 
		 *
		 * The AddressType must be unique per a subclass of FTransmissionAddress
		 * as it is used to determine the validity of a static_cast from FTransmissionAddress
		 * to the derived address type.
		 */
		virtual FName GetAddressType() const = 0;

		UE_API bool operator==(const FTransmissionAddress& InOther) const;
		UE_API bool operator!=(const FTransmissionAddress& InOther) const;

		/** Return the data type. */
		virtual FName GetDataType() const = 0;

		/** Clone the address. */
		virtual TUniquePtr<FTransmissionAddress> Clone() const = 0;

		/** Return a string representation of the address for debugging. */
		virtual FString ToString() const = 0;

	private:
		friend uint32 GetTypeHash(const ::Metasound::FTransmissionAddress&);

		template<typename DerivedAddressType>
		friend const DerivedAddressType* CastAddressType(const FTransmissionAddress&);

		template<typename DerivedAddressType>
		struct THasStaticAddressType
		{
		private:
			
			// SFINAE: If the static member `T::AddressType` exists, then this template is used.
			template<
				typename T, 
				typename = typename std::enable_if_t<!std::is_member_pointer_v<decltype(&T::AddressType)>>
			>
			static std::true_type CheckIfStaticMemberExists(int32);

			// SFINAE: If the static member `T::AddressType` does not exist, then this template is used.
			template<typename>
			static std::false_type CheckIfStaticMemberExists(...);

		public:
			// True if the static member T::AddressType exists. 
			static constexpr bool Value = decltype(CheckIfStaticMemberExists<DerivedAddressType>(0))::value;
		};

		/** Create a hash of the address. */
		virtual uint32 GetHash() const = 0;

		/** Determine if the given address is equal to this address. */
		virtual bool IsEqual(const FTransmissionAddress& InOther) const = 0;
	};

	/** Attempt to cast a FTransmissionAddress to a derived address type. 
	 *
	 * This function uses the FTransmissionAddress::GetAddressType() method to 
	 * determine whether the cast is legal. Each subclass of FTransmissionAddress
	 * *MUST* provide a unique address type to protect against invalid casts.
	 */
	template<typename DerivedAddressType>
	const DerivedAddressType* CastAddressType(const FTransmissionAddress& InOther)
	{
		// Helper struct to determine if the DerivedAddressType class has a `DerivedAddressType::AddressType` static member

		// Static assert on invalid cast attempts
		static_assert(std::is_base_of<FTransmissionAddress, DerivedAddressType>::value, "The derived address type must be derived from FTransmissionAddress");
		static_assert(FTransmissionAddress::THasStaticAddressType<DerivedAddressType>::Value, "The derived address type must have a static member `AddressType` to perform casts");

		// Check if AddressType is equal
		if (InOther.GetAddressType() == DerivedAddressType::AddressType)
		{
			return static_cast<const DerivedAddressType*>(&InOther);
		}
		return nullptr;
	}

	/** FSendAddress is used as a addressing system for MetaSound Send nodes, Recieve
	 * nodes and MetaSound script communication
	 */
	class FSendAddress : public FTransmissionAddress
	{
	public:
		static UE_API const FLazyName AddressType;

		FSendAddress() = default;

		// Create an address without a data type or instance ID
		UE_API FSendAddress(const FString& InChannelName);

		// Create an address with a channel name, data type and optionally an instance ID. 
		UE_API FSendAddress(const FName& InChannelName, const FName& InDataType, uint64 InInstanceID=INDEX_NONE);

		UE_API virtual FName GetAddressType() const override;
		UE_API virtual FName GetDataType() const override;
		UE_API virtual TUniquePtr<FTransmissionAddress> Clone() const override;
		UE_API virtual FString ToString() const override;

		UE_API const FName& GetChannelName() const;
		UE_API uint64 GetInstanceID() const;

	private:

		UE_API virtual uint32 GetHash() const override;
		UE_API virtual bool IsEqual(const FTransmissionAddress& InOther) const override;

		FName ChannelName;
		FName DataType;

		// For instance-specific addresses, this uint64 is the instance of this metasound.
		uint64 InstanceID = INDEX_NONE;
	};

	inline uint32 GetTypeHash(const Metasound::FTransmissionAddress& InAddress)
	{
		return InAddress.GetHash();
	}

	// Base class that defines shared state and utilities for senders, receivers, and the shared channel state.
	class IDataTransmissionBase
	{
	public:
		IDataTransmissionBase() = delete;

		IDataTransmissionBase(FName InDataType)
			: DataType(InDataType)
		{
		}

		virtual ~IDataTransmissionBase() = default;
		
		template<typename TSenderImplementation>
		bool CheckType() const
		{
			FName DestinationTypeName = TSenderImplementation::GetDataTypeName();
			return ensureAlwaysMsgf(DataType == DestinationTypeName, TEXT("Tried to downcast type %s to %s!"), *DataType.ToString(), *DestinationTypeName.ToString());
		}

		template<typename TSenderImplementation>
		const TSenderImplementation& GetAs() const
		{
			check(CheckType<TSenderImplementation>());
			return static_cast<TSenderImplementation&>(*this);
		}

		template<typename TSenderImplementation>
		TSenderImplementation& GetAs()
		{
			check(CheckType<TSenderImplementation>());
			return static_cast<TSenderImplementation&>(*this);
		}

		FName GetDataType() const
		{
			return DataType;
		}

	protected:
		// The data type for this sender.
		FName DataType;
	};

	class IDataChannel;

	// This is a handle to something that will retrieve a datatype.
	// Intentionally opaque- to use, one will need to call IReceiver::GetAs<TReceiver<YourDataType>>().
	class IReceiver : public IDataTransmissionBase
	{
	public:
		IReceiver(TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel, FName DataType)
			: IDataTransmissionBase(DataType)
			, DataChannel(InDataChannel)
		{}


		UE_API virtual ~IReceiver();

	protected:
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> DataChannel;
	};

	// This is a handle to something that will retrieve a datatype.
	// Intentionally opaque- to use, one will need to call ISender::GetAs<TSender<YourDataType>>().
	class ISender : public IDataTransmissionBase
	{
	public:
		ISender(TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel, FName DataType)
			: IDataTransmissionBase(DataType)
			, DataChannel(InDataChannel)
		{}

		UE_API virtual ~ISender();

		virtual int64 GetPayloadID() const = 0;

		// Constructs the value with a literal and pushes to data channel. 
		virtual bool PushLiteral(const FLiteral& InLiteral) = 0;

	protected:
		TSharedPtr<IDataChannel, ESPMode::ThreadSafe> DataChannel;
	};

	struct FSenderInitParams
	{
		FOperatorSettings OperatorSettings;
		float DelayTimeInSeconds;
	};

	struct FReceiverInitParams
	{
		FOperatorSettings OperatorSettings;
	};

	// This contains an intermediary buffer for use between the send and receive nodes.
	// Typically only used by ISender and IReceiver implementations.
	class IDataChannel : public IDataTransmissionBase, public TSharedFromThis<IDataChannel, ESPMode::ThreadSafe>
	{
	public:
		virtual ~IDataChannel() = default;

		IDataChannel(FName DataType)
			: IDataTransmissionBase(DataType)
		{}

		TUniquePtr<IReceiver> NewReceiver(const FReceiverInitParams& InitParams)
		{
			NumAliveReceivers.Increment();
			return ConstructNewReceiverImplementation(InitParams);
		}

		TUniquePtr<ISender> NewSender(const FSenderInitParams& InitParams)
		{
			NumAliveSenders.Increment();
			return ConstructNewSenderImplementation(InitParams);
		}

		uint32 GetNumActiveReceivers() const
		{
			return NumAliveReceivers.GetValue();
		}

		uint32 GetNumActiveSenders() const
		{
			return NumAliveSenders.GetValue();
		}

		virtual int64 GetPayloadID() const { return INDEX_NONE; }
		virtual int64 CreateNewPayloadID() = 0;
		virtual bool PushOpaque(ISender& InSender) { return true; };
		virtual void PopOpaque(IReceiver& InReceiver) {};
		virtual bool IsEmpty() const { return true; };

		virtual FName GetDataType() { return FName(); };

		virtual bool PushLiteral(const FLiteral& InParam) { return false; }

	protected:
		virtual TUniquePtr<IReceiver> ConstructNewReceiverImplementation(const FReceiverInitParams& InitParams) { return nullptr; };
		virtual TUniquePtr<ISender> ConstructNewSenderImplementation(const FSenderInitParams& InitParams) { return nullptr; };

	private:
		void OnSenderDestroyed()
		{
			check(NumAliveSenders.GetValue() > 0);
			NumAliveSenders.Decrement();
		}

		void OnReceiverDestroyed()
		{
			check(NumAliveReceivers.GetValue() > 0);
			NumAliveReceivers.Decrement();
		}

		friend class ISender;
		friend class IReceiver;

		FThreadSafeCounter NumAliveReceivers;
		FThreadSafeCounter NumAliveSenders;
	};

	// Generic templates:
	template<typename TDataType>
	class TNonOperationalSender : public ISender
	{
	public:
		bool Push(const TDataType& InValue) { return true; }
	};

	template<typename TDataType>
	class TNonOperationalReceiver : public IReceiver
	{
	public:
		bool Pop(TDataType& OutValue) { return true; }
	};

	template<typename TDataType>
	class TNonOperationalDataChannel : public IDataChannel
	{

	public:

		TNonOperationalDataChannel(const FOperatorSettings& InOperatorSettings)
			: IDataChannel(FName())
			, PayloadIDCounter(0)
		{}

		virtual bool PushOpaque(ISender& InSender) override 
		{
			return true;
		}

		virtual void PopOpaque(IReceiver& InReceiver) override
		{

		}

		virtual int64 CreateNewPayloadID() override
		{
			// Atomic return and post increment.
			return PayloadIDCounter++;
		}

		virtual bool IsEmpty() const override
		{
			return true;
		}

		virtual FName GetDataType() override
		{
			return GetDataTypeName();
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

	protected:
		virtual TUniquePtr<IReceiver> ConstructNewReceiverImplementation(const FReceiverInitParams& InitParams) override
		{
			return TUniquePtr<IReceiver>();
		}

		virtual TUniquePtr<ISender> ConstructNewSenderImplementation(const FSenderInitParams& InitParams) override
		{
			return TUniquePtr<ISender>();
		}
	private:
		std::atomic<int64> PayloadIDCounter;
	};

	// Specializations for trivially copyable types:
	template<typename TDataType>
	class TSender : public ISender
	{
		static_assert(std::is_copy_constructible<TDataType>::value, "The generic TDataChannel requires the DataType it is specialized with to be copy assignable.");

		// This can be changed to change when we reallocate the internal buffer for this sender.
		static constexpr float BufferResizeThreshold = 2.0f;

		bool bMessageDataTypeError = true;

	public:

		bool Push(const TDataType& InElement)
		{
			SenderBuffer.Push(InElement);

			if (SenderBuffer.Num() >= NumElementsToDelayBy)
			{
				return PushToDataChannel();
			}
			else
			{
				return true;
			}
		}

		bool PushLiteral(const FLiteral& InLiteral) override
		{
			if (!TLiteralTraits<TDataType>::IsParsable(InLiteral))
			{
				if (bMessageDataTypeError)
				{
					UE_LOGF(LogMetaSound, Error, "Failed to send data. Data type [TypeName:%ls] cannot be parsed from literal [Literal:%ls].", *GetMetasoundDataTypeString<TDataType>(), *LexToString(InLiteral));
					bMessageDataTypeError = false;
				}
				return false;
			}
			else
			{
				return TSender<TDataType>::Push(TDataTypeLiteralFactory<TDataType>::CreateExplicitArgs(Params.OperatorSettings, InLiteral));
			}
		}

		// Resets the delay for this specific sender. if this goes beyond the threshold set by BufferResizeThreshold, we reallocate.
		bool SetDelay(float InSeconds)
		{
			Params.DelayTimeInSeconds = InSeconds;
			SenderBuffer.SetCapacity(2);
			return true;
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		TSender(const FSenderInitParams& InitParams, TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel)
			: ISender(InDataChannel, GetDataTypeName())
			, Params(InitParams)
		{
			SetDelay(InitParams.DelayTimeInSeconds);
		}

		int64 GetPayloadID() const override
		{
			return PayloadID;
		}

		TDataType RetrievePayload()
		{
			check(Payload.IsValid());
			return MoveTempIfPossible(*Payload);
		}

		// Push from the SenderBuffer to the data channel.
		bool PushToDataChannel()
		{
			PayloadID = DataChannel->CreateNewPayloadID();
			if (!Payload.IsValid())
			{
				Payload = MakeUnique<TDataType>(SenderBuffer.Pop());
			}
			else
			{
				*Payload = SenderBuffer.Pop();
			}

			return DataChannel->PushOpaque(*this);
		}

	private:
		// This buffer acts as a delay line for this sender specifically.
		Audio::TCircularAudioBuffer<TDataType> SenderBuffer;

		int64 PayloadID = 0;

		// lazily initialized holder for payload, allocated on first call to Push.
		// Placed on the heap so that we don't have to assume we know how to construct TDataType.
		TUniquePtr<TDataType> Payload;

		FSenderInitParams Params;

		// The amount of pop operations before elements before this sender pushed appear.
		uint32 NumElementsToDelayBy = 0;
	};

	template<typename TDataType>
	class TReceiver : public IReceiver
	{
	
	public:

		bool CanPop() const
		{
			if (DataChannel->GetPayloadID() == LastPayloadID)
			{
				return false;
			}

			if (DataChannel->IsEmpty())
			{
				return false;
			}

			return true;
		}

		// Pop the latest value from the data channel.
		bool Pop(TDataType& OutValue)
		{
			DataChannel->PopOpaque(*this);

			if constexpr (std::is_copy_assignable<TDataType>::value)
			{
				check(Payload.IsValid());
				OutValue = *Payload;
				return true;
			}
			else
			{
				return false;
			}
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		TReceiver(const FReceiverInitParams& InitParams, TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel)
			: IReceiver(InDataChannel, GetDataTypeName())
			, OperatorSettings(InitParams.OperatorSettings)
		{
		}

		void PushPayload(int64 InPayloadID, const TDataType& InDataPayload)
		{
			// Attempting to send the same payload again, so ignore.
			if (InPayloadID == LastPayloadID)
			{
				return;
			}

			LastPayloadID = InPayloadID;
			if (!Payload.IsValid())
			{
				Payload = MakeUnique<TDataType>(InDataPayload);
			}
			else
			{
				*Payload = InDataPayload;
			}
		}

	private:
		int64 LastPayloadID = INDEX_NONE;
		TUniquePtr<TDataType> Payload;
		FOperatorSettings OperatorSettings;
	};

	// Generic templated implementation of IDataChannel that can use for any copyable type.
	template<typename TDataType>
	class TCopyableDataChannel : public IDataChannel
	{
		

	public:

		TCopyableDataChannel(const FOperatorSettings& InOperatorSettings)
			: IDataChannel(GetDataTypeName())
			, OperatorSettings(InOperatorSettings)
			, PayloadIDCounter(0)
		{}

		virtual int64 GetPayloadID() const override
		{
			return PayloadID;
		}

		virtual int64 CreateNewPayloadID() override
		{
			// Atomic return and post increment.
			return PayloadIDCounter++;
		}

		virtual bool PushOpaque(ISender& InSender) override
		{
			TSender<TDataType>& CastSender = InSender.GetAs<TSender<TDataType>>();

			{
				FScopeLock ScopeLock(&AtomicDataLock);
				PayloadID = InSender.GetPayloadID();
				if (!AtomicData.IsValid())
				{
					AtomicData = MakeUnique<TDataType>(CastSender.RetrievePayload());
				}
				else
				{
					*AtomicData = CastSender.RetrievePayload();
				}
			}

			return true;
		}

		virtual void PopOpaque(IReceiver& InReceiver) override
		{
			TReceiver<TDataType>& CastReceiver = InReceiver.GetAs<TReceiver<TDataType>>();

			FScopeLock ScopeLock(&AtomicDataLock);
			if (AtomicData.IsValid())
			{
				CastReceiver.PushPayload(PayloadID, *AtomicData);
			}
		}

		virtual bool IsEmpty() const override
		{
			return !AtomicData.IsValid();
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		virtual FName GetDataType() override
		{
			return GetDataTypeName();
		}

		virtual bool PushLiteral(const FLiteral& InParam)
		{
			if (TLiteralTraits<TDataType>::IsParsable(InParam))
			{
				TDataType DataToPush = TDataTypeLiteralFactory<TDataType>::CreateExplicitArgs(OperatorSettings, InParam);

				FScopeLock ScopeLock(&AtomicDataLock);
				if (!AtomicData.IsValid())
				{
					AtomicData = MakeUnique<TDataType>(DataToPush);
				}
				else
				{
					*AtomicData = DataToPush;
				}

				return true;
			}
			else
			{
				return false;
			}
		}

	protected:
		virtual TUniquePtr<IReceiver> ConstructNewReceiverImplementation(const FReceiverInitParams& InitParams) override
		{
			return TUniquePtr<IReceiver>(new TReceiver<TDataType>(InitParams, this->AsShared()));
		}

		virtual TUniquePtr<ISender> ConstructNewSenderImplementation(const FSenderInitParams& InitParams) override
		{
			return TUniquePtr<ISender>(new TSender<TDataType>(InitParams, this->AsShared()));
		}

	private:
		// NOTE- In the future, this could be changed to use a TCircualrAudioBuffer or a TQueue for lockless operation.
		// The primary challenge with this is handling multiple senders and receivers.
		// In order to support multiple senders and receivers at scale, we'd need a good implementation of a bounded MPMC queue for arbitrary datatypes.
		TUniquePtr<TDataType> AtomicData;
		FCriticalSection AtomicDataLock;

		int64 PayloadID = INDEX_NONE;

		FOperatorSettings OperatorSettings;

		std::atomic<int64> PayloadIDCounter;
	};

	// AUDIO SPECIALIZATIONS:
	template<typename TDataType>
	class TAudioSender : public ISender
	{
		static constexpr float BufferResizeThreshold = 2.0f;
		static const int32 MaxChannels = 8;
	public:

		bool Push(const TDataType& InElement)
		{
			const int32 NumChannelsToPush = InElement.GetNumChannels();
			check(NumChannelsToPush <= MaxChannels);

			const TArrayView<const FAudioBufferReadRef> InputBuffers = InElement.GetBuffers();

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannelsToPush; ChannelIndex++)
			{
				const FAudioBuffer& ChannelBuffer = *InputBuffers[ChannelIndex];
				DelayLines[ChannelIndex].Push(ChannelBuffer.GetData(), ChannelBuffer.Num());

				if (DelayLines[ChannelIndex].Num() > DelayTimeInFrames)
				{
					Audio::FAlignedFloatBuffer& IntermediateBuffer = IntermediaryBuffers[ChannelIndex];
					DelayLines[ChannelIndex].Pop(IntermediateBuffer.GetData(), IntermediateBuffer.Num());
					DataChannelInputs[ChannelIndex].PushAudio(IntermediateBuffer.GetData(), IntermediateBuffer.Num());
				}
			}

			return true;
		}

		int64 GetPayloadID() const override
		{
			return INDEX_NONE;
		}

		bool PushLiteral(const FLiteral& InLiteral) override
		{
			ensureMsgf(false, TEXT("Cannot push literal to audio format"));
			return false;
		}

		// Resets the delay for this sender.
		bool SetDelay(float InSeconds)
		{
			Params.DelayTimeInSeconds = InSeconds;
			DelayTimeInFrames = FMath::Min<int32>(Params.DelayTimeInSeconds * Params.OperatorSettings.GetSampleRate(), Params.OperatorSettings.GetNumFramesPerBlock());
			
			if (DelayTimeInFrames >= DelayLines[0].Num())
			{
				int32 NumSamplesToBuffer = DelayTimeInFrames * BufferResizeThreshold;
				DelayLines.Reset();
				IntermediaryBuffers.Reset();

				// For now, all senders/receivers/data channels will just allocate 8 delay lines, but this can be changed in the future.
				for (int32 Index = 0; Index < MaxChannels; Index++)
				{
					DelayLines.Emplace(NumSamplesToBuffer);
					Audio::FAlignedFloatBuffer BufferForChannel;
					BufferForChannel.AddZeroed(Params.OperatorSettings.GetNumFramesPerBlock());
					IntermediaryBuffers.Add(MoveTemp(BufferForChannel));
				}
			}

			return true;
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		TAudioSender(const FSenderInitParams& InitParams, TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel, TArray<Audio::FPatchInput>&& Inputs)
			: ISender(InDataChannel, GetDataTypeName())
			, DataChannelInputs(MoveTemp(Inputs))
			, Params(InitParams)
		{
			SetDelay(InitParams.DelayTimeInSeconds);
		}

	private:
		// This buffer acts as a delay line for this sender specifically.
		TArray<Audio::TCircularAudioBuffer<float>> DelayLines;

		uint32 DelayTimeInFrames;

		TArray<Audio::FPatchInput> DataChannelInputs;

		TArray<Audio::FAlignedFloatBuffer> IntermediaryBuffers;

		FSenderInitParams Params;
	};

	template<typename TDataType>
	class TAudioReceiver : public IReceiver
	{
		static constexpr float BufferResizeThreshold = 2.0f;
		static const int32 MaxChannels = 8;
	public:

		bool CanPop() const
		{
			// Audio receivers can and should always pop audio,
			// because we zero out audio when a datachannel underruns.
			return true;
		}

		bool Pop(TDataType& OutElement)
		{
			const int32 NumChannelsToPop = OutElement.GetNumChannels();
			check(NumChannelsToPop <= MaxChannels);

			DataChannel->PopOpaque(*this);

			TArrayView<const FAudioBufferWriteRef> OutputBuffers = OutElement.GetBuffers();

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannelsToPop; ChannelIndex++)
			{
				FAudioBuffer& OutBuffer = *OutputBuffers[ChannelIndex];
				DataChannelOutputs[ChannelIndex]->PopAudio(OutBuffer.GetData(), OutBuffer.Num(), false);
			}

			return true;
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		TAudioReceiver(const FReceiverInitParams& InitParams, TSharedPtr<IDataChannel, ESPMode::ThreadSafe> InDataChannel, TArray<Audio::FPatchOutputStrongPtr>&& Outputs)
			: IReceiver(InDataChannel, GetDataTypeName())
			, DataChannelOutputs(MoveTemp(Outputs))
			, Params(InitParams)
		{
		}

	private:

		TArray<Audio::FPatchOutputStrongPtr> DataChannelOutputs;

		FReceiverInitParams Params;
	};

	template<typename TDataType>
	class TAudioDataChannel : public IDataChannel
	{
	private:
		static const int32 MaxChannels = 8;
	public:
		TAudioDataChannel(const FOperatorSettings& InOperatorSettings)
			: IDataChannel(GetDataTypeName())
			, PayloadIDCounter(0)
		{
			AudioBuses.AddDefaulted(MaxChannels);
		}

		virtual bool PushOpaque(ISender& InSender) override
		{
			return true;
		}

		virtual void PopOpaque(IReceiver& InReceiver) override
		{
			for (Audio::FPatchMixerSplitter& Bus : AudioBuses)
			{
				Bus.ProcessAudio();
			}
		}

		virtual int64 CreateNewPayloadID() override
		{
			// Atomic return and post increment.
			return PayloadIDCounter++;
		}

		virtual bool IsEmpty() const override
		{
			return false;
		}

		static FName GetDataTypeName()
		{
			return GetMetasoundDataTypeName<TDataType>();
		}

		virtual FName GetDataType() override
		{
			return GetDataTypeName();
		}

	protected:
		virtual TUniquePtr<IReceiver> ConstructNewReceiverImplementation(const FReceiverInitParams& InitParams) override
		{
			TArray<Audio::FPatchOutputStrongPtr> Outputs;
			for (Audio::FPatchMixerSplitter& Bus : AudioBuses)
			{
				Outputs.Add(Bus.AddNewOutput(8096, 1.0f));
			}

			return TUniquePtr<IReceiver>(new TAudioReceiver<TDataType>(InitParams, this->AsShared(), MoveTemp(Outputs)));
		}

		virtual TUniquePtr<ISender> ConstructNewSenderImplementation(const FSenderInitParams& InitParams) override
		{
			TArray<Audio::FPatchInput> Inputs;
			for (Audio::FPatchMixerSplitter& Bus : AudioBuses)
			{
				Inputs.Add(Bus.AddNewInput(8096, 1.0f));
			}

			return TUniquePtr<ISender>(new TAudioSender<TDataType>(InitParams, this->AsShared(), MoveTemp(Inputs)));
		}

	private:
		TArray<Audio::FPatchMixerSplitter> AudioBuses;
		std::atomic<int64> PayloadIDCounter;
	};


	// Coalesce both our standard copying Sender/Receivers and our audio-specific sender/receiver implementations into TSenderPtr and TReceiverPtr:

	template<typename TDataType, typename USenderType = typename std::conditional<TIsDerivedFrom<TDataType, IAudioDataType>::Value, TAudioSender<TDataType>, TSender<TDataType>>::type>
	using TSenderPtr = TUniquePtr<USenderType>;

	template<typename TDataType, typename UReceiverType = typename std::conditional<TIsDerivedFrom<TDataType, IAudioDataType>::Value, TAudioReceiver<TDataType>, TReceiver<TDataType>>::type>
	using TReceiverPtr = TUniquePtr<UReceiverType>;

	// SFINAE for creating the correct data channel type for the given datatype:

	template<typename TDataType, typename TEnableIf<std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSharedRef<IDataChannel, ESPMode::ThreadSafe> MakeDataChannel(const FOperatorSettings& InSettings)
	{
		return MakeShareable(new TCopyableDataChannel<TDataType>(InSettings));
	}

	template<typename TDataType, typename TEnableIf<TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSharedRef<IDataChannel, ESPMode::ThreadSafe> MakeDataChannel(const FOperatorSettings& InSettings)
	{
		return MakeShareable(new TAudioDataChannel<TDataType>(InSettings));
	}

	template<typename TDataType, typename TEnableIf<!std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSharedRef<IDataChannel, ESPMode::ThreadSafe> MakeDataChannel(const FOperatorSettings& InSettings)
	{
		return MakeShareable(new TNonOperationalDataChannel<TDataType>(InSettings));
	}

	// Utility function for properly downcasting to the correct type based on whether TDataType is an audio stream:

	// For Audio Senders:
	template <typename TDataType, typename TEnableIf<TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSenderPtr<TDataType> Downcast(TUniquePtr<ISender>&& InPtr)
	{
		checkf(InPtr->CheckType<TAudioSender<TDataType>>(), TEXT("Tried to downcast an ISender of type %s to a TSender of type %s!"), *(InPtr->GetDataType().ToString()), *(TAudioSender<TDataType>::GetDataTypeName().ToString()));
		return TSenderPtr<TDataType>(static_cast<TAudioSender<TDataType>*>(InPtr.Release()));
	}

	// For generic senders for copyable types:
	template <typename TDataType, typename TEnableIf<std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSenderPtr<TDataType> Downcast(TUniquePtr<ISender>&& InPtr)
	{
		checkf(InPtr->CheckType<TSender<TDataType>>(), TEXT("Tried to downcast an ISender of type %s to a TSender of type %s!"), *(InPtr->GetDataType().ToString()), *(TSender<TDataType>::GetDataTypeName().ToString()));
		return TSenderPtr<TDataType>(static_cast<TSender<TDataType>*>(InPtr.Release()));
	}

	// For invalid types (non-copyable AND not an audio type):
	template <typename TDataType, typename TEnableIf<!std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TSenderPtr<TDataType> Downcast(TUniquePtr<ISender>&& InPtr)
	{
		return nullptr;
	}

	// For audio receivers:
	template <typename TDataType, typename TEnableIf<TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TReceiverPtr<TDataType> Downcast(TUniquePtr<IReceiver>&& InPtr)
	{
		checkf(InPtr->CheckType<TAudioReceiver<TDataType>>(), TEXT("Tried to downcast an IReceiver of type %s to a TSender of type %s!"), *(InPtr->GetDataType().ToString()), *(TAudioReceiver<TDataType>::GetDataTypeName().ToString()));
		return TReceiverPtr<TDataType>(static_cast<TAudioReceiver<TDataType>*>(InPtr.Release()));
	}

	// For generic receivers for copyable types:
	template <typename TDataType, typename TEnableIf<std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TReceiverPtr<TDataType> Downcast(TUniquePtr<IReceiver>&& InPtr)
	{
		checkf(InPtr->CheckType<TReceiver<TDataType>>(), TEXT("Tried to downcast an IReceiver of type %s to a TSender of type %s!"), *(InPtr->GetDataType().ToString()), *(TReceiver<TDataType>::GetDataTypeName().ToString()));
		return TReceiverPtr<TDataType>(static_cast<TReceiver<TDataType>*>(InPtr.Release()));
	}

	// For invalid types (non-copyable AND not an audio type):
	template <typename TDataType, typename TEnableIf<!std::is_copy_constructible<TDataType>::value && !TIsDerivedFrom<TDataType, IAudioDataType>::Value, bool>::Type = true>
	TReceiverPtr<TDataType> Downcast(TUniquePtr<IReceiver>&& InPtr)
	{
		return nullptr;
	}

	// Basic router that takes an FName address, 
	class FAddressRouter
	{

	public:

		UE_API TSharedPtr<IDataChannel, ESPMode::ThreadSafe> FindDataChannel(const FTransmissionAddress& InAddress);
		UE_API TSharedPtr<IDataChannel, ESPMode::ThreadSafe> GetDataChannel(const FTransmissionAddress& InAddress, const FOperatorSettings& InOperatorSettings);

		UE_API TUniquePtr<ISender> RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams);
		UE_API TUniquePtr<IReceiver> RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams);

		template<typename TDataType>
		TReceiverPtr<TDataType> RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams)
		{
			TReceiverPtr<TDataType> Receiver;

			TUniquePtr<IReceiver> ReceiverBase = RegisterNewReceiver(InAddress, InitParams);
			if (ReceiverBase.IsValid())
			{
				Receiver = Downcast<TDataType>(MoveTemp(ReceiverBase));
			}

			return Receiver;
		}

		template<typename TDataType>
		TSenderPtr<TDataType> RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams)
		{
			TSenderPtr<TDataType> Sender;
			TUniquePtr<ISender> SenderBase = RegisterNewSender(InAddress, InitParams);

			if (SenderBase.IsValid())
			{
				Sender = Downcast<TDataType>(MoveTemp(SenderBase));
			}

			return Sender;
		}

		UE_DEPRECATED(5.1, "Pushing literals from the Metasound::FAddressRouter will no longer be supported in future released.")
		UE_API bool PushLiteral(const FName& InDataTypeName, const FName& InChannelName, const FLiteral& InParam);

		UE_API bool UnregisterDataChannel(const FTransmissionAddress& InAddress);
		UE_API bool UnregisterDataChannelIfUnconnected(const FTransmissionAddress& InAddress);

		UE_API FAddressRouter(const FAddressRouter& Other);
		FAddressRouter() = default;

	private:

		// Wrapper class for having an FTransmissionAddress work in a TMap<>
		// This implements the copy constructor and copy assignment operator
		// by calling the virtual Clone() method. 
		class FTransmissionAddressKey : public FTransmissionAddress
		{
		public:
			FTransmissionAddressKey(const FTransmissionAddress& InCopy);
			FTransmissionAddressKey(const FTransmissionAddressKey& InCopy);
			FTransmissionAddressKey(TUniquePtr<FTransmissionAddress>&& InImpl);
			virtual ~FTransmissionAddressKey() = default;

			FTransmissionAddressKey& operator=(const FTransmissionAddress& InOther);
			virtual FName GetAddressType() const override;
			virtual FName GetDataType() const override;
			virtual TUniquePtr<FTransmissionAddress> Clone() const override;
			virtual FString ToString() const override;

		private:
			virtual uint32 GetHash() const override;
			virtual bool IsEqual(const FTransmissionAddress& InOther) const override;

			TUniquePtr<FTransmissionAddress> PImpl;
		};

		TMap<FTransmissionAddressKey, TSharedRef<IDataChannel, ESPMode::ThreadSafe>> DataChannelMap;
		FCriticalSection DataChannelMapMutationLock;
	};


	// Main entry point for all sender/receiver registration.
	class FDataTransmissionCenter
	{

	public:

		// Returns the universal router.
		static UE_API FDataTransmissionCenter& Get();

		// Creates a new object to push data to an address.
		// Returns a new sender, or nullptr if registration failed.
		UE_API TUniquePtr<ISender> RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams);

		// Creates a new object to push data to an address.
		// Returns a new sender, or nullptr if registration failed.
		template<typename TDataType>
		TSenderPtr<TDataType> RegisterNewSender(const FTransmissionAddress& InAddress, const FSenderInitParams& InitParams)
		{
			TSenderPtr<TDataType> Sender;
			TUniquePtr<ISender> SenderBase = RegisterNewSender(InAddress, InitParams);

			if (SenderBase.IsValid())
			{
				Sender = Downcast<TDataType>(MoveTemp(SenderBase));
			}

			return Sender;
		}

		// Registers a new object to poll data from an address.
		// Returns a new receiver, or nullptr if registration failed.
		UE_API TUniquePtr<IReceiver> RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams);

		// Registers a new object to poll data from an address.
		// Returns a new receiver, or nullptr if registration failed.
		template<typename TDataType>
		TReceiverPtr<TDataType> RegisterNewReceiver(const FTransmissionAddress& InAddress, const FReceiverInitParams& InitParams)
		{
			TReceiverPtr<TDataType> Receiver;
			TUniquePtr<IReceiver> ReceiverBase = RegisterNewReceiver(InAddress, InitParams);

			if (ReceiverBase.IsValid())
			{
				Receiver = Downcast<TDataType>(MoveTemp(ReceiverBase));
			}

			return Receiver;
		}

		// Unregisters DataChannel irrespective of number of receivers or senders still active.
		UE_API bool UnregisterDataChannel(const FTransmissionAddress& InAddress);

		// Unregister a data channel if there are no senders or receivers
		UE_API bool UnregisterDataChannelIfUnconnected(const FTransmissionAddress& InAddress);

		// Pushes a literal parameter to a specific data channel in the global router.
		// returns false if the literal type isn't supported.
		UE_DEPRECATED(5.1, "Pushing literals from the Metasound::FDataTransmissionCenter will no longer be supported in future released.")
		UE_API bool PushLiteral(FName DataTypeName, FName GlobalChannelName, const FLiteral& InParam);

	private:
		// Single map of FNames to IDataChannels
		FAddressRouter GlobalRouter;

		FDataTransmissionCenter() = default;
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FSendAddress, METASOUNDFRONTEND_API, FSendAddressTypeInfo, FSendAddressReadRef, FSendAddressWriteRef)
}


#undef UE_API
