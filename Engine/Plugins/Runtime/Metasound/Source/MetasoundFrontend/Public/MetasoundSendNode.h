// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundBuilderInterface.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundRouter.h"
#include "MetasoundVertex.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "MetasoundFrontend"


namespace Metasound
{
	namespace SendVertexNames
	{
		METASOUND_PARAM(AddressInput, "Address", "Address")
	}

	namespace SendNodePrivate
	{
		UE_INTERNAL
		METASOUNDFRONTEND_API FVertexInterface CreateVertexInterface(const FName& InDataTypeName);

		UE_INTERNAL
		METASOUNDFRONTEND_API FNodeClassName CreateNodeClassName(const FName& InDataTypeName);

		UE_INTERNAL
		METASOUNDFRONTEND_API Frontend::FNodeClassRegistryKey CreateNodeClassRegistryKey(const FName& InDataTypeName);

		UE_INTERNAL
		METASOUNDFRONTEND_API FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FText& InDataTypeDisplayText);

		UE_INTERNAL
		METASOUNDFRONTEND_API const FName& GetSendInputName(const FName& InDataTypeName);

		template<typename TDataType>
		class TSendOperator : public TExecutableOperator<TSendOperator<TDataType>>
		{
		public:


			TSendOperator(TDataReadReference<TDataType> InInputData, TDataReadReference<FSendAddress> InSendAddress, const FOperatorSettings& InOperatorSettings)
				: InputData(InInputData)
				, SendAddress(InSendAddress)
				, CachedSendAddress(*InSendAddress)
				, CachedSenderParams({InOperatorSettings, 0.0f})
				, Sender(nullptr)
			{
				Sender = CreateNewSender();
			}

			virtual ~TSendOperator() 
			{
				ResetSenderAndCleanupChannel();
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				using namespace SendVertexNames; 
				using namespace SendNodePrivate;

				InOutVertexData.BindReadVertex<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), SendAddress);
				InOutVertexData.BindReadVertex<TDataType>(GetSendInputName(GetMetasoundDataTypeName<TDataType>()), InputData);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
			}

			void Execute()
			{
				if (*SendAddress != CachedSendAddress)
				{
					ResetSenderAndCleanupChannel();
					CachedSendAddress = *SendAddress;
					Sender = CreateNewSender();
					check(Sender.IsValid());
				}

				Sender->Push(*InputData);
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				ResetSenderAndCleanupChannel();
				CachedSendAddress = *SendAddress;
				Sender = CreateNewSender();
				check(Sender.IsValid());
			}

			static FVertexInterface DeclareVertexInterface()
			{
				return CreateVertexInterface(GetMetasoundDataTypeName<TDataType>());
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				static const FNodeClassMetadata Info = CreateNodeClassMetadata(GetMetasoundDataTypeName<TDataType>(), GetMetasoundDataTypeDisplayText<TDataType>());
				return Info;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				using namespace SendVertexNames;
				using namespace SendNodePrivate;

				if (InParams.InputData.IsVertexBound(GetSendInputName(GetMetasoundDataTypeName<TDataType>())))
				{
					return MakeUnique<SendNodePrivate::TSendOperator<TDataType>>(
						InParams.InputData.GetOrCreateDefaultDataReadReference<TDataType>(GetSendInputName(GetMetasoundDataTypeName<TDataType>()), InParams.OperatorSettings),
						InParams.InputData.GetOrCreateDefaultDataReadReference<FSendAddress>(METASOUND_GET_PARAM_NAME(AddressInput), InParams.OperatorSettings),
						InParams.OperatorSettings
					);
				}
				else
				{
					// No input hook up to send, so this node can no-op
					return MakeUnique<FNoOpOperator>();
				}
			}


		private:
			FSendAddress GetSendAddressWithDataType(const FSendAddress& InAddress) const 
			{
				// The data type of a send address is inferred by the underlying
				// data type of this node. A full send address, including the data type,
				// cannot be constructed from a literal. 
				return FSendAddress{ InAddress.GetChannelName(), GetMetasoundDataTypeName<TDataType>(), InAddress.GetInstanceID() };
			}

			TSenderPtr<TDataType> CreateNewSender() const
			{
				if (ensure(SendAddress->GetDataType().IsNone() || (GetMetasoundDataTypeName<TDataType>() == SendAddress->GetDataType())))
				{
					return FDataTransmissionCenter::Get().RegisterNewSender<TDataType>(GetSendAddressWithDataType(*SendAddress), CachedSenderParams);
				}
				return TSenderPtr<TDataType>(nullptr);
			}

			void ResetSenderAndCleanupChannel()
			{
				Sender.Reset();
				FDataTransmissionCenter::Get().UnregisterDataChannelIfUnconnected(GetSendAddressWithDataType(CachedSendAddress));
			}

			TDataReadReference<TDataType> InputData;
			TDataReadReference<FSendAddress> SendAddress;
			FSendAddress CachedSendAddress;
			FSenderInitParams CachedSenderParams;

			TSenderPtr<TDataType> Sender;
		};
	}



	template<typename TDataType>
	class TSendNode : public TNodeFacade<SendNodePrivate::TSendOperator<TDataType>>
	{
	public:
		using FOperator = SendNodePrivate::TSendOperator<TDataType>;
		using FSuper = TNodeFacade<SendNodePrivate::TSendOperator<TDataType>>;
		using FSuper::FSuper;

		UE_DEPRECATED(5.8, "This will no longer be supported")
		static const FVertexName& GetSendInputName()
		{
			return SendNodePrivate::GetSendInputName(GetMetasoundDataTypeName<TDataType>());
		}

		UE_DEPRECATED(5.8, "This will no longer be supported")
		static FVertexInterface DeclareVertexInterface()
		{
			return FOperator::DeclareVertexInterface();
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			return FOperator::GetNodeInfo();
		}
	};
}
#undef LOCTEXT_NAMESPACE
