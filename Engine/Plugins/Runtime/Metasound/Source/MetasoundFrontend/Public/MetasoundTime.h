// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundDataReferenceMacro.h"
#include "MetasoundOperatorSettings.h"

namespace Metasound
{
	class FSampleCounter;

	// Strongly typed time class
	class FTime
	{
		using TimeType = double;
		static_assert(TIsFloatingPoint<TimeType>::Value, "TimeType must be floating point.");

		TimeType Time = 0;

		public:
			FTime()
			{
			}

			explicit FTime(TimeType InTime)
				: Time(InTime)
			{
			}

			explicit FTime(float InTime)
			: Time(static_cast<TimeType>(InTime))
			{
			}
			
			FTime(const FTime& InOtherTime)
				: Time(InOtherTime.Time)
			{
			}

			static const FTime& Zero()
			{
				static const FTime ZeroTime;
				return ZeroTime;
			}

			/** Return the time as seconds. */
			TimeType GetSeconds() const
			{ 
				return Time;
			}

			FTime& operator=(const FTime& InRHS)
			{
				Time = InRHS.Time;
				return *this;
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator*=(ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time *= InRHS.Time;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend FTime& operator*=(FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time *= InRHS;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator/=(ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time /= InRHS.Time;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend FTime& operator/=(FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time /= InRHS;
				return InLHS;
			}

			FTime& operator-=(const FTime& InRHS)
			{
				Time -= InRHS.Time;
				return *this;
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator-=(ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS -= InRHS.Time;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend FTime& operator-=(FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time -= InRHS;
				return InLHS;
			}

			FTime& operator+=(const FTime& InRHS)
			{
				Time += InRHS.Time;
				return *this;
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator+=(ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS += InRHS.Time;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend FTime& operator+=(FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				InLHS.Time += InRHS;
				return InLHS;
			}

			FTime operator-(const FTime& InRHS) const
			{
				return FTime(Time - InRHS.Time);
			}

			template<typename ArithmeticType>
			friend ArithmeticType operator-(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS - InRHS.Time;
			}

			template<typename ArithmeticType>
			friend FTime operator-(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(InLHS.Time - InRHS);
			}

			FTime operator+(const FTime& InRHS) const
			{
				return FTime(Time + InRHS.Time);
			}

			template<typename ArithmeticType>
			friend ArithmeticType operator+(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS + InRHS.Time;
			}

			template<typename ArithmeticType>
			friend FTime operator+(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(InLHS.Time + InRHS);
			}

			template<typename ArithmeticType>
			ArithmeticType operator*(const FTime& InRHS) const
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return static_cast<ArithmeticType>(Time * InRHS.Time);
			}

			template<typename ArithmeticType>
			friend ArithmeticType operator*(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS * InRHS.Time;
			}

			template<typename ArithmeticType>
			friend FTime operator*(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(InLHS.Time * InRHS);
			}

			template<typename ArithmeticType>
			ArithmeticType operator/(const FTime& InRHS) const
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return static_cast<ArithmeticType>(Time / InRHS.Time);
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator/(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS / InRHS.Time;
			}

			template<typename ArithmeticType>
			friend FTime operator/(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(InLHS.Time / InRHS);
			}

			bool operator<(const FTime& InRHS) const
			{
				return Time < InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator<(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS < InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator<(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time < InRHS;
			}

			bool operator<=(const FTime& InRHS) const
			{
				return Time <= InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator<=(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS <= InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator<=(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time <= InRHS;
			}

			bool operator>(const FTime& InRHS) const
			{
				return Time > InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator>(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time > InRHS;
			}

			template<typename ArithmeticType>
			friend bool operator>(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS > InRHS.Time;
			}

			bool operator>=(const FTime& InRHS) const
			{
				return Time >= InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator>=(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS >= InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator>=(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time >= InRHS;
			}

			template<typename ArithmeticType>
			friend bool operator!=(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS != InRHS.Time;
			}

			bool operator!=(const FTime& InRHS) const
			{
				return Time != InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator!=(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time != InRHS;
			}

			friend bool operator==(const FTime& InLHS, const FTime& InRHS)
			{
				return InLHS.Time == InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator==(const ArithmeticType& InLHS, const FTime& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS == InRHS.Time;
			}

			template<typename ArithmeticType>
			friend bool operator==(const FTime& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return InLHS.Time == InRHS;
			}

			template<typename ArithmeticType = TimeType>
			static ArithmeticType ToMilliseconds(const FTime& InTime)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return static_cast<ArithmeticType>(InTime.GetSeconds() * 1e3);
			}

			template<typename ArithmeticType = TimeType>
			static ArithmeticType ToMicroseconds(const FTime& InTime)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return static_cast<ArithmeticType>(InTime.GetSeconds() * 1e6);
			}

			template<typename ArithmeticType>
			static FTime FromSeconds(const ArithmeticType& InSeconds)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(static_cast<TimeType>(InSeconds));
			}

			template<typename ArithmeticType>
			static FTime FromMilliseconds(const ArithmeticType& InMilliseconds)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(static_cast<TimeType>(InMilliseconds * 1e-3));
			}

			template<typename ArithmeticType>
			static FTime FromMicroseconds(const ArithmeticType& InMicroseconds)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Must be arithmetic type.");
				return FTime(static_cast<TimeType>(InMicroseconds * 1e-6));
			}
			
			friend inline uint32 GetTypeHash(const FTime& InTime)
			{
				return ::GetTypeHash(InTime.Time);
			}
			
			friend FSampleCounter;
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FTime, METASOUNDFRONTEND_API, FTimeTypeInfo, FTimeReadRef, FTimeWriteRef);
}
