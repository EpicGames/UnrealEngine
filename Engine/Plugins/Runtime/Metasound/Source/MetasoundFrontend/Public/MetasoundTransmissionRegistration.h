// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundRouter.h"
#include "MetasoundFrontendRegistries.h"

#include <type_traits>

namespace Metasound
{
	namespace MetasoundTransmissionPrivate
	{
		// Determines whether send/receive nodes are enabled for a specific data type.
		template<typename DataType>
		struct TEnableTransmission
		{
			static constexpr bool Value = true;
		};

		// TTransmissionSupport determines whether the send/receive system is 
		// supported for a given data type. It is used to add send/receive
		// support when possible, and avoid errors when registering data types
		// that do not support the transmission system. 
		template<typename DataType>
		struct TTransmissionSupport
		{
		private:
			static constexpr bool bEnabled = TEnableTransmission<DataType>::Value;

			// All types that support copy assignment and copy construction can be
			// used in the transmission system. 
			static constexpr bool bIsCopyable = std::is_copy_assignable<DataType>::value && std::is_copy_constructible<DataType>::value;

			// IAudioDataType derived classes have specialized templates for the
			// transmission system.
			static constexpr bool bIsAudioDataType = std::is_base_of<IAudioDataType, DataType>::value;

		public:
			
			static constexpr bool bIsTransmissionSupported = bEnabled && (bIsCopyable || bIsAudioDataType);
		};

		// At the time of writing this code, TArray incorrectly defines a copy constructor
		// even when the underlying elements or not copyable. Normally this results
		// in compilation errors when trying to utilize the erroneously defined 
		// TArray copy constructor. This specialization checks the underlying TArray
		// element type to determine if the array is copyable.
		template<typename ElementType>
		struct TTransmissionSupport<TArray<ElementType>>
		{
			static constexpr bool bIsCopyable = std::is_copy_assignable<ElementType>::value && std::is_copy_constructible<ElementType>::value;

			static constexpr bool bEnabled = TEnableTransmission<TArray<ElementType>>::Value;

		public:
			
			static constexpr bool bIsTransmissionSupported = bIsCopyable && bEnabled;
		};
	}

	struct FTransmissionDataChannelFactory
	{
		/**  Create a transmission IDataChannel given a data type.
		 *
		 * This function is defined if a data type is supported by the transmission system. 
		 */
		template<
			typename DataType,
			typename std::enable_if<
				MetasoundTransmissionPrivate::TTransmissionSupport<DataType>::bIsTransmissionSupported, 
				bool
			>::type = true
		>
		static TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FOperatorSettings& InOperatorSettings)
		{
			return MakeDataChannel<DataType>(InOperatorSettings);
		}

		/**  Create a transmission IDataChannel given a data type.
		 *
		 * This function is defined if a data type is not supported by the transmission system. 
		 * It returns null data channel. 
		 */
		template<
			typename DataType,
			typename std::enable_if<
				!MetasoundTransmissionPrivate::TTransmissionSupport<DataType>::bIsTransmissionSupported, 
				bool
			>::type = true
		>
		static TSharedPtr<IDataChannel, ESPMode::ThreadSafe> CreateDataChannel(const FOperatorSettings& InOperatorSettings)
		{
			return TSharedPtr<IDataChannel, ESPMode::ThreadSafe>(nullptr);
		}
	};
}
