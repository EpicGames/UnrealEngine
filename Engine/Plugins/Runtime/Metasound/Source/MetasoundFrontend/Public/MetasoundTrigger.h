// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundTime.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound
{
	/** FTrigger supports sample accurate triggering, sample accurate internal tracking,
	 * and a convenient interface for running trigger-aligned audio signal processing
	 * routines on buffers.. 
	 *
	 * FTriggers are triggered using FTrigger::TriggerTime or FTrigger::TriggerFrame.
	 * FTriggers track time internally by calling FTrigger::Advance.
	 * Executing audio signal processing on buffers can be performed by calling
	 * FTrigger::ExecuteBlock or FTrigger::LookAhead.
	 */
	class FTrigger
	{
		public:
			/** FTrigger constructor. 
			 *
			 * @param InSettings - Operator settings.
			 * @param bShouldTrigger - If true, triggers first sample.
			 */
			UE_API explicit FTrigger(const FOperatorSettings& InSettings, bool bShouldTrigger);

			/** FTrigger constructor. 
			 *
			 * @param InFrameToTrigger - Set specific frame to trigger.
			 * @param InSettings - Operator settings.
			 */
			UE_API explicit FTrigger(const FOperatorSettings& InSettings, int32 InFrameToTrigger);

			/** FTrigger constructor. By default it is not triggered.
			 *
			 * @param InSettings - Operator settings.
			 */
			UE_API FTrigger(const FOperatorSettings& InSettings);

			/** For use when a Trigger request is found in a Parameter Pack
			 *
			 * @param ParamPackPayload - A pointer to a bool which should always be 'true'.
			 */
			UE_API void AssignRawParameter(const void* ParamPackPayload);

			/** Trigger a specific frame in the future.
			 *
			 * @param InFrameToTrigger - Index of frame to trigger.
			 */
			UE_API void TriggerFrame(int32 InFrameToTrigger);

			/** Advance internal frame counters by block size. */
			UE_API void AdvanceBlock();

			/** Advance internal frame counters by specific frame count. */
			UE_API void Advance(int32 InNumFrames);

			/** Number of triggered frames. */
			UE_API int32 Num() const;

			/** Returns true if there is a trigger in the current block of audio */
			UE_API int32 NumTriggeredInBlock() const;

			/** Returns frame index for a given trigger index. 
			 *
			 * @param InIndex - Index of trigger. Must be a value between 0 and Num().
			 *
			 * @return The frame of the trigger.
			 */
			UE_API int32 operator[](int32 InTriggerIndex) const;

			/** Returns frame index for the first trigger in the block.
			 *
			 * @return The frame of the first trigger in the block or -1 if there is no trigger in the block.
			 */
			UE_API int32 First() const;

			/** Returns frame index for the last trigger in the block.
			 *
			 * @return The frame of the last trigger in the block or -1 if there is no trigger in the block.
			 */
			UE_API int32 Last() const;

			/** Returns true if there are any triggered frames. */
			UE_API bool IsTriggered() const;

			/** Returns true there is a trigger in the current block of audio. */
			UE_API bool IsTriggeredInBlock() const;

			/** Implicit conversion of FTrigger into bool by calling IsTriggeredInBlock()
			 *
			 * @return Returns true if frame is triggered in current block.
			 */
			UE_API operator bool() const;

			/** Removes all triggered frames. */
			UE_API void Reset();

			/** Removes all triggers which occur after the frame index. */
			UE_API void RemoveAfter(int32 InFrameIndex);

			/** Executes one block of frames and calls underlying InPreTrigger and InOnTrigger
			 * functions with frame indices.
			 *
			 * @param InPreTrigger - A function which handles frames before the first 
			 *                   trigger in the current block. The function must accept 
			 *                   the arguments (int32 StartFrame, int32 EndFrame).
			 * @param InOnTrigger - A function which handles frames starting with the 
			 *                  triggers index and ending the next trigger index or the
			 *                  number of frames in a block.. The function must
			 *                  accept the arguments (int32 StartFrame, int32 EndFrame).
			 */
			template<typename PreTriggerType, typename OnTriggerType>
			void ExecuteBlock(PreTriggerType InPreTrigger, OnTriggerType InOnTrigger) const
			{
				ExecuteFrames(NumFramesPerBlock, LastTriggerIndexInBlock, InPreTrigger, InOnTrigger);
			}

			/** Executes a desired number of frames and calls underlying InPreTrigger 
			 * and InOnTrigger functions with frame indices.
			 *
			 * @param InNumFrames - The number of frames to look ahead.
			 * @param InPreTrigger - A function which handles frames before the first 
			 *                   trigger in the range of frames. The function must accept 
			 *                   the arguments (int32 StartFrame, int32 EndFrame).
			 * @param InOnTrigger - A function which handles frames starting with the 
			 *                  triggers index and ending the next trigger index or InNumFrames
			 *             		The function must accept the arguments 
			 *             		(int32 StartFrame, int32 EndFrame).
			 */
			template<typename PreTriggerType, typename OnTriggerType>
			void LookAhead(int32 InNumFrames, PreTriggerType InPreTrigger, OnTriggerType InOnTrigger) const
			{
				if (InNumFrames > 0)
				{
					int32 LastTriggerIndexInLookAhead= Algo::LowerBound(TriggeredFrames, InNumFrames);
					
					ExecuteFrames(InNumFrames, LastTriggerIndexInLookAhead, InPreTrigger, InOnTrigger);
				}
			}

			const TArray<int32>& GetTriggeredFrames() const
			{
				return TriggeredFrames;
			}

		private:
			UE_API void UpdateLastTriggerIndexInBlock();

			template<typename PreTriggerType, typename OnTriggerType>
			void ExecuteFrames(int32 InNumFrames, int32 InLastTriggerIndex, PreTriggerType InPreTrigger, OnTriggerType InOnTrigger) const
			{
				if (InLastTriggerIndex <= 0)
				{
					InPreTrigger(0, InNumFrames);
					return;
				}

				const int32* TriggeredFramesData = TriggeredFrames.GetData();

				if (TriggeredFramesData[0] > 0)
				{
					InPreTrigger(0, TriggeredFramesData[0]);
				}

				const int32 NextToLastTriggerIndex = InLastTriggerIndex - 1;

				for (int32 i = 0; i < NextToLastTriggerIndex; i++)
				{
					InOnTrigger(TriggeredFramesData[i], TriggeredFramesData[i + 1]);
				}

				InOnTrigger(TriggeredFramesData[NextToLastTriggerIndex], InNumFrames);
			}


			TArray<int32> TriggeredFrames;

			bool bHasAdvanced = false;
			bool bTriggeredFromInit = false;
			bool bHasTrigger = false;
			int32 NumFramesPerBlock = 0;
			FSampleRate SampleRate = 0;
			int32 LastTriggerIndexInBlock = 0;
	};

	template<>
	struct TPostExecutableDataType<FTrigger>
	{
		static constexpr bool bIsPostExecutable = true;

		static void PostExecute(FTrigger& InOutData)
		{
			InOutData.AdvanceBlock();
		}
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FTrigger, METASOUNDFRONTEND_API, FTriggerTypeInfo, FTriggerReadRef, FTriggerWriteRef);
}

UE_API const TCHAR* LexToString(const Metasound::FTrigger& Trigger);

#undef UE_API
