// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataFactory.h"
#include <type_traits>

namespace Metasound
{
	namespace Frontend
	{
		/** Allow or disallow types to be registered as MetaSound Variables. */
		template<typename DataType>
		struct TEnableVariables
		{
			// By default, all registered MetaSound data types are variables.
			static constexpr bool Value = true;
		};

		/** Determine if data type supports necessary operations required of all
		 * MetaSound variable types.
		 */
		template<typename DataType>
		struct TVariablesSupport
		{

		private:

			static constexpr bool bIsCopyAssignable = std::is_copy_assignable<DataType>::value;
			static constexpr bool bIsParsableFromLiteral = TLiteralTraits<DataType>::bIsParsableFromAnyLiteralType;

		public:

			/* MetaSound variables must be copy-assignable and parsable from a FLiteral */
			static constexpr bool bIsVariableSupported = bIsCopyAssignable && bIsParsableFromLiteral;
		};
	}
}

