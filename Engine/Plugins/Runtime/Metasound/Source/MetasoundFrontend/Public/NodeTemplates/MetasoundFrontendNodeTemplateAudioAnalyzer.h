// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendNodeTemplateRegistry.h"

#define UE_API METASOUNDFRONTEND_API

namespace Metasound::Frontend
{
	class FAudioAnalyzerNodeTemplate : public FNodeTemplateBase
	{
	public:
		static UE_API const FMetasoundFrontendClassName ClassName;
		static UE_API const FMetasoundFrontendVersionNumber VersionNumber;

		UE_API virtual FMetasoundFrontendNodeInterface GenerateNodeInterface(FNodeTemplateGenerateInterfaceParams InParams) const override;
		UE_API virtual const FMetasoundFrontendClassName& GetClassName() const override;
		UE_API virtual TUniquePtr<INodeTemplateTransform> GenerateNodeTransform() const override;
		UE_API virtual const FMetasoundFrontendClass& GetFrontendClass() const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;
		UE_API virtual const FMetasoundFrontendVersionNumber& GetVersionNumber() const override;
		UE_API virtual bool IsInputAccessTypeDynamic() const override;
		UE_API virtual bool IsInputConnectionUserModifiable() const override;
		UE_API virtual bool IsOutputAccessTypeDynamic() const override;
		UE_API virtual bool IsOutputConnectionUserModifiable() const override;
		UE_API virtual bool IsValidNodeInterface(const FMetasoundFrontendNodeInterface& InNodeInterface) const override;
	};
} // namespace Metasound::Frontend

#undef UE_API
