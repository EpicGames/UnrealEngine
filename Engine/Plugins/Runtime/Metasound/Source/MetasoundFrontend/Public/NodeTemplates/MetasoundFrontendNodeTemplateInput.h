// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendNodeTemplateReroute.h"
#include "MetasoundFrontendTransform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"

#define UE_API METASOUNDFRONTEND_API


// Forward Declarations
struct FMetaSoundFrontendDocumentBuilder;


namespace Metasound::Frontend
{
	// Specialized node that connects an input node's single output to various input destinations.
	// While similar to reroute nodes, primarily exists to visually distinguish an input having multiple
	// locations in a visual graph while sharing implementation at runtime, while also differentiating
	// general input style from a typical reroute.
	class FInputNodeTemplate : public FRerouteNodeTemplate
	{
	public:
		static UE_API const FMetasoundFrontendClassName ClassName;
		static UE_API const FMetasoundFrontendVersionNumber VersionNumber;

		static UE_API const FInputNodeTemplate& GetChecked();
		static UE_API const FNodeRegistryKey& GetRegistryKey();

		virtual ~FInputNodeTemplate() = default;

#if WITH_EDITOR
		// Adds template node and connects with the input of the provided name on the page with
		// the provided id (defaults to builder's build page ID if not provided)
		static UE_API const FMetasoundFrontendNode* CreateNode(FMetaSoundFrontendDocumentBuilder& InOutBuilder, FName InputName, const FGuid* InPageID = nullptr);
#endif // WITH_EDITOR

		UE_API virtual const TArray<FMetasoundFrontendClassInputDefault>* FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const override;
		UE_API virtual const FMetasoundFrontendClassName& GetClassName() const override;

#if WITH_EDITOR
		UE_API virtual FText GetNodeDisplayName(const IMetaSoundDocumentInterface& Interface, const FGuid& InPageID, const FGuid& InNodeID) const override;
#endif // WITH_EDITOR

		UE_API virtual const FMetasoundFrontendClass& GetFrontendClass() const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;

#if WITH_EDITOR
		UE_API virtual FText GetOutputVertexDisplayName(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName OutputName) const override;
		UE_API virtual bool HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage = nullptr) const override;

		// Injects template nodes between builder's document inputs not connected
		// to existing template inputs, copying locational data from the represented
		// input metadata. If bForceNodeCreation is false, only generates a template
		// input node if a connection between the input and other nodes exists. If true,
		// will inject template node irrespective of whether or not the input has connections.
		UE_API bool Inject(FMetaSoundFrontendDocumentBuilder& InOutBuilder, bool bForceNodeCreation = false) const;
#endif // WITH_EDITOR

		UE_API virtual bool IsInputAccessTypeDynamic() const override;
		UE_API virtual bool IsInputConnectionUserModifiable() const override;
		UE_API virtual bool IsOutputAccessTypeDynamic() const override;
	};
} // namespace Metasound::Frontend

#undef UE_API
