// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundFrontendController.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundFrontendDocumentBuilder.h"
#include "MetasoundFrontendNodeTemplateRegistry.h"
#include "MetasoundFrontendTransform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"

#define UE_API METASOUNDFRONTEND_API


namespace Metasound::Frontend
{
	class FRerouteNodeTemplate : public FNodeTemplateBase
	{
	public:
		static UE_API const FMetasoundFrontendClassName ClassName;

		static UE_API const FMetasoundFrontendVersionNumber VersionNumber;

		static UE_API const FNodeRegistryKey& GetRegistryKey();

		UE_DEPRECATED(5.5, "Look-up template via registry and use non-static GenerateNodeInterface instead with provided params")
		static FMetasoundFrontendNodeInterface CreateNodeInterfaceFromDataType(FName InDataType) { return { }; }

		virtual ~FRerouteNodeTemplate() = default;

		UE_API virtual const FMetasoundFrontendClassName& GetClassName() const override;

#if WITH_EDITOR
		UE_API virtual FText GetNodeDisplayName(const IMetaSoundDocumentInterface& DocumentInterface, const FGuid& InPageID, const FGuid& InNodeID) const override;
#endif // WITH_EDITOR

		UE_API virtual FMetasoundFrontendNodeInterface GenerateNodeInterface(FNodeTemplateGenerateInterfaceParams InParams) const override;
		UE_API virtual TUniquePtr<INodeTemplateTransform> GenerateNodeTransform() const override;
		UE_API virtual const FMetasoundFrontendClass& GetFrontendClass() const override;
		UE_API virtual const TArray<FMetasoundFrontendClassInputDefault>* FindNodeClassInputDefaults(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FName VertexName) const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeInputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;
		UE_API virtual EMetasoundFrontendVertexAccessType GetNodeOutputAccessType(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, const FGuid& InVertexID) const override;
		UE_API virtual const FMetasoundFrontendVersionNumber& GetVersionNumber() const override;
		UE_API virtual bool IsInputAccessTypeDynamic() const override;
		UE_API virtual bool IsInputConnectionUserModifiable() const override;
		UE_API virtual bool IsOutputConnectionUserModifiable() const override;
		UE_API virtual bool IsOutputAccessTypeDynamic() const override;
		UE_API virtual bool IsValidNodeInterface(const FMetasoundFrontendNodeInterface& InNodeInterface) const override;

#if WITH_EDITOR
		UE_API virtual bool HasRequiredConnections(const FMetaSoundFrontendDocumentBuilder& InBuilder, const FGuid& InPageID, const FGuid& InNodeID, FString* OutMessage = nullptr) const override;

		UE_API virtual bool OnEdgeAdded(const FTemplateNodeEdgeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnNodeAdded(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnRemoveSwappingEdge(const FTemplateNodeEdgeRemoveTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
		UE_API virtual bool OnRemoveSwappingNode(const FTemplateNodeTransactionContext& TransactionContext, DynamicGraph::FDynamicOperatorTransactor& OutTransactor) const override;
#endif // WITH_EDITOR
	};
} // namespace Metasound::Frontend

#undef UE_API
