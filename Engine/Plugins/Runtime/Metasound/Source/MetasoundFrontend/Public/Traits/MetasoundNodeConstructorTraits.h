// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>

#include "MetasoundNodeInterface.h"
#include "Templates/SharedPointer.h"

namespace Metasound
{
	/** Determines whether the constructor provided by the node is usable in node
	 * registration. */
	template<typename T>
	struct TIsNodeConstructorSupported
	{
	private:
		static constexpr bool bIsConstructibleWithFNodeData = std::is_constructible_v<T, ::Metasound::FNodeData>;
		static constexpr bool bIsConstructibleWithFNodeDataAndClassMetadata = std::is_constructible_v<T, ::Metasound::FNodeData, TSharedRef<const FNodeClassMetadata>>;

	public:
		static constexpr bool Value = bIsConstructibleWithFNodeData || bIsConstructibleWithFNodeDataAndClassMetadata;
	};
}

