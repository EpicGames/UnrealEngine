// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>

#include "MetasoundNodeInterface.h"

namespace Metasound
{
	template <typename U>
	class TIsCreateNodeClassMetadataDeclared
	{
		private:
			template<typename T, T> 
			struct Helper;

			template<typename T>
			static uint8 Check(Helper<FNodeClassMetadata(*)(), &T::CreateNodeClassMetadata>*);

			template<typename T> static uint16 Check(...);

		public:

			// If the function exists, then "Value" is true. Otherwise "Value" is false.
			static constexpr bool Value = sizeof(Check<U>(0)) == sizeof(uint8);
	};
}
