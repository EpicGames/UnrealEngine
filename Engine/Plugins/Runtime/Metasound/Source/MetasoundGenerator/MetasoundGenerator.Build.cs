// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundGenerator : ModuleRules
	{
        public MetasoundGenerator(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SignalProcessing",
					"Engine",
					"AudioExtensions",
					"TraceLog",
					"AudioChannelAgnosticCore"
				}
			);

            PublicDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
					"CoreUObject",
					"AudioMixer",
					"MetasoundGraphCore",
					"MetasoundStandardNodes",
					"MetasoundFrontend"
                }
            );
		}
	}
}
