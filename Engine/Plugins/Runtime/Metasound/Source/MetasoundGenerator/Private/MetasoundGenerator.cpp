// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundGenerator.h"

#include "AudioParameter.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundGeneratorBuilder.h"
#include "MetasoundGeneratorModuleImpl.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOutputNode.h"
#include "MetasoundOperatorCache.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundRouter.h"
#include "MetasoundThreadLocalDebug.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertexData.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "DSP/FloatArrayMath.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "Modules/ModuleManager.h"
#include "AutoRTFM.h"
#include "ChannelAgnostic/ChannelAgnosticTranscoding.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "DSP/MultiMono.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

#ifndef ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING 
#define ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING !UE_BUILD_SHIPPING
#endif

CSV_DECLARE_CATEGORY_MODULE_EXTERN(METASOUNDGRAPHCORE_API, Audio_Metasound);
LLM_DEFINE_TAG(Audio_MetaSound_Generator_Execute);

namespace Metasound
{
	namespace ConsoleVariables
	{
		static bool bEnableAsyncMetaSoundGeneratorBuilder = true;
		static bool bEnableExperimentalAutoCachingForOneShotOperators = false;
		static bool bEnableExperimentalAutoCachingForAllOperators = false;
#if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING
		static bool bEnableMetaSoundGeneratorNonFiniteLogging = false;
		static bool bEnableMetaSoundGeneratorInvalidSampleValueLogging = false;
		static float MetasoundGeneratorSampleValueThreshold = 2.f;
#endif // if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING
	}

	namespace MetasoundGeneratorPrivate
	{
#if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING
		void LogInvalidAudioSampleValues(const FTopLevelAssetPath& InAssetPath, const TArray<FAudioBufferReadRef>& InAudioBuffers)
		{
			if (ConsoleVariables::bEnableMetaSoundGeneratorNonFiniteLogging || ConsoleVariables::bEnableMetaSoundGeneratorInvalidSampleValueLogging)
			{
				const int32 NumChannels = InAudioBuffers.Num();

				// Check outputs for non finite values if any sample value logging is enabled.
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
					const FAudioBuffer& Buffer = *InAudioBuffers[ChannelIndex];
					const float* Data = Buffer.GetData();
					const int32 Num = Buffer.Num();

					for (int32 i = 0; i < Num; i++)
					{
						if (!FMath::IsFinite(Data[i]))
						{
							UE_LOGF(LogMetaSound, Error, "Found non-finite sample (%f) in channel %d of MetaSound %ls", Data[i], ChannelIndex, *InAssetPath.ToString());
							break;
						}
					}

					// Only check threshold if explicitly enabled
					if (ConsoleVariables::bEnableMetaSoundGeneratorInvalidSampleValueLogging)
					{
						const float Threshold = FMath::Abs(ConsoleVariables::MetasoundGeneratorSampleValueThreshold);
						const float MaxAbsValue = Audio::ArrayMaxAbsValue(Buffer);
						if (MaxAbsValue > Threshold)
						{
							UE_LOGF(LogMetaSound, Warning, "Found sample (absolute value: %f) exceeding threshold (%f) in channel %d of MetaSound %ls", MaxAbsValue, Threshold, ChannelIndex, *InAssetPath.ToString());
						}
					}
				}
			}
		}
#endif // if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING

		struct FRenderTimer
		{
			FRenderTimer(const FOperatorSettings& InSettings, double InAnalysisDuration)
			{
				// Use single pole IIR filter to smooth data.
				
				const double AnalysisAudioFrameCount = FMath::Max(1.0, InAnalysisDuration * InSettings.GetSampleRate());
				const double AnalysisRenderBlockCount = FMath::Max(1.0, AnalysisAudioFrameCount / FMath::Max(1, InSettings.GetNumFramesPerBlock()));
				const double DigitalCutoff = 1. / AnalysisRenderBlockCount;

				SmoothingAlpha = 1. - FMath::Exp(-UE_PI * DigitalCutoff);
				SmoothingAlpha = FMath::Clamp(SmoothingAlpha, 0.0, 1.0 - UE_DOUBLE_SMALL_NUMBER);
				SecondsOfAudioProducedPerBlock = static_cast<double>(InSettings.GetNumFramesPerBlock()) / FMath::Max(1., static_cast<double>(InSettings.GetSampleRate()));
			}

			double UpdateCPUCoreUtilization()
			{
				double CPUSecondsToRenderBlock = FPlatformTime::ToSeconds64(AccumulatedCycles);
				if (CPUSecondsToRenderBlock > 0.0)
				{
					double NewCPUUtil = CPUSecondsToRenderBlock / SecondsOfAudioProducedPerBlock;
					if (CPUCoreUtilization >= 0.0)
					{
						CPUCoreUtilization = SmoothingAlpha * NewCPUUtil + (1. - SmoothingAlpha) * CPUCoreUtilization;
					}
					else
					{
						CPUCoreUtilization = NewCPUUtil;
					}
				}
				AccumulatedCycles = 0;
				return CPUCoreUtilization;
			}

			FORCEINLINE void AccumulateCycles(uint64 Cycles)
			{
				AccumulatedCycles += Cycles;
			}

		private:
			uint64 AccumulatedCycles = 0;
			double CPUCoreUtilization = -1.0;
			double SmoothingAlpha = 1.0;
			double SecondsOfAudioProducedPerBlock = 0.0;
		};

		struct FBlockRenderScope
		{
			FBlockRenderScope(FRenderTimer* InTimer)
			: Timer(InTimer)
			{
				StartCycle = FPlatformTime::Cycles64();
			}

			~FBlockRenderScope()
			{
				uint64 EndCycle = FPlatformTime::Cycles64();
				if (Timer)
				{
					Timer->AccumulateCycles(EndCycle - StartCycle);
				}
			}
			
		private:
			uint64 StartCycle = 0;
			FRenderTimer* Timer = nullptr;
		};

#ifndef ENABLE_METASOUND_CONSTANT_OUTPUT_VERTEX_ERROR_LOG
#define ENABLE_METASOUND_CONSTANT_OUTPUT_VERTEX_ERROR_LOG DO_CHECK
#endif

#if ENABLE_METASOUND_CONSTANT_OUTPUT_VERTEX_ERROR_LOG
		// Check whether a vertex name represents an output audio buffer. MetaSound
		// Generators cannot dynamically update their output audio buffers. This
		// check is here to inform future developers of this limitation.
		static const FLazyName ParameterNameComponentUE = "UE";
		static const FLazyName ParameterNameComponentOutputFormat = "OutputFormat";
		bool IsAudioOutputVertex(const FVertexName& InVertexName)
		{
			FName Namespace;
			FName Remaining;
			Audio::FParameterPath::SplitName(InVertexName, Namespace, Remaining);
			if (ParameterNameComponentUE == Namespace)
			{
				FName Group;
				FName Name;
				Audio::FParameterPath::SplitName(Remaining, Group, Name);
				return (ParameterNameComponentOutputFormat == Name);
			}
			return false;
		}

		void LogErrorIfIsAudioVertex(const FVertexName& InVertexName, const TCHAR* Message)
		{
			if (IsAudioOutputVertex(InVertexName))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot modify vertex %ls. %ls", *InVertexName.ToString(), Message);
			}
		};
#else
		void LogErrorIfIsAudioVertex(const FVertexName& InVertexName, const TCHAR* Message)
		{
		}
#endif
	}	
}

FAutoConsoleVariableRef CVarMetaSoundEnableAsyncGeneratorBuilder(
	TEXT("au.MetaSound.EnableAsyncGeneratorBuilder"),
	Metasound::ConsoleVariables::bEnableAsyncMetaSoundGeneratorBuilder,
	TEXT("Enables async building of FMetaSoundGenerators\n")
	TEXT("Default: true"),
	ECVF_Default);

FAutoConsoleVariableRef CVarMetaSoundEnableExperimentalAutoCachingForOneShotOperators(
	TEXT("au.MetaSound.Experimental.EnableAutoCachingForOneShotOperators"),
	Metasound::ConsoleVariables::bEnableExperimentalAutoCachingForOneShotOperators,
	TEXT("Enables auto-caching of MetaSound operators using the OneShot source interface.\n")
	TEXT("(see MetasoundOperatorCacheSubsystem.h for manual path).\n")
	TEXT("Default: false"),
	ECVF_Default);

FAutoConsoleVariableRef CVarMetaSoundEnableExperimentalAutoCachingForAllOperators(
	TEXT("au.MetaSound.Experimental.EnableAutoCachingForAllOperators"),
	Metasound::ConsoleVariables::bEnableExperimentalAutoCachingForAllOperators,
	TEXT("Enables auto-caching of all MetaSound operators.\n")
	TEXT("(see MetasoundOperatorCacheSubsystem.h for manual path).\n")
	TEXT("Default: false"),
	ECVF_Default);

#if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING

FAutoConsoleVariableRef CVarMetaSoundEnableGeneratorNonFiniteLogging(
	TEXT("au.MetaSound.EnableGeneratorNonFiniteLogging"),
	Metasound::ConsoleVariables::bEnableMetaSoundGeneratorNonFiniteLogging,
	TEXT("Enables logging of non-finite (NaN/inf) audio samples values produced from a FMetaSoundGenerator\n")
	TEXT("Default: false"),
	ECVF_Default);

FAutoConsoleVariableRef CVarMetaSoundEnableGeneratorInvalidSampleValueLogging(
	TEXT("au.MetaSound.EnableGeneratorInvalidSampleValueLogging"),
	Metasound::ConsoleVariables::bEnableMetaSoundGeneratorInvalidSampleValueLogging,
	TEXT("Enables logging of audio samples values produced from a FMetaSoundGenerator which exceed the absolute sample value threshold\n")
	TEXT("Default: false"),
	ECVF_Default);

FAutoConsoleVariableRef CVarMetaSoundGeneratorSampleValueThrehshold(
	TEXT("au.MetaSound.GeneratorSampleValueThreshold"),
	Metasound::ConsoleVariables::MetasoundGeneratorSampleValueThreshold,
	TEXT("If invalid sample value logging is enabled, this sets the maximum abs value threshold for logging samples\n")
	TEXT("Default: 2.0"),
	ECVF_Default);

#endif // if !UE_BUILD_SHIPPING

namespace Metasound
{
	namespace MetasoundGeneratorPrivate
	{
		bool HasOneShotInterface(const FVertexInterface& Interface)
		{
			return Interface.GetOutputInterface().Contains(Frontend::SourceOneShotInterface::Outputs::OnFinished);
		}
	}

	void FGeneratorInitParams::Reset(FGeneratorInitParams& InParams)
	{
		InParams.Environment = {};
		InParams.AssetPath = {};
		InParams.AudioOutputNames = {};
		InParams.DefaultParameters = {};
		InParams.DataChannel.Reset();
		InParams.GraphRenderCost.Reset();
		InParams.SourceFormatName = NAME_None;
	}

	void FMetasoundDynamicGraphGeneratorInitParams::Reset(FMetasoundDynamicGraphGeneratorInitParams& InParams)
	{
		FGeneratorInitParams::Reset(InParams);
		InParams.TransformQueue.Reset();
	}

	FMetasoundGenerator::FMetasoundGenerator(const FOperatorSettings& InOperatorSettings)
		: OperatorSettings(InOperatorSettings)
#if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
		, InstanceCounter(FModuleManager::GetModuleChecked<FMetasoundGeneratorModule>("MetasoundGenerator").GetOperatorInstanceCounterManager())
#endif // if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
		, OnFinishedTriggerRef(FTriggerWriteRef::CreateNew(InOperatorSettings))
#if ENABLE_METASOUND_GENERATOR_RENDER_TIMING
		, bDoRuntimeRenderTiming(1)
#endif // ENABLE_METASOUND_GENERATOR_RENDER_TIMING
		, NumFramesPerExecute(InOperatorSettings.GetNumFramesPerBlock())
	{
	}

	FMetasoundGenerator::~FMetasoundGenerator()
	{
		// Remove routing for parameter packs
		ParameterPackReceiver.Reset();
		FDataTransmissionCenter::Get().UnregisterDataChannelIfUnconnected(ParameterPackSendAddress);
	}

	FDelegateHandle FMetasoundGenerator::AddGraphSetCallback(FOnSetGraph::FDelegate&& Delegate)
	{
		FScopeLock SetPendingGraphLock(&PendingGraphMutex);
		// If we already have a graph give the delegate an initial call
		if (!bIsNewGraphPending && !bIsWaitingForFirstGraph)
		{
			Delegate.ExecuteIfBound();
		}
		return OnSetGraph.Add(Delegate);
	}

	// Clamp a channel count to the nearest mixer-compatible value (1, 2, 4, 6, 8).
	static int32 ClampToMixerChannelCount(int32 InChannels)
	{
		if (InChannels <= 1) return 1;
		if (InChannels <= 2) return 2;
		if (InChannels <= 4) return 4;
		if (InChannels <= 6) return 6;
		return 8;
	}

	// Resolve the mixer-compatible channel count from a CAT SourceFormatName.
	static int32 ResolveNumChannelsFromFormat(FName InSourceFormatName)
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		if (const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindChannel(InSourceFormatName))
		{
			return ClampToMixerChannelCount(Found->NumChannels());
		}
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		UE_LOGF(LogMetaSound, Warning, "CAT format '%ls' not found in channel registry; defaulting to stereo.", *InSourceFormatName.ToString());
		return 2;
	}

	void FMetasoundGenerator::InitBase(FGeneratorInitParams& InInitParams)
	{
		AssetPath = InInitParams.AssetPath;
		SourceFormatName = InInitParams.SourceFormatName;

		if (!SourceFormatName.IsNone())
		{
			// CAT path: derive mixer-compatible channel count from format name.
			NumChannels = ResolveNumChannelsFromFormat(SourceFormatName);
		}
		else
		{
			// Discrete path: channel count equals number of audio output vertices.
			NumChannels = InInitParams.AudioOutputNames.Num();
		}
		NumSamplesPerExecute = NumChannels * NumFramesPerExecute;

#if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
		InstanceCounter.Init(AssetPath);
#endif // if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING

		// Data channels
		ParameterQueue = InInitParams.DataChannel;

		// Create the routing for parameter packs
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ParameterPackSendAddress = UMetasoundParameterPack::CreateSendAddressFromEnvironment(InInitParams.Environment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ParameterPackReceiver = FDataTransmissionCenter::Get().RegisterNewReceiver<FMetasoundParameterStorageWrapper>(ParameterPackSendAddress, FReceiverInitParams{ OperatorSettings });
	}

	bool FMetasoundGenerator::RemoveGraphSetCallback(const FDelegateHandle& Handle)
	{
		return OnSetGraph.Remove(Handle);
	}

	void FMetasoundGenerator::SetPendingGraph(MetasoundGeneratorPrivate::FMetasoundGeneratorData&& InData, bool bTriggerGraph)
	{
		using namespace MetasoundGeneratorPrivate;

		FScopeLock SetPendingGraphLock(&PendingGraphMutex);
		{
			PendingGraphData = MakeUnique<FMetasoundGeneratorData>(MoveTemp(InData));
			bPendingGraphTrigger = bTriggerGraph;
			bIsNewGraphPending = true;
		}
	}

	void FMetasoundGenerator::SetPendingGraphBuildFailed()
	{
		FScopeLock SetPendingGraphLock(&PendingGraphMutex);
		{
			PendingGraphData.Reset();
			bPendingGraphTrigger = false;
			bIsNewGraphPending = true;
		}
	}

	bool FMetasoundGenerator::UpdateGraphIfPending()
	{
		FScopeLock GraphLock(&PendingGraphMutex);
		if (bIsNewGraphPending)
		{
			SetGraph(MoveTemp(PendingGraphData), bPendingGraphTrigger);
			bIsNewGraphPending = false;
			return true;
		}

		return false;
	}

	void FMetasoundGenerator::SetGraph(TUniquePtr<MetasoundGeneratorPrivate::FMetasoundGeneratorData>&& InData, bool bTriggerGraph)
	{
		if (!InData.IsValid())
		{
			return;
		}

		InterleavedAudioBuffer.Reset();

		// Copy off all vertex interface data
		VertexInterfaceData = MoveTemp(InData->VertexInterfaceData);

		GraphOutputAudio.Reset();
		GraphCatOutput.Reset();

		if (InData->OutputChannelAgnosticRef.IsSet())
		{
			// The MixerOutput must be a format AudioMixer supports (1/2/4/6/8 channels).
			// SourceFormatName carries the CAT format identity for the graph side (via Environment),
			// but the mixer-facing output is always determined from NumChannels (already clamped to
			// mixer-compatible values). This is intentionally lossy — the transcoder handles the
			// conversion from whatever the graph produces to the mixer's discrete format.
			//
			// IMPORTANT: Do NOT early-return from this block. SetGraph must complete operator
			// installation below regardless of CAT setup success. Early returns after buffer
			// resets and MoveTemp leave the generator in a stranded partial-init state with
			// no retry path (the pending graph has already been consumed).
			const TSharedPtr<const Audio::FDiscreteChannelTypeFamily> MixerFormat =
				Audio::FChannelAgnosticUtils::FindDiscreteFormatFromNumChannels(NumChannels);
			const Audio::FChannelTypeFamily* MixerChannel = nullptr;
			if (MixerFormat.IsValid())
			{
				MixerChannel = Audio::GetChannelRegistry().FindConcreteChannel(MixerFormat->GetName());
			}

			if (ensureMsgf(MixerChannel, TEXT("Failed to resolve mixer channel format for %d channels. CAT output will be silent."), NumChannels))
			{
				// Check if the graph output format already matches the mixer format.
				// If so, skip the intermediate buffer and transcoder entirely — just interleave
				// directly from the graph output. This avoids allocating a redundant MixerOutput
				// buffer and a no-op memcpy transcode on every block. (Addresses Phil Popp's
				// memory bloat concern: most CAT sources output a mixer-compatible format.)
				const Audio::FChannelTypeFamily& GraphOutputType = InData->OutputChannelAgnosticRef->Get()->GetType();
				const bool bIdentity = (&GraphOutputType == MixerChannel) || (GraphOutputType.GetName() == MixerChannel->GetName());

				if (bIdentity)
				{
					// Identity: graph output IS the mixer format. No transcode, no extra buffer.
					GraphCatOutput.Emplace(FChannelAgnosticState{
						*InData->OutputChannelAgnosticRef,	// GraphOutput
						{},					// MixerOutput — not allocated
						{},					// Transcoder — not needed
						true					// bIdentityTranscode
					});
				}
				else
				{
					// Non-identity: allocate mixer buffer and resolve transcoder.
					Audio::FChannelAgnosticType MixOut(*MixerChannel, NumFramesPerExecute);
					Audio::ChannelAgnosticTranscoder::FTranscoder Transcoder = Audio::FTranscoderResolver::Resolve(
						GraphOutputType,
						{
							.ToType = MixOut.GetType(),
							.TranscodeMethod = Audio::EChannelTranscodeMethod::MixUpOrDown,
							.MixMethod = Audio::EChannelMapMonoUpmixMethod::EqualPower,
						});
					if (ensureMsgf(Transcoder.IsSet(), TEXT("Failed to resolve CAT transcoder from '%s' to '%s'. CAT output will be silent."),
						*GraphOutputType.GetName().ToString(), *MixerChannel->GetName().ToString()))
					{
						GraphCatOutput.Emplace(FChannelAgnosticState{
							*InData->OutputChannelAgnosticRef,	// GraphOutput
							MoveTemp(MixOut),			// MixerOutput
							MoveTemp(Transcoder),			// Transcoder
							false					// bIdentityTranscode
						});
					}
					// On transcoder failure: GraphCatOutput stays empty → InterleaveGeneratedAudio
					// skips CAT path → silence for this source, but generator completes init normally.
				}
			}
			// On MixerChannel failure: GraphCatOutput stays empty → silence, but generator init completes.
		}
		else if (InData->OutputBuffers.Num() == NumChannels)
		{
			if (InData->OutputBuffers.Num() > 0)
			{
				GraphOutputAudio.Append(InData->OutputBuffers.GetData(), InData->OutputBuffers.Num());
			}
		}
		else
		{
			int32 FoundNumChannels = InData->OutputBuffers.Num();

			UE_LOGF(LogMetaSound, Warning, "Metasound generator expected %d number of channels, found %d", NumChannels, FoundNumChannels);

			int32 NumChannelsToCopy = FMath::Min(FoundNumChannels, NumChannels);
			int32 NumChannelsToCreate = NumChannels - NumChannelsToCopy;

			if (NumChannelsToCopy > 0)
			{
				GraphOutputAudio.Append(InData->OutputBuffers.GetData(), NumChannelsToCopy);
			}
			for (int32 i = 0; i < NumChannelsToCreate; i++)
			{
				GraphOutputAudio.Add(TDataReadReference<FAudioBuffer>::CreateNew(InData->OperatorSettings));
			}
		}

		OnFinishedTriggerRef = InData->TriggerOnFinishRef;

		// The graph operator and graph audio output contain all the values needed
		// by the sound generator.
		RootExecuter.SetOperator(MoveTemp(InData->GraphOperator));


		// Validate that either discrete output buffers or CAT output is configured.
		// If CAT setup failed (ensureMsgf above), both may be empty — that's a valid
		// degraded state (silence), not a crash. Only assert if we have neither AND
		// NumChannels expects output (which would cause OOB in InterleaveGeneratedAudio).
		ensureMsgf(NumChannels == GraphOutputAudio.Num() || GraphCatOutput.IsSet() || NumChannels == 0,
			TEXT("MetaSound generator has %d expected channels but no audio output configured (GraphOutputAudio=%d, CatOutput=%s)"),
			NumChannels, GraphOutputAudio.Num(), GraphCatOutput.IsSet() ? TEXT("set") : TEXT("unset"));

		if (NumSamplesPerExecute > 0)
		{
			// Preallocate interleaved buffer as it is necessary for any audio generation calls.
			InterleavedAudioBuffer.AddUninitialized(NumSamplesPerExecute);
		}

		GraphAnalyzer = MoveTemp(InData->GraphAnalyzer);

		ParameterSetters = MoveTemp(InData->ParameterSetters);
		ParameterPackSetters = MoveTemp(InData->ParameterPackSetters);

		if (bTriggerGraph)
		{
			InData->TriggerOnPlayRef->TriggerFrame(0);
		}

		bIsWaitingForFirstGraph = false;
		OnSetGraph.Broadcast();

		for (const auto& Binding : VertexInterfaceData.GetInputs())
		{
			VertexInterfaceChangesSinceLastBroadcast.Add({ Binding.GetVertex().VertexName, EMetasoundFrontendClassType::Input, EVertexInterfaceChangeType::Added });
		}

		for (const auto& Binding : VertexInterfaceData.GetOutputs())
		{
			VertexInterfaceChangesSinceLastBroadcast.Add({ Binding.GetVertex().VertexName, EMetasoundFrontendClassType::Output, EVertexInterfaceChangeType::Added });
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TUniquePtr<IOperator> FMetasoundGenerator::ReleaseGraphOperator()
	{
		return RootExecuter.ReleaseOperator();
	}

	FInputVertexInterfaceData FMetasoundGenerator::ReleaseInputVertexData()
	{
		return MoveTemp(VertexInterfaceData.GetInputs());
	}

	void FMetasoundGenerator::ClearGraph()
	{
		for (const auto& Binding : VertexInterfaceData.GetInputs())
		{
			VertexInterfaceChangesSinceLastBroadcast.Add({ Binding.GetVertex().VertexName, EMetasoundFrontendClassType::Input, EVertexInterfaceChangeType::Removed });
		}

		for (const auto& Binding : VertexInterfaceData.GetOutputs())
		{
			VertexInterfaceChangesSinceLastBroadcast.Add({ Binding.GetVertex().VertexName, EMetasoundFrontendClassType::Output, EVertexInterfaceChangeType::Removed });
		}

		RootExecuter.ReleaseOperator();
		VertexInterfaceData = FVertexInterfaceData();
		GraphOutputAudio.Reset();
		GraphCatOutput.Reset();
		OnFinishedTriggerRef = TDataWriteReference<FTrigger>::CreateNew(OperatorSettings);
		GraphAnalyzer.Reset();
		OutputAnalyzers.Reset();
		ParameterSetters.Reset();
		ParameterPackSetters.Reset();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FMetasoundGenerator::QueueParameterPack(TSharedPtr<FMetasoundParameterPackStorage> ParameterPack)
	{
		// We can't start processing Metasound data until the Verse transaction completes, since it might be rolled back.
		UE_AUTORTFM_ONCOMMIT(this, ParameterPack)
		{
			ParameterPackQueue.Enqueue(ParameterPack);
		};
	}

	int32 FMetasoundGenerator::GetNumChannels() const
	{
		return NumChannels;
	}

	bool AnalyzerAddressesReferToSameGeneratorOutput(const Frontend::FAnalyzerAddress& Lhs, const Frontend::FAnalyzerAddress& Rhs)
	{
		return Lhs.OutputName == Rhs.OutputName && Lhs.AnalyzerName == Rhs.AnalyzerName;
	}
	
	void FMetasoundGenerator::AddOutputVertexAnalyzer(const Frontend::FAnalyzerAddress& AnalyzerAddress)
	{
		OutputAnalyzerModificationQueue.Enqueue([this, AnalyzerAddress]
		{
			const FAnyDataReference* OutputReference = VertexInterfaceData.GetOutputs().FindDataReference(AnalyzerAddress.OutputName);

			if (nullptr == OutputReference)
			{
				return;
			}

			TUniquePtr<Frontend::IVertexAnalyzer>* ExistingAnalyzer = OutputAnalyzers.FindByPredicate([AnalyzerAddress](const TUniquePtr<Frontend::IVertexAnalyzer>& Analyzer)
				{
					return AnalyzerAddressesReferToSameGeneratorOutput(AnalyzerAddress, Analyzer->GetAnalyzerAddress());
				});

			if (ExistingAnalyzer && ExistingAnalyzer->IsValid())
			{
				if (ExistingAnalyzer->Get()->GetDataReferenceId() != GetDataReferenceID(*OutputReference))
				{
					ExistingAnalyzer->Get()->SetDataReference(*OutputReference);
				}

				return;
			}
			
			const Frontend::IVertexAnalyzerFactory* Factory = Frontend::IVertexAnalyzerRegistry::Get().FindAnalyzerFactory(AnalyzerAddress.AnalyzerName);
			
			if (nullptr == Factory)
			{
				return;
			}
			
			TUniquePtr<Frontend::IVertexAnalyzer> Analyzer = Factory->CreateAnalyzer({ AnalyzerAddress, OperatorSettings, *OutputReference});
			
			if (nullptr == Analyzer)
			{
				return;
			}

			if (!Analyzer->OnOutputDataChanged.IsBound())
			{
				Analyzer->OnOutputDataChanged.BindLambda(
					[this, AnalyzerAddress](const FName AnalyzerOutputName, TSharedPtr<IOutputStorage> OutputData)
					{
						OnOutputChanged.Broadcast(
							AnalyzerAddress.AnalyzerName,
							AnalyzerAddress.OutputName,
							AnalyzerOutputName,
							OutputData);
					});
			}
			
			OutputAnalyzers.Emplace(MoveTemp(Analyzer));
		});
	}

	void FMetasoundGenerator::RemoveOutputVertexAnalyzer(const Frontend::FAnalyzerAddress& AnalyzerAddress)
	{
		OutputAnalyzerModificationQueue.Enqueue([this, AnalyzerAddress]
		{
			OutputAnalyzers.RemoveAll([AnalyzerAddress](const TUniquePtr<Frontend::IVertexAnalyzer>& Analyzer)
			{
				return AnalyzerAddressesReferToSameGeneratorOutput(AnalyzerAddress, Analyzer->GetAnalyzerAddress());
			});
		});
	}

	int32 FMetasoundGenerator::OnGenerateAudio(float* OutAudio, int32 NumSamplesRemaining)
	{
		METASOUND_LLM_SCOPE;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MetasoundGenerator::OnGenerateAudio %s"), *AssetPath.GetAssetName().ToString()));
		CSV_SCOPED_TIMING_STAT(Audio_Metasound, OnGenerateAudio);

		// Defer finishing the metasound generator one block
		if (bIsFinishTriggered)
		{
			bIsFinished = true;
		}

		if (NumSamplesRemaining <= 0)
		{
			return 0;
		}

		UpdateGraphIfPending();

		HandleRenderTimingEnableDisable();

		// Output silent audio if we're still building a graph
		if (bIsWaitingForFirstGraph)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MetasoundGenerator::OnGenerateAudio::MissedRenderDeadline %s"), *AssetPath.GetAssetName().ToString()));
			CSV_CUSTOM_STAT(Audio_Metasound, WaitingForGraphBuildOnPlaybackCount, 1, ECsvCustomStatOp::Accumulate);
			FMemory::Memset(OutAudio, 0, sizeof(float)* NumSamplesRemaining);
			return NumSamplesRemaining;
		}

		// If no longer pending and executer is no-op, kill the MetaSound.
		// Covers case where there was an error when building, resulting in
		// Executer operator being assigned to NoOp.
		else if (RootExecuter.IsNoOp() || NumSamplesPerExecute < 1)
		{
			bIsFinished = true;
			FMemory::Memset(OutAudio, 0, sizeof(float) * NumSamplesRemaining);
			return NumSamplesRemaining;
		}

		// Modify the output analyzers as needed
		{
			while (TOptional<TUniqueFunction<void()>> ModFn = OutputAnalyzerModificationQueue.Dequeue())
			{
				(*ModFn)();
			}
		}

		// If we have any audio left in the internal overflow buffer from 
		// previous calls, write that to the output before generating more audio.
		int32 NumSamplesWritten = FillWithBuffer(OverflowBuffer, OutAudio, NumSamplesRemaining);

		if (NumSamplesWritten > 0)
		{
			NumSamplesRemaining -= NumSamplesWritten;
			OverflowBuffer.RemoveAtSwap(0 /* Index */, NumSamplesWritten /* Count */, EAllowShrinking::No);
		}

		while (NumSamplesRemaining > 0 && bIsFinishTriggered == 0)
		{
			if (GraphRenderCost)
			{
				// Set all render node costs to zero to start
				GraphRenderCost->ResetNodeRenderCosts();
			}
			// Create a scoped timed section for the bulk of the processing...
			{
				MetasoundGeneratorPrivate::FBlockRenderScope RuntimelBlockRenderScope(RenderTimer.Get());
				
				LLM_SCOPE_BYTAG(Audio_MetaSound_Generator_Execute);
				
				ApplyPendingUpdatesToInputs();

				// Call metasound graph operator.
 				RootExecuter.Execute();
				
				// Check if generated finished during this execute call
				if (*OnFinishedTriggerRef)
				{
					FinishSample = FMath::Max(0, ((*OnFinishedTriggerRef)[0] * NumChannels));
				}

				// Interleave audio because ISoundGenerator interface expects interleaved audio.
				InterleaveGeneratedAudio();


				// Add audio generated during graph execution to the output buffer.
				const int32 ThisLoopNumSamplesWritten = FillWithBuffer(InterleavedAudioBuffer, &OutAudio[NumSamplesWritten], NumSamplesRemaining);

				NumSamplesRemaining -= ThisLoopNumSamplesWritten;
				NumSamplesWritten += ThisLoopNumSamplesWritten;

				// If not all the samples were written, then we have to save the 
				// additional samples to the overflow buffer.
				if (ThisLoopNumSamplesWritten < InterleavedAudioBuffer.Num())
				{
					int32 OverflowCount = InterleavedAudioBuffer.Num() - ThisLoopNumSamplesWritten;

					OverflowBuffer.Reset();
					OverflowBuffer.AddUninitialized(OverflowCount);

					FMemory::Memcpy(OverflowBuffer.GetData(), &InterleavedAudioBuffer.GetData()[ThisLoopNumSamplesWritten], OverflowCount * sizeof(float));
				}

				// Execute the output analyzers
				for (const TUniquePtr<Frontend::IVertexAnalyzer>& Analyzer : OutputAnalyzers)
				{
					Analyzer->Execute();
				}
				
				if (ThisLoopNumSamplesWritten <= 0)
				{
					UE_LOGF(LogMetasoundGenerator, Warning, "Terminating OnGenerateAudio() loop for MetaSound %ls -- FillWithBuffer() returned %i", *AssetPath.ToString(), ThisLoopNumSamplesWritten);
					break;
				}
			}

			// Don't time the graph analyzer. It is only used for graph visualization.

			if (GraphAnalyzer.IsValid())
			{
				GraphAnalyzer->Execute();
			}

			// Create a scoped timed section for the post processing...
			{
				MetasoundGeneratorPrivate::FBlockRenderScope RuntimelBlockRenderScope(RenderTimer.Get());
				RootExecuter.PostExecute();
			}

			// Update timer if there is one...
			if (RenderTimer)
			{
				RenderTime = RenderTimer->UpdateCPUCoreUtilization();
			}

			if (GraphRenderCost)
			{
				// Require the cost to be a minimum of 1. 
				RelativeRenderCost = FMath::Max(1.f, GraphRenderCost->ComputeGraphRenderCost());
			}

		}

		// If the vertex interface changed, notify listeners
		if (!VertexInterfaceChangesSinceLastBroadcast.IsEmpty())
		{
			OnVertexInterfaceDataUpdated.Broadcast(VertexInterfaceData);
			OnVertexInterfaceDataUpdatedWithChanges.Broadcast(VertexInterfaceChangesSinceLastBroadcast);
			VertexInterfaceChangesSinceLastBroadcast.Empty();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			bVertexInterfaceHasChanged.store(false);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		return NumSamplesWritten;
	}

	int32 FMetasoundGenerator::GetDesiredNumSamplesToRenderPerCallback() const
	{
		return NumFramesPerExecute * NumChannels;
	}

	bool FMetasoundGenerator::IsFinished() const
	{
		return bIsFinished;
	}

	double FMetasoundGenerator::GetCPUCoreUtilization() const
	{
		return RenderTime;
	}

	float FMetasoundGenerator::GetRelativeRenderCost() const
	{
		return RelativeRenderCost;
	}

	int32 FMetasoundGenerator::FillWithBuffer(const Audio::FAlignedFloatBuffer& InBuffer, float* OutAudio, int32 MaxNumOutputSamples)
	{
		const int32 InNum = InBuffer.Num();
		if (!InNum)
		{
			return 0;
		}

		const int32 NumSamplesToCopy = FMath::Min(InNum, MaxNumOutputSamples);
		
		if (bIsFinishTriggered)
		{
			FMemory::Memcpy(OutAudio, InBuffer.GetData(), MaxNumOutputSamples * sizeof(float));
			return NumSamplesToCopy;
		}
		
		
		if (FinishSample != INDEX_NONE)
		{
			if (FinishSample > NumSamplesToCopy)
			{
				FinishSample -= NumSamplesToCopy;
			}
			else
			{
				bIsFinishTriggered = 1;
			}
		}
		
		const int32 NumFramesToWrite = bIsFinishTriggered ? FinishSample : NumSamplesToCopy;
		const int32 NumFramesToZero = MaxNumOutputSamples - NumFramesToWrite;
		FinishSample = bIsFinishTriggered ? INDEX_NONE : FinishSample;
		
		FMemory::Memcpy(OutAudio, InBuffer.GetData(), NumFramesToWrite * sizeof(float));
		FMemory::Memzero(OutAudio + NumFramesToWrite, NumFramesToZero * sizeof(float));
		
		return NumSamplesToCopy;
	}

	void FMetasoundGenerator::InterleaveGeneratedAudio()
	{
		// Prepare output buffer
		InterleavedAudioBuffer.Reset();

		if (NumSamplesPerExecute > 0)
		{
			InterleavedAudioBuffer.AddUninitialized(NumSamplesPerExecute);
		}

#if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING
		MetasoundGeneratorPrivate::LogInvalidAudioSampleValues(AssetPath, GraphOutputAudio);
#endif // if ENABLE_METASOUNDGENERATOR_INVALID_SAMPLE_VALUE_LOGGING

		if (GraphCatOutput.IsSet())
		{
			// Pointer to the multi-mono data we'll interleave from.
			const Audio::FChannelAgnosticType* SourceData = nullptr;

			if (GraphCatOutput->bIdentityTranscode)
			{
				// Fast path: graph output format matches mixer format — no transcode needed,
				// no intermediate buffer allocated. Read directly from graph output.
				SourceData = GraphCatOutput->GraphOutput.operator->();
			}
			else
			{
				// Transcode path: graph output format differs from mixer format.
				// Convert into the pre-allocated MixerOutput buffer, then interleave from that.
				check(GraphCatOutput->MixerOutput.IsSet());
				Audio::TStackArrayOfPointers<const float> Src = Audio::MakeMultiMonoPointersFromView(GraphCatOutput->GraphOutput->GetRawMultiMono(), NumFramesPerExecute, GraphCatOutput->GraphOutput->NumChannels());
				Audio::TStackArrayOfPointers<float> Dst = Audio::MakeMultiMonoPointersFromView(GraphCatOutput->MixerOutput->GetRawMultiMono(), NumFramesPerExecute, GraphCatOutput->MixerOutput->NumChannels());
				GraphCatOutput->Transcoder(Src, Dst, NumFramesPerExecute);
				SourceData = &GraphCatOutput->MixerOutput.GetValue();
			}

			// Interleave from SourceData into the output buffer.
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				const TArrayView<const float> InputBuffer = SourceData->GetChannel(ChannelIndex);

				const float* InputPtr = InputBuffer.GetData();
				float* OutputPtr = &InterleavedAudioBuffer.GetData()[ChannelIndex];

				// Assign values to output for single channel.
				for (int32 FrameIndex = 0; FrameIndex < NumFramesPerExecute; FrameIndex++)
				{
					*OutputPtr = InputPtr[FrameIndex];
					OutputPtr += NumChannels;
				}
			}
		}
		else
		{
			// Iterate over channels
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				const FAudioBuffer& InputBuffer = *GraphOutputAudio[ChannelIndex];

				const float* InputPtr = InputBuffer.GetData();
				float* OutputPtr = &InterleavedAudioBuffer.GetData()[ChannelIndex];

				// Assign values to output for single channel.
				for (int32 FrameIndex = 0; FrameIndex < NumFramesPerExecute; FrameIndex++)
				{
					*OutputPtr = InputPtr[FrameIndex];
					OutputPtr += NumChannels;
				}
			}
		}
		// TODO: memcpy for single channel. 
	}

	void FMetasoundGenerator::ApplyPendingUpdatesToInputs()
	{
		using namespace Frontend;
		using namespace MetasoundGeneratorPrivate;

		auto ProcessPack = [&](FMetasoundParameterPackStorage* Pack)
			{
				for (auto Walker = Pack->begin(); Walker != Pack->end(); ++Walker)
				{
					if (const FParameterPackSetter* ParameterPackSetter = ParameterPackSetters.Find(Walker->Name))
					{
						if (ParameterPackSetter->DataType == Walker->TypeName)
						{
							ParameterPackSetter->SetParameterWithPayload(Walker->GetPayload());
						}
					}
				}
			};

		// Handle parameters from the FMetasoundParameterTransmitter
		if (ParameterQueue.IsValid())
		{
			TOptional<FMetaSoundParameterTransmitter::FParameter> Parameter;
			while ((Parameter = ParameterQueue->Dequeue()))
			{
				if (FParameterSetter* Setter = ParameterSetters.Find(Parameter->Name))
				{
					Setter->Assign(OperatorSettings, Parameter->Value, Setter->DataReference);
				}
			}
		}

		// Handle parameter packs that have come from the IAudioParameterInterface system...
		if (ParameterPackReceiver.IsValid())
		{
			FMetasoundParameterStorageWrapper Pack;
			while (ParameterPackReceiver->CanPop())
			{
				ParameterPackReceiver->Pop(Pack);
				ProcessPack(Pack.Get().Get());
			}
		}

		// Handle parameter packs that came from QueueParameterPack...
		TSharedPtr<FMetasoundParameterPackStorage> QueuedParameterPack;
		while (ParameterPackQueue.Dequeue(QueuedParameterPack))
		{
			ProcessPack(QueuedParameterPack.Get());
		}
	}

	void FMetasoundGenerator::HandleRenderTimingEnableDisable()
	{
		if (bDoRuntimeRenderTiming && !RenderTimer)
		{
			RenderTimer = MakeUnique<MetasoundGeneratorPrivate::FRenderTimer>(OperatorSettings, 1. /* AnalysisPeriod */);
		}
		else if (!bDoRuntimeRenderTiming && RenderTimer)
		{
			RenderTimer = nullptr;
			RenderTime = 0.0;
		}
	}

	FMetasoundConstGraphGenerator::FMetasoundConstGraphGenerator(FGeneratorInitParams&& InInitParams)
	: FMetasoundGenerator(InInitParams.OperatorSettings)
	{
		Init(MoveTemp(InInitParams));
	}

	FMetasoundConstGraphGenerator::FMetasoundConstGraphGenerator(const FOperatorSettings& InOperatorSettings)
	: FMetasoundGenerator(InOperatorSettings)
	{
	}

	static FThreadSafeCounter NumActiveConstGraphs;
	void FMetasoundConstGraphGenerator::Init(FGeneratorInitParams&& InParams)
	{
		NumActiveConstGraphs.Increment();
		CSV_CUSTOM_STAT(Audio_Metasound, NumActiveConstGraphs, NumActiveConstGraphs.GetValue(), ECsvCustomStatOp::Set);

		InitBase(InParams);
		// attempt to use operator cache instead of building a new operator.
		bool bDidUseCachedOperator = false;
		const bool bIsOperatorPoolEnabled = ConsoleVariables::bEnableExperimentalAutoCachingForOneShotOperators || ConsoleVariables::bEnableExperimentalAutoCachingForAllOperators;
		// Dynamic operators cannot use the operator cache because they can change their internal structure. 
		// The operator cache assumes that the operator is unchanged from it's original structure. 
		if (bIsOperatorPoolEnabled)
		{
			bUseOperatorPool = ConsoleVariables::bEnableExperimentalAutoCachingForAllOperators || MetasoundGeneratorPrivate::HasOneShotInterface(InParams.Graph->GetVertexInterface());
		}

		// check the cache for manually pre-cached operators
		// even if the cvars disable automatic cache population.
		bDidUseCachedOperator = TryUseCachedOperator(InParams, true /* bTriggerGenerator */);

		if (!bDidUseCachedOperator)
		{
			GraphRenderCost = FGraphRenderCost::MakeGraphRenderCost();
			InParams.GraphRenderCost = GraphRenderCost;
			BuildGraph(MoveTemp(InParams));
		}
	}

	FMetasoundConstGraphGenerator::~FMetasoundConstGraphGenerator()
	{
		NumActiveConstGraphs.Decrement();
		CSV_CUSTOM_STAT(Audio_Metasound, NumActiveConstGraphs, NumActiveConstGraphs.GetValue(), ECsvCustomStatOp::Set);

		if (BuilderTask.IsValid())
		{
			BuilderTask->EnsureCompletion();
			BuilderTask = nullptr;
		}

		if (bUseOperatorPool)
		{
			UpdateGraphIfPending();
			ReleaseOperatorToCache();
		}
	}

	bool FMetasoundConstGraphGenerator::TryUseCachedOperator(FGeneratorInitParams& InInitParams, bool bInTriggerGraph)
	{
		using namespace MetasoundGeneratorPrivate;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundConstGraphGenerator::TryUseCachedOperator);

		FMetasoundGeneratorModule& Module = FModuleManager::GetModuleChecked<FMetasoundGeneratorModule>("MetasoundGenerator");
		TSharedPtr<FOperatorPool> OperatorPool = Module.GetOperatorPool();
		if (OperatorPool.IsValid())
		{
			// See if the cache has an operator with matching OperatorID
			OperatorPoolID = FOperatorPoolEntryID{InInitParams.Graph->GetInstanceID(), InInitParams.OperatorSettings};
			FOperatorAndInputs GraphOperatorAndInputs = OperatorPool->ClaimOperator(*OperatorPoolID, FOperatorContext::FromInitParams(InInitParams));
			if (GraphOperatorAndInputs.Operator.IsValid())
			{
				UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Using cached operator %ls for MetaSound %ls", *OperatorPoolID->ToString(), *InInitParams.AssetPath.ToString());
				bUseOperatorPool = true; // raise this flag to make sure we put the operator back in the pool (regardless of CVAR state)

				// Apply and default inputs to the operator.
				GeneratorBuilder::ResetGraphOperatorInputs(OperatorSettings, MoveTemp(InInitParams.DefaultParameters), GraphOperatorAndInputs.Inputs);

				GraphRenderCost = GraphOperatorAndInputs.RenderCost;
				if (!GraphRenderCost)
				{
					GraphRenderCost = FGraphRenderCost::MakeGraphRenderCost();
				}
				
				InInitParams.GraphRenderCost = GraphRenderCost;

				// Reset operator internal state before playing it.
				if (IOperator::FResetFunction Reset = GraphOperatorAndInputs.Operator->GetResetFunction())
				{
					EnvironmentPtr = MakeUnique<FMetasoundEnvironment>(InInitParams.Environment);
					IOperator::FResetParams ResetParams {OperatorSettings, *EnvironmentPtr, GraphRenderCost.Get()};
					Reset(GraphOperatorAndInputs.Operator.Get(), ResetParams);
				}
				
				// Create data needed for MetaSound Generator
				FMetasoundGeneratorData Data = GeneratorBuilder::BuildGeneratorData(OperatorSettings, InInitParams, MoveTemp(GraphOperatorAndInputs), nullptr);

				SetGraph(MakeUnique<FMetasoundGeneratorData>(MoveTemp(Data)), bInTriggerGraph);
				
				return true;
			}
		}

		return false;
	}

	void FMetasoundConstGraphGenerator::ReleaseOperatorToCache()
	{
		using namespace MetasoundGeneratorPrivate;
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(FMetasoundConstGraphGenerator::ReleaseOperatorToCache);

		if (OperatorPoolID.IsSet())
		{
			FMetasoundGeneratorModule& Module = FModuleManager::GetModuleChecked<FMetasoundGeneratorModule>("MetasoundGenerator");
			TSharedPtr<FOperatorPool> OperatorPool = Module.GetOperatorPool();
			TUniquePtr<IOperator> GraphOperator = ReleaseGraphOperator();
			FInputVertexInterfaceData InputData = ReleaseInputVertexData();

			if (OperatorPool.IsValid() && GraphOperator.IsValid())
			{
				// Release graph operator and input data to the cache
				UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Caching operator %ls", *OperatorPoolID->ToString());

				// give the operator a chance to reduce its memory footprint before being cached
				// in the future this should be a conanical phase of an operator's lifecycle (i.e. Reset, Execute, Hybernate)
				if (EnvironmentPtr.IsValid())
				{
					if (IOperator::FResetFunction Reset = GraphOperator->GetResetFunction())
					{
						IOperator::FResetParams ResetParams {OperatorSettings, *EnvironmentPtr.Get()};
						Reset(GraphOperator.Get(), ResetParams);
					}
				}

				// Clear out any internal references to graph data before adding to
				// operator pool. MetaSound data references are not thread safe which
				// requires that data references be removed before transferring 
				// data references to the operator pool. The operator pool is accessed
				// from multiple threads and so failure to remove internal data
				// references may result in a race condition on deallocating references.
				ClearGraph();

				OperatorPool->AddOperator(*OperatorPoolID, MoveTemp(GraphOperator), MoveTemp(InputData), MoveTemp(GraphRenderCost));
			}
		}
	}

	void FMetasoundConstGraphGenerator::BuildGraph(FGeneratorInitParams&& InInitParams)
	{
		const bool bBuildAsync = !InInitParams.bBuildSynchronous && Metasound::ConsoleVariables::bEnableAsyncMetaSoundGeneratorBuilder;

		BuilderTask = MakeUnique<FAsyncTask<FAsyncMetaSoundBuilder>>(this, MoveTemp(InInitParams), true /* bTriggerGenerator */);
		
		if (bBuildAsync)
		{
			// Build operator asynchronously
			BuilderTask->StartBackgroundTask(GBackgroundPriorityThreadPool);
		}
		else
		{
			// Build operator synchronously
			BuilderTask->StartSynchronousTask();
			BuilderTask = nullptr;
			UpdateGraphIfPending();
		}
	}

	FMetasoundDynamicGraphGenerator::FMetasoundDynamicGraphGenerator(const FOperatorSettings& InOperatorSettings)
	: FMetasoundGenerator(InOperatorSettings)
	{
	}

	static FThreadSafeCounter NumActiveDynamicGraphs;
	FMetasoundDynamicGraphGenerator::~FMetasoundDynamicGraphGenerator()
	{
		NumActiveDynamicGraphs.Decrement();
		CSV_CUSTOM_STAT(Audio_Metasound, NumActiveDynamicGraphs, NumActiveDynamicGraphs.GetValue(), ECsvCustomStatOp::Set);

		if (BuilderTask.IsValid())
		{
			BuilderTask->EnsureCompletion();
			BuilderTask = nullptr;
		}
	}
	
	void FMetasoundDynamicGraphGenerator::Init(FMetasoundDynamicGraphGeneratorInitParams&& InParams)
	{
		NumActiveDynamicGraphs.Increment();
		CSV_CUSTOM_STAT(Audio_Metasound, NumActiveDynamicGraphs, NumActiveDynamicGraphs.GetValue(), ECsvCustomStatOp::Set);

		InitBase(InParams);
		AudioOutputNames = InParams.AudioOutputNames;
		BuildGraph(MoveTemp(InParams));
	}

	void FMetasoundDynamicGraphGenerator::BuildGraph(FMetasoundDynamicGraphGeneratorInitParams&& InParams)
	{
		const bool bBuildAsync = !InParams.bBuildSynchronous && Metasound::ConsoleVariables::bEnableAsyncMetaSoundGeneratorBuilder;

		BuilderTask = MakeUnique<FAsyncTask<FAsyncDynamicMetaSoundBuilder>>(this, MoveTemp(InParams), true /* bTriggerGenerator */);

		if (bBuildAsync)
		{
			// Build operator asynchronously
			BuilderTask->StartBackgroundTask(GBackgroundPriorityThreadPool);
		}
		else
		{
			// Build operator synchronously
			BuilderTask->StartSynchronousTask();
			BuilderTask = nullptr;
			UpdateGraphIfPending();
		}
	}

	void FMetasoundDynamicGraphGenerator::OnInputAdded(const FVertexName& InVertexName, const FInputVertexInterfaceData& InGraphInputs)
	{
		// Update vertex data and set parameters. 
		VertexInterfaceData.GetInputs() = InGraphInputs;
		GeneratorBuilder::AddParameterSetterIfWritable(InVertexName, VertexInterfaceData.GetInputs(), ParameterSetters, ParameterPackSetters);

		VertexInterfaceChangesSinceLastBroadcast.Add({ InVertexName, EMetasoundFrontendClassType::Input, EVertexInterfaceChangeType::Added });
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FMetasoundDynamicGraphGenerator::OnInputRemoved(const FVertexName& InVertexName, const FInputVertexInterfaceData& InGraphInputs)
	{
		VertexInterfaceData.GetInputs() = InGraphInputs;
		ParameterSetters.Remove(InVertexName);
		ParameterPackSetters.Remove(InVertexName);
		VertexInterfaceChangesSinceLastBroadcast.Add({ InVertexName, EMetasoundFrontendClassType::Input, EVertexInterfaceChangeType::Removed });
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FMetasoundDynamicGraphGenerator::OnOutputAdded(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InGraphOutputs)
	{
		MetasoundGeneratorPrivate::LogErrorIfIsAudioVertex(InVertexName, TEXT("The MetaSound Generator cannot dynamically add audio outputs"));
		VertexInterfaceData.GetOutputs() = InGraphOutputs;

		if (Frontend::SourceOneShotInterface::Outputs::OnFinished == InVertexName)
		{
			OnFinishedTriggerRef = InGraphOutputs.GetDataReadReference<FTrigger>(InVertexName);
		}
		
		VertexInterfaceChangesSinceLastBroadcast.Add({ InVertexName, EMetasoundFrontendClassType::Output, EVertexInterfaceChangeType::Added });
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FMetasoundDynamicGraphGenerator::OnOutputUpdated(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InGraphOutputs)
	{
		VertexInterfaceData.GetOutputs() = InGraphOutputs;

		// If it's an output audio buffer, update the internal reference	
		int32 AudioOutputIndex = AudioOutputNames.IndexOfByKey(InVertexName);
		if (INDEX_NONE != AudioOutputIndex)
		{
			// For CAT sources, the audio output vertex carries FChannelAgnosticType instead of
			// FAudioBuffer. GraphOutputAudio is empty in the CAT path, so update GraphCatOutput
			// instead. MixerOutput buffer and Transcoder are format-dependent (not reference-
			// dependent) and remain valid across reference updates.
			if (GraphCatOutput.IsSet())
			{
				GraphCatOutput->GraphOutput = InGraphOutputs.GetDataReadReference<FChannelAgnosticType>(InVertexName);
			}
			else
			{
				GraphOutputAudio[AudioOutputIndex] = InGraphOutputs.GetDataReadReference<FAudioBuffer>(InVertexName);
			}
		}
		// If it's the on-finished trigger, update the internal reference
		else if (Frontend::SourceOneShotInterface::Outputs::OnFinished == InVertexName)
		{
			OnFinishedTriggerRef = InGraphOutputs.GetDataReadReference<FTrigger>(InVertexName);
		}
		
		VertexInterfaceChangesSinceLastBroadcast.Add({ InVertexName, EMetasoundFrontendClassType::Output, EVertexInterfaceChangeType::Updated });
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FMetasoundDynamicGraphGenerator::OnOutputRemoved(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InGraphOutputs)
	{
		MetasoundGeneratorPrivate::LogErrorIfIsAudioVertex(InVertexName, TEXT("The MetaSound Generator cannot dynamically remove audio outputs"));

		VertexInterfaceData.GetOutputs() = InGraphOutputs;
		VertexInterfaceChangesSinceLastBroadcast.Add({ InVertexName, EMetasoundFrontendClassType::Output, EVertexInterfaceChangeType::Removed });
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bVertexInterfaceHasChanged.store(true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TUniquePtr<IOperator> FMetasoundDynamicGraphGenerator::ReleaseGraphOperator()
	{
		FMetasoundGenerator::ReleaseGraphOperator();

		// Never allow the dynamic operator to leave this object because it contains 
		// callbacks pointing to this object instance. If the dynamic operator left
		// this object, it could attempt to execute member functions on this object
		// after this object has been destroyed. (X╭╮X) 
		return nullptr;
	}

} 
