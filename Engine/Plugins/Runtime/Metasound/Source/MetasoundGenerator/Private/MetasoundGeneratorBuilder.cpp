// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGeneratorBuilder.h"

#include "AudioParameter.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundFrontendDataTypeRegistry.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOutputNode.h"
#include "MetasoundParameterTransmitter.h"
#include "MetasoundRouter.h"
#include "MetasoundThreadLocalDebug.h"
#include "MetasoundTrace.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertexData.h"
#include "Analysis/MetasoundFrontendAnalyzerFactory.h"
#include "Analysis/MetasoundFrontendAnalyzerRegistry.h"
#include "Analysis/MetasoundFrontendVertexAnalyzer.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

namespace Metasound
{
	LLM_DEFINE_TAG(Audio_MetaSound_Generator_Build);
	
	FAsyncMetaSoundBuilderBase::FAsyncMetaSoundBuilderBase(FMetasoundGenerator* InGenerator, bool bInTriggerGenerator)
	: Generator(InGenerator)
	, bTriggerGenerator(bInTriggerGenerator)
	{
	}

	void FAsyncMetaSoundBuilderBase::SetGraphOnGenerator(FOperatorAndInputs&& InOpAndInputs, FBuildResults&& InBuildResults)
	{
		using namespace Audio;
		using namespace Frontend;
		using namespace MetasoundGeneratorPrivate;

		const FGeneratorInitParams& InitParams = GetGeneratorParams();

		if (InOpAndInputs.Operator.IsValid())
		{
			// Create graph analyzer
			TUniquePtr<FGraphAnalyzer> GraphAnalyzer;
			if (InitParams.BuilderSettings.bPopulateInternalDataReferences)
			{
				GraphAnalyzer = GeneratorBuilder::BuildGraphAnalyzer(MoveTemp(InBuildResults.InternalDataReferences), InitParams.Environment, Generator->OperatorSettings);
			}
	
			// Collect data for generator
			FMetasoundGeneratorData GeneratorData = GeneratorBuilder::BuildGeneratorData(Generator->OperatorSettings, InitParams, MoveTemp(InOpAndInputs), MoveTemp(GraphAnalyzer));

			Generator->SetPendingGraph(MoveTemp(GeneratorData), bTriggerGenerator);
		}
		else 
		{
			UE_LOGF(LogMetaSound, Error, "Failed to build Metasound operator from graph in MetasoundSource [%ls]", *InitParams.AssetPath.ToString());
			// Set null generator data to inform that generator failed to build. 
			// Otherwise, generator will continually wait for a new generator.
			Generator->SetPendingGraphBuildFailed();
		}
	}

	FAsyncMetaSoundBuilder::FAsyncMetaSoundBuilder(FMetasoundConstGraphGenerator* InGenerator, FGeneratorInitParams&& InInitParams, bool bInTriggerGenerator)
		: FAsyncMetaSoundBuilderBase(InGenerator, bInTriggerGenerator)
		, InitParams(MoveTemp(InInitParams))
	{
	}

	void FAsyncMetaSoundBuilder::DoWork()
	{
		using namespace Audio;
		using namespace Frontend;
		using namespace MetasoundGeneratorPrivate;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("AsyncMetaSoundBuilder::DoWork %s"), *InitParams.AssetPath.GetAssetName().ToString()));

		// Create an instance of the new graph
		FBuildResults BuildResults;
		FOperatorAndInputs GraphOperatorAndInputs = GeneratorBuilder::BuildGraphOperator(Generator->OperatorSettings, InitParams, BuildResults);
		GeneratorBuilder::LogBuildErrors(InitParams.AssetPath, BuildResults);

		SetGraphOnGenerator(MoveTemp(GraphOperatorAndInputs), MoveTemp(BuildResults));

		FGeneratorInitParams::Reset(InitParams);
	}

	const FGeneratorInitParams& FAsyncMetaSoundBuilder::GetGeneratorParams() const
	{
		return InitParams;
	}

	FAsyncDynamicMetaSoundBuilder::FAsyncDynamicMetaSoundBuilder(FMetasoundDynamicGraphGenerator* InGenerator, FMetasoundDynamicGraphGeneratorInitParams&& InInitParams, bool bInTriggerGenerator)
	: FAsyncMetaSoundBuilderBase(InGenerator, bInTriggerGenerator)
	, DynamicGenerator(InGenerator)
	, InitParams(MoveTemp(InInitParams))
	{
	}

	void FAsyncDynamicMetaSoundBuilder::DoWork()
	{
		using namespace Audio;
		using namespace Frontend;
		using namespace MetasoundGeneratorPrivate;
		using namespace DynamicGraph;

		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("AsyncMetaSoundBuilder::DoWork %s"), *InitParams.AssetPath.GetAssetName().ToString()));

		FDynamicOperatorUpdateCallbacks UpdateCallbacks;

		// Note: The operator which calls these functions is owned by the FMetasoundDynamicGraphGenerator
		// Capturing the "DynamicGenerator" pointer here is safe as long as the operator's lifetime
		// is less than or equal to the lifetime of the FMetasoundDynamicGraphGenerator. These
		// callbacks are triggered during calls to the base class ISoundGenerator::OnGeneratorAudio(...)
		// and should not be triggered at any other time. 
		//
		// The dynamic operator should never be cached.
		UpdateCallbacks.OnInputAdded = [DynamicGenerator = DynamicGenerator](const FVertexName& InVertexName, const FInputVertexInterfaceData& InData) { DynamicGenerator->OnInputAdded(InVertexName, InData); };
		UpdateCallbacks.OnInputRemoved = [DynamicGenerator = DynamicGenerator](const FVertexName& InVertexName, const FInputVertexInterfaceData& InData) { DynamicGenerator->OnInputRemoved(InVertexName, InData); };
		UpdateCallbacks.OnOutputAdded = [DynamicGenerator = DynamicGenerator](const FVertexName& InVertexName, const FOutputVertexInterfaceData& InData) { DynamicGenerator->OnOutputAdded(InVertexName, InData); };
		UpdateCallbacks.OnOutputUpdated = [DynamicGenerator = DynamicGenerator](const FVertexName& InVertexName, const FOutputVertexInterfaceData& InData) { DynamicGenerator->OnOutputUpdated(InVertexName, InData); };
		UpdateCallbacks.OnOutputRemoved = [DynamicGenerator = DynamicGenerator](const FVertexName& InVertexName, const FOutputVertexInterfaceData& InData) { DynamicGenerator->OnOutputRemoved(InVertexName, InData); };

		// Create an instance of the new graph
		FBuildResults BuildResults;
		FOperatorAndInputs GraphOperatorAndInputs = GeneratorBuilder::BuildDynamicGraphOperator(Generator->OperatorSettings, InitParams, UpdateCallbacks, BuildResults);
		GeneratorBuilder::LogBuildErrors(InitParams.AssetPath, BuildResults);

		SetGraphOnGenerator(MoveTemp(GraphOperatorAndInputs), MoveTemp(BuildResults));

		FMetasoundDynamicGraphGeneratorInitParams::Reset(InitParams);
	}

	const FGeneratorInitParams& FAsyncDynamicMetaSoundBuilder::GetGeneratorParams() const
	{
		return InitParams;
	}

	namespace GeneratorBuilder
	{
		TOptional<FChannelAgnosticTypeReadRef> FindOutputChannelAgnostic(const TArray<FVertexName>& InAudioVertexNames, const FVertexInterfaceData& InVertexData, const FOperatorSettings& InOperatorSettings, const FTopLevelAssetPath& InAssetPath)
		{
			const FOutputVertexInterfaceData& OutputVertexData = InVertexData.GetOutputs();

			for (const FVertexName& AudioOutputName : InAudioVertexNames)
			{
				if (OutputVertexData.IsVertexBound(AudioOutputName))
				{
					if (const FAnyDataReference* Ref = OutputVertexData.FindDataReference(AudioOutputName))
					{
						if (IsCastable(Ref->GetDataTypeName(), GetMetasoundDataTypeName<FChannelAgnosticType>()))
						{
							return Ref->GetDataReadReference<FChannelAgnosticType>();
						}
					}
				}
			}
			return {};
		}
		
		TArray<FAudioBufferReadRef> FindOutputAudioBuffers(const TArray<FVertexName>& InAudioVertexNames, const FVertexInterfaceData& InVertexData, const FOperatorSettings& InOperatorSettings, const FTopLevelAssetPath& InAssetPath)
		{
			TArray<FAudioBufferReadRef> OutputBuffers;

			const FOutputVertexInterfaceData& OutputVertexData = InVertexData.GetOutputs();

			// Get output audio buffers.
			for (const FVertexName& AudioOutputName : InAudioVertexNames)
			{
				if (const FAnyDataReference* Ref = OutputVertexData.FindDataReference(AudioOutputName))
				{
					if (Ref->GetDataTypeName() == Metasound::GetMetasoundDataTypeName<FAudioBuffer>())
					{
						OutputBuffers.Add(OutputVertexData.GetOrConstructDataReadReference<FAudioBuffer>(AudioOutputName, InOperatorSettings));
					}
				}
			}

			return OutputBuffers;
		}

		void LogBuildErrors(const FTopLevelAssetPath& InAssetPath, const FBuildResults& InBuildResults)
		{
			// Log build errors
			for (const IOperatorBuilder::FBuildErrorPtr& Error : InBuildResults.Errors)
			{
				if (Error.IsValid())
				{
					UE_LOGF(LogMetaSound, Warning, "MetasoundSource [%ls] build error [%ls] \"%ls\"", *InAssetPath.ToString(), *(Error->GetErrorType().ToString()), *(Error->GetErrorDescription().ToString()));
				}
			}
		}

		TUniquePtr<Frontend::FGraphAnalyzer> BuildGraphAnalyzer(TMap<FGuid, FDataReferenceCollection>&& InInternalDataReferences, const FMetasoundEnvironment& InEnvironment, const FOperatorSettings& InOperatorSettings)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::BuildGraphAnalyzer);
			using namespace Frontend;

			const uint64 InstanceID = InEnvironment.GetValue<uint64>(CoreInterface::Environment::InstanceID);
			return MakeUnique<FGraphAnalyzer>(InOperatorSettings, InstanceID, MoveTemp(InInternalDataReferences));
		}

		FInputVertexInterfaceData BuildGraphOperatorInputs(const FOperatorSettings& InOperatorSettings, FGeneratorInitParams& InInitParams)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::BuildGraphOperatorInputs);

			using namespace Frontend;

			// Choose which type of data reference access to create depending upon the access of the vertex.
			auto VertexAccessTypeToDataReferenceAccessType = [](EVertexAccessType InVertexAccessType) -> EDataReferenceAccessType
			{
				switch(InVertexAccessType)
				{
					case EVertexAccessType::Value:
						return EDataReferenceAccessType::Value;

					case EVertexAccessType::Reference:
					default:
						return EDataReferenceAccessType::Write;
				}
			};

			if (!InInitParams.Graph)
			{
				UE_LOGF(LogMetaSound, Error, "Unable to build graph operator inputs for null graph in MetaSoundSource [%ls]", *InInitParams.AssetPath.ToString());
				return FInputVertexInterfaceData();
			}

			const FInputVertexInterface& InputInterface = InInitParams.Graph->GetVertexInterface().GetInputInterface();
			FInputVertexInterfaceData InputData(InputInterface);

			// Set input data based on the input parameters and the input interface
			IDataTypeRegistry& DataRegistry = IDataTypeRegistry::Get();
			for (FAudioParameter& Parameter : InInitParams.DefaultParameters)
			{
				const FName ParamName = Parameter.ParamName;
				if (const FInputDataVertex* InputVertex = InputInterface.Find(ParamName))
				{
					FLiteral Literal = Frontend::ConvertParameterToLiteral(MoveTemp(Parameter));

					TOptional<FAnyDataReference> DataReference = DataRegistry.CreateDataReference(InputVertex->DataTypeName, VertexAccessTypeToDataReferenceAccessType(InputVertex->AccessType), Literal, InOperatorSettings);

					if (DataReference)
					{
						InputData.BindVertex(ParamName, *DataReference);
					}
					else if(MetaSoundParameterEnableWarningOnIgnoredParameterCVar)
					{
						UE_LOGF(LogMetaSound, Warning, "Failed to create initial input data reference from parameter %ls of type %ls on graph in MetaSoundSource [%ls]", *ParamName.ToString(), *InputVertex->DataTypeName.ToString(), *InInitParams.AssetPath.ToString());
					}
				}
				else if(MetaSoundParameterEnableWarningOnIgnoredParameterCVar)
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to set initial input parameter %ls on graph in MetaSoundSource [%ls]", *ParamName.ToString(), *InInitParams.AssetPath.ToString());
				}
			}

			// Set any remaining inputs to their default values.
			for (const VertexDataPrivate::FInputBinding& Binding : InputData)
			{
				// Only create data reference if something does not already exist. 
				if (!Binding.IsBound())
				{
					const FInputDataVertex& InputVertex = Binding.GetVertex();
					EDataReferenceAccessType AccessType = VertexAccessTypeToDataReferenceAccessType(InputVertex.AccessType);
					TOptional<FAnyDataReference> DataReference = DataRegistry.CreateDataReference(InputVertex.DataTypeName, VertexAccessTypeToDataReferenceAccessType(InputVertex.AccessType), InputVertex.GetDefaultLiteral(), InOperatorSettings);

					if (DataReference)
					{
						InputData.BindVertex(InputVertex.VertexName, *DataReference);
					}
				}
			}

			// Reset as elements in array have been moved.
			InInitParams.DefaultParameters.Reset();

			return MoveTemp(InputData);
		}

		FOperatorAndInputs BuildGraphOperator(const FOperatorSettings& InOperatorSettings, FGeneratorInitParams& InInitParams, FBuildResults& OutBuildResults)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::BuildGraphOperator);

			// Create inputs to graph operator
			FOperatorAndInputs OpAndInputs;
			OpAndInputs.Inputs = BuildGraphOperatorInputs(InOperatorSettings, InInitParams);

			if (!InInitParams.Graph)
			{
				UE_LOGF(LogMetaSound, Error, "Unable to build graph operator for null graph in MetaSoundSource [%ls]", *InInitParams.AssetPath.ToString());
				return OpAndInputs;
			}
			// Create an instance of the new graph operator
			FBuildGraphOperatorParams BuildParams { *InInitParams.Graph, InOperatorSettings, OpAndInputs.Inputs, InInitParams.Environment, InInitParams.GraphRenderCost.Get()};
			FOperatorBuilder Builder(InInitParams.BuilderSettings);
			
			LLM_SCOPE_BYTAG(Audio_MetaSound_Generator_Build);

			// asset-level trace asset path
			METASOUND_DEBUG_DECLARE_SCOPE;
			METASOUND_DEBUG_SET_ACTIVE_ASSET_SCOPE(InInitParams.AssetPath.GetAssetName(), InInitParams.ClassName, InInitParams.AssetPath.GetPackageName());
			
			OpAndInputs.Operator = Builder.BuildGraphOperator(BuildParams, OutBuildResults);
			OpAndInputs.RenderCost = InInitParams.GraphRenderCost;
			return OpAndInputs;
		}

		FOperatorAndInputs BuildDynamicGraphOperator(const FOperatorSettings& InOperatorSettings, FMetasoundDynamicGraphGeneratorInitParams& InInitParams, const DynamicGraph::FDynamicOperatorUpdateCallbacks& InCallbacks, FBuildResults& OutBuildResults)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::BuildDynamicGraphOperator);

			FOperatorAndInputs OpAndInputs;
			OpAndInputs.Inputs = BuildGraphOperatorInputs(InOperatorSettings, InInitParams);

			// Create an instance of the new graph
			FBuildDynamicGraphOperatorParams BuildParams 
			{ 
				{ 
					*InInitParams.Graph,
					InOperatorSettings, 
					OpAndInputs.Inputs, 
					InInitParams.Environment,
					InInitParams.GraphRenderCost.Get()
				},
				MoveTemp(InInitParams.TransformQueue),
				InCallbacks
			};
			FOperatorBuilder Builder(InInitParams.BuilderSettings);

			OpAndInputs.Operator = Builder.BuildDynamicGraphOperator(BuildParams, OutBuildResults);

			return OpAndInputs;
		}

		MetasoundGeneratorPrivate::FMetasoundGeneratorData BuildGeneratorData(const FOperatorSettings& InOperatorSettings, const FGeneratorInitParams& InInitParams, FOperatorAndInputs&& InGraphOperatorAndInputs, TUniquePtr<Frontend::FGraphAnalyzer> InAnalyzer)
		{
			using namespace Audio;
			using namespace Frontend;
			using namespace MetasoundGeneratorPrivate;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::BuildGeneratorData);

			checkf(InGraphOperatorAndInputs.Operator.IsValid(), TEXT("Graph operator must be a valid object"))

			// Gather relevant input and output references
			FVertexInterfaceData VertexData(InInitParams.Graph->GetVertexInterface());
			InGraphOperatorAndInputs.Operator->BindInputs(VertexData.GetInputs());
			InGraphOperatorAndInputs.Operator->BindOutputs(VertexData.GetOutputs());

			// Replace input data with writable inputs
			VertexData.GetInputs() = InGraphOperatorAndInputs.Inputs;

			// Get inputs
			FTriggerWriteRef PlayTrigger = VertexData.GetInputs().GetOrCreateDefaultDataWriteReference<FTrigger>(SourceInterface::Inputs::OnPlay, InOperatorSettings);

			// Get outputs
			TArray<FAudioBufferReadRef> OutputAudioBuffers = FindOutputAudioBuffers(InInitParams.AudioOutputNames, VertexData, InOperatorSettings, InInitParams.AssetPath);
			TOptional<FChannelAgnosticTypeReadRef> OutputCat = FindOutputChannelAgnostic(InInitParams.AudioOutputNames, VertexData, InOperatorSettings, InInitParams.AssetPath);
			
			FTriggerReadRef FinishTrigger = TDataReadReferenceFactory<FTrigger>::CreateExplicitArgs(InOperatorSettings, false);

			if (const FAnyDataReference* FinishTriggerRef = VertexData.GetOutputs().FindDataReference(SourceOneShotInterface::Outputs::OnFinished))
			{
 
				FinishTrigger = FinishTriggerRef->GetDataReadReference<FTrigger>();
			}

			
			// Initialize parameter setters for routing inputs to a MetaSound Generator.
			FParameterSetterSortedMap ParameterSetters;
			TMap<FName, FParameterPackSetter> ParameterPackSetters;
			InitializeParameterSetters(VertexData.GetInputs(), ParameterSetters, ParameterPackSetters);

			// Set data needed for graph
			return FMetasoundGeneratorData 
			{
				InOperatorSettings,
				MoveTemp(InGraphOperatorAndInputs.Operator),
				MoveTemp(VertexData),
				MoveTemp(ParameterSetters),
				MoveTemp(ParameterPackSetters),
				MoveTemp(InAnalyzer),
				MoveTemp(OutputAudioBuffers),
				MoveTemp(PlayTrigger),
				MoveTemp(FinishTrigger),
				MoveTemp(OutputCat)
			};
		}

		void ResetGraphOperatorInputs(const FOperatorSettings& InOperatorSettings, TArray<FAudioParameter> InParameterOverrides, FInputVertexInterfaceData& InOutInterface)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(GeneratorBuilder::ResetGraphOperatorInputs);

			Frontend::IDataTypeRegistry& DataTypeRegistry = Frontend::IDataTypeRegistry::Get();

			for (VertexDataPrivate::FInputBinding& Binding : InOutInterface)
			{
				if (const FAnyDataReference* Ref = Binding.GetDataReference())
				{
					if (EDataReferenceAccessType::Write == Ref->GetAccessType())
					{
						Frontend::FLiteralAssignmentFunction LiteralSetter = DataTypeRegistry.GetLiteralAssignmentFunction(Ref->GetDataTypeName());
						if (LiteralSetter)
						{
							auto IsParameterNameEqualToVertexName = [&Name=Binding.GetVertex().VertexName](const FAudioParameter& InParam) -> bool
							{
								return Name == InParam.ParamName;
							};

							if (FAudioParameter* Parameter = InParameterOverrides.FindByPredicate(IsParameterNameEqualToVertexName))
							{
								FLiteral Literal = Frontend::ConvertParameterToLiteral(MoveTemp(*Parameter)); 
								LiteralSetter(InOperatorSettings, Literal, *Ref);
							}
							else
							{
								LiteralSetter(InOperatorSettings, Binding.GetVertex().GetDefaultLiteral(), *Ref);
							}
						}
					}
				}
			}
		}

		void InitializeParameterSetters(FInputVertexInterfaceData& InputData, MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters)
		{
			using namespace Frontend;

			// Create the parameter setter map so parameters and parameter packs 
			// can be assigned appropriately
			const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

			for (const VertexDataPrivate::FInputBinding& Binding : InputData)
			{
				// Only assign inputs that are writable. 
				if (EDataReferenceAccessType::Write == Binding.GetAccessType())
				{
					if (const FAnyDataReference* DataRef = Binding.GetDataReference())
					{
						const FInputDataVertex& InputVertex = Binding.GetVertex();
						AddParameterSetter(DataTypeRegistry, InputVertex, *DataRef, OutParamSetters, OutParamPackSetters);
					}
				}
			}
		}

		void AddParameterSetterIfWritable(const FVertexName& InVertexName, FInputVertexInterfaceData& InputData, MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters)
		{
			using namespace Frontend;

			// Create the parameter setter map so parameters and parameter packs 
			// can be assigned appropriately
			const IDataTypeRegistry& DataTypeRegistry = IDataTypeRegistry::Get();

			if (const FAnyDataReference* DataRef = InputData.FindDataReference(InVertexName))
			{
				// Only assign inputs that are writable. 
				if (EDataReferenceAccessType::Write == DataRef->GetAccessType())
				{
					const FInputDataVertex& InputVertex = InputData.GetVertex(InVertexName);
					AddParameterSetter(DataTypeRegistry, InputVertex, *DataRef, OutParamSetters, OutParamPackSetters);
				}
			}
		}

		void AddParameterSetter(const Frontend::IDataTypeRegistry& InDataTypeRegistry, const FInputDataVertex& InVertex, const FAnyDataReference& InDataRef,  MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters)
		{
			using namespace MetasoundGeneratorPrivate;
			
			checkf(EDataReferenceAccessType::Write == InDataRef.GetAccessType(), TEXT("Only writable inputs can have parameter setters."));

			const Frontend::IParameterAssignmentFunction& PackSetter = InDataTypeRegistry.GetRawAssignmentFunction(InVertex.DataTypeName);
			if (PackSetter)
			{
				FParameterPackSetter ParameterPackSetter(InVertex.DataTypeName, InDataRef.GetRaw(), PackSetter);
				OutParamPackSetters.Add(InVertex.VertexName, ParameterPackSetter);
			}

			Frontend::FLiteralAssignmentFunction LiteralSetter = InDataTypeRegistry.GetLiteralAssignmentFunction(InVertex.DataTypeName);
			if (LiteralSetter)
			{
				OutParamSetters.Add(InVertex.VertexName, FParameterSetter{LiteralSetter, InDataRef});
			}
		}
	}
}
