// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundGenerator.h"

#include "Async/AsyncWork.h"
#include "Containers/Map.h"
#include "MetasoundEnvironment.h"
#include "MetasoundGenerator.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorCache.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "Misc/Guid.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	class FMetasoundGenerator;
	class FMetasoundFixedGraphGenerator;
	class FMetasoundDynamicGraphGenerator;

	namespace Frontend
	{
		class IDataTypeRegistry;
	}

	namespace DynamicGraph
	{
		struct FDynamicOperatorUpdateCallbacks;
	}

	// Base class for building a metasound generator
	class FAsyncMetaSoundBuilderBase : public FNonAbandonableTask
	{
	public:
		FAsyncMetaSoundBuilderBase(FMetasoundGenerator* InGenerator, bool bInTriggerGenerator);

		virtual ~FAsyncMetaSoundBuilderBase() = default;

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncMetaSoundBuilder, STATGROUP_ThreadPoolAsyncTasks);
		}
	protected:

		FMetasoundGenerator* Generator;
		bool bTriggerGenerator;

		void SetGraphOnGenerator(FOperatorAndInputs&& InOpAndInputs, FBuildResults&& InBuildResults);

	private:
		virtual const FGeneratorInitParams& GetGeneratorParams() const = 0;

	};

	// Task class for asynchronously building a FMetasoundGenerator
	class FAsyncMetaSoundBuilder : public FAsyncMetaSoundBuilderBase
	{
	public:
		FAsyncMetaSoundBuilder(FMetasoundConstGraphGenerator* InGenerator, FGeneratorInitParams&& InInitParams, bool bInTriggerGenerator);

		virtual ~FAsyncMetaSoundBuilder() = default;

		void DoWork();

	private:
		virtual const FGeneratorInitParams& GetGeneratorParams() const override;
		FGeneratorInitParams InitParams;
	};

	// Task class for asynchronously building a FMetasoundDynamicGraphGenerator
	class FAsyncDynamicMetaSoundBuilder : public FAsyncMetaSoundBuilderBase
	{
	public:
		FAsyncDynamicMetaSoundBuilder(FMetasoundDynamicGraphGenerator* InGenerator, FMetasoundDynamicGraphGeneratorInitParams&& InInitParams, bool bInTriggerGenerator);

		virtual ~FAsyncDynamicMetaSoundBuilder() = default;

		void DoWork();

	private:
		virtual const FGeneratorInitParams& GetGeneratorParams() const override;
		FMetasoundDynamicGraphGenerator* DynamicGenerator;
		FMetasoundDynamicGraphGeneratorInitParams InitParams;
	};

	namespace GeneratorBuilder
	{
		// Finds an array of audio buffer read references from the VertexData
		TArray<FAudioBufferReadRef> FindOutputAudioBuffers(const TArray<FVertexName>& InAudioVertexNames, const FVertexInterfaceData& InVertexData, const FOperatorSettings& InOperatorSettings, const FTopLevelAssetPath& InAssetPath);

		// Log any errors in the build results
		void LogBuildErrors(const FTopLevelAssetPath& InAssetPath, const FBuildResults& InBuildResults);

		// Build an analyzer for a graph
		TUniquePtr<Frontend::FGraphAnalyzer> BuildGraphAnalyzer(TMap<FGuid, FDataReferenceCollection>&& InInternalDataReferences, const FMetasoundEnvironment& InEnvironment, const FOperatorSettings& InOperatorSettings);

		// Instantiate the input data references for a generator.
		FInputVertexInterfaceData BuildGraphOperatorInputs(const FOperatorSettings& InOperatorSettings, FGeneratorInitParams& InInitParams);
		
		// Build a standard graph operatpr
		FOperatorAndInputs BuildGraphOperator(const FOperatorSettings& InOperatorSettings, FGeneratorInitParams& InInitParams, FBuildResults& OutBuildResults);
		
		// Build a dynamic graph operator
		FOperatorAndInputs BuildDynamicGraphOperator(const FOperatorSettings& InOperatorSettings, FMetasoundDynamicGraphGeneratorInitParams& InInitParams, const DynamicGraph::FDynamicOperatorUpdateCallbacks& InCallbacks, FBuildResults& OutBuildResults);

		// Create all the data required to run a FMetasoundGenerator
		MetasoundGeneratorPrivate::FMetasoundGeneratorData BuildGeneratorData(const FOperatorSettings& InOperatorSettings, const FGeneratorInitParams& InInitParams, FOperatorAndInputs&& InGraphOperatorAndInputs, TUniquePtr<Frontend::FGraphAnalyzer> InAnalyzer);

		// Resets FInputVertexInterfaceData to it's initial default state using the default literals stored ther-in. If a parameter exists in InParmaeterOverrides, that value is used instead. 
		void ResetGraphOperatorInputs(const FOperatorSettings& InOperatorSettings, TArray<FAudioParameter> InParameterOverrides, FInputVertexInterfaceData& InOutInterface);

		// Initialize the maps of parameter setters for applying async input data to metasound inputs.
		void InitializeParameterSetters(FInputVertexInterfaceData& InputData, MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters);

		// Add a parameter setter to the parameter map if the data bound to the input vertex is writable.
		void AddParameterSetterIfWritable(const FVertexName& InVertexName, FInputVertexInterfaceData& InputData, MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters);

		// Add a parameter setter to the parameter map for the given vertex.
		void AddParameterSetter(const Frontend::IDataTypeRegistry& InDataTypeRegistry, const FInputDataVertex& InVertex, const FAnyDataReference& InDataRef,  MetasoundGeneratorPrivate::FParameterSetterSortedMap& OutParamSetters, TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter>& OutParamPackSetters);
	}
}
