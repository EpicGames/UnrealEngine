// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGeneratorModule.h"
#include "MetasoundGeneratorModuleImpl.h"

#include "AudioDeviceManager.h"
#include "Containers/Array.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "MetasoundOperatorCache.h"
#include "MetasoundOperatorCacheStatTracker.h"
#include "Misc/CString.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"

DEFINE_LOG_CATEGORY(LogMetasoundGenerator);

CSV_DEFINE_CATEGORY(MetaSound_ActiveOperators, true);

// Note: disabled by default as it bloats the csvs quite a bit.
static TAutoConsoleVariable<bool> CVarRecordActiveOperatorsToCsv(
	TEXT("au.MetaSound.RecordActiveMetasoundsToCsv"),
	false,
	TEXT("Record the name of each active Metasound when csv profiling is recording.")
);

namespace Metasound::Private
{
	static void ApplyOperatorPoolMaxNumOperators(int32 MaxNumOperators)
	{
		if (MaxNumOperators < 0)
		{
			return;
		}

		FMetasoundGeneratorModule* Module = FModuleManager::GetModulePtr<FMetasoundGeneratorModule>("MetasoundGenerator");
		if (!Module)
		{
			// Module not loaded yet. StartupModule re-applies the current cvar value once the pool exists.
			return;
		}

		TSharedPtr<FOperatorPool> OperatorPool = Module->GetOperatorPool();
		if (OperatorPool.IsValid())
		{
			OperatorPool->SetMaxNumOperators(static_cast<uint32>(MaxNumOperators));
			UE_LOGF(LogMetasoundGenerator, Display, "Operator cache size set to %d operators.", MaxNumOperators);
		}
	}
}

static TAutoConsoleVariable<int32> CVarMetaSoundExperimentalOperatorPoolSetMaxNumOperators(
	TEXT("au.MetaSound.Experimental.OperatorPool.SetMaxNumOperators"),
	-1,
	TEXT("Set the maximum number of operators in the MetaSound operator cache. -1 leaves the default unchanged; negative values are otherwise ignored."),
	FConsoleVariableDelegate::CreateStatic([](IConsoleVariable* Var)
	{
		Metasound::Private::ApplyOperatorPoolMaxNumOperators(Var->GetInt());
	}),
	ECVF_Default
);

namespace Metasound
{
	void FMetasoundGeneratorModule::StartupModule()
	{
		OperatorPool = MakeShared<FOperatorPool>();
		OperatorInstanceCounterManager = MakeShared<FConcurrentInstanceCounterManager>(TEXT("MetaSound/Active_Generators"));

		// The cvar may have been set from [ConsoleVariables] ini before this module loaded;
		// at that point the OnChanged delegate ran with no pool to apply to. Re-apply now.
		Private::ApplyOperatorPoolMaxNumOperators(CVarMetaSoundExperimentalOperatorPoolSetMaxNumOperators.GetValueOnAnyThread());

		PreAudioDeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnPreAudioDeviceDestroyed.AddLambda(
			[WeakPool = OperatorPool.ToWeakPtr()](Audio::FDeviceId /* DeviceId */)
			{
				if (TSharedPtr<FOperatorPool> Pool = WeakPool.Pin())
				{
					Pool->StopAsyncTasks();
				}
			});

#if CSV_PROFILER && !UE_BUILD_SHIPPING && METASOUND_OPERATORCACHEPROFILER_ENABLED
		CsvEndFrameDelegateHandle = FCsvProfiler::Get()->OnCSVProfileEndFrame().AddLambda([WeakCounterManager = OperatorInstanceCounterManager.ToWeakPtr()]
		{
			if (!CVarRecordActiveOperatorsToCsv->GetBool())
			{
				return;
			}

			if (TSharedPtr<FConcurrentInstanceCounterManager> CounterManager = WeakCounterManager.Pin())
			{
				CounterManager->VisitStats([](const FTopLevelAssetPath& InAssetPath, int64 Value)
				{
					Metasound::Engine::RecordOperatorStat(InAssetPath, CSV_CATEGORY_INDEX(MetaSound_ActiveOperators), (int32)Value, ECsvCustomStatOp::Set);
				});
			}
		});
#endif // #if CSV_PROFILER && !UE_BUILD_SHIPPING && METASOUND_OPERATORCACHEPROFILER_ENABLED
	}

	void FMetasoundGeneratorModule::ShutdownModule()
	{
		FAudioDeviceManagerDelegates::OnPreAudioDeviceDestroyed.Remove(PreAudioDeviceDestroyedHandle);
		PreAudioDeviceDestroyedHandle.Reset();

#if CSV_PROFILER && !UE_BUILD_SHIPPING
		FCsvProfiler::Get()->OnCSVProfileEndFrame().Remove(CsvEndFrameDelegateHandle);
		CsvEndFrameDelegateHandle.Reset();
#endif // #if CSV_PROFILER && !UE_BUILD_SHIPPING

		if (OperatorPool.IsValid())
		{
			TSharedPtr<FOperatorPool> PoolShuttingDown = OperatorPool;
			OperatorPool.Reset();

			// Clear the pool reference and cancel independent of resetting
			// the shared pointer to ensure if any references are held elsewhere,
			// they are properly invalidated.
			PoolShuttingDown->StopAsyncTasks();
		}

		OperatorInstanceCounterManager.Reset();
	}

	TSharedPtr<FOperatorPool> FMetasoundGeneratorModule::GetOperatorPool()
	{
		return OperatorPool;
	}

	TSharedPtr<FConcurrentInstanceCounterManager> FMetasoundGeneratorModule::GetOperatorInstanceCounterManager()
	{
		return OperatorInstanceCounterManager;
	}
}

IMPLEMENT_MODULE(Metasound::FMetasoundGeneratorModule, MetasoundGenerator);

