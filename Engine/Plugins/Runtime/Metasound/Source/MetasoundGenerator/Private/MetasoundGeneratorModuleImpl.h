// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundGeneratorModule.h"

#include "MetasoundOperatorCache.h"
#include "MetasoundInstanceCounter.h"

namespace Metasound
{
	class FMetasoundGeneratorModule : public IMetasoundGeneratorModule
	{
	public:

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		virtual TSharedPtr<FOperatorPool> GetOperatorPool() override;
		virtual TSharedPtr<FConcurrentInstanceCounterManager> GetOperatorInstanceCounterManager() override;

	private:

		TSharedPtr<FOperatorPool> OperatorPool;
		TSharedPtr<FConcurrentInstanceCounterManager> OperatorInstanceCounterManager;

		FDelegateHandle CsvEndFrameDelegateHandle;
		FDelegateHandle PreAudioDeviceDestroyedHandle;
	};
}
