// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundInstanceCounter.h"

namespace Metasound
{
	FConcurrentInstanceCounterManager::FConcurrentInstanceCounterManager(const FString InCategoryName)
	: CategoryName(InCategoryName)
	{
	}

	void FConcurrentInstanceCounterManager::Increment(const FTopLevelAssetPath& InstancePath)
	{
		FScopeLock Lock(&MapCritSec);
		GetOrAddStats(InstancePath).Increment();
	}

	void FConcurrentInstanceCounterManager::Decrement(const FTopLevelAssetPath& InstancePath)
	{
		FScopeLock Lock(&MapCritSec);
		GetOrAddStats(InstancePath).Decrement();
	}

	int64 FConcurrentInstanceCounterManager::GetCountForPath(const FTopLevelAssetPath& InPath)
	{
		FScopeLock Lock(&MapCritSec);
		if (FStats* Stats = StatsMap.Find(InPath))
		{
			return Stats->GetCount();
		}

		return 0;
	}

	int64 FConcurrentInstanceCounterManager::GetPeakCountForPath(const FTopLevelAssetPath& InPath)
	{
		FScopeLock Lock(&MapCritSec);
		if (FStats* Stats = StatsMap.Find(InPath))
		{
			return Stats->GetPeakCount();
		}

		return 0;
	}

	void FConcurrentInstanceCounterManager::VisitStats(TFunctionRef<void(const FTopLevelAssetPath&, int64)> Visitor)
	{
		FScopeLock Lock(&MapCritSec);

		for (const TPair<FTopLevelAssetPath, FStats>& Pair : StatsMap)
		{
			Visitor(Pair.Key, Pair.Value.GetCount());
		}
	}

#if COUNTERSTRACE_ENABLED
	FConcurrentInstanceCounterManager::FStats::FStats(const FString& InCategoryName, const FTopLevelAssetPath& InPath)
	: TraceCounter(MakeUnique<FCountersTrace::FCounterInt>(
		TraceCounterNameType_Dynamic,
		*FString::Printf(TEXT("%s - %s"), *InCategoryName, *InPath.ToString()),
		TraceCounterDisplayHint_None))
	{
	}
#endif

	void FConcurrentInstanceCounterManager::FStats::Increment()
	{
		ensure(TraceCounter);
		TraceCounter->Increment();
		PeakCount = FMath::Max(PeakCount, GetCount());
	}


	void FConcurrentInstanceCounterManager::FStats::Decrement()
	{
		ensure(TraceCounter);
		TraceCounter->Decrement();
	}

	int64 FConcurrentInstanceCounterManager::FStats::GetCount() const
	{
		ensure(TraceCounter);
#if COUNTERSTRACE_ENABLED
		return TraceCounter->Get();
#else
		return TraceCounter->GetValue();
#endif
	}

	int64 FConcurrentInstanceCounterManager::FStats::GetPeakCount() const
	{
		return PeakCount;
	}

	FConcurrentInstanceCounterManager::FStats& FConcurrentInstanceCounterManager::GetOrAddStats(const FTopLevelAssetPath& InstancePath)
	{
		FScopeLock Lock(&MapCritSec);

		// avoid re-constructing the trace counter string unless it's new
		if (StatsMap.Contains(InstancePath))
		{
			return StatsMap[InstancePath];
		}

#if COUNTERSTRACE_ENABLED
		return StatsMap.Emplace(InstancePath, FStats(CategoryName, InstancePath));
#else
		return StatsMap.Add(InstancePath);
#endif
	}

	FConcurrentInstanceCounter::FConcurrentInstanceCounter(TSharedPtr<FConcurrentInstanceCounterManager> InCounterManager)
	: ManagerPtr(InCounterManager)
	{
		check(ManagerPtr);
	}

	FConcurrentInstanceCounter::FConcurrentInstanceCounter(const FTopLevelAssetPath& InPath, TSharedPtr<FConcurrentInstanceCounterManager> InCounterManager)
	: InstancePath(InPath)
	, ManagerPtr(InCounterManager)
	{
		check(ManagerPtr);
		GetManagerChecked().Increment(InstancePath);
	}

	// dtor
	FConcurrentInstanceCounter::~FConcurrentInstanceCounter()
	{
		GetManagerChecked().Decrement(InstancePath);
	}

	void FConcurrentInstanceCounter::Init(const FTopLevelAssetPath& InPath)
	{
		InstancePath = InPath;
		GetManagerChecked().Increment(InstancePath);
	}

	FConcurrentInstanceCounterManager& FConcurrentInstanceCounter::GetManagerChecked()
	{
		check(ManagerPtr);
		return *ManagerPtr;
	}

	// static interface
} // namespace Metasound
