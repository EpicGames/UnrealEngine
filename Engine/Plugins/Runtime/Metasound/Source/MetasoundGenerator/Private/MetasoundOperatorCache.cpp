// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorCache.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "MetasoundGeneratorModuleImpl.h"
#include "MetasoundGeneratorBuilder.h"
#include "MetasoundOperatorCacheStatTracker.h"
#include "MetasoundTrace.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	LLM_DEFINE_TAG(Audio_Metasound_OperatorPool);

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
	TRACE_DECLARE_INT_COUNTER(MetaSound_OperatorPool_NumOperators, TEXT("MetaSound/OperatorPool/NumOperatorsInPool"));
	TRACE_DECLARE_FLOAT_COUNTER(MetaSound_OperatorPool_HitRatio, TEXT("MetaSound/OperatorPool/HitRatio"));
	TRACE_DECLARE_FLOAT_COUNTER(MetaSound_OperatorPool_WindowedHitRatio, TEXT("MetaSound/OperatorPool/WindowedHitRatio"));

	CSV_DEFINE_CATEGORY(MetaSound_OperatorPool, true);
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

	namespace OperatorPoolPrivate
	{
		constexpr int32 DefaultSampleRateForDeprecatedAPI = 48000;
		constexpr float BlockRate = 100.f;

		static bool bMetasoundPoolSyncGraphRetrieval = true;
		FAutoConsoleVariableRef CVarMetasoundPoolSyncGraphRetrieval(
			TEXT("au.MetaSound.OperatorPoolSyncGraphRetrieval"),
			bMetasoundPoolSyncGraphRetrieval,
			TEXT("Retrieves graph on the requesting thread prior to asynchronous task to create instance.\n"),
			ECVF_Default);

		// TODO: Move this into the OperatorCacheStatTracker.
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
		static std::atomic<uint32> CacheHitCount = 0;
		static std::atomic<uint32> CacheAttemptCount = 0;

		static float MetasoundPoolHitRateWindowSecondsCVar = 1.0f;
		FAutoConsoleVariableRef CVarMetasoundPoolHitRateWindowSeconds(
			TEXT("au.MetaSound.OperatorPoolHitRateWindowSeconds"),
			MetasoundPoolHitRateWindowSecondsCVar,
			TEXT("Control how long hit/miss results matter for the success rate reporting.\n"),
			ECVF_Default);

		double GetHitRatio()
		{
			uint32 NumHits = CacheHitCount;
			uint32 Total = CacheAttemptCount;
			if (Total > 0)
			{
				return static_cast<double>(NumHits) / static_cast<double>(Total);
			}
			else
			{
				return 0.f;
			}
		}

		FWindowedHitRate::FWindowedHitRate()
		: CurrTTLSeconds(MetasoundPoolHitRateWindowSecondsCVar)
		{
		}

		void FWindowedHitRate::Update()
		{
			if (bIsFirstUpdate)
			{
				bIsFirstUpdate = false;
				FirstUpdate();
			}

			if (CurrTTLSeconds != MetasoundPoolHitRateWindowSecondsCVar)
			{
				SetWindowLength(MetasoundPoolHitRateWindowSecondsCVar);
			}
	
			// Incorporate latest results
			// note:	there a sliver of a race condition here between the 2 values
			// 			but we should be able to afford the occasional off-by-one and
			// 			avoid mutex contention
			IntermediateResult Result( { CurrHitCount, CurrTotal });
			CurrHitCount = 0;
			CurrTotal = 0;
			RunningHitCount += Result.NumHits;
			RunningTotal += Result.Total;
			Result.TTLSeconds = CurrTTLSeconds;
			History.Emplace(MoveTemp(Result));
	
			// calculate delta time, update time
			const uint64 CurrentTimeCycles = FPlatformTime::Cycles64();
			const float DeltaTimeSeconds = FPlatformTime::ToSeconds64(CurrentTimeCycles - PreviousTimeCycles);
			PreviousTimeCycles = CurrentTimeCycles;
	
			// tick down intermediate results, remove any which have expired
			TickResults(DeltaTimeSeconds);
			
			if (RunningTotal > 0)
			{
				const float HitRatio = RunningHitCount / static_cast<float>(RunningTotal);
				TRACE_COUNTER_SET(MetaSound_OperatorPool_WindowedHitRatio, HitRatio);

				CSV_CUSTOM_STAT(MetaSound_OperatorPool, WindowedCacheHitRatio, HitRatio, ECsvCustomStatOp::Set);
			}
		}

		void FWindowedHitRate::AddHit()
		{
			++CurrHitCount;
			++CurrTotal;
		}
	
		void FWindowedHitRate::AddMiss()
		{
			++CurrTotal;
		}
	
		void FWindowedHitRate::SetWindowLength(const float InNewLengthSeconds)
		{
			if (!ensure(InNewLengthSeconds))
			{
				return;
			}
	
			const float Delta = InNewLengthSeconds - CurrTTLSeconds;
			CurrTTLSeconds = InNewLengthSeconds;
	
			// Delta is positive if the new length is longer than the old length.
			// perform an "inverse" Tick to adjust the TTLs in the History
			TickResults(-Delta);
		}
	
		void FWindowedHitRate::FirstUpdate()
		{
			PreviousTimeCycles = FPlatformTime::Cycles64();
		}
	
		void FWindowedHitRate::ExpireResult(const IntermediateResult& InResultToExpire)
		{	
			RunningHitCount -= InResultToExpire.NumHits;
			RunningTotal -= InResultToExpire.Total;
		}
	
		void FWindowedHitRate::TickResults(const float DeltaTimeSeconds)
		{
			// tick down intermediate results, remove any which have expired
			const int32 NumEntries = History.Num();
			for (int i = NumEntries - 1; i >= 0; --i)
			{
				IntermediateResult& Result = History[i];
				Result.TTLSeconds -= DeltaTimeSeconds;
				if (Result.TTLSeconds < 0.f)
				{
					ExpireResult(Result);
					History.RemoveAtSwap(i); // note: Result ref is no longer valid!
				}
			}
		}
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

#if CSV_PROFILER
		template <typename TValue>
		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, const TValue& Value, ECsvCustomStatOp StatOp)
		{
			// Transient MetaSound graphs are guaranteed to have unique names, whereas asset
			// packages may have the same name but are guaranteed to have different package names.
			if (const UPackage* TransientPackage = GetTransientPackage())
			{
				if (InAssetPath.GetPackageName() == TransientPackage->GetFName())
				{
					FCsvProfiler::Get()->RecordCustomStat(InAssetPath.GetAssetName(), CategoryIndex, Value, StatOp);
					return;
				}
			}

			FCsvProfiler::Get()->RecordCustomStat(InAssetPath.GetPackageName(), CategoryIndex, Value, StatOp);
		}
#endif // CSV_PROFILER
	} // namespace OperatorPoolPrivate

#if CSV_PROFILER
	namespace Engine
	{
		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, int32 Value, ECsvCustomStatOp StatOp)
		{
			OperatorPoolPrivate::RecordOperatorStat(InAssetPath, CategoryIndex, Value, StatOp);
		}

		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, float Value, ECsvCustomStatOp StatOp)
		{
			OperatorPoolPrivate::RecordOperatorStat(InAssetPath, CategoryIndex, Value, StatOp);
		}

		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, double Value, ECsvCustomStatOp StatOp)
		{
			OperatorPoolPrivate::RecordOperatorStat(InAssetPath, CategoryIndex, Value, StatOp);
		}
	} // namespace Engine
#endif // CSV_PROFILER

	FOperatorBuildData::FOperatorBuildData(
		  FGeneratorInitParams&& InInitParams
		, Frontend::FGraphRegistryKey InRegistryKey
		, FGuid InAssetClassID
		, int32 InNumInstances
		, bool bInTouchExisting
	)
	: InitParams(InInitParams)
	, RegistryKey(InRegistryKey)
	, AssetClassID(InAssetClassID)
	, NumInstances(InNumInstances)
	, bTouchExisting(bInTouchExisting)
	{
	}

	FOperatorContext FOperatorContext::FromInitParams(const FGeneratorInitParams& InParams)
	{
		return FOperatorContext
		{
			.GraphAssetPath = InParams.Graph ? InParams.Graph->GetAssetPath() : InParams.AssetPath,
			.AssetPath = InParams.AssetPath
		};
	}

	FOperatorPoolEntryID::FOperatorPoolEntryID(FGuid InOperatorID, FOperatorSettings InSettings)
	: OperatorID(MoveTemp(InOperatorID))
	, OperatorSettings(MoveTemp(InSettings))
	{
	}

	FString FOperatorPoolEntryID::ToString() const
	{
		return FString::Printf(TEXT("%s %s"), *OperatorID.ToString(), *OperatorSettings.ToString());
	}

	bool operator<(const FOperatorPoolEntryID& InLHS, const FOperatorPoolEntryID& InRHS)
	{
		if (InLHS.OperatorID < InRHS.OperatorID)
		{
			return true;
		}
		else if (InRHS.OperatorID < InLHS.OperatorID)
		{
			return false;
		}
		else 
		{
			return InLHS.OperatorSettings < InRHS.OperatorSettings;
		}
	}

	bool operator==(const FOperatorPoolEntryID& InLHS, const FOperatorPoolEntryID& InRHS)
	{
		return (InLHS.OperatorID == InRHS.OperatorID) && (InLHS.OperatorSettings == InRHS.OperatorSettings);
	}


	FOperatorPool::FOperatorPool(FOperatorPoolSettings InSettings)
	: Settings(MoveTemp(InSettings))
	, AsyncBuildPipe(UE_SOURCE_LOCATION)
	{
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
		CacheStatTracker = MakeUnique<Engine::FOperatorCacheStatTracker>();
#endif
	}

	FOperatorPool::~FOperatorPool()
	{
		StopAsyncTasks();
	}

	FOperatorAndInputs FOperatorPool::ClaimOperator(const FOperatorPoolEntryID& InOperatorID, const FOperatorContext& InContext)
	{
		FOperatorAndInputs OpAndInputs;

		if (!IsStopping())
		{
			FScopeLock Lock(&CriticalSection);

			bool bCacheHit = false;
			if (TArray<FOperatorAndInputs>* OperatorsWithID = Operators.Find(InOperatorID))
			{
				if (OperatorsWithID->Num() > 0)
				{
					OpAndInputs = OperatorsWithID->Pop();
					Stack.RemoveAt(Stack.FindLast(InOperatorID));
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
					OperatorPoolPrivate::CacheHitCount++;
					bCacheHit = true;
					TRACE_COUNTER_DECREMENT(MetaSound_OperatorPool_NumOperators);
#endif
				}
			}

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
			bCacheHit ? HitRateTracker.AddHit() : HitRateTracker.AddMiss();
			OperatorPoolPrivate::CacheAttemptCount++;
			TRACE_COUNTER_SET(MetaSound_OperatorPool_HitRatio, OperatorPoolPrivate::GetHitRatio());
			CacheStatTracker->RecordCacheEvent(InOperatorID, bCacheHit, InContext);
#endif
			UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Attempt to claim operator with ID %ls from operator pool resulted in %ls", *InOperatorID.ToString(), *::LexToString(bCacheHit));
		}

		return OpAndInputs;
	}
	
	void FOperatorPool::AddOperator(const FOperatorPoolEntryID& InOperatorID, TUniquePtr<IOperator>&& InOperator, FInputVertexInterfaceData&& InInputData, TSharedPtr<FGraphRenderCost>&& InRenderCost)
	{
		AddOperator(InOperatorID, { MoveTemp(InOperator), MoveTemp(InInputData), MoveTemp(InRenderCost) });
	}

	bool FOperatorPool::ExecuteTaskAsync(FOperatorPool::FTaskFunction&& InFunction)
	{
		using namespace UE::Tasks;

		if (IsStopping())
		{
			return false;
		}

		TWeakPtr<FOperatorPool> WeakOpPool = AsShared();
		const int32 TaskId = ++LastTaskId;
		FTask NewTask = AsyncBuildPipe.Launch(UE_SOURCE_LOCATION, [WeakOpPool, TaskId, PoolFunction = MoveTemp(InFunction)]() mutable
		{
			PoolFunction(TaskId, WeakOpPool);

			if (TSharedPtr<FOperatorPool> ThisPool = WeakOpPool.Pin())
			{
				FScopeLock Lock(&ThisPool->CriticalSection);
				ThisPool->ActiveBuildTasks.Remove(TaskId);
			}
		});

		{
			FScopeLock Lock(&CriticalSection);
			ActiveBuildTasks.Add(TaskId, MoveTemp(NewTask));
		}

		return true;
	}

	void FOperatorPool::AddOperator(const FOperatorPoolEntryID& InOperatorID, FOperatorAndInputs&& OperatorAndInputs)
	{
		using namespace UE::Tasks;

		check(OperatorAndInputs.Operator.IsValid());

		ExecuteTaskAsync([OperatorID = InOperatorID, OpAndInputs = MoveTemp(OperatorAndInputs)](FTaskId, TWeakPtr<FOperatorPool> WeakPoolPtr) mutable
		{
			if (TSharedPtr<FOperatorPool> Pool = WeakPoolPtr.Pin())
			{
				Pool->AddOperatorInternal(OperatorID, MoveTemp(OpAndInputs));
			}
		});
	}	

	void FOperatorPool::AddOperatorInternal(const FOperatorPoolEntryID& InOperatorID, FOperatorAndInputs && OperatorAndInputs)
	{
		if (!OperatorAndInputs.Operator.IsValid())
		{
			return;
		}

		{
			FScopeLock Lock(&CriticalSection);
			Stack.Add(InOperatorID);
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
			TRACE_COUNTER_INCREMENT(MetaSound_OperatorPool_NumOperators);
			CacheStatTracker->OnOperatorAdded(InOperatorID);
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

			UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Adding operator with ID %ls to operator pool", *InOperatorID.ToString());

			if (TArray<FOperatorAndInputs>* OperatorArray = Operators.Find(InOperatorID))
			{
				// add to existing array
				OperatorArray->Add(MoveTemp(OperatorAndInputs));
			}
			else
			{
				// create a new array and add it to the map
				TArray<FOperatorAndInputs> NewOperatorArray;
				NewOperatorArray.Add(MoveTemp(OperatorAndInputs));
				Operators.Add(InOperatorID, MoveTemp(NewOperatorArray));
			}
		}

		// Call Trim() outside the lock so any evicted operators are destroyed
		// after CritSec is released (see Trim() for deadlock details).
		// Note: another thread may claim the just-added operator between the
		// lock release and Trim(); Trim() re-reads Stack.Num() under its own
		// lock, so it will simply find nothing to trim in that case.
		Trim();
	}

	bool FOperatorPool::IsStopping() const
	{
		return bStopping.load();
	}

	void FOperatorPool::StopAsyncTasks()
	{
		using namespace UE::Tasks;

		bStopping.store(true);

		// Move tasks to local copy in crit section to allow for safe mutation
		// of ActiveBuildTasks from within tasks and avoid deadlocks with
		// mutation of other pool resources while canceling remaining tasks.
		TMap<FTaskId, FTask> TasksToCancel;
		{
			FScopeLock Lock(&CriticalSection);
			TasksToCancel = MoveTemp(ActiveBuildTasks);
			ActiveBuildTasks.Reset();
		}

		if (!TasksToCancel.IsEmpty())
		{
			UE_LOGF(LogMetasoundGenerator, Display, "Cancelling active MetaSound Cache Pool Operator build requests...");

			for (TPair<FTaskId, FTask>& Pair : TasksToCancel)
			{
				FTask& TaskToCancel = Pair.Value;
				if (!TaskToCancel.IsCompleted())
				{
					TaskToCancel.Wait();
				}
			}
		}

		bStopping.store(false);
	}

	void FOperatorPool::BuildAndAddOperator(TUniquePtr<FOperatorBuildData> InBuildData)
	{
		using namespace UE::Tasks;
		using namespace OperatorPoolPrivate;

		if (!ensure(InBuildData))
		{
			return;
		}

		TSharedPtr<const Metasound::FGraph> Graph;
		if (bMetasoundPoolSyncGraphRetrieval)
		{
			// get the metasound graph and add to init params (might wait for async registration to complete)
			Graph = FMetasoundFrontendRegistryContainer::Get()->GetGraph(InBuildData->RegistryKey);
			if (!Graph.IsValid())
			{
				UE_LOGF(LogMetasoundGenerator, Error, "Failed to retrieve graph '%ls' synchronously when attempting to BuildAndAddOperator to pool", *InBuildData->RegistryKey.ToString());
				return;
			}
		}

		// Build operations should never keep the operator pool alive as this can delay app shutdown arbitrarily.
		ExecuteTaskAsync([Graph, PreCacheData = MoveTemp(InBuildData)](FTaskId, TWeakPtr<FOperatorPool> WeakPoolPtr)
		{
			using namespace OperatorPoolPrivate;

			LLM_SCOPE_BYTAG(Audio_Metasound_OperatorPool);
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorPool::AsyncOperatorPrecache)

			if (!ensure(PreCacheData))
			{
				return;
			}

			if (Graph.IsValid())
			{
				PreCacheData->InitParams.Graph = Graph;
			}
			else
			{
				if (bMetasoundPoolSyncGraphRetrieval)
				{
					return;
				}
				else
				{
					// get the metasound graph and add to init params (might wait for async registration to complete)
					PreCacheData->InitParams.Graph = FMetasoundFrontendRegistryContainer::Get()->GetGraph(PreCacheData->RegistryKey);
				}
			}

			if (!PreCacheData->InitParams.Graph)
			{
				UE_LOGF(LogMetasoundGenerator, Error, "Failed to retrieve graph '%ls' async when attempting to BuildAndAddOperator to pool", *PreCacheData->RegistryKey.ToString());
				return;
			}

			int32 NumToBuild = PreCacheData->NumInstances;
			const FOperatorPoolEntryID EntryID{PreCacheData->InitParams.Graph->GetInstanceID(), PreCacheData->InitParams.OperatorSettings};

			if (PreCacheData->bTouchExisting)
			{
				TSharedPtr<FOperatorPool> OperatorPool = WeakPoolPtr.Pin();
				if (OperatorPool.IsValid())
				{
					// Get the number of instances already in the cache & move pre-existing to the top of the cache
					const int32 NumInCache = OperatorPool->GetNumCachedOperatorsWithID(EntryID);
					OperatorPool->TouchOperators(EntryID, FMath::Min(NumInCache, NumToBuild));
					NumToBuild -= NumInCache;
				}
			}

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
			if (TSharedPtr<FOperatorPool> OperatorPool = WeakPoolPtr.Pin();
				OperatorPool.IsValid())
			{
				const int32 NumInCache = OperatorPool->GetNumCachedOperatorsWithID(EntryID);
				OperatorPool->CacheStatTracker->RecordPreCacheRequest(*PreCacheData, NumToBuild, NumInCache);
			}
#endif // METASOUND_OPERATORCACHEPROFILER_ENABLED

			for (int32 i = 0; i < NumToBuild; ++i)
			{
				// These build operations can take a fair bit of time, so
				// check continually for the validity of the operator pool
				// on each build request to abort if necessary if cancellation
				// is requested.
				TSharedPtr<FOperatorPool> OperatorPool = WeakPoolPtr.Pin();
				if (!OperatorPool.IsValid() || OperatorPool->IsStopping())
				{
					return;
				}

				FBuildResults BuildResults;
				// First operator to add will use the given graph render cost if set, 
				// but subsequent ones must be recreated so they aren't shared between multiple instances
				if (i != 0 || !PreCacheData->InitParams.GraphRenderCost)
				{
					PreCacheData->InitParams.GraphRenderCost = FGraphRenderCost::MakeGraphRenderCost();
				}

				FOperatorAndInputs OperatorAndInputs = GeneratorBuilder::BuildGraphOperator(PreCacheData->InitParams.OperatorSettings, PreCacheData->InitParams, BuildResults);
				GeneratorBuilder::LogBuildErrors(PreCacheData->InitParams.AssetPath, BuildResults);

				OperatorPool->AddOperatorInternal(EntryID, MoveTemp(OperatorAndInputs));
				OperatorPool->AddAssetIdToGraphIdLookUpInternal(PreCacheData->AssetClassID, EntryID);
			}
		});
	}

	void FOperatorPool::TouchOperators(const FOperatorPoolEntryID& InOperatorID, int32 NumToTouch)
	{
		if (!IsStopping())
		{
			FScopeLock Lock(&CriticalSection);


			const int32 NumToMove = FMath::Min(NumToTouch, GetNumCachedOperatorsWithID(InOperatorID));
			UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Touching operator %d operators with ID %ls in operator pool", NumToMove, *InOperatorID.ToString());
			if (!NumToMove)
			{
				return;
			}

			// add to the "top" (end)
			for (int32 i = 0; i < NumToMove; ++i)
			{
				Stack.Add(InOperatorID);
			}

			// remove from the "bottom" (beginning)
			for (int32 i = 0; i < NumToMove; ++i)
			{
				Stack.RemoveSingle(InOperatorID);
			}
		}
	}

	void FOperatorPool::TouchOperatorsViaAssetClassID(const FGuid& InAssetClassID, int32 NumToTouch)
	{
		if (!IsStopping())
		{
			FScopeLock Lock(&CriticalSection);
			FOperatorPoolEntryID* GraphIdPtr = AssetIdToGraphIdLookUp.Find(InAssetClassID);
			if (GraphIdPtr)
			{
				TouchOperators(*GraphIdPtr, NumToTouch);
			}
		}
	}

	void FOperatorPool::RemoveOperatorsWithID(const FOperatorPoolEntryID& InOperatorID)
	{
		using namespace UE::Tasks;

		if (IsStopping())
		{
			return;
		}

		ExecuteTaskAsync([OperatorID = InOperatorID](FTaskId, TWeakPtr<FOperatorPool> WeakPoolPtr)
		{
			if (TSharedPtr<FOperatorPool> Pool = WeakPoolPtr.Pin())
			{
				UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Removing operators with ID %ls from operator pool", *OperatorID.ToString());

				// Move operators out of the map while holding CritSec, then destroy
				// them after the lock is released. Operator dtors (e.g.
				// FEnvelopeFollowerGenerator::~) call OnAudioDeviceDestroyed.Remove(),
				// which acquires the TS delegate write lock. Destroying under CritSec
				// while the broadcast thread holds the read lock causes an A->B / B->A
				// deadlock. By moving first, CritSec is released before destruction.
				TArray<FOperatorAndInputs> OperatorsToDestroy;
				int32 NumRemoved = 0;
				{
					FScopeLock Lock(&Pool->CriticalSection);
					if (TArray<FOperatorAndInputs>* Found = Pool->Operators.Find(OperatorID))
					{
						OperatorsToDestroy = MoveTemp(*Found);
					}
					Pool->Operators.Remove(OperatorID);
					NumRemoved = Pool->Stack.Remove(OperatorID);

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
					TRACE_COUNTER_SUBTRACT(MetaSound_OperatorPool_NumOperators, int64(NumRemoved));

					Pool->CacheStatTracker->OnOperatorRemoved(OperatorID);
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED
				}
				
				OperatorsToDestroy.Empty();
			}
		});
	}

	void FOperatorPool::RemoveOperatorsWithAssetClassID(const FGuid& InAssetClassID)
	{
		FScopeLock Lock(&CriticalSection);
		const FOperatorPoolEntryID* GraphIdPtr = AssetIdToGraphIdLookUp.Find(InAssetClassID);
		if (GraphIdPtr)
		{
			if (GraphIdToAssetIdLookUp.Num(*GraphIdPtr) <= 1)
			{
				// Only remove operators if there are no other assets pointing to the same operator
				RemoveOperatorsWithID(*GraphIdPtr);
			}

			GraphIdToAssetIdLookUp.Remove(*GraphIdPtr, InAssetClassID);
			AssetIdToGraphIdLookUp.Remove(InAssetClassID);
		}
	}
		
	void FOperatorPool::SetMaxNumOperators(uint32 InMaxNumOperators)
	{
		if (!IsStopping())
		{
			{
				FScopeLock Lock(&CriticalSection);
				Settings.MaxNumOperators = InMaxNumOperators;
			}

			// Call Trim() outside the lock so any evicted operators are destroyed
			// after CritSec is released (see Trim() for deadlock details).
			Trim();
		}
	}

	int32 FOperatorPool::GetNumCachedOperatorsWithID(const FOperatorPoolEntryID& InOperatorID) const
	{
		FScopeLock Lock(&CriticalSection);
		if (TArray<FOperatorAndInputs> const* OperatorsWithID = Operators.Find(InOperatorID))
		{
			return OperatorsWithID->Num();
		}

		return 0;
	}

	int32 FOperatorPool::GetNumCachedOperatorsWithAssetClassID(const FGuid& InAssetClassID) const
	{
		FScopeLock Lock(&CriticalSection);
		if (const FOperatorPoolEntryID* GraphIdPtr = AssetIdToGraphIdLookUp.Find(InAssetClassID))
		{
			return GetNumCachedOperatorsWithID(*GraphIdPtr);
		}

		return 0;
	}

	void FOperatorPool::AddAssetIdToGraphIdLookUpInternal(const FGuid& InAssetClassID, const FOperatorPoolEntryID& InOperatorID)
	{
		FScopeLock Lock(&CriticalSection);
		AssetIdToGraphIdLookUp.Add(InAssetClassID, InOperatorID);
		GraphIdToAssetIdLookUp.AddUnique(InOperatorID, InAssetClassID);
	}

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
	void FOperatorPool::UpdateHitRateTracker()
	{
		HitRateTracker.Update();
	}
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

	void FOperatorPool::Trim()
	{
		// Collect evicted operators outside the lock scope, then destroy them
		// after CritSec is released. Operator dtors (e.g.
		// FEnvelopeFollowerGenerator::~) call OnAudioDeviceDestroyed.Remove(),
		// which acquires the TS delegate write lock. Destroying under CritSec
		// while the broadcast thread holds the read lock can cause deadlock
		TArray<FOperatorAndInputs> TrimmedOperators;
		{
			FScopeLock Lock(&CriticalSection);
			int32 NumToTrim = Stack.Num() - Settings.MaxNumOperators;
			if (NumToTrim > 0)
			{
				TrimmedOperators.Reserve(NumToTrim);
				UE_LOGF(LogMetasoundGenerator, Verbose, "Trimming %d operators", NumToTrim);
				for (int32 i = 0; i < NumToTrim; i++)
				{
					UE_LOGF(LogMetasoundGenerator, VeryVerbose, "Trimming operator with ID %ls", *Stack[i].ToString());
					TArray<FOperatorAndInputs>* OperatorArray = Operators.Find(Stack[i]);
					if (ensure(OperatorArray))
					{
						if (ensure(OperatorArray->Num() > 0))
						{
							TrimmedOperators.Add(OperatorArray->Pop(EAllowShrinking::No));
							if (OperatorArray->Num() == 0)
							{
								Operators.Remove(Stack[i]);
								// Remove related asset IDs
								TArray<FGuid> RelatedAssetIDs;
								GraphIdToAssetIdLookUp.MultiFind(Stack[i], RelatedAssetIDs);

								for (const FGuid& AssetID : RelatedAssetIDs)
								{
									AssetIdToGraphIdLookUp.Remove(AssetID);
									GraphIdToAssetIdLookUp.Remove(Stack[i], AssetID);
								}
							}

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
							CacheStatTracker->OnOperatorTrimmed(Stack[i]);
#endif // METASOUND_OPERATORCACHEPROFILER_ENABLED
						}
					}
				}
				Stack.RemoveAt(0, NumToTrim);
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
				TRACE_COUNTER_SUBTRACT(MetaSound_OperatorPool_NumOperators, NumToTrim);
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED
			}
		}
		// TrimmedOperators destructs here — CritSec is NOT held
	}
} // namespace Metasound
