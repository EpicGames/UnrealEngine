// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorCacheStatTracker.h"

#include "MetasoundOperatorCache.h"
#include "MetasoundGeneratorModule.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
namespace Metasound
{
	CSV_DECLARE_CATEGORY_EXTERN(MetaSound_OperatorPool);
}

namespace Metasound::Engine
{
	CSV_DEFINE_CATEGORY(MetaSound_OperatorCacheUtilization, true);
	CSV_DEFINE_CATEGORY(MetaSound_AvailableCachedOperators, true);
	CSV_DEFINE_CATEGORY(Metasound_OperatorCacheMiss, true);

	namespace Private
	{
		static bool bMetasoundOperatorPoolCsvStatsEnabled = false;
		static FAutoConsoleVariableRef CVarMetasoundOperatorPoolCsvStatsEnabled(
			TEXT("au.MetaSound.OperatorPool.CsvStatsEnabled"),
			bMetasoundOperatorPoolCsvStatsEnabled,
			TEXT("If we should record operator pool stats to the csv.")
		);

		static TAutoConsoleVariable<bool> CVarCacheMissCsvStatsEnabled(
			TEXT("au.MetaSound.OperatorPool.CacheMissCsvStatsEnabled"),
			true,
			TEXT("Record which metasounds incur a cache miss when building their graph.")
		);
	} // namespace Private

	FOperatorCacheStatTracker::FOperatorCacheStatTracker()
	{
#if CSV_PROFILER_STATS 
		CsvEndFrameDelegateHandle = FCsvProfiler::Get()->OnCSVProfileEndFrame().AddRaw(this, &FOperatorCacheStatTracker::OnCsvProfileEndFrame);
#endif // CSV_PROFILER_STATS 
	}

	FOperatorCacheStatTracker::~FOperatorCacheStatTracker()
	{
#if CSV_PROFILER_STATS 
		FCsvProfiler::Get()->OnCSVProfileEndFrame().Remove(CsvEndFrameDelegateHandle);
		CsvEndFrameDelegateHandle.Reset();
#endif // CSV_PROFILER_STATS 
	}

	void FOperatorCacheStatTracker::RecordPreCacheRequest(const FOperatorBuildData& BuildData, int32 NumInstancesToBuild, int32 NumInstancesInCache)
	{
		if (BuildData.NumInstances <= 0)
		{
			return;
		}

		const FOperatorPoolEntryID EntryID{ BuildData.InitParams.Graph->GetInstanceID(), BuildData.InitParams.OperatorSettings };

		FScopeLock Lock(&CriticalSection);

		if (FStatEntry* Entry = StatEntries.Find(EntryID))
		{
			Entry->NumCacheSlots += NumInstancesToBuild;

			// Show how much we're increasing the existing cache for this sound by.
			UE_LOGF(LogMetasoundGenerator, Log,
				"Pre-cached MetaSound: %ls [Graph: %ls]. Added %d instances, Total: %d.",
				*BuildData.InitParams.AssetPath.ToString(),
				*Entry->GraphAssetPath.ToString(),
				NumInstancesToBuild,
				Entry->NumCacheSlots);
		}
		else
		{
			FStatEntry& StatEntry = StatEntries.Add(EntryID, FStatEntry
			{
				.GraphAssetPath = BuildData.InitParams.Graph->GetAssetPath(),
				.NumInstancesBuilt = NumInstancesToBuild,
				.NumCacheSlots = NumInstancesToBuild
			});

			if (StatEntry.GraphAssetPath == BuildData.InitParams.AssetPath)
			{
				UE_LOGF(LogMetasoundGenerator, Log,
					"Pre-cached MetaSound: %ls. Requested: %d, Built: %d.",
					*BuildData.InitParams.AssetPath.ToString(),
					BuildData.NumInstances,
					NumInstancesToBuild);
			}
			else
			{
				// Include the parent graph if preset so it's clearer which entry the build instances contribute to.
				UE_LOGF(LogMetasoundGenerator, Log,
					"Pre-cached MetaSound: %ls [Graph: %ls] Requested: %d, Built: %d.",
					*BuildData.InitParams.AssetPath.ToString(),
					*StatEntry.GraphAssetPath.ToString(),
					BuildData.NumInstances,
					NumInstancesToBuild);
			}
		}

		// HACK: Validate the num of tracked cache slots/available instances matches the pool.
		// This is a temporary work-around to an existing issue:
		// 
		// 1. A sound is pre-cached and the operator is added to the pool.
		// 2. That sound is played so the operator is claimed from the cache.
		// 3. The pre-cached operators are removed from the cached (eg. due to match ending), but the sound is still playing.
		// 4. The operator that was claimed is now returned to the cache.
		// 5. The sounds are pre-cached again (eg. new match starting). <-- This is where the mismatch occurs.
		// 6. The operator is claimed from the cache, but the stat tracker's number of available slots doesn't match the operator pool, so we can end up with negative available.
		// 
		// Further detail:
		// - In step 3, when removing an operator from the cache we remove the corresponding stat entry.
		// - In step 4 when the operator is returned, the operator pool will add it as a new operator, but since our entry was removed we don't update that it's back in the cache.
		// - During step 5 when we pre-cache again, let's say 1 instance is pre-cached.
		//		- If touchExisting is set, we'll see we already have 1 instance in the operator pool so we don't build any additional ones. We inform the stat tracker a pre-cache request
		//		  happened but no slots are added since numInstancesToBuild is 0. So we've added the stat entry but it has 0 cache slots which doesn't match the pool.
		//		  Furthermore, since no instances were built, AddOperatorInternal is never called so we also don't increment NumAvailableInCache.
		//		- If touchExisting isn't set we have a similar issue. Newly cached instances will be correctly tracked by the tracker but we'll always be off by however many were already in the pool.
		// - This ultimately results in NumAvailableInCache going negative as there are more instances of the operator in the pool than we're aware of.
		{
			FStatEntry& StatEntry = StatEntries[EntryID];
			const int32 ExpectedCacheSlots = NumInstancesToBuild + NumInstancesInCache;
			// We only correct if expected is greater since the num in the cache is affected by the number currently in use.
			// So if an operator was claimed then we pre-cached that same operator, NumInCache would be less than the actual value once it's returned.
			if (ExpectedCacheSlots > StatEntry.NumCacheSlots)
			{
				const int32 NumMissing = ExpectedCacheSlots - StatEntry.NumCacheSlots;
				UE_LOGF(LogMetasoundGenerator, Log,
					"FOperatorCacheStatTracker detected a cache slot mismatch for %ls. Have %d, expected %d. Updating to expected value.",
					*StatEntry.GraphAssetPath.ToString(),
					StatEntry.NumCacheSlots,
					ExpectedCacheSlots);

				StatEntry.NumCacheSlots = ExpectedCacheSlots;
				for (int32 Index = 0; Index < NumMissing; ++Index)
				{
					OnOperatorAdded(EntryID);
				}
			}
		}
	}

	void FOperatorCacheStatTracker::RecordCacheEvent(const FOperatorPoolEntryID& OperatorID, bool bCacheHit, const FOperatorContext& Context)
	{
		if (!bCacheHit)
		{
#if CSV_PROFILER_STATS 
			if (Private::bMetasoundOperatorPoolCsvStatsEnabled &&
				Private::CVarCacheMissCsvStatsEnabled->GetBool() &&
				Context.GraphAssetPath.IsValid())
			{
				RecordOperatorStat(Context.GraphAssetPath, CSV_CATEGORY_INDEX(Metasound_OperatorCacheMiss), 1, ECsvCustomStatOp::Accumulate);
			}
#endif // CSV_PROFILER_STATS 
			return;
		}

		FScopeLock Lock(&CriticalSection);

		if (FStatEntry* StatEntry = StatEntries.Find(OperatorID))
		{
			--StatEntry->NumAvailableInCache;
			ensure(StatEntry->NumAvailableInCache >= 0);
		}

		--NumInCache;
		ensure(NumInCache >= 0);
	}

	void FOperatorCacheStatTracker::OnOperatorAdded(const FOperatorPoolEntryID& OperatorID)
	{
		FScopeLock Lock(&CriticalSection);

		if (FStatEntry* StatEntry = StatEntries.Find(OperatorID))
		{
			++StatEntry->NumAvailableInCache;
		}

		++NumInCache;
	}

	void FOperatorCacheStatTracker::OnOperatorTrimmed(const FOperatorPoolEntryID& OperatorID)
	{
		FScopeLock Lock(&CriticalSection);

		if (FStatEntry* StatEntry = StatEntries.Find(OperatorID))
		{
			--StatEntry->NumCacheSlots;
			--StatEntry->NumAvailableInCache;
			ensure(StatEntry->NumCacheSlots >= 0);
			ensure(StatEntry->NumAvailableInCache >= 0);

			if (StatEntry->NumCacheSlots == 0)
			{
				UE_LOGF(LogMetasoundGenerator, Log, "Evicted %ls from the Operator Pool.", *StatEntry->GraphAssetPath.ToString())
			}
			else
			{
				UE_LOGF(LogMetasoundGenerator, Log,
					"Trimmed 1 instance of %ls from the Operator Pool. %d instances remaining.",
					*StatEntry->GraphAssetPath.ToString(),
					StatEntry->NumCacheSlots);
			}
		}

		--NumInCache;
		ensure(NumInCache >= 0);
	}

	void FOperatorCacheStatTracker::OnOperatorRemoved(const FOperatorPoolEntryID& OperatorID)
	{
		FScopeLock Lock(&CriticalSection);

		if (FStatEntry* StatEntry = StatEntries.Find(OperatorID))
		{
			NumInCache -= StatEntry->NumAvailableInCache;
		}

		StatEntries.Remove(OperatorID);

		ensure(NumInCache >= 0);
	}

	void FOperatorCacheStatTracker::OnCsvProfileEndFrame()
	{
#if CSV_PROFILER_STATS
		if (!Private::bMetasoundOperatorPoolCsvStatsEnabled)
		{
			return;
		}

		QUICK_SCOPE_CYCLE_COUNTER(OperatorCacheStatTracker_RecordStats)

		FScopeLock Lock(&CriticalSection);

		CSV_CUSTOM_STAT(MetaSound_OperatorPool, TotalCachedOperators, NumInCache, ECsvCustomStatOp::Set);

		for (auto It = StatEntries.CreateIterator(); It; ++It)
		{
			const FOperatorPoolEntryID& PoolEntryID = It->Key;
			FStatEntry& Entry = It->Value;

			// Remove any nodes that have been evicted from the cache.
			if (Entry.NumCacheSlots <= 0)
			{
				It.RemoveCurrent();
				continue;
			}

			// Record cache utilization stats.
			if (ensure(Entry.NumCacheSlots > 0))
			{
				RecordOperatorStat(Entry.GraphAssetPath, CSV_CATEGORY_INDEX(MetaSound_AvailableCachedOperators), Entry.NumAvailableInCache, ECsvCustomStatOp::Set);

				const float UtilizationRatio = (Entry.NumCacheSlots - Entry.NumAvailableInCache) / static_cast<float>(Entry.NumCacheSlots);
				RecordOperatorStat(Entry.GraphAssetPath, CSV_CATEGORY_INDEX(MetaSound_OperatorCacheUtilization), UtilizationRatio, ECsvCustomStatOp::Set);
			}
		}
#endif // CSV_PROFILER_STATS 
	}
} // namespace Metasound::Engine
#endif // METASOUND_OPERATORCACHEPROFILER_ENABLED
