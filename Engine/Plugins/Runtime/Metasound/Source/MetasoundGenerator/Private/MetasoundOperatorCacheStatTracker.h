// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundGenerator.h"
#include "ProfilingDebugging/CountersTrace.h"

#ifndef METASOUND_OPERATORCACHEPROFILER_ENABLED
#define METASOUND_OPERATORCACHEPROFILER_ENABLED COUNTERSTRACE_ENABLED
#endif

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
namespace Metasound
{
	struct FOperatorBuildData;
	struct FOperatorContext;
}

namespace Metasound::Engine
{
	class FOperatorCacheStatTracker final
	{
	public:
		FOperatorCacheStatTracker();
		~FOperatorCacheStatTracker();

		// Called when we're pre-caching an operator. Called before AddOperator.
		void RecordPreCacheRequest(const FOperatorBuildData& BuildData, int32 NumInstancesToBuild, int32 NumInstancesInCache);
		void RecordCacheEvent(const FOperatorPoolEntryID& OperatorID, bool bCacheHit, const FOperatorContext& Context);
		void OnOperatorAdded(const FOperatorPoolEntryID& OperatorID);
		void OnOperatorTrimmed(const FOperatorPoolEntryID& OperatorID);
		void OnOperatorRemoved(const FOperatorPoolEntryID& OperatorID);

	private:
		void OnCsvProfileEndFrame();

		struct FStatEntry
		{
			// Stored as a TopLevelPath to avoid conversions writing out csv stats.
			FTopLevelAssetPath GraphAssetPath;

			// The actual number of instances we built.
			int32 NumInstancesBuilt;

			// The number of instances we have space to cache.
			// This can be different from the number currently in the stack as those are removed when in use.
			int32 NumCacheSlots;

			// The number of operators sitting in the pool waiting to be used.
			// Set exclusively by OnOperatorAdded/OnOperatorTrimmed.
			int32 NumAvailableInCache = 0;
		};

		TMap<FOperatorPoolEntryID, FStatEntry> StatEntries;
		mutable FCriticalSection CriticalSection;
		FDelegateHandle CsvEndFrameDelegateHandle;
		int32 NumInCache = 0;
	};
} // namespace Metasound::Engine
#endif // METASOUND_OPERATORCACHEPROFILER_ENABLED