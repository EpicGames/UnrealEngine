// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundGraphOperator.h"
#include "MetasoundInstanceCounter.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParameterPack.h"
#include "MetasoundRouter.h"
#include "MetasoundTrigger.h"
#include "MetasoundRenderCost.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"

#include "Analysis/MetasoundFrontendGraphAnalyzer.h"
#include "Async/AsyncWork.h"
#include "ChannelAgnostic/ChannelAgnosticTranscoding.h"
#include "Containers/MpscQueue.h"
#include "Containers/SpscQueue.h"
#include "Sound/SoundGenerator.h"

#define UE_API METASOUNDGENERATOR_API

#ifndef ENABLE_METASOUND_GENERATOR_RENDER_TIMING
#define ENABLE_METASOUND_GENERATOR_RENDER_TIMING WITH_EDITOR
#endif // ifndef ENABLE_METASOUND_GENERATOR_RENDER_TIMING

#ifndef ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
#define ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#endif // ifndef ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
namespace Metasound
{
	namespace DynamicGraph
	{
		class IDynamicOperatorTransform;
	}

	namespace MetasoundGeneratorPrivate
	{
		struct FRenderTimer;

		struct FParameterSetter
		{
			Frontend::FLiteralAssignmentFunction Assign;
			FAnyDataReference DataReference;
		};

		// In order to use FName as a key to a TSortedMap you have to explicitly 
		// choose which comparison implementation you want to use.  Declaring the
		// type here helps minimize the confusion over why there are so many 
		// template arguments. 
		using FParameterSetterSortedMap  = TSortedMap<FName, FParameterSetter, FDefaultAllocator, FNameFastLess>;

		// A struct that provides a method of pushing "raw" data from a parameter pack into a specific metasound input node.
		struct FParameterPackSetter
		{
			FName DataType;
			void* Destination;
			const Frontend::IParameterAssignmentFunction& Setter;
			FParameterPackSetter(FName InDataType, void* InDestination, const Frontend::IParameterAssignmentFunction& InSetter)
				: DataType(InDataType)
				, Destination(InDestination)
				, Setter(InSetter)
			{}
			void SetParameterWithPayload(const void* ParameterPayload) const
			{
				Setter(ParameterPayload, Destination);
			}
		};

		struct FMetasoundGeneratorData
		{
			FOperatorSettings OperatorSettings;
			TUniquePtr<IOperator> GraphOperator;
			FVertexInterfaceData VertexInterfaceData;
			FParameterSetterSortedMap ParameterSetters;
			TMap<FName, FParameterPackSetter> ParameterPackSetters;
			TUniquePtr<Frontend::FGraphAnalyzer> GraphAnalyzer;
			TArray<TDataReadReference<FAudioBuffer>> OutputBuffers;
			FTriggerWriteRef TriggerOnPlayRef;
			FTriggerReadRef TriggerOnFinishRef;
			TOptional<FChannelAgnosticTypeReadRef> OutputChannelAgnosticRef;
		};
	}

	/** ID for looking up a operator in the operator pool */
	struct FOperatorPoolEntryID final
	{
		/** Construct an ID
		 *
		 * InOperatorID - ID of the operator.
		 * InSettings - Operator settings used to create the operator.
		 */
		UE_API FOperatorPoolEntryID(FGuid InOperatorID, FOperatorSettings InSettings);

		UE_API FString ToString() const;

		METASOUNDGENERATOR_API friend bool operator<(const FOperatorPoolEntryID& InLHS, const FOperatorPoolEntryID& InRHS);
		METASOUNDGENERATOR_API friend bool operator==(const FOperatorPoolEntryID& InLHS, const FOperatorPoolEntryID& InRHS);
		friend inline uint32 GetTypeHash(const FOperatorPoolEntryID& InID)
		{
			return HashCombineFast(GetTypeHash(InID.OperatorID), GetTypeHash(InID.OperatorSettings));
		}

	private:
		FGuid OperatorID;
		FOperatorSettings OperatorSettings;
	};

	struct FGeneratorInitParams
	{
		FOperatorSettings OperatorSettings;
		FOperatorBuilderSettings BuilderSettings;
		TSharedPtr<const IGraph, ESPMode::ThreadSafe> Graph;
		FMetasoundEnvironment Environment;

		TArray<FVertexName> AudioOutputNames;
		TArray<FAudioParameter> DefaultParameters;
		bool bBuildSynchronous = false;
		TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> DataChannel;
		TSharedPtr<FGraphRenderCost> GraphRenderCost;
		FName ClassName;
		FTopLevelAssetPath AssetPath;

		// CAT format identity (e.g. "Cat:Mono1Dot0"). Empty for non-CAT sources.
		// The generator derives the mixer-compatible channel count internally:
		//   - CAT path:     from SourceFormatName via channel registry lookup + clamp.
		//   - Discrete path: from AudioOutputNames.Num().
		FName SourceFormatName;

		static UE_API void Reset(FGeneratorInitParams& InParams);
	};

	// Initialization params required for building a MetaSound generator
	struct UE_DEPRECATED(5.7, "Use FGeneratorInitParams instead (MetaSoundName is deprecated in favor of AssetPath") FMetasoundGeneratorInitParams : public FGeneratorInitParams
	{
#if WITH_EDITORONLY_DATA
		FString MetaSoundName = { };
#endif // WITH_EDITORONLY_DATA
	};

	enum class EVertexInterfaceChangeType : uint8
	{
		Added,
		Updated,
		Removed
	};

	struct FVertexInterfaceChange
	{
		FVertexName VertexName;
		EMetasoundFrontendClassType VertexType; // Input or Output
		EVertexInterfaceChangeType ChangeType;
	};

	DECLARE_TS_MULTICAST_DELEGATE(FOnSetGraph);

	class FMetasoundGenerator : public ISoundGenerator
	{
	public:
		using FOperatorUniquePtr = TUniquePtr<Metasound::IOperator>;
		using FAudioBufferReadRef = Metasound::FAudioBufferReadRef;

		const FOperatorSettings OperatorSettings;

		/** Create the generator with a graph operator and an output audio reference.
		 *
		 * @param InParams - The generator initialization parameters
		 */

		UE_API explicit FMetasoundGenerator(const FOperatorSettings& InOperatorSettings);

		UE_API virtual ~FMetasoundGenerator();

		/** Set the value of a graph's input data using the assignment operator.
		 *
		 * @param InName - The name of the graph's input data reference.
		 * @param InData - The value to assign.
		 */
		template<typename DataType>
		void SetInputValue(const FVertexName& InName, DataType InData)
		{
			if (const FAnyDataReference* Ref = VertexInterfaceData.GetInputs().FindDataReference(InName))
			{
				*(Ref->GetDataWriteReference<typename TDecay<DataType>::Type>()) = InData;
			}
		}

		/** Apply a function to the graph's input data.
		 *
		 * @param InName - The name of the graph's input data reference.
		 * @param InFunc - A function which takes the DataType as an input.
		 */ 
		template<typename DataType>
		void ApplyToInputValue(const FVertexName& InName, TFunctionRef<void(DataType&)> InFunc)
		{
			if (const FAnyDataReference* Ref = VertexInterfaceData.GetInputs().FindDataReference(InName))
			{
				InFunc(*(Ref->GetDataWriteReference<typename TDecay<DataType>::Type>()));
			}
		}

		UE_API void QueueParameterPack(TSharedPtr<FMetasoundParameterPackStorage> ParameterPack);

		/**
		 * Get a write reference to one of the generator's inputs, if it exists.
		 * NOTE: This reference is only safe to use immediately on the same thread that this generator's
		 * OnGenerateAudio() is called.
		 *
		 * @tparam DataType - The expected data type of the input
		 * @param InputName - The user-defined name of the input
		 */
		template<typename DataType>
		TOptional<TDataWriteReference<DataType>> GetInputWriteReference(const FVertexName InputName)
		{
			TOptional<TDataWriteReference<DataType>> WriteRef;
			
			if (const FAnyDataReference* Ref = VertexInterfaceData.GetInputs().FindDataReference(InputName))
			{
				WriteRef = Ref->GetDataWriteReference<typename TDecay<DataType>::Type>();
			}
			
			return WriteRef;
		}
		
		/**
		 * Get a read reference to one of the generator's outputs, if it exists.
		 * NOTE: This reference is only safe to use immediately on the same thread that this generator's
		 * OnGenerateAudio() is called.
		 *
		 * @tparam DataType - The expected data type of the output
		 * @param OutputName - The user-defined name of the output
		 */
		template<typename DataType>
		TOptional<TDataReadReference<DataType>> GetOutputReadReference(const FVertexName OutputName)
		{
			TOptional<TDataReadReference<DataType>> ReadRef;

			if (const FAnyDataReference* Ref = VertexInterfaceData.GetOutputs().FindDataReference(OutputName))
			{
				ReadRef = Ref->GetDataReadReference<typename TDecay<DataType>::Type>();
			}
			
			return ReadRef;
		}

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnVertexInterfaceDataUpdated, FVertexInterfaceData);
		FOnVertexInterfaceDataUpdated OnVertexInterfaceDataUpdated;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnVertexInterfaceDataUpdatedWithChanges, const TArray<FVertexInterfaceChange>&);
		FOnVertexInterfaceDataUpdatedWithChanges OnVertexInterfaceDataUpdatedWithChanges;

		/**
		 * Add a vertex analyzer for a named output with the given address info.
		 *
		 * @param AnalyzerAddress - Address information for the analyzer
		 */
		UE_API void AddOutputVertexAnalyzer(const Frontend::FAnalyzerAddress& AnalyzerAddress);
		
		/**
		 * Remove a vertex analyzer for a named output
		 *
		 * @param AnalyzerAddress - Address information for the analyzer
		 */
		UE_API void RemoveOutputVertexAnalyzer(const Frontend::FAnalyzerAddress& AnalyzerAddress);
		
		DECLARE_TS_MULTICAST_DELEGATE_FourParams(FOnOutputChanged, FName, FName, FName, TSharedPtr<IOutputStorage>);
		FOnOutputChanged OnOutputChanged;
		
		/** Return the number of audio channels. */
		UE_API virtual int32 GetNumChannels() const override;

		//~ Begin FSoundGenerator
		UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;
		UE_API virtual int32 GetDesiredNumSamplesToRenderPerCallback() const override;
		UE_API virtual bool IsFinished() const override;
		UE_API virtual float GetRelativeRenderCost() const override;
		//~ End FSoundGenerator

		/** Enables the performance timing of the metasound rendering process. You
		 * must call this before "GetCPUCoreUtilization" or the results will
		 * always be 0.0.
		 */
		void EnableRuntimeRenderTiming(bool Enable) { bDoRuntimeRenderTiming = Enable; }

		/** Fraction of a single CPU core used to render audio on a scale of 0.0 to 1.0 */
		UE_API double GetCPUCoreUtilization() const;

		// Called when a new graph has been "compiled" and set up as this generator's graph.
		// Note: We don't allow direct assignment to the FOnSetGraph delegate
		// because we want to give the Delegate an initial immediate callback if the generator 
		// already has a graph. 
		UE_API FDelegateHandle AddGraphSetCallback(FOnSetGraph::FDelegate&& Delegate);
		UE_API bool RemoveGraphSetCallback(const FDelegateHandle& Handle);

		// Enqueues a command for this generator to execute when its next buffer is
		// requested by the mixer.  Enqueued commands are executed before OnGenerateAudio,
		// and on the same thread.  They can safely access generator state.
		void OnNextBuffer(TFunction<void(FMetasoundGenerator&)> Command)
		{
			SynthCommand([this, Command = MoveTemp(Command)]() { Command(*this); });
		}

	protected:

		UE_API void InitBase(FGeneratorInitParams& InInitParams);


		/** SetGraph directly sets graph. Callers must ensure that no race conditions exist. */
		UE_API void SetGraph(TUniquePtr<MetasoundGeneratorPrivate::FMetasoundGeneratorData>&& InData, bool bTriggerGraph);


		UE_API virtual TUniquePtr<IOperator> ReleaseGraphOperator();
		UE_API FInputVertexInterfaceData ReleaseInputVertexData();
#if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING
		FConcurrentInstanceCounter InstanceCounter; 
#endif // if ENABLE_METASOUND_GENERATOR_INSTANCE_COUNTING

		/** Release the graph operator and remove any references to data owned by
		 * the graph operator.
		 */
		UE_API void ClearGraph();
		UE_API bool UpdateGraphIfPending();

		UE_DEPRECATED(5.5, "Use VertexInterfaceChangesSinceLastBroadcast to determine if changes have occurred.")
		std::atomic<bool> bVertexInterfaceHasChanged{ false };

	private:
		friend class FAsyncMetaSoundBuilderBase;

		UE_API void SetPendingGraphBuildFailed();

		/** Update the current graph operator with a new graph operator. The number of channels
		 * of InGraphOutputAudioRef must match the existing number of channels reported by
		 * GetNumChannels() in order for this function to successfully replace the graph operator.
		 *
		 * @param InData - Metasound data of built graph.
		 * @param bTriggerGraph - If true, "OnPlay" will be triggered on the new graph.
		 */
		UE_API void SetPendingGraph(MetasoundGeneratorPrivate::FMetasoundGeneratorData&& InData, bool bTriggerGraph);

		// Fill OutAudio with data in InBuffer, up to maximum number of samples.
		// Returns the number of samples used.
		UE_API int32 FillWithBuffer(const Audio::FAlignedFloatBuffer& InBuffer, float* OutAudio, int32 MaxNumOutputSamples);

		// Metasound creates deinterleaved audio while sound generator requires interleaved audio.
		UE_API void InterleaveGeneratedAudio();
		
		UE_API void ApplyPendingUpdatesToInputs();

		UE_API void HandleRenderTimingEnableDisable();

	protected:
		FExecuter RootExecuter;
		FVertexInterfaceData VertexInterfaceData;
		TArray<FVertexInterfaceChange> VertexInterfaceChangesSinceLastBroadcast;

		TArray<FAudioBufferReadRef> GraphOutputAudio; 
		
		struct FChannelAgnosticState
		{
			FChannelAgnosticTypeReadRef GraphOutput;
			TOptional<Audio::FChannelAgnosticType> MixerOutput;		// Only allocated when transcode is needed (graph format != mixer format).
			Audio::ChannelAgnosticTranscoder::FTranscoder Transcoder;	// Empty when identity — no transcode needed.
			bool bIdentityTranscode = false;				// True when graph output format matches mixer format (skip transcode + extra buffer).
		};
		TOptional<FChannelAgnosticState> GraphCatOutput;			// This is the CAT reference from the graph.

		// Triggered when metasound is finished
		FTriggerReadRef OnFinishedTriggerRef;

		MetasoundGeneratorPrivate::FParameterSetterSortedMap ParameterSetters;

		// This map provides setters for all of the input nodes in the metasound graph. 
		// It is used when processing named parameters in a parameter pack.
		TMap<FName, MetasoundGeneratorPrivate::FParameterPackSetter> ParameterPackSetters;
		TSharedPtr<FGraphRenderCost> GraphRenderCost;

	private:
		FTopLevelAssetPath AssetPath;

		uint8 bIsGraphBuilding : 1 = 0;
		uint8 bIsFinishTriggered : 1 = 0;
		uint8 bIsFinished : 1 = 0;
		uint8 bPendingGraphTrigger : 1 = 1;
		uint8 bIsNewGraphPending : 1 = 0;
		uint8 bIsWaitingForFirstGraph : 1 = 1;
		uint8 bDoRuntimeRenderTiming : 1 = 0;

		int32 FinishSample = INDEX_NONE;
		int32 NumChannels = 0;
		FName SourceFormatName; // Graph-side CAT format identity. Set in Environment upstream (MetasoundSource); held here for reference only. Mixer output uses NumChannels separately.
		int32 NumFramesPerExecute = 0;
		int32 NumSamplesPerExecute = 0;

		Audio::FAlignedFloatBuffer InterleavedAudioBuffer;
		Audio::FAlignedFloatBuffer OverflowBuffer;

		FCriticalSection PendingGraphMutex;
		TUniquePtr<MetasoundGeneratorPrivate::FMetasoundGeneratorData> PendingGraphData;
		TUniquePtr<Frontend::FGraphAnalyzer> GraphAnalyzer;
		TSharedPtr<TSpscQueue<FMetaSoundParameterTransmitter::FParameter>> ParameterQueue;

		// These next items are needed to provide a destination for the FAudioDevice, etc. to
		// send parameter packs to. Every playing metasound will have a parameter destination
		// that can accept parameter packs.
		FSendAddress ParameterPackSendAddress;
		TReceiverPtr<FMetasoundParameterStorageWrapper> ParameterPackReceiver;

		// While parameter packs may arrive via the IAudioParameterInterface system,
		// a faster method of sending parameters is via the QueueParameterPack function 
		// and this queue.
		TMpscQueue<TSharedPtr<FMetasoundParameterPackStorage>> ParameterPackQueue;

		TMpscQueue<TUniqueFunction<void()>> OutputAnalyzerModificationQueue;
		TArray<TUniquePtr<Frontend::IVertexAnalyzer>> OutputAnalyzers;

		FOnSetGraph OnSetGraph;

		double RenderTime = 0.0;
		TUniquePtr<MetasoundGeneratorPrivate::FRenderTimer> RenderTimer;
		
		std::atomic<float> RelativeRenderCost { 1.f };
	};

	/** FMetasoundConstGraphGenerator generates audio from a given metasound IOperator
	 * which produces a multichannel audio output.
	 */
	class FMetasoundConstGraphGenerator : public FMetasoundGenerator
	{
	public:

		UE_API explicit FMetasoundConstGraphGenerator(FGeneratorInitParams&& InParams);

		UE_API explicit FMetasoundConstGraphGenerator(const FOperatorSettings& InOperatorSettings);

		UE_API void Init(FGeneratorInitParams&& InParams);

		UE_API virtual ~FMetasoundConstGraphGenerator() override;

	private:
		friend class FAsyncMetaSoundBuilder;
		UE_API void BuildGraph(FGeneratorInitParams&& InInitParams);
		UE_API bool TryUseCachedOperator(FGeneratorInitParams& InParams, bool bTriggerGenerator);
		UE_API void ReleaseOperatorToCache();

		TUniquePtr<FMetasoundEnvironment> EnvironmentPtr;
		TUniquePtr<FAsyncTaskBase> BuilderTask;
		TOptional<FOperatorPoolEntryID> OperatorPoolID;
		bool bUseOperatorPool = false;
	};

	struct FMetasoundDynamicGraphGeneratorInitParams : FGeneratorInitParams
	{
		TSharedPtr<TSpscQueue<TUniquePtr<DynamicGraph::IDynamicOperatorTransform>>> TransformQueue;
		static UE_API void Reset(FMetasoundDynamicGraphGeneratorInitParams& InParams);
	};

	/** FMetasoundDynamicGraphGenerator generates audio from the given a dynamic operator. It also
	 * reacts to updates to inputs and outputs of the dynamic operator.
	 */
	class FMetasoundDynamicGraphGenerator : public FMetasoundGenerator
	{
	public:

		/** Create the generator with a graph operator and an output audio reference.
		 *
		 * @param InParams - The generator initialization parameters
		 */
		UE_API explicit FMetasoundDynamicGraphGenerator(const FOperatorSettings& InOperatorSettings);

		UE_API void Init(FMetasoundDynamicGraphGeneratorInitParams&& InParams);

		UE_API virtual ~FMetasoundDynamicGraphGenerator();

		// The callbacks are executed when the equivalent change happens on the owned dynamic operator.
		UE_API void OnInputAdded(const FVertexName& InVertexName, const FInputVertexInterfaceData& InInputData);
		UE_API void OnInputRemoved(const FVertexName& InVertexName, const FInputVertexInterfaceData& InInputData);
		UE_API void OnOutputAdded(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InOutputData);
		UE_API void OnOutputUpdated(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InOutputData);
		UE_API void OnOutputRemoved(const FVertexName& InVertexName, const FOutputVertexInterfaceData& InOutputData);

	protected:
		UE_API virtual TUniquePtr<IOperator> ReleaseGraphOperator() override;

	private:
		void BuildGraph(FMetasoundDynamicGraphGeneratorInitParams&& InParams);


		TArray<FVertexName> AudioOutputNames;
		TUniquePtr<FAsyncTaskBase> BuilderTask;
	};
}

#undef UE_API
