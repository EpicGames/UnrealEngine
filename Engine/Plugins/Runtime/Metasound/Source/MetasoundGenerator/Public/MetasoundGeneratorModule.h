// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMetasoundGenerator, Log, All);

namespace Metasound
{
	// forward
	class FOperatorPool;
	class FConcurrentInstanceCounterManager;

	class IMetasoundGeneratorModule : public IModuleInterface
	{
	public:
		virtual TSharedPtr<FOperatorPool> GetOperatorPool() = 0;
		virtual TSharedPtr<FConcurrentInstanceCounterManager> GetOperatorInstanceCounterManager() = 0;
	};
} // namespace Metasound
