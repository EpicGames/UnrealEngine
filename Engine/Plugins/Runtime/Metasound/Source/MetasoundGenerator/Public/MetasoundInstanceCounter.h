// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

// todo: comments

namespace Metasound
{
#if COUNTERSTRACE_ENABLED
	using CounterType = FCountersTrace::FCounterInt;
#else
	using CounterType = FThreadSafeCounter64;
#endif

	// forward
	class FConcurrentInstanceCounter;

	class FConcurrentInstanceCounterManager
	{
	public:
		FConcurrentInstanceCounterManager(const FString InCategoryName);

		void Increment(const FTopLevelAssetPath& InstancePath);
		void Decrement(const FTopLevelAssetPath& InstancePath);

		int64 GetCountForPath(const FTopLevelAssetPath& InPath);
		int64 GetPeakCountForPath(const FTopLevelAssetPath& InPath);

		void VisitStats(TFunctionRef<void(const FTopLevelAssetPath&, int64)> Visitor);

	private:
		struct FStats
		{
		public:
#if COUNTERSTRACE_ENABLED
			FStats(const FString& InCategoryName, const FTopLevelAssetPath& InPath);
#else
			FStats() = default;
#endif // COUNTERSTRACE_ENABLED

			void Increment();
			void Decrement();
	
			int64 GetCount() const;
			int64 GetPeakCount() const;
	
		private:
			TUniquePtr<CounterType> TraceCounter;
			int64 PeakCount;
	
		};

		FStats& GetOrAddStats(const FTopLevelAssetPath& AssetPath);

		TMap<FTopLevelAssetPath, FStats> StatsMap;
		FCriticalSection MapCritSec;
		FString CategoryName;

	}; // class FConcurrentInstanceCounterManager



	class FConcurrentInstanceCounter
	{
	public:
		// ctor
		FConcurrentInstanceCounter() = delete; // must init ManagerPtr
		FConcurrentInstanceCounter(TSharedPtr<FConcurrentInstanceCounterManager> InCounterManager);
		FConcurrentInstanceCounter(const FTopLevelAssetPath& InPath, TSharedPtr<FConcurrentInstanceCounterManager> InCounterManager);
	
		// dtor
		virtual ~FConcurrentInstanceCounter();
	
		// for non-RAII clients
		void Init(const FTopLevelAssetPath& InPath);
	
	private:
		FConcurrentInstanceCounterManager& GetManagerChecked();

		FTopLevelAssetPath InstancePath;
		TSharedPtr<FConcurrentInstanceCounterManager> ManagerPtr;
	}; // class FConcurrentInstanceCounter
} // namespace Metasound
