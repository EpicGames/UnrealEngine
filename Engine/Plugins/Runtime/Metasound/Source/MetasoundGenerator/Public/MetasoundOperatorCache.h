// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundGenerator.h"
#include "Misc/Guid.h"
#include "HAL/CriticalSection.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDGENERATOR_API

#ifndef METASOUND_OPERATORCACHEPROFILER_ENABLED
#define METASOUND_OPERATORCACHEPROFILER_ENABLED COUNTERSTRACE_ENABLED
#endif
namespace Metasound
{
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
	namespace Engine
	{
		class FOperatorCacheStatTracker;

#if CSV_PROFILER
		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, int32 Value, ECsvCustomStatOp StatOp);
		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, float Value, ECsvCustomStatOp StatOp);
		void RecordOperatorStat(const FTopLevelAssetPath& InAssetPath, int32 CategoryIndex, double Value, ECsvCustomStatOp StatOp);
#endif // CSV_PROFILER
	} // namespace Engine

	namespace OperatorPoolPrivate
	{
		class FWindowedHitRate
		{
		public:
			// ctor
			FWindowedHitRate();
			void Update();
			void AddHit();
			void AddMiss();
	
		private:
			struct IntermediateResult
			{
				uint32 NumHits = 0;
				uint32 Total = 0;
				float TTLSeconds;
			};
	
			TArray<IntermediateResult> History;
	
			uint32 CurrHitCount = 0;
			uint32 CurrTotal = 0;
			uint32 RunningHitCount = 0;
			uint32 RunningTotal = 0;
	
			float CurrTTLSeconds = 0.f;
	
			uint64 PreviousTimeCycles = 0;
			bool bIsFirstUpdate = true;
	
			void FirstUpdate();
			void SetWindowLength(const float InNewLengthSeconds);
			void ExpireResult(const IntermediateResult& InResultToExpire);
			void TickResults(const float DeltaTimeSeconds);
	
		};
	} // namespace OperatorPoolPrivate
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED


	struct FOperatorPoolSettings
	{
		uint32 MaxNumOperators = 64;
	};


	// Data required to build an operator without immediately playing it
	struct FOperatorBuildData
	{
		FGeneratorInitParams InitParams;
		Frontend::FGraphRegistryKey RegistryKey;
		FGuid AssetClassID;
		int32 NumInstances;

		// If true, touches existing assets and only builds remaining number if required
		bool bTouchExisting = false; 

		FOperatorBuildData() = delete;
		UE_API FOperatorBuildData(
			  FGeneratorInitParams&& InInitParams
			, Frontend::FGraphRegistryKey InRegistryKey
			, FGuid InAssetID
			, int32 InNumInstances = 1
			, bool bInTouchExisting = false
		);

	}; // struct FOperatorPrecacheData

	// Provides additional debug context for the operator the pool is interacting with.
	struct FOperatorContext
	{
		// Path to underlying graph asset
		FTopLevelAssetPath GraphAssetPath;

		// Path to asset with given operator (presets have different paths than graph asset)
		FTopLevelAssetPath AssetPath;

		static UE_API FOperatorContext FromInitParams(const FGeneratorInitParams& InParams);
	};

	// Pool of re-useable metasound operators to be used / put back by the metasound generator
	// operators can also be pre-constructed via the UMetasoundCacheSubsystem BP api.
	class FOperatorPool : public TSharedFromThis<FOperatorPool>
	{
	public:

		UE_API FOperatorPool(FOperatorPoolSettings InSettings = { });
		UE_API ~FOperatorPool();

		UE_API FOperatorAndInputs ClaimOperator(const FOperatorPoolEntryID& InOperatorID, const FOperatorContext& InContext);

		UE_API void AddOperator(const FOperatorPoolEntryID& InOperatorID, TUniquePtr<IOperator>&& InOperator, FInputVertexInterfaceData&& InputData, TSharedPtr<FGraphRenderCost>&& InRenderCost = {});
		UE_API void AddOperator(const FOperatorPoolEntryID& InOperatorID, FOperatorAndInputs&& OperatorAndInputs);

		UE_API void BuildAndAddOperator(TUniquePtr<FOperatorBuildData> InBuildData);

		UE_API void TouchOperators(const FOperatorPoolEntryID& InOperatorID, int32 NumToTouch = 1);
		UE_API void TouchOperatorsViaAssetClassID(const FGuid& InAssetClassID, int32 NumToTouch = 1);

		UE_API bool IsStopping() const;

		UE_API void RemoveOperatorsWithID(const FOperatorPoolEntryID& InOperatorID);
		UE_API void RemoveOperatorsWithAssetClassID(const FGuid& InAssetClassID);

		UE_API int32 GetNumCachedOperatorsWithID(const FOperatorPoolEntryID& InOperatorID) const;
		UE_API int32 GetNumCachedOperatorsWithAssetClassID(const FGuid& InAssetClassID) const;

		UE_API void SetMaxNumOperators(uint32 InMaxNumOperators);
#if METASOUND_OPERATORCACHEPROFILER_ENABLED
		UE_API void UpdateHitRateTracker();
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

		UE_API void StopAsyncTasks();

		using FTaskId = int32;
		using FTaskFunction = TUniqueFunction<void(FOperatorPool::FTaskId, TWeakPtr<FOperatorPool>)>;

	private:
		FTaskId LastTaskId = 0;

		UE_API void AddAssetIdToGraphIdLookUpInternal(const FGuid& InAssetClassID, const FOperatorPoolEntryID& InOperatorID);
		UE_API void AddOperatorInternal(const FOperatorPoolEntryID& InOperatorID, FOperatorAndInputs&& OperatorAndInputs);
		UE_API bool ExecuteTaskAsync(FTaskFunction&& InFunction);
		UE_API void Trim();

		FOperatorPoolSettings Settings;
		mutable FCriticalSection CriticalSection;

#if METASOUND_OPERATORCACHEPROFILER_ENABLED
		OperatorPoolPrivate::FWindowedHitRate HitRateTracker;
		TUniquePtr<Engine::FOperatorCacheStatTracker> CacheStatTracker;
#endif // #if METASOUND_OPERATORCACHEPROFILER_ENABLED

		// Notifies active build tasks to abort as soon as possible
		// and gates additional build tasks from being added.
		std::atomic<bool> bStopping;

		TMap<FTaskId, UE::Tasks::FTask> ActiveBuildTasks;
		UE::Tasks::FPipe AsyncBuildPipe;

		TMap<FOperatorPoolEntryID, TArray<FOperatorAndInputs>> Operators;
		TMap<FGuid, FOperatorPoolEntryID> AssetIdToGraphIdLookUp;
		TMultiMap<FOperatorPoolEntryID, FGuid> GraphIdToAssetIdLookUp;
		TArray<FOperatorPoolEntryID> Stack;
	};
} // namespace Metasound




#undef UE_API
