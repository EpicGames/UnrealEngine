// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundGraphCore : ModuleRules
	{
		public MetasoundGraphCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SignalProcessing",
					"AudioExtensions"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MathCore",
					"TraceLog"
				}
			);
		}
	}
}
