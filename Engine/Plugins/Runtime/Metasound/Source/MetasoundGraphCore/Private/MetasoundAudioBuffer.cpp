// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReferenceMacro.h"

DEFINE_METASOUND_DATA_TYPE(Metasound::FAudioBuffer, "Audio");

