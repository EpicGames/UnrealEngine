// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundBasicNode.h"

#include "MetasoundNodeInterface.h"
#include "Templates/SharedPointer.h"

namespace Metasound
{
	FBasicNode::FBasicNode(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata)
	: NodeData(InNodeData)
	, ClassMetadata(InClassMetadata)
	{
	}

	/** Return the name of this specific instance of the node class. */
	const FName& FBasicNode::GetInstanceName() const
	{
		return NodeData.Name;
	}

	/** Return the ID of this specific instance of the node class. */
	const FGuid& FBasicNode::GetInstanceID() const
	{
		return NodeData.ID;
	}

	/** Return the type name of this node. */
	const FNodeClassMetadata& FBasicNode::GetMetadata() const
	{
		return *ClassMetadata;
	}

	const FVertexInterface& FBasicNode::GetVertexInterface() const
	{
		return NodeData.Interface;
	}
	
	TSharedPtr<const IOperatorData> FBasicNode::GetOperatorData() const 
	{
		return NodeData.OperatorData;
	}

	void FBasicNode::SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		if (FInputDataVertex* Vertex = NodeData.Interface.GetInputInterface().Find(InVertexName))
		{
			Vertex->SetDefaultLiteral(InLiteral);
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Could not set default input. Could not find input vertex %ls on node %ls of node class %ls", *InVertexName.ToString(), *NodeData.Name.ToString(), *ClassMetadata->ClassName.ToString());
		}
	}
}
