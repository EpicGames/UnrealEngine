// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundBuildError.h"

#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"

#define LOCTEXT_NAMESPACE "MetasoundBuildError"

namespace Metasound
{
	FBuildErrorBase::FBuildErrorBase(const FName& InErrorType, const FText& InErrorDescription)
	:	ErrorType(InErrorType)
	,	ErrorDescription(InErrorDescription)
	{
	}

	/** Returns the type of error. */
	const FName& FBuildErrorBase::GetErrorType() const
	{
		return ErrorType;
	}

	/** Returns a human readable error description. */
	const FText& FBuildErrorBase::GetErrorDescription() const
	{
		return ErrorDescription;
	}

	/** Returns an array of destinations associated with the error. */
	const TArray<FInputDataDestination>& FBuildErrorBase::GetInputDataDestinations() const
	{
		return Destinations;
	}
	
	/** Returns an array of sources associated with the error. */
	const TArray<FOutputDataSource>& FBuildErrorBase::GetOutputDataSources() const
	{
		return Sources;
	}

	/** Returns an array of Nodes associated with the error. */
	const TArray<const INode*>& FBuildErrorBase::GetNodes() const 
	{
		return Nodes;
	}

	/** Returns an array of edges associated with the error. */
	const TArray<FDataEdge>& FBuildErrorBase::GetDataEdges() const
	{
		return Edges;
	}

	void FBuildErrorBase::AddInputDataDestination(const FInputDataDestination& InDestination)
	{
		Destinations.Add(InDestination);
	}

	void FBuildErrorBase::AddInputDataDestinations(TArrayView<const FInputDataDestination> InDestinations)
	{
		Destinations.Append(InDestinations.GetData(), InDestinations.Num());
	}

	void FBuildErrorBase::AddOutputDataSource(const FOutputDataSource& InSource)
	{
		Sources.Add(InSource);
	}

	void FBuildErrorBase::AddOutputDataSources(TArrayView<const FOutputDataSource> InSources)
	{
		Sources.Append(InSources.GetData(), InSources.Num());
	}

	void FBuildErrorBase::AddDataEdge(const FDataEdge& InEdge)
	{
		Edges.Add(InEdge);
	}

	void FBuildErrorBase::AddDataEdges(TArrayView<const FDataEdge> InEdges)
	{
		Edges.Append(InEdges.GetData(), InEdges.Num());
	}

	void FBuildErrorBase::AddNode(const INode& InNode)
	{
		Nodes.Add(&InNode);
	}

	void FBuildErrorBase::AddNodes(TArrayView<INode const * const> InNodes)
	{
		Nodes.Append(InNodes.GetData(), InNodes.Num());
	}

	const FName FDanglingVertexError::ErrorType = FName(TEXT("MetasoundDanglingVertex"));

	FDanglingVertexError::FDanglingVertexError()
	:	FBuildErrorBase(ErrorType, LOCTEXT("DanglingVertexError", "Edge is not connected"))
	{}

	FDanglingVertexError::FDanglingVertexError(const FInputDataDestination& InDestination)
	:	FDanglingVertexError()
	{
		AddInputDataDestination(InDestination);
	}

	FDanglingVertexError::FDanglingVertexError(const FOutputDataSource& InSource)
	:	FDanglingVertexError()
	{
		AddOutputDataSource(InSource);
	}

	FDanglingVertexError::FDanglingVertexError(const FDataEdge& InEdge)
	:	FDanglingVertexError()
	{
		AddDataEdge(InEdge);
	}

	const FName FMissingVertexError::ErrorType = FName(TEXT("MetasoundMissingVertex"));

	FMissingVertexError::FMissingVertexError(const FInputDataDestination& InDest)
	:	FBuildErrorBase(ErrorType, LOCTEXT("MissingVertexError", "Node is missing a vertex"))
	{
		AddInputDataDestination(InDest);
	}

	FMissingVertexError::FMissingVertexError(const FOutputDataSource& InSource)
	:	FBuildErrorBase(ErrorType, LOCTEXT("MissingVertexError", "Node is missing a vertex"))
	{
		AddOutputDataSource(InSource);
	}

	const FName FDuplicateInputError::ErrorType = FName(TEXT("MetasoundDuplicateInput"));

	FDuplicateInputError::FDuplicateInputError(const TArrayView<FDataEdge> InEdges)
	:	FBuildErrorBase(ErrorType, LOCTEXT("DuplicateInputError", "Node has duplicate connections to input"))
	{
		AddDataEdges(InEdges);
	}

	const FName FGraphCycleError::ErrorType = FName(TEXT("MetasoundGraphCycleError"));

	FGraphCycleError::FGraphCycleError(TArrayView<INode const* const> InNodes, const TArray<FDataEdge>& InEdges)
	:	FBuildErrorBase(ErrorType, LOCTEXT("GraphCycleError", "Graph contains cycles."))
	{
		AddNodes(InNodes);
		AddDataEdges(InEdges);
	}

	const FName FNodePrunedError::ErrorType = FName(TEXT("MetasoundNodePrunedError"));

	FNodePrunedError::FNodePrunedError(const INode* InNode)
	:	FBuildErrorBase(ErrorType, LOCTEXT("NodePrunedError", "Unreachable node pruned from graph."))
	{
		if (ensure(nullptr != InNode))
		{
			AddNode(*InNode);
		}
	}

	const FName FInternalError::ErrorType = FName(TEXT("MetasoundInternalError"));

	FInternalError::FInternalError(const FString& InFileName, int32 InLineNumber)
	:	FBuildErrorBase(ErrorType, LOCTEXT("InternalError", "Internal error."))
	,	FileName(InFileName)
	,	LineNumber(InLineNumber)
	{
	}

	const FString& FInternalError::GetFileName() const
	{
		return FileName;
	}

	int32 FInternalError::GetLineNumber() const
	{
		return LineNumber;
	}

	const FName FMissingInputDataReferenceError::ErrorType = FName(TEXT("MetasoundMissingInputDataReferenceError"));
	
	FMissingInputDataReferenceError::FMissingInputDataReferenceError(const FInputDataDestination& InInputDataDestination)
	:	FBuildErrorBase(ErrorType, LOCTEXT("MissingInputDataReferenceError", "Missing input data reference."))
	{
		AddInputDataDestination(InInputDataDestination);
	}


	const FName FMissingOutputDataReferenceError::ErrorType = FName(TEXT("MetasoundMissingOutputDataReferenceError"));
	
	FMissingOutputDataReferenceError::FMissingOutputDataReferenceError(const FOutputDataSource& InOutputDataSource)
	:	FBuildErrorBase(ErrorType, LOCTEXT("MissingOutputDataReferenceError", "Missing output data reference."))
	{
		AddOutputDataSource(InOutputDataSource);
	}

	const FName FInvalidConnectionDataTypeError::ErrorType = FName(TEXT("MetasoundInvalidConnectionDataTypeError"));

	FInvalidConnectionDataTypeError::FInvalidConnectionDataTypeError(const FDataEdge& InEdge)
	:	FBuildErrorBase(ErrorType, LOCTEXT("InvalidConnectionDataTypeError", "Source and destination data types do not match."))
	{
		AddDataEdge(InEdge);
	}

	const FName FInputReceiverInitializationError::ErrorType = "InputReceiverInitializationError";

	FInputReceiverInitializationError::FInputReceiverInitializationError(const INode& InInputNode, const FName& InVertexKey, const FName& InDataType)
		: FBuildErrorBase(ErrorType, FText::Format(LOCTEXT("InputReceiverInitializationError", "Failed to create transmission receiever for input '{0}' of type '{1}'.")
			, FText::FromName(InVertexKey)
			, FText::FromName(InDataType)
		))
	{
		AddNode(InInputNode);
	}
}

#undef LOCTEXT_NAMESPACE
