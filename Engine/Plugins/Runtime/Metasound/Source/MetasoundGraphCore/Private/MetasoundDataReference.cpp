// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceMacro.h"

namespace Metasound
{
	const TCHAR* TDataReferenceTypeInfo<void>::TypeName = TEXT("void");
	const FText& TDataReferenceTypeInfo<void>::GetTypeDisplayText()
	{
		return FText::GetEmpty();
	}
	const void* const TDataReferenceTypeInfo<void>::TypePtr = nullptr;
	const void* const TDataReferenceTypeInfo<void>::TypeId = static_cast<const void* const>(TDataReferenceTypeInfo<void>::TypePtr);

	const FName IDataReference::RouterName = "DataReference";

	FName CreateArrayTypeNameFromElementTypeName(const FName InTypeName)
	{
		return FName(InTypeName.ToString() + TEXT(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER));
	}

	FName CreateElementTypeNameFromArrayTypeName(const FName InArrayTypeName)
	{
		auto GetSuffixLength = []() { return FString(TEXT(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER)).Len(); };
		static const int32 SuffixLength = GetSuffixLength();
		return *InArrayTypeName.ToString().LeftChop(SuffixLength);
	}
}

FString LexToString(Metasound::EDataReferenceAccessType InAccessType)
{
	using namespace Metasound;

	switch (InAccessType)
	{
		case EDataReferenceAccessType::None:
			return FString(TEXT("None"));

		case EDataReferenceAccessType::Read:
			return FString(TEXT("Read"));

		case EDataReferenceAccessType::Write:
			return FString(TEXT("Write"));

		case EDataReferenceAccessType::Value:
			return FString(TEXT("Value"));

		default:
			{
				checkNoEntry();
			}
	}
	return FString(TEXT(""));
}

