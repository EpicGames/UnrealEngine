// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundDataReferenceCollection.h"

#include "CoreMinimal.h"
#include "MetasoundDataReference.h"

namespace Metasound
{
	bool FDataReferenceCollection::ContainsDataReadReference(const FVertexName& InName, const FName& InTypeName) const
	{
		if (const FAnyDataReference* DataRef = FindDataReference(InName))
		{
			if (InTypeName == DataRef->GetDataTypeName())
			{
				const EDataReferenceAccessType AccessType = DataRef->GetAccessType();
				return (AccessType == EDataReferenceAccessType::Read) || (AccessType == EDataReferenceAccessType::Write);
			}
		}

		return false;
	}

	bool FDataReferenceCollection::ContainsDataWriteReference(const FVertexName& InName, const FName& InTypeName) const
	{
		if (const FAnyDataReference* DataRef = FindDataReference(InName))
		{
			if (InTypeName == DataRef->GetDataTypeName())
			{
				const EDataReferenceAccessType AccessType = DataRef->GetAccessType();
				return (AccessType == EDataReferenceAccessType::Write);
			}
		}

		return false;
	}

	const FAnyDataReference* FDataReferenceCollection::FindDataReference(const FVertexName& InName) const
	{
		return DataRefMap.Find(InName);
	}


	bool FDataReferenceCollection::AddDataReadReferenceFrom(const FVertexName& InName, const FDataReferenceCollection& OtherCollection, const FVertexName& OtherName, const FName& OtherTypeName)
	{
		if (const FAnyDataReference* OtherRef = OtherCollection.FindDataReference(OtherName))
		{
			if (OtherRef->GetDataTypeName() == OtherTypeName)
			{
				DataRefMap.Add(InName, *OtherRef);
				return true;
			}
		}

		return false;
	}

	bool FDataReferenceCollection::AddDataWriteReferenceFrom(const FVertexName& InName, const FDataReferenceCollection& OtherCollection, const FVertexName& OtherName, const FName& OtherTypeName)
	{
		if (const FAnyDataReference* OtherRef = OtherCollection.FindDataReference(OtherName))
		{
			if (OtherRef->GetDataTypeName() == OtherTypeName)
			{
				if (OtherRef->GetAccessType() == EDataReferenceAccessType::Write)
				{
					DataRefMap.Add(InName, *OtherRef);
					return true;
				}
			}
		}
		return false;
	}

	void FDataReferenceCollection::AddDataReference(const FVertexName& InName, FAnyDataReference&& InDataRef)
	{
		DataRefMap.Add(InName, InDataRef);
	}
}
