// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicGraphAlgo.h"

#include "Algo/IsSorted.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "MetasoundDataReference.h"
#include "MetasoundGraphAlgo.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundLog.h"
#include "MetasoundTrace.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"


namespace Metasound
{
	namespace DynamicGraph
	{
		namespace DynamicGraphAlgoPrivate
		{
			template<typename EntryType>
			struct TGetOrdinal
			{
				int32 operator()(const EntryType& InEntry) const
				{
					return InEntry.Ordinal;
				}
			};

			template<typename EntryType>
			void SortExecutionTable(TArrayView<EntryType> InTable)
			{
				Algo::SortBy(InTable, TGetOrdinal<EntryType>());
			}

			template<typename EntryType>
			void SetOrdinalsAndSortTable(const TMap<FOperatorID, int32>& InOrdinals, TArray<EntryType>& InOutTable)
			{
				// Assign new ordinals to all entries
				for (EntryType& Entry : InOutTable)
				{
					if (const int32* Ordinal = InOrdinals.Find(Entry.OperatorID))
					{
						Entry.Ordinal = *Ordinal;
					}
					else
					{
						Entry.Ordinal = ORDINAL_NONE;
					}
				}

				// Sort table by ordinal
				SortExecutionTable<EntryType>(InOutTable);

				// Remove entries without an ordinal
				InOutTable.RemoveAll([](const EntryType& InEntry) { return InEntry.Ordinal == ORDINAL_NONE; });
			}

			template<typename EntryType>
			void SwapOrdinalsAndSortTable(int32 InMinOrdinal, int32 InMaxOrdinal, const TArray<FOrdinalSwap>& InSwaps, TArray<EntryType>& InOutTable)
			{
				int32 StartTableIndex = Algo::LowerBoundBy(InOutTable, InMinOrdinal, TGetOrdinal<EntryType>());
				int32 EndTableIndex = Algo::UpperBoundBy(InOutTable, InMaxOrdinal, TGetOrdinal<EntryType>());

				if ((StartTableIndex >= InOutTable.Num()) || (EndTableIndex <= StartTableIndex))
				{
					// No entries exist in the table which match the swaps. There
					// is nothing to update.
					return;
				}

				const FOrdinalSwap* SwapPtr = InSwaps.GetData();
				const FOrdinalSwap* const EndSwapPtr = SwapPtr + InSwaps.Num();
				EntryType* EntryPtr = InOutTable.GetData() + StartTableIndex;
				EntryType* const EndEntryPtr = InOutTable.GetData() + EndTableIndex;

				// Iterate through swaps and entries until we've worked through all the
				// swaps or all the table entries in the range.
				while ((SwapPtr != EndSwapPtr) && (EntryPtr != EndEntryPtr))
				{
					if (SwapPtr->OriginalOrdinal == EntryPtr->Ordinal)
					{
						// Found a match. Update ordinal and increment both pointers. 
						EntryPtr->Ordinal = SwapPtr->NewOrdinal;
						EntryPtr++;
						SwapPtr++;
					}
					else if (SwapPtr->OriginalOrdinal < EntryPtr->Ordinal)
					{
						SwapPtr++;
					}
					else // if (EntryPtr->Ordinal < SwapPtr->OriginalOrdinal) <-- assumed because of earlier logic checks.
					{
						EntryPtr++;
					}
				}

				// Sort the entries by ordinal. Only update table entries in the given range. 
				SortExecutionTable(TArrayView<const EntryType>(InOutTable.GetData() + StartTableIndex, EndTableIndex - StartTableIndex));
			}

			template<typename EntryType, typename FunctionType>
			void UpdateTableEntry(const FOperatorID& InOperatorID, FOperatorInfo& InOperatorInfo, TArray<EntryType>& InOutTable, FunctionType InFunction)
			{
				using namespace DirectedGraphAlgo;

				int32 OperatorOrdinal = InOperatorInfo.Ordinal;
				int32 EntryIndex = Algo::BinarySearchBy(InOutTable, OperatorOrdinal, TGetOrdinal<EntryType>());

				bool bEntryShouldExist = (nullptr != InFunction); // Remove operators if they have no function for this table
				bool bEntryCurrentlyExists = (EntryIndex != INDEX_NONE);

				if (bEntryCurrentlyExists && bEntryShouldExist)
				{
					// Update the existing entry
					InOutTable[EntryIndex].Function = InFunction;
					InOutTable[EntryIndex].Operator = InOperatorInfo.Operator.Get();
				}
				else if (bEntryCurrentlyExists)
				{
					// Remove the existing entry
					InOutTable.RemoveAt(EntryIndex);
				}
				else if (bEntryShouldExist)
				{
					check(InOperatorInfo.Operator.IsValid());

					// Add missing entry
					int32 InsertLocation = Algo::UpperBoundBy(InOutTable, OperatorOrdinal, TGetOrdinal<EntryType>());
					InOutTable.Insert(EntryType{OperatorOrdinal, InOperatorID, *InOperatorInfo.Operator, InFunction}, InsertLocation);
				}
			}

			void UpdateGraphRuntimeTableEntries(const FOperatorID& InOperatorID, FOperatorInfo& InOperatorInfo, FDynamicGraphOperatorData& InOutGraphOperatorData)
			{
				using namespace DynamicGraphAlgoPrivate;
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::UpdateOperatorRuntimeTableEntries)
				IOperator* Operator = InOperatorInfo.Operator.Get();

				UpdateTableEntry(InOperatorID, InOperatorInfo, InOutGraphOperatorData.ExecuteTable, Operator ? Operator->GetExecuteFunction() : nullptr);
				UpdateTableEntry(InOperatorID, InOperatorInfo, InOutGraphOperatorData.PostExecuteTable, Operator ? Operator->GetPostExecuteFunction() : nullptr);
				UpdateTableEntry(InOperatorID, InOperatorInfo, InOutGraphOperatorData.ResetTable, Operator ? Operator->GetResetFunction() : nullptr);
			}

			void SetOutputVertexData(FDynamicGraphOperatorData& InOutGraphOperatorData)
			{
				using namespace DirectedGraphAlgo;

				// Iterate through the output operators and force their output data references
				// to be reflected in the graph's FOutputVertexInterfaceData
				for (const TPair<FVertexName, FOperatorID>& OutputVertexInfo : InOutGraphOperatorData.OutputVertexMap)
				{
					const FVertexName& VertexName = OutputVertexInfo.Get<0>();
					const FOperatorID& OperatorID = OutputVertexInfo.Get<1>();

					if (const FGraphOperatorData::FOperatorInfo* OperatorInfo = InOutGraphOperatorData.OperatorMap.Find(OperatorID))
					{
						if (const FAnyDataReference* Ref = OperatorInfo->VertexData.GetOutputs().FindDataReference(OutputVertexInfo.Get<0>()))
						{
							InOutGraphOperatorData.VertexData.GetOutputs().SetVertex(VertexName, *Ref);
						}
						else if (InOutGraphOperatorData.VertexData.GetOutputs().IsVertexBound(VertexName))
						{
							UE_LOGF(LogMetaSound, Error, "Output vertex (%ls) lost data reference after rebinding graph", *VertexName.ToString());
						}
					}
					else
					{
						UE_LOGF(LogMetaSound, Error, "Failed to update graph operator outputs. Could not find output operator info with ID %ls for vertex %ls", *LexToString(OperatorID), *VertexName.ToString());
					}
				}
			}
		} // namespace DynamicGraphAlgoPrivate

		void SetOrdinalsAndSort(const TMap<FOperatorID, int32>& InOrdinals, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DynamicGraphAlgoPrivate;

			// Assign ordinals to operator map
			for (TPair<FOperatorID, FOperatorInfo>& Pair : InOutGraphOperatorData.OperatorMap)
			{
				if (const int32* Ordinal = InOrdinals.Find(Pair.Key))
				{
					Pair.Value.Ordinal = *Ordinal;
				}
				else
				{
					Pair.Value.Ordinal = ORDINAL_NONE;
				}
			}

			// set ordinals to on execution tables. 
			SetOrdinalsAndSortTable(InOrdinals, InOutGraphOperatorData.ExecuteTable);
			SetOrdinalsAndSortTable(InOrdinals, InOutGraphOperatorData.PostExecuteTable);
			SetOrdinalsAndSortTable(InOrdinals, InOutGraphOperatorData.ResetTable);
		}

		void SwapOrdinalsAndSort(const TArray<FOrdinalSwap>& InSwaps, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DynamicGraphAlgoPrivate;

			checkf(Algo::IsSorted(InSwaps, FOrdinalSwap::OriginalOrdinalLessThan), TEXT("Dynamic MetaSound ordinal swaps must be presorted by the original ordinal."));

			if (InSwaps.Num() < 1)
			{
				return;
			}

			const int32 MinOrdinal = InSwaps[0].OriginalOrdinal;
			const int32 MaxOrdinal = InSwaps.Last().OriginalOrdinal;
			
			// Update operator map
			for (const FOrdinalSwap& Swap : InSwaps)
			{
				if (FOperatorInfo* OpInfo = InOutGraphOperatorData.OperatorMap.Find(Swap.OperatorID))
				{
					OpInfo->Ordinal = Swap.NewOrdinal;
				}
			}

			// Update execution tables
			SwapOrdinalsAndSortTable(MinOrdinal, MaxOrdinal, InSwaps, InOutGraphOperatorData.ExecuteTable);
			SwapOrdinalsAndSortTable(MinOrdinal, MaxOrdinal, InSwaps, InOutGraphOperatorData.PostExecuteTable);
			SwapOrdinalsAndSortTable(MinOrdinal, MaxOrdinal, InSwaps, InOutGraphOperatorData.ResetTable);
		}

		// Apply updates to data references through all the operators by following connections described in the FOperatorInfo map.
		void PropagateBindUpdate(FOperatorID InInitialOperatorID, const FVertexName& InVertexName, const FAnyDataReference& InNewReference, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;
			using namespace DynamicGraphAlgoPrivate;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::PropagateBindUpdate)

			struct FInputToUpdate
			{
				FOperatorID OperatorID;
				FVertexName VertexName;
				FAnyDataReference DataReference;
			};

			TArray<FInputToUpdate> PropagateStack;
			PropagateStack.Emplace(FInputToUpdate{InInitialOperatorID, InVertexName, InNewReference});

			TArray<FVertexDataState> InitialOutputState;
			TSortedVertexNameMap<FAnyDataReference> OutputUpdates;
			while (PropagateStack.Num())
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::PropagateBindUpdate_Iteration)
				FInputToUpdate Current = PropagateStack.Pop();
				if (FOperatorInfo* OpInfo = InOutGraphOperatorData.OperatorMap.Find(Current.OperatorID))
				{
					IOperator& Operator = *(OpInfo->Operator);
					// Get current outputs
					InitialOutputState.Reset();
					GetVertexInterfaceDataState(OpInfo->VertexData.GetOutputs(), InitialOutputState);

					// Set new input.
					OpInfo->VertexData.GetInputs().SetVertex(Current.VertexName, MoveTemp(Current.DataReference));

					// Bind inputs and outputs
					Operator.BindInputs(OpInfo->VertexData.GetInputs());
					Operator.BindOutputs(OpInfo->VertexData.GetOutputs());

					// Update execute/postexecute/reset tables in case those have changed after rebinding.
					UpdateGraphRuntimeTableEntries(Current.OperatorID, *OpInfo, InOutGraphOperatorData);

					// See if binding altered the outputs. 
					OutputUpdates.Reset();
					CompareVertexInterfaceDataToPriorState(OpInfo->VertexData.GetOutputs(), InitialOutputState, OutputUpdates);

					// Any updates to the outputs need to be propagated through the graph.
					for (const TPair<FVertexName, FAnyDataReference>& OutputUpdate : OutputUpdates)
					{
						const FVertexName& OutputVertexName = OutputUpdate.Get<0>();
						const FAnyDataReference& OutputDataReference = OutputUpdate.Get<1>();

						if (const TArray<FGraphOperatorData::FVertexDestination>* Destinations = OpInfo->OutputConnections.Find(OutputVertexName))
						{
							for (const FGraphOperatorData::FVertexDestination& Destination : *Destinations)
							{
								PropagateStack.Push({Destination.OperatorID, Destination.VertexName, OutputDataReference});
							}
						}
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Failed to rebind graph operator state. Could not find operator info with ID %ls", *LexToString(Current.OperatorID));
				}
			}
		}

		FExecuteEntry::FExecuteEntry(int32 InOrdinal, DirectedGraphAlgo::FOperatorID InOperatorID, IOperator& InOperator, IOperator::FExecuteFunction InFunc)
		: Ordinal(InOrdinal)
		, OperatorID(InOperatorID)
		, Operator(&InOperator)
		, Function(InFunc)
		{
		}

		FPostExecuteEntry::FPostExecuteEntry(int32 InOrdinal, DirectedGraphAlgo::FOperatorID InOperatorID, IOperator& InOperator, IOperator::FPostExecuteFunction InFunc)
		: Ordinal(InOrdinal)
		, OperatorID(InOperatorID)
		, Operator(&InOperator)
		, Function(InFunc)
		{
		}

		FResetEntry::FResetEntry(int32 InOrdinal, DirectedGraphAlgo::FOperatorID InOperatorID, IOperator& InOperator, IOperator::FResetFunction InFunc)
		: Ordinal(InOrdinal)
		, OperatorID(InOperatorID)
		, Operator(&InOperator)
		, Function(InFunc)
		{
		}

		FDynamicGraphOperatorData::FDynamicGraphOperatorData(const FOperatorSettings& InSettings)
		: DirectedGraphAlgo::FGraphOperatorData(InSettings)
		{
		}


		FDynamicGraphOperatorData::FDynamicGraphOperatorData(const FOperatorSettings& InSettings, const FDynamicOperatorUpdateCallbacks& InCallbacks)
		: DirectedGraphAlgo::FGraphOperatorData(InSettings)
		, OperatorUpdateCallbacks(InCallbacks)
		{
		}

		void FDynamicGraphOperatorData::InitTables()
		{
			// Populate execute/postexecute/reset stacks
			for (TPair<FOperatorID, FOperatorInfo>& Pair : OperatorMap)
			{
				const FOperatorID OperatorID = Pair.Key;
				FOperatorInfo& OperatorInfo = Pair.Value;

				check(OperatorInfo.Operator.IsValid());
				IOperator& Operator = *OperatorInfo.Operator;

				if (IOperator::FExecuteFunction ExecuteFunc = Operator.GetExecuteFunction())
				{
					ExecuteTable.Emplace(FExecuteEntry{OperatorInfo.Ordinal, OperatorID, Operator, ExecuteFunc});
				}

				if (IOperator::FPostExecuteFunction PostExecuteFunc = Operator.GetPostExecuteFunction())
				{
					PostExecuteTable.Emplace(FPostExecuteEntry{OperatorInfo.Ordinal, OperatorID, Operator, PostExecuteFunc});
				}

				if (IOperator::FResetFunction ResetFunc = Operator.GetResetFunction())
				{
					ResetTable.Emplace(FResetEntry{OperatorInfo.Ordinal, OperatorID, Operator, ResetFunc});
				}
			}

			// Sort execution stacks
			DynamicGraphAlgoPrivate::SortExecutionTable<FExecuteEntry>(ExecuteTable);
			DynamicGraphAlgoPrivate::SortExecutionTable<FPostExecuteEntry>(PostExecuteTable);
			DynamicGraphAlgoPrivate::SortExecutionTable<FResetEntry>(ResetTable);
		}

		void UpdateOutputVertexData(FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;
			using namespace DynamicGraphAlgoPrivate;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::UpdateOutputVertexData)

			// If we have operator updates to call out to, we need to check for
			// any changes to the output vertex data.
			if (InOutGraphOperatorData.OperatorUpdateCallbacks.OnOutputUpdated)
			{

				// Cache current graph output vertex data state. 	
				TArray<FVertexDataState> OriginalOutputVertexState;
				GetVertexInterfaceDataState(InOutGraphOperatorData.VertexData.GetOutputs(), OriginalOutputVertexState);

				// Force updates
				SetOutputVertexData(InOutGraphOperatorData);

				// Check for any changes to the output vertex data state. 
				TSortedVertexNameMap<FAnyDataReference> OutputVertexUpdates;
				CompareVertexInterfaceDataToPriorState(InOutGraphOperatorData.VertexData.GetOutputs(), OriginalOutputVertexState, OutputVertexUpdates);

				// Report any updates. 
				for (const TPair<FVertexName, FAnyDataReference>& Updates : OutputVertexUpdates)
				{
					InOutGraphOperatorData.OperatorUpdateCallbacks.OnOutputUpdated(Updates.Key, InOutGraphOperatorData.VertexData.GetOutputs());
				}
				
			}
			else
			{
				// Force updates on output. No need to update outside callers if
				// there is no callback set. 
				SetOutputVertexData(InOutGraphOperatorData);
			}
		}

		void RebindWrappedOperator(const FOperatorID& InOperatorID, FOperatorInfo& InOperatorInfo, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;
			using namespace DynamicGraphAlgoPrivate;

			check(InOperatorInfo.Operator);
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::RebindWrappedOperator)

			/* Bind and diff the graph's interface to determine if there is an update to any vertices */

			// Cache current state of operators output data. 
			FOutputVertexInterfaceData& OutputVertexData = InOperatorInfo.VertexData.GetOutputs();
			TArray<FVertexDataState> InitialVertexDataState;
			GetVertexInterfaceDataState(OutputVertexData, InitialVertexDataState);

			// Bind the operator to trigger any updates. 
			InOperatorInfo.Operator->BindInputs(InOperatorInfo.VertexData.GetInputs());
			InOperatorInfo.Operator->BindOutputs(InOperatorInfo.VertexData.GetOutputs());

			// Update any execution tables that need updating after wrapping
			UpdateGraphRuntimeTableEntries(InOperatorID, InOperatorInfo, InOutGraphOperatorData);

			// Determine if there have been changes to `OutputVertexData`. 
			TSortedVertexNameMap<FAnyDataReference> OutputsToUpdate;
			CompareVertexInterfaceDataToPriorState(OutputVertexData, InitialVertexDataState, OutputsToUpdate);

			// If there have been any changes to `OutputVertexData`, then these need to be propagated
			// through the graph to route them to operator inputs and to handle any knock-on updates
			// to other data references. 

			// Update the graph data by propagating the updates through the inputs nodes. 
			for (const TPair<FVertexName, FAnyDataReference>& OutputToUpdate : OutputsToUpdate)
			{
				const FVertexName& VertexName = OutputToUpdate.Get<0>();
				if (const TArray<FGraphOperatorData::FVertexDestination>* Destinations  = InOperatorInfo.OutputConnections.Find(VertexName))
				{
					for (const FGraphOperatorData::FVertexDestination& Destination : *Destinations)
					{
						PropagateBindUpdate(Destination.OperatorID, Destination.VertexName, OutputToUpdate.Get<1>(), InOutGraphOperatorData);
					}
				}
			}

			// Refresh output vertex interface data in case any graph output 
			// nodes were updated when bind updates were propagated through 
			// the graph.
			UpdateOutputVertexData(InOutGraphOperatorData);
		}
		
		void RebindGraphInputs(FInputVertexInterfaceData& InOutVertexData, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::RebindGraphInputs)

			// Bind and diff the graph's interface to determine if there is an update to any vertices
			FInputVertexInterfaceData& InputVertexData = InOutGraphOperatorData.VertexData.GetInputs();
			TArray<FVertexDataState> InitialVertexDataState;
			GetVertexInterfaceDataState(InputVertexData, InitialVertexDataState);

			// Binding an input vertex interface may update `InputVertexData`
			InOutVertexData.Bind(InputVertexData);

			// Determine if there have been changes to `InputVertexData`. 
			TSortedVertexNameMap<FAnyDataReference> GraphInputsToUpdate;
			CompareVertexInterfaceDataToPriorState(InputVertexData, InitialVertexDataState, GraphInputsToUpdate);

			// If there have been any changes to `InputVertexData`, then these need to be propagated
			// through the graph to route them to operator inputs and to handle any knock-on updates
			// to other data references. 
			if (GraphInputsToUpdate.Num() > 0)
			{
				// Update the graph data by propagating the updates through the inputs nodes. 
				for (const TPair<FVertexName, FAnyDataReference>& InputToUpdate : GraphInputsToUpdate)
				{
					const FVertexName& VertexName = InputToUpdate.Get<0>();
					if (const FOperatorID* OperatorID = InOutGraphOperatorData.InputVertexMap.Find(VertexName))
					{
						PropagateBindUpdate(*OperatorID, VertexName, InputToUpdate.Get<1>(), InOutGraphOperatorData);
					}
					else
					{
						UE_LOGF(LogMetaSound, Error, "No input operator exists for input vertex %ls", *VertexName.ToString());
					}
				}

				// Refresh output vertex interface data in case any output nodes were updated
				// when bind updates were propagated through the graph.
				UpdateOutputVertexData(InOutGraphOperatorData);
			}
		}

		void RebindGraphOutputs(FOutputVertexInterfaceData& InOutVertexData, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicGraphAlgo::RebindGraphOutputs)
			// Output rebinding does not alter data references in an operator. Here we can get away with
			// simply reading the latest values.
			InOutVertexData.Bind(InOutGraphOperatorData.VertexData.GetOutputs());
		}

		void InsertOperator(FOperatorID InOperatorID, FOperatorInfo InOperatorInfo, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DynamicGraphAlgoPrivate;

			IOperator* Operator = InOperatorInfo.Operator.Get();
			if (nullptr == Operator)
			{
				return;
			}

			if (FOperatorInfo* ExistingInfo = InOutGraphOperatorData.OperatorMap.Find(InOperatorID))
			{
				// The options here are not good. The prior operator will be 
				// removed and replaced with this new operator.
				// Another option would be to leave the existing operator unchanged. 
				// Neither option is satisfactory.
				UE_LOG(LogMetaSound, Warning, TEXT("Overriding existing operator with the same operator ID %" UPTRINT_FMT ". Duplicate operator IDs will lead to undefined behavior. Remove existing operators before adding a new one with the same ID"), InOperatorID);

				TArray<FOperatorID> ConnectionsToRemove;
				RemoveOperator(InOperatorID, ConnectionsToRemove, InOutGraphOperatorData);
			}

			// insert operator to execution tables
			if (InOperatorInfo.Ordinal != ORDINAL_NONE)
			{
				if (IOperator::FExecuteFunction ExecuteFunc = Operator->GetExecuteFunction())
				{
					int32 InsertLocation = Algo::UpperBoundBy(InOutGraphOperatorData.ExecuteTable, InOperatorInfo.Ordinal, TGetOrdinal<FExecuteEntry>());
					InOutGraphOperatorData.ExecuteTable.Insert(FExecuteEntry(InOperatorInfo.Ordinal, InOperatorID, *Operator, ExecuteFunc), InsertLocation);
				}

				if (IOperator::FPostExecuteFunction PostExecuteFunc = Operator->GetPostExecuteFunction())
				{
					int32 InsertLocation = Algo::UpperBoundBy(InOutGraphOperatorData.PostExecuteTable, InOperatorInfo.Ordinal, TGetOrdinal<FPostExecuteEntry>());
					InOutGraphOperatorData.PostExecuteTable.Insert(FPostExecuteEntry(InOperatorInfo.Ordinal, InOperatorID, *Operator, PostExecuteFunc), InsertLocation);
				}

				if (IOperator::FResetFunction ResetFunc = Operator->GetResetFunction())
				{
					int32 InsertLocation = Algo::UpperBoundBy(InOutGraphOperatorData.ResetTable, InOperatorInfo.Ordinal, TGetOrdinal<FResetEntry>());
					InOutGraphOperatorData.ResetTable.Insert(FResetEntry(InOperatorInfo.Ordinal, InOperatorID, *Operator, ResetFunc), InsertLocation);
				}
			}

			// Move operator info to local map.
			InOutGraphOperatorData.OperatorMap.Add(InOperatorID, MoveTemp(InOperatorInfo));
		}

		void RemoveOperator(FOperatorID InOperatorID, const TArray<FOperatorID>& InOperatorsConnectedToInput, FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;

			// Remove any other nodes connected to this node. 
			for (const FOperatorID& ConnectedOperatorID : InOperatorsConnectedToInput)
			{
				if (FOperatorInfo* ConnectedOperatorInfo = InOutGraphOperatorData.OperatorMap.Find(ConnectedOperatorID))
				{
					for (TPair<FVertexName, TArray<FGraphOperatorData::FVertexDestination>>& VertexDestinations : ConnectedOperatorInfo->OutputConnections)
					{
						VertexDestinations.Value.RemoveAllSwap([&InOperatorID](const FGraphOperatorData::FVertexDestination& Destination) { return Destination.OperatorID == InOperatorID; });
					}
				}
			}

			// remove from map of operators
			InOutGraphOperatorData.OperatorMap.Remove(InOperatorID);

			// Update execution tables
			InOutGraphOperatorData.ExecuteTable.RemoveAll([&InOperatorID](const FExecuteEntry& InEntry) { return InEntry.OperatorID == InOperatorID; });
			InOutGraphOperatorData.PostExecuteTable.RemoveAll([&InOperatorID](const FPostExecuteEntry& InEntry) { return InEntry.OperatorID == InOperatorID; });
			InOutGraphOperatorData.ResetTable.RemoveAll([&InOperatorID](const FResetEntry& InEntry) { return InEntry.OperatorID == InOperatorID; });
		}

		namespace Debug
		{
			template<typename EntryType>
			void EnsureIfDynamicGraphOperatorDataTableIsCorrupt(const FDynamicGraphOperatorData& InData, const TArray<EntryType>& InTable)
			{
				TSet<FOperatorID> OperatorIDs;
				TSet<const IOperator*> Operators;
				TSet<int32> Ordinals;

				for (const EntryType& Entry : InTable)
				{
					bool bIsAlreadyInSet = false;

					Ordinals.Add(Entry.Ordinal, &bIsAlreadyInSet);
					ensure(!bIsAlreadyInSet);
					OperatorIDs.Add(Entry.OperatorID, &bIsAlreadyInSet);
					ensure(!bIsAlreadyInSet);
					Operators.Add(Entry.Operator, &bIsAlreadyInSet);
					ensure(!bIsAlreadyInSet);
					ensure(nullptr != Entry.Function);

					ensure(InData.OperatorMap.Contains(Entry.OperatorID));
					const FOperatorInfo& Info = InData.OperatorMap[Entry.OperatorID];
					ensure(Info.Ordinal == Entry.Ordinal);
					ensure(Info.Operator.Get() == Entry.Operator);
				}
			}

			void EnsureIfDynamicGraphOperatorDataIsCorrupt(const FDynamicGraphOperatorData& InData)
			{
				EnsureIfDynamicGraphOperatorDataTableIsCorrupt<FExecuteEntry>(InData, InData.ExecuteTable);
				EnsureIfDynamicGraphOperatorDataTableIsCorrupt<FPostExecuteEntry>(InData, InData.PostExecuteTable);
				EnsureIfDynamicGraphOperatorDataTableIsCorrupt<FResetEntry>(InData, InData.ResetTable);
			}
		}
	}
}
