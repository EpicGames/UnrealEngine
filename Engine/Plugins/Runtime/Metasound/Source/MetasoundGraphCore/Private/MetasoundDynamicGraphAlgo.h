// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/NumericLimits.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"

namespace Metasound
{
	namespace DynamicGraph
	{
		struct FDynamicGraphOperatorData;

		using FOperatorInfo = DirectedGraphAlgo::FGraphOperatorData::FOperatorInfo;
		using FOperatorID = DirectedGraphAlgo::FOperatorID;

		/* Convenience wrapper for execute function of an IOperator. */
		struct FExecuteEntry final
		{
			FExecuteEntry(int32 InOrdinal, FOperatorID InOperatorID, IOperator& InOperator, IOperator::FExecuteFunction InFunc);

			void Execute()
			{
				check(Operator);
				check(Function);
				Function(Operator);
			}

			int32 Ordinal;
			FOperatorID OperatorID;
			IOperator* Operator;
			IOperator::FExecuteFunction Function;	
		};

		/* Convenience wrapper for post execute function of an IOperator. */
		struct FPostExecuteEntry final
		{
			FPostExecuteEntry(int32 InOrdinal, FOperatorID InOperatorID, IOperator& InOperator, IOperator::FPostExecuteFunction InFunc);

			void PostExecute()
			{
				check(Operator);
				check(Function);
				Function(Operator);
			}

			int32 Ordinal;
			FOperatorID OperatorID;
			IOperator* Operator;
			IOperator::FPostExecuteFunction Function;	
		};

		/* Convenience wrapper for reset function of an IOperator. */
		struct FResetEntry final
		{
			FResetEntry(int32 InOrdinal, FOperatorID InOperatorID, IOperator& InOperator, IOperator::FResetFunction InFunc);

			void Reset(const IOperator::FResetParams& InParams)
			{
				check(Operator);
				check(Function);
				Function(Operator, InParams);
			}

			int32 Ordinal;
			FOperatorID OperatorID;
			IOperator* Operator;
			IOperator::FResetFunction Function;	
		};


		/** Collection of data needed to support a dynamic operator*/
		struct FDynamicGraphOperatorData : DirectedGraphAlgo::FGraphOperatorData
		{
			FDynamicGraphOperatorData(const FOperatorSettings& InSettings);
			FDynamicGraphOperatorData(const FOperatorSettings& InSettings, const FDynamicOperatorUpdateCallbacks& InCallbacks);

			// Initialize the Execute/PostExecute/Reset tables.
			void InitTables();

			// A collection of optional callbacks which can be invoked when various
			// updates are made to this collection of data.
			FDynamicOperatorUpdateCallbacks OperatorUpdateCallbacks;

			TArray<FExecuteEntry> ExecuteTable;
			TArray<FPostExecuteEntry> PostExecuteTable;
			TArray<FResetEntry> ResetTable;
		};

		/** Interface to allow FOperatorBuilder special access to internal FDynamicGraphOperatorData
		 * structures when the operator is being built.
		 */
		class IDynamicGraphInPlaceBuildable 
		{
		public:
			virtual ~IDynamicGraphInPlaceBuildable() = default;
		private:
			friend class ::Metasound::FOperatorBuilder;

			/** This function gives FOperatorBuilder access to the graph's internal data. 
			 * This allows the FOperatorBuilder to build the operator in place by modifying
			 * the internal data structure of the operator. An alternative approach 
			 * would be to pass the data to the operator as a constructor argument. 
			 * "In Place" building was chosen because it simplifies and streamlines the
			 * build process within the FOperatorBuilder.
			 */
			virtual FDynamicGraphOperatorData& GetDynamicGraphOperatorData() = 0;
		};

		/** Sets the ordinals of operators and sorts executions tables. 
		 * @param InOrdinals - A map of FOperatorID to ordinal.
		 * @param InOutGraphOperatorData - The graph data where the updated ordinals should be applied. 
		 */
		void SetOrdinalsAndSort(const TMap<FOperatorID, int32>& InOrdinals, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Applies the ordinal swaps to operators and sorts execution tables.
		 * @param InSwaps - An array of ordinal swaps.
		 * @param InOutGraphOperatorData - The graph data where the updated ordinals should be applied. 
		 */
		void SwapOrdinalsAndSort(const TArray<FOrdinalSwap>& InSwaps, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Propagate FVertexInterfaceData updates through the operators in the Dynamic Graph Operator Data. 
		 *
		 * A change to an operator's input may result in a change to the operator's output. The updates to the
		 * operator's output and any subsequent knock-on operator output updates need to be propagated through
		 * all the relevant operators in the graph.
		 *
		 * @param InInitialOperatorID - ID of operator which will have it's input updated.
		 * @param InVertexName - Vertex name on the initial operator which will have it's input updated.
		 * @param InNewReference - New data reference to apply to the operators input vertex.
		 * @param InOutGraphOperatorData - Graph data which will be updated with new references. 
		 */
		void PropagateBindUpdate(FOperatorID InInitialOperatorID, const FVertexName& InVertexName, const FAnyDataReference& InNewReference, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Iterate through the output operators and make force that their output data references
		 * are reflected in the graph's FOutputVertexInterfaceData 
		 */
		void UpdateOutputVertexData(FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Rebinds an operator which is wrapping another operator. */
		void RebindWrappedOperator(const FOperatorID& InOperatorID, FOperatorInfo& InOperatorInfo, FDynamicGraphOperatorData& InGraphOperatorData);

		/** Rebind the graph inputs, updating internal operator bindings as needed. */
		void RebindGraphInputs(FInputVertexInterfaceData& InOutVertexData, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Rebind the graph inputs, updating internal operator bindings as needed. */
		void RebindGraphOutputs(FOutputVertexInterfaceData& InOutVertexData, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Insert an operator into the graph data and add to execution tables.
		 *
		 * @param InOperatorID - ID of new operator.
		 * @param InOperatorInfo - Info containing describing operator.
		 * @param InOutGraphOperatorData - Graph data where the operator will be inserted. 
		 */
		void InsertOperator(FOperatorID InOperatorID, FOperatorInfo InOperatorInfo, FDynamicGraphOperatorData& InOutGraphOperatorData);

		/** Remove operator and related connections from the graph data.
		 *
		 * @param InOperatorID - ID of operator to remove.
		 * @param InOperatorsConnectedToInput - An array of operator IDs denoting which operators connect to the input of operator being removed. 
		 * @param InOutGraphOperatorData - Graph data where the operator will be removed from.
		 */
		void RemoveOperator(FOperatorID InOperatorID, const TArray<FOperatorID>& InOperatorsConnectedToInput, FDynamicGraphOperatorData& InOutGraphOperatorData);

		namespace Debug
		{
			void EnsureIfDynamicGraphOperatorDataIsCorrupt(const FDynamicGraphOperatorData& InData);
		}
	}
}

