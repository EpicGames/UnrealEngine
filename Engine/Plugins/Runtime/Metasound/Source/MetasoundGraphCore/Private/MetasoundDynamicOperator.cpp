// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicOperator.h"

#include "Containers/Array.h"
#include "HAL/IConsoleManager.h"
#include "MetasoundDynamicGraphAlgo.h"
#include "MetasoundDynamicOperatorAudioFade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "MetasoundTrace.h"
#include "Misc/ReverseIterate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	namespace DynamicGraph
	{
		const TCHAR* LexToString(EDynamicOperatorTransformQueueAction InTransformResult)
		{
			switch (InTransformResult)
			{
				case EDynamicOperatorTransformQueueAction::Continue:
					return TEXT("Continue");

				case EDynamicOperatorTransformQueueAction::Fence:
					return TEXT("Fence");

				default:
					{
						checkNoEntry();
						return TEXT("Unhandled");
					}
			}
		}

		namespace DynamicOperatorPrivate
		{
			float MetaSoundExperimentalTransformTimeoutInSeconds = -1.0f;
			FAutoConsoleVariableRef CVarMetaSoundExperimentalTransformTimeoutInSeconds(
				TEXT("au.MetaSound.Experimental.DynamicOperatorTransformTimeoutInSeconds"),
				MetaSoundExperimentalTransformTimeoutInSeconds,
				TEXT("Sets the number of seconds allowed to process pending dynamic graph transformations for a single MetaSound render cycle .\n")
				TEXT("[Less than zero]: Disabled, [Greater than zero]: Enabled, (disabled by default)"),
				ECVF_Default);
			

			template<typename VertexInterfaceDataType>
			class TScopeUnfreeze final
			{
			public:
				explicit TScopeUnfreeze(VertexInterfaceDataType& InVertexData)
				: bIsOriginallyFrozen(InVertexData.IsVertexInterfaceFrozen())
				, VertexData(InVertexData)
				{
					VertexData.SetIsVertexInterfaceFrozen(false);
				}

				~TScopeUnfreeze()
				{
					VertexData.SetIsVertexInterfaceFrozen(bIsOriginallyFrozen);
				}

			private:
				bool bIsOriginallyFrozen;
				VertexInterfaceDataType& VertexData;
			};


		} // namespace DynamicOperatorPrivate

		FDynamicOperator::FDynamicOperator(const FOperatorSettings& InSettings)
		: DynamicOperatorData(InSettings)
		{
			// Ensure that a transform queue exists. 
			if (!TransformQueue.IsValid())
			{
				TransformQueue = MakeShared<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>>();
			}
		}

		FDynamicOperator::FDynamicOperator(const FOperatorSettings& InSettings, TSharedPtr<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> InTransformQueue, const FDynamicOperatorUpdateCallbacks& InOperatorUpdateCallbacks)
		: DynamicOperatorData(InSettings, InOperatorUpdateCallbacks)
		, TransformQueue(InTransformQueue)
		{
			// Ensure that a transform queue exists. 
			if (!TransformQueue.IsValid())
			{
				TransformQueue = MakeShared<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>>();
			}
		}

		void FDynamicOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
		{
			RebindGraphInputs(InOutVertexData, DynamicOperatorData);
		}

		void FDynamicOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
		{
			RebindGraphOutputs(InOutVertexData, DynamicOperatorData);
		}

		IOperator::FResetFunction FDynamicOperator::GetResetFunction()
		{
			return &StaticReset;
		}

		IOperator::FExecuteFunction FDynamicOperator::GetExecuteFunction()
		{
			return &StaticExecute;
		}

		IOperator::FPostExecuteFunction FDynamicOperator::GetPostExecuteFunction()
		{
			return &StaticPostExecute;
		}

		void FDynamicOperator::FlushEnqueuedTransforms()
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperator::FlushEnqueuedTransforms)

			while (TOptional<TUniquePtr<IDynamicOperatorTransform>> Transform = TransformQueue->Dequeue())
			{
				if (Transform.IsSet())
				{
					if (Transform->IsValid())
					{
						(*Transform)->Transform(DynamicOperatorData);
					}
				}
			}
		}

		FDynamicGraphOperatorData& FDynamicOperator::GetDynamicGraphOperatorData()
		{
			return DynamicOperatorData;
		}

		void FDynamicOperator::ApplyTransformsUntilFence()
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperator::ApplyTransformsUntilFence)

			if (!bExecuteFenceIsSet)
			{
				while (TOptional<TUniquePtr<IDynamicOperatorTransform>> Transform = TransformQueue->Dequeue())
				{
					if (Transform.IsSet())
					{
						if (Transform->IsValid())
						{
							EDynamicOperatorTransformQueueAction Result = (*Transform)->Transform(DynamicOperatorData);
							if (EDynamicOperatorTransformQueueAction::Fence == Result)
							{
								bExecuteFenceIsSet = true;
								break;
							}
						}
					}
				}
			}
		}

		void FDynamicOperator::ApplyTransformsUntilFenceOrTimeout(double InTimeoutInSeconds)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperator::ApplyTransformsUntilFenceOrTimeout)

			if (bExecuteFenceIsSet)
			{
				// Execute fence needs to be cleared before applying any transforms.
				return;
			}

			TOptional<TUniquePtr<IDynamicOperatorTransform>> Transform = TransformQueue->Dequeue();

			if (Transform)
			{
				check(FPlatformTime::GetSecondsPerCycle() > 0);
				const uint64 BreakTimeInCycles = FPlatformTime::Cycles64() + static_cast<uint64>(InTimeoutInSeconds / FPlatformTime::GetSecondsPerCycle());
				do
				{
					if (Transform.IsSet() && Transform->IsValid())
					{
						EDynamicOperatorTransformQueueAction Result = (*Transform)->Transform(DynamicOperatorData);
						if (EDynamicOperatorTransformQueueAction::Fence == Result)
						{
							bExecuteFenceIsSet = true;
							break;
						}
					}

					if (FPlatformTime::Cycles64() >= BreakTimeInCycles)
					{
						UE_LOGF(LogMetaSound, Verbose, "Transforms exceeded duration.");
						break;
					}
				}
				while((Transform = TransformQueue->Dequeue()));
			}
		}

		void FDynamicOperator::Execute()
		{
			using namespace DynamicOperatorPrivate;

			if (MetaSoundExperimentalTransformTimeoutInSeconds > 0)
			{
				ApplyTransformsUntilFenceOrTimeout(MetaSoundExperimentalTransformTimeoutInSeconds);
			}
			else
			{
				ApplyTransformsUntilFence();
			}

			for (FExecuteEntry& Entry : DynamicOperatorData.ExecuteTable)
			{
				Entry.Execute();
			}
		}

		void FDynamicOperator::PostExecute()
		{
			for (FPostExecuteEntry& Entry : ReverseIterate(DynamicOperatorData.PostExecuteTable))
			{
				Entry.PostExecute();
			}

			bExecuteFenceIsSet = false;
		}

		void FDynamicOperator::Reset(const IOperator::FResetParams& InParams)
		{
			FlushEnqueuedTransforms();
			for (FResetEntry& Entry : DynamicOperatorData.ResetTable)
			{
				Entry.Reset(InParams);
			}
		}

		void FDynamicOperator::StaticReset(IOperator* InOperator, const IOperator::FResetParams& InParams)
		{
			static_cast<FDynamicOperator*>(InOperator)->Reset(InParams);
		}

		void FDynamicOperator::StaticExecute(IOperator* InOperator)
		{
			static_cast<FDynamicOperator*>(InOperator)->Execute();
		}

		void FDynamicOperator::StaticPostExecute(IOperator* InOperator)
		{
			static_cast<FDynamicOperator*>(InOperator)->PostExecute();
		}

		EDynamicOperatorTransformQueueAction FNullTransform::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Set the order of operators in the graph
		FSetOperatorOrdinalsAndSort::FSetOperatorOrdinalsAndSort(TMap<FOperatorID, int32> InOrdinals)
		: Ordinals(MoveTemp(InOrdinals))
		{
		}

		EDynamicOperatorTransformQueueAction FSetOperatorOrdinalsAndSort::Transform(FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::SetOperatorOrdinalsAndSort)

			SetOrdinalsAndSort(Ordinals, InOutGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Swaps order of operators. 
		FSwapOperatorOrdinalsAndSort::FSwapOperatorOrdinalsAndSort(TArray<FOrdinalSwap> InSwaps)
		: Swaps(MoveTemp(InSwaps))
		{
		}

		EDynamicOperatorTransformQueueAction FSwapOperatorOrdinalsAndSort::Transform(FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::SwapOperatorOrdinalsAndSort)

			SwapOrdinalsAndSort(Swaps, InOutGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Add an operator to the grpah
		FInsertOperator::FInsertOperator(FOperatorID InOperatorID, FOperatorInfo InInfo)
		: OperatorID(InOperatorID)
		, OperatorInfo(MoveTemp(InInfo))
		{
		}

		EDynamicOperatorTransformQueueAction FInsertOperator::Transform(FDynamicGraphOperatorData& InOutGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::InsertOperator)

			InsertOperator(OperatorID, MoveTemp(OperatorInfo), InOutGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Remove an operator from the graph
		FRemoveOperator::FRemoveOperator(FOperatorID InOperatorID, TArray<FOperatorID> InOperatorsConnectedToInput)
		: OperatorID(InOperatorID)
		, OperatorsConnectedToInput(MoveTemp(InOperatorsConnectedToInput))
		{
		}

		EDynamicOperatorTransformQueueAction FRemoveOperator::Transform(FDynamicGraphOperatorData& InOutGraphOperatorData) 
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::RemoveOperator)

			RemoveOperator(OperatorID, OperatorsConnectedToInput, InOutGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Add an input to the graph
		FAddInput::FAddInput(FOperatorID InOperatorID, const FVertexName& InVertexName, FAnyDataReference InDataReference)
		: OperatorID(InOperatorID)
		, VertexName(InVertexName)
		, DataReference(InDataReference)
		{
		}

		EDynamicOperatorTransformQueueAction FAddInput::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::AddInput)

			FOperatorInfo* OpInfo = InGraphOperatorData.OperatorMap.Find(OperatorID);
			if (nullptr == OpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when adding input %s."), OperatorID, *VertexName.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			const FInputDataVertex& OperatorInputVertex = OpInfo->VertexData.GetInputs().GetVertex(VertexName);

			FInputVertexInterfaceData& GraphInputData = InGraphOperatorData.VertexData.GetInputs();
			{
				// Unfreeze interface so a new vertex can be added. 
				DynamicOperatorPrivate::TScopeUnfreeze Unfreeze(GraphInputData);

				// Update unfrozen vertex interface data.
				GraphInputData.AddVertex(OperatorInputVertex);
			}
			GraphInputData.SetVertex(VertexName, DataReference);

			InGraphOperatorData.InputVertexMap.Add(VertexName, OperatorID);

			// Update listeners that an input has been added.
			if (InGraphOperatorData.OperatorUpdateCallbacks.OnInputAdded)
			{
				InGraphOperatorData.OperatorUpdateCallbacks.OnInputAdded(VertexName, InGraphOperatorData.VertexData.GetInputs());
			}

			// Propagate the data reference update through the graph
			PropagateBindUpdate(OperatorID, VertexName, DataReference, InGraphOperatorData);

			// Refresh output vertex interface data in case any output nodes were updated
			// when bind updates were propagated through the graph.
			UpdateOutputVertexData(InGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Remove an input from the graph
		FRemoveInput::FRemoveInput(const FVertexName& InVertexName)
		: VertexName(InVertexName)
		{
		}

		EDynamicOperatorTransformQueueAction FRemoveInput::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::RemoveInput)

			InGraphOperatorData.InputVertexMap.Remove(VertexName);
			{
				// Unfreeze vertex data so vertex can be removed
				DynamicOperatorPrivate::TScopeUnfreeze Unfreeze(InGraphOperatorData.VertexData.GetInputs());
				InGraphOperatorData.VertexData.GetInputs().RemoveVertex(VertexName);
			}

			// Update listeners that an input has been removed.
			if (InGraphOperatorData.OperatorUpdateCallbacks.OnInputRemoved)
			{
				InGraphOperatorData.OperatorUpdateCallbacks.OnInputRemoved(VertexName, InGraphOperatorData.VertexData.GetInputs());
			}

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Add an output to the graph
		FAddOutput::FAddOutput(FOperatorID InOperatorID, const FVertexName& InVertexName)
		: OperatorID(InOperatorID)
		, VertexName(InVertexName)
		{
		}

		EDynamicOperatorTransformQueueAction FAddOutput::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::AddOutput)

			FOperatorInfo* OpInfo = InGraphOperatorData.OperatorMap.Find(OperatorID);
			if (nullptr == OpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when adding output %s."), OperatorID, *VertexName.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			const FOutputDataVertex& OperatorOutputVertex = OpInfo->VertexData.GetOutputs().GetVertex(VertexName);
			const FAnyDataReference* Ref = OpInfo->VertexData.GetOutputs().FindDataReference(VertexName);

			if (nullptr == Ref)
			{
				UE_LOGF(LogMetaSound, Error, "Could not find data reference when creating output %ls", *VertexName.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			FOutputVertexInterfaceData& GraphOutputData = InGraphOperatorData.VertexData.GetOutputs();
			{
				// Unfreeze interface so a new vertex can be added. 
				DynamicOperatorPrivate::TScopeUnfreeze Unfreeze(GraphOutputData);

				// Update unfrozen vertex interface data.
				GraphOutputData.AddVertex(OperatorOutputVertex);
			}
			GraphOutputData.SetVertex(VertexName, *Ref);

			InGraphOperatorData.OutputVertexMap.Add(VertexName, OperatorID);

			// Update listeners that an input has been added.
			if (InGraphOperatorData.OperatorUpdateCallbacks.OnOutputAdded)
			{
				InGraphOperatorData.OperatorUpdateCallbacks.OnOutputAdded(VertexName, InGraphOperatorData.VertexData.GetOutputs());
			}

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Remove an output from the graph
		FRemoveOutput::FRemoveOutput(const FVertexName& InVertexName)
		: VertexName(InVertexName)
		{
		}

		EDynamicOperatorTransformQueueAction FRemoveOutput::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::RemoveOutput)

			InGraphOperatorData.OutputVertexMap.Remove(VertexName);
			{
				// Unfreeze vertex data so vertex can be removed
				DynamicOperatorPrivate::TScopeUnfreeze Unfreeze(InGraphOperatorData.VertexData.GetOutputs());
				InGraphOperatorData.VertexData.GetOutputs().RemoveVertex(VertexName);
			}

			// Update listeners that an output has been removed.
			if (InGraphOperatorData.OperatorUpdateCallbacks.OnOutputRemoved)
			{
				InGraphOperatorData.OperatorUpdateCallbacks.OnOutputRemoved(VertexName, InGraphOperatorData.VertexData.GetOutputs());
			}

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Trigger a execution fence.
		EDynamicOperatorTransformQueueAction FExecuteFence::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			return EDynamicOperatorTransformQueueAction::Fence;
		}

		// Connect two operators in the graph
		FConnectOperators::FConnectOperators(FOperatorID InFromOpID, const FName& InFromVert, FOperatorID InToOpID, const FName& InToVert)
		: FromOpID(InFromOpID)
		, ToOpID(InToOpID)
		, FromVert(InFromVert)
		, ToVert(InToVert)
		{
		}

		EDynamicOperatorTransformQueueAction FConnectOperators::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::ConnectOperators)

			FOperatorInfo* FromOpInfo = InGraphOperatorData.OperatorMap.Find(FromOpID);
			FOperatorInfo* ToOpInfo = InGraphOperatorData.OperatorMap.Find(ToOpID);

			if (nullptr == FromOpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when connecting from %" UPTRINT_FMT ":%s to %" UPTRINT_FMT ":%s"), FromOpID, FromOpID, *FromVert.ToString(), ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			if (nullptr == ToOpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when connecting from %" UPTRINT_FMT ":%s to %" UPTRINT_FMT ":%s"), ToOpID, FromOpID, *FromVert.ToString(), ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			const FAnyDataReference* FromRef = FromOpInfo->VertexData.GetOutputs().FindDataReference(FromVert);
			if (nullptr == FromRef)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find output data reference with vertex name %s when connecting from %" UPTRINT_FMT ":%s to %" UPTRINT_FMT ":%s"), *FromVert.ToString(), FromOpID, *FromVert.ToString(), ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			// Propagate the data reference update through the graph
			PropagateBindUpdate(ToOpID, ToVert, *FromRef, InGraphOperatorData);

			// Refresh output vertex interface data in case any output nodes were updated
			// when bind updates were propagated through the graph.
			UpdateOutputVertexData(InGraphOperatorData);

			FromOpInfo->OutputConnections.FindOrAdd(FromVert).Add(FGraphOperatorData::FVertexDestination{ToOpID, ToVert});

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Disconnect a specific edge
		FSwapOperatorConnection::FSwapOperatorConnection(FOperatorID InOriginalFromOpID, const FName& InOriginalFromVert, FOperatorID InNewFromOpID, const FName& InNewFromVert, FOperatorID InToOpID, const FName& InToVert)
		: ConnectTransform(InNewFromOpID, InNewFromVert, InToOpID, InToVert)
		, OriginalFromOpID(InOriginalFromOpID)
		, ToOpID(InToOpID)
		, OriginalFromVert(InOriginalFromVert)
		, ToVert(InToVert)
		{
		}

		EDynamicOperatorTransformQueueAction FSwapOperatorConnection::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::DisconnectOperators)

			// Make new connection. We can skip propagating updates and updating callbacks
			// here because those are handled in the FConnectOperators transform.
			EDynamicOperatorTransformQueueAction NextAction = ConnectTransform.Transform(InGraphOperatorData);
			check(NextAction == EDynamicOperatorTransformQueueAction::Continue); // this should always be continue since next step must be performed. 

			// Clean up the old connection.
			FOperatorInfo* OriginalFromOpInfo = InGraphOperatorData.OperatorMap.Find(OriginalFromOpID);

			if (nullptr == OriginalFromOpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when disconnecting from %" UPTRINT_FMT ":%s to %" UPTRINT_FMT ":%s"), OriginalFromOpID, OriginalFromOpID, *OriginalFromVert.ToString(), ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			// Remove destinations from operator map.
			auto IsDestinationToRemove = [&](const FGraphOperatorData::FVertexDestination& Dst) 
			{ 
				return (Dst.OperatorID == ToOpID) && (Dst.VertexName == ToVert);
			};
			OriginalFromOpInfo->OutputConnections.FindOrAdd(OriginalFromVert).RemoveAll(IsDestinationToRemove);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Disconnect a specific edge
		FRemoveOperatorConnection::FRemoveOperatorConnection(FOperatorID InFromOpID, const FName& InFromVertName, FOperatorID InToOpID, const FName& InToVertName, FAnyDataReference InDataReference)
			: SetOperatorInputTransform(InToOpID, InToVertName, MoveTemp(InDataReference))
			, FromOpID(InFromOpID)
			, ToOpID(InToOpID)
			, FromVertName(InFromVertName)
			, ToVertName(InToVertName)
		{
		}

		EDynamicOperatorTransformQueueAction FRemoveOperatorConnection::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::DisconnectOperators)

			// Clean up the old connection.
			FOperatorInfo* FromOpInfo = InGraphOperatorData.OperatorMap.Find(FromOpID);

			if (nullptr == FromOpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when disconnecting from %" UPTRINT_FMT ":%s to %" UPTRINT_FMT ":%s"), FromOpID, FromOpID, *FromVertName.ToString(), ToOpID, *ToVertName.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			// Remove destinations from operator map.
			auto IsDestinationToRemove = [&](const FGraphOperatorData::FVertexDestination& Dst)
			{
				return (Dst.OperatorID == ToOpID) && (Dst.VertexName == ToVertName);
			};
			FromOpInfo->OutputConnections.FindOrAdd(FromVertName).RemoveAll(IsDestinationToRemove); 

			// Set data reference on newly unconnected input 
			EDynamicOperatorTransformQueueAction NextAction = SetOperatorInputTransform.Transform(InGraphOperatorData);
			check(NextAction == EDynamicOperatorTransformQueueAction::Continue);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Set the value on an unconnected operator input
		FSetOperatorInput::FSetOperatorInput(FOperatorID InToOpID, const FName& InToVert, FAnyDataReference InDataRef)
		: ToOpID(InToOpID)
		, ToVert(InToVert)
		, DataRef(InDataRef)
		{
		}

		EDynamicOperatorTransformQueueAction FSetOperatorInput::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::SetOperatorInput)

			FOperatorInfo* ToOpInfo = InGraphOperatorData.OperatorMap.Find(ToOpID);

			if (nullptr == ToOpInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when setting value for %" UPTRINT_FMT ":%s"), ToOpID, ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}

			const FAnyDataReference* Ref = ToOpInfo->VertexData.GetInputs().FindDataReference(ToVert);
			if (nullptr == Ref)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find input data reference with vertex name %s when setting %" UPTRINT_FMT ":%s"), *ToVert.ToString(), ToOpID, *ToVert.ToString());
				return EDynamicOperatorTransformQueueAction::Continue;
			}
						
			// Propagate the data reference update through the graph
			PropagateBindUpdate(ToOpID, ToVert, MoveTemp(DataRef), InGraphOperatorData);

			// Refresh output vertex interface data in case any output nodes were updated
			// when bind updates were propagated through the graph.
			UpdateOutputVertexData(InGraphOperatorData);

			return EDynamicOperatorTransformQueueAction::Continue;
		}

		// Perform several transformations at once without executing the graph.
		FAtomicTransform::FAtomicTransform(TArray<TUniquePtr<IDynamicOperatorTransform>> InTransforms)
		: Transforms(MoveTemp(InTransforms))
		{
		}

		EDynamicOperatorTransformQueueAction FAtomicTransform::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::AtomicTransform)

			EDynamicOperatorTransformQueueAction Result = EDynamicOperatorTransformQueueAction::Continue;

			for (TUniquePtr<IDynamicOperatorTransform>& TransformPtr : Transforms)
			{
				if (TransformPtr.IsValid())
				{
					if (EDynamicOperatorTransformQueueAction::Continue != Result)
					{
						UE_LOGF(LogMetaSound, Error, "Encountered unsupported dynamic operator transform result (%ls) during atomic operator trasnform.", LexToString(Result));
					}

					Result = TransformPtr->Transform(InGraphOperatorData);
				}
			}

			return Result;
		}

		FBeginAudioFadeTransform::FBeginAudioFadeTransform(FOperatorID InOperatorIDToFade, EAudioFadeType InFadeType, TArrayView<const FVertexName> InInputVerticesToFade, TArrayView<const FVertexName> InOutputVerticesToFade)
		: OperatorIDToFade(InOperatorIDToFade)
		, InitFadeState(InFadeType == EAudioFadeType::FadeIn ? FAudioFadeOperatorWrapper::EFadeState::FadingIn : FAudioFadeOperatorWrapper::EFadeState::FadingOut)
		, InputVerticesToFade(InInputVerticesToFade)
		, OutputVerticesToFade(InOutputVerticesToFade)
		{
		}

		EDynamicOperatorTransformQueueAction FBeginAudioFadeTransform::FBeginAudioFadeTransform::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::BeginAudioFadeTransform)

		  	if (FOperatorInfo* OperatorInfo = InGraphOperatorData.OperatorMap.Find(OperatorIDToFade))
			{
				// Make wrapped operator
				OperatorInfo->Operator = MakeUnique<FAudioFadeOperatorWrapper>(InitFadeState, InGraphOperatorData.OperatorSettings, OperatorInfo->VertexData.GetInputs(), MoveTemp(OperatorInfo->Operator), InputVerticesToFade, OutputVerticesToFade);

				// Update data references in graph
				RebindWrappedOperator(OperatorIDToFade, *OperatorInfo, InGraphOperatorData);
			}
			else
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when applying audio fade out."), OperatorIDToFade);
			}
			
			return EDynamicOperatorTransformQueueAction::Continue;
		}

		FEndAudioFadeTransform::FEndAudioFadeTransform(FOperatorID InOperatorIDToFade)
		: OperatorIDToFade(InOperatorIDToFade)
		{
		}

		EDynamicOperatorTransformQueueAction FEndAudioFadeTransform::Transform(FDynamicGraphOperatorData& InGraphOperatorData)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::DynamicOperator::EndAudioFadeTransform)

		  	if (FOperatorInfo* OperatorInfo = InGraphOperatorData.OperatorMap.Find(OperatorIDToFade))
			{
				FAudioFadeOperatorWrapper* Wrapper = static_cast<FAudioFadeOperatorWrapper*>(OperatorInfo->Operator.Get());
				check(Wrapper);

				// Unwrap operator
				OperatorInfo->Operator = Wrapper->ReleaseOperator();

				// Update data references in graph
				RebindWrappedOperator(OperatorIDToFade, *OperatorInfo, InGraphOperatorData);
			}
			else
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find operator with ID %" UPTRINT_FMT " when applying audio fade out."), OperatorIDToFade);
			}
			
			return EDynamicOperatorTransformQueueAction::Continue;
		}
	} // namespace DynamicGraph
}

