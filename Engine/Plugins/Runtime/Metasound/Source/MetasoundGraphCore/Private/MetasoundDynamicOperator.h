// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SpscQueue.h"
#include "MetasoundDynamicGraphAlgo.h"
#include "MetasoundDynamicOperatorAudioFade.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"

#ifndef METASOUND_DEBUG_DYNAMIC_TRANSACTOR
#define METASOUND_DEBUG_DYNAMIC_TRANSACTOR !UE_BUILD_SHIPPING
#endif

namespace Metasound
{
	namespace DynamicGraph
	{
		struct FDynamicGraphOperatorData;
		class FDynamicOperator;
		class FDynamicOperatorTransactor;
		
#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
		namespace Debug
		{
			class FDynamicOperatorDebugger;
		}
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR

		using FLiteralAssignmentFunction = void(*)(const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral, const FAnyDataReference& OutDataRef);

		/** Action to perform after a single transformation. */
		enum class EDynamicOperatorTransformQueueAction : uint8
		{
			Continue, //< Perform next operator if it exists. 
			Fence     //< Wait to perform next operator until the operator has been executed. 
		};

		/** Interface for transformation of dynamic graph operator data. */
		class IDynamicOperatorTransform
		{
		public:
			virtual ~IDynamicOperatorTransform() = default;
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) = 0;
		};

		/** An FDynamicOperator is a MetaSound operator which can dynamically
		 * change it's topology. Changes are communicated to the dynamic operator
		 * through a TransformationQueue. 
		 */
		class FDynamicOperator : public IOperator, public IDynamicGraphInPlaceBuildable
		{
		public:

			FDynamicOperator(const FOperatorSettings& InSettings);
			FDynamicOperator(const FOperatorSettings& InSettings, TSharedPtr<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> TransformQueue, const FDynamicOperatorUpdateCallbacks& InOperatorUpdateCallbacks);

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			virtual IOperator::FResetFunction GetResetFunction() override;

			virtual IOperator::FExecuteFunction GetExecuteFunction() override;

			virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override;

			// Force all transformations in the transaction queue to be applied. 
			void FlushEnqueuedTransforms();

		private:
#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
			friend class Debug::FDynamicOperatorDebugger;
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR

			virtual FDynamicGraphOperatorData& GetDynamicGraphOperatorData() override;

			void ApplyTransformsUntilFence();
			void ApplyTransformsUntilFenceOrTimeout(double InTimeoutInSeconds);
			void Execute();
			void PostExecute();
			void Reset(const IOperator::FResetParams& InParams);

			static void StaticReset(IOperator* InOperator, const IOperator::FResetParams& InParams);
			static void StaticExecute(IOperator* InOperator);
			static void StaticPostExecute(IOperator* InOperator);

			FDynamicGraphOperatorData DynamicOperatorData;

			TSharedPtr<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> TransformQueue;
			bool bExecuteFenceIsSet = false;
		};

		/** A transform which does nothing. */
		class FNullTransform : public IDynamicOperatorTransform
		{
		public:
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		};

		/** A transform which determines the order of execution for operators. */
		class FSetOperatorOrdinalsAndSort : public IDynamicOperatorTransform
		{
		public:
			FSetOperatorOrdinalsAndSort(TMap<FOperatorID, int32> InOrdinals);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			TMap<FOperatorID, int32> Ordinals;
		};

		/** A transform which applies operator ordinal swaps. */
		class FSwapOperatorOrdinalsAndSort : public IDynamicOperatorTransform
		{
		public:
			FSwapOperatorOrdinalsAndSort(TArray<FOrdinalSwap> InSwaps);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			TArray<FOrdinalSwap> Swaps;
		};

		/** A transform which adds an operator. */
		class FInsertOperator : public IDynamicOperatorTransform
		{
		public:
			FInsertOperator(FOperatorID InOperatorID, FOperatorInfo InInfo);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorID;
			FOperatorInfo OperatorInfo;
		};

		/** A transform which removes an operator. */
		class FRemoveOperator : public IDynamicOperatorTransform
		{
		public:
			FRemoveOperator(FOperatorID InOperatorID, TArray<FOperatorID> InOperatorsConnectedToInput);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorID;
			TArray<FOperatorID> OperatorsConnectedToInput;
		};

		/** A transform which exposes an input to the graph. */
		class FAddInput : public IDynamicOperatorTransform
		{
		public:
			FAddInput(FOperatorID InOperatorID, const FVertexName& InVertexName, FAnyDataReference InDataReference);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorID;
			FVertexName VertexName;
			FAnyDataReference DataReference;
		};

		/** A transform which removes an input to the graph. */
		class FRemoveInput : public IDynamicOperatorTransform
		{
		public:
			FRemoveInput(const FVertexName& InVertexName);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FVertexName VertexName;
		};

		/** A transform which exposes an output from the graph. */
		class FAddOutput: public IDynamicOperatorTransform
		{
		public:
			FAddOutput(FOperatorID InOperatorID, const FVertexName& InVertexName);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorID;
			FVertexName VertexName;
		};

		/** A transform which removes an output from the graph. */
		class FRemoveOutput: public IDynamicOperatorTransform
		{
		public:
			FRemoveOutput(const FVertexName& InVertexName);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FVertexName VertexName;
		};

		/** A transform which pauses the transformation queue until the dynamic operator has executed. */
		class FExecuteFence : public IDynamicOperatorTransform
		{
		public:
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		};

		/** A transform which connects two vertices in the graph. */
		class FConnectOperators : public IDynamicOperatorTransform
		{
		public:
			FConnectOperators(FOperatorID InFromOpID, const FName& InFromVert, FOperatorID InToOpID, const FName& InToVert);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID FromOpID;
			FOperatorID ToOpID;
			FName FromVert;
			FName ToVert;
		};

		/** A transform which disconnects two vertices in the graph. */
		class FSwapOperatorConnection : public IDynamicOperatorTransform
		{
		public:
			FSwapOperatorConnection(FOperatorID InOriginalFromOpID, const FName& InOriginalFromVert, FOperatorID InNewFromOpID, const FName& InNewFromVert, FOperatorID InToOpID, const FName& InToVert);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FConnectOperators ConnectTransform;

			FOperatorID OriginalFromOpID;
			FOperatorID ToOpID;
			FName OriginalFromVert;
			FName ToVert;
		};

		/** A transform which sets the input to an operator to a specified literal. */
		class FSetOperatorInput : public IDynamicOperatorTransform
		{
		public:
			FSetOperatorInput(FOperatorID InToOpID, const FName& InToVert, FAnyDataReference InDataRef);

			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;

		private:
			FOperatorID ToOpID;
			FName ToVert;
			FAnyDataReference DataRef;
		};

		/** A transform which disconnects two vertices in the graph, replacing the input with a given data reference. */
		class FRemoveOperatorConnection : public IDynamicOperatorTransform
		{
		public:
			FRemoveOperatorConnection(FOperatorID InFromOpID, const FName& InFromVertName,  FOperatorID InToOpID, const FName& InToVertName, FAnyDataReference InReplacementDataRef);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;

		private:
			FSetOperatorInput SetOperatorInputTransform;

			FOperatorID FromOpID;
			FOperatorID ToOpID;
			FName FromVertName;
			FName ToVertName;
		};

		/** A transform which groups multiple transformations together and forces them to all complete
		 * before the dynamic operator executes. */
		class FAtomicTransform : public IDynamicOperatorTransform
		{
		public:
			FAtomicTransform(TArray<TUniquePtr<IDynamicOperatorTransform>> InTransforms);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			TArray<TUniquePtr<IDynamicOperatorTransform>> Transforms;
		};

		enum class EAudioFadeType : uint8
		{
			FadeIn, //< Fade from silent to full volume.
			FadeOut //< Fade from full volume to silent.
		};

		/* Marks the beginning of an audio fade. 
		 *
		 * When scheduling fade transformations on a dynamic operator, an FBeginAudioFadeTransform 
		 * must be matched with an FEndAudioTransform with an FExecuteFence between them. 
		 *
		 * The FBeginAudioFadeTransform sets up the graph to perform a set of audio
		 * fades. 
		 * The FExecuteFence forces the fade to occur before any additional transforms 
		 * are performed.
		 * The FEndAudioFadeTransform cleans up any temporary state that was needed
		 * to perform the fade. 
		 * */
		class FBeginAudioFadeTransform : public IDynamicOperatorTransform
		{
		public:

			FBeginAudioFadeTransform(FOperatorID InOperatorIDToFade, EAudioFadeType InFadeType, TArrayView<const FVertexName> InInputVerticesToFade, TArrayView<const FVertexName> InOutputVerticesToFade);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorIDToFade;
			FAudioFadeOperatorWrapper::EFadeState InitFadeState;
			TArray<FVertexName> InputVerticesToFade;
			TArray<FVertexName> OutputVerticesToFade;
		};

		/** Marks the end of an audio fade. */
		class FEndAudioFadeTransform : public IDynamicOperatorTransform
		{
		public:
			FEndAudioFadeTransform(FOperatorID InOperatorIDToFade);
			virtual EDynamicOperatorTransformQueueAction Transform(FDynamicGraphOperatorData& InGraphOperatorData) override;
		private:
			FOperatorID OperatorIDToFade;
		};
	}
}
