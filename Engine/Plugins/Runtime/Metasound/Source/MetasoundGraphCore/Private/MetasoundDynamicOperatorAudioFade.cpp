// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicOperatorAudioFade.h"

#include "Containers/Array.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/UnrealMemory.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"

namespace Metasound::DynamicGraph
{
	FAudioFadeOperatorWrapper::FAudioFadeOperatorWrapper(EFadeState InInitialFadeState, const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData, TUniquePtr<IOperator> InOperator, TArrayView<const FVertexName> InInputVerticesToFade, TArrayView<const FVertexName> InOutputVerticesToFade)
	: WrappedOperator(MoveTemp(InOperator))
	, InputVerticesToFade(InInputVerticesToFade)
	, OutputVerticesToFade(InOutputVerticesToFade)
	, InitialFadeState(InInitialFadeState)
	, FadeState(InInitialFadeState)
	{
		// allocate internal input buffers and data references
		InitInputBuffers(InOperatorSettings, InInputData);

		SetWrappedOperatorInputs(InInputData);

		// allocate internal output buffers and data references
		InitOutputBuffers(InOperatorSettings);
	}

	TUniquePtr<IOperator> FAudioFadeOperatorWrapper::ReleaseOperator() 
	{
		// Reset internal arrays so we don't hold onto data references 
		InputVerticesToFade.Reset();
		OutputVerticesToFade.Reset();

		WrappedInputAudioBuffers.Reset();
		FadedInputAudioBuffers.Reset();
		WrappedOutputAudioBuffers.Reset();
		FadedOutputAudioBuffers.Reset();

		return WrappedOperator.ReleaseOperator();
	}

	void FAudioFadeOperatorWrapper::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		// Bind input audio buffers that will be faded
		for (int i = 0; i < InputVerticesToFade.Num(); i++)
		{
			const FVertexName& VertexName = InputVerticesToFade[i];
			InOutVertexData.BindVertex(VertexName, WrappedInputAudioBuffers[i]);
		}

		// Pass remaining vertex data to be bound with wrapped operator
		SetWrappedOperatorInputs(InOutVertexData);
	}

	void FAudioFadeOperatorWrapper::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		// Bind wrapped operator
		WrappedOperator.BindOutputs(InOutVertexData);

		// Override any output bindings are related to faded outputs .
		for (int i = 0; i < OutputVerticesToFade.Num(); i++)
		{
			const FVertexName& VertexName = OutputVerticesToFade[i];

			// Update internal references to wrapped operator's audio buffers
			if (const FAnyDataReference* DataRef = InOutVertexData.FindDataReference(VertexName))
			{
				WrappedOutputAudioBuffers[i] = DataRef->GetDataReadReference<FAudioBuffer>();
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed to find output audio buffer '%ls' when attempting to fade out audio buffer.", *VertexName.ToString());
			}

			// Override which output buffers are exposed from this operator.
			InOutVertexData.BindVertex(VertexName, FadedOutputAudioBuffers[i]);
		}
	}

	void FAudioFadeOperatorWrapper::Execute()
	{
		// Fade inputs
		FadeBuffers(WrappedInputAudioBuffers, FadedInputAudioBuffers);

		WrappedOperator.Execute();

		// Fade outputs
		FadeBuffers(WrappedOutputAudioBuffers, FadedOutputAudioBuffers);

		// Update fade state
		if (FadeState == EFadeState::FadingIn)
		{
			FadeState = EFadeState::FullVolume;
		}
		else if (FadeState == EFadeState::FadingOut)
		{
			FadeState = EFadeState::Silence;
		}
	}

	void FAudioFadeOperatorWrapper::PostExecute()
	{
		WrappedOperator.PostExecute();
		
		if (FadeState == EFadeState::Silence)
		{
			// If it is silent, zero output buffers to avoid having to rezero output buffers every
			// time the operator is executed 
			for (TDataWriteReference<FAudioBuffer>& FadedBufferRef : FadedInputAudioBuffers)
			{
				FadedBufferRef->Zero();
			}
			for (TDataWriteReference<FAudioBuffer>& FadedBufferRef : FadedOutputAudioBuffers)
			{
				FadedBufferRef->Zero();
			}
		}
	}

	void FAudioFadeOperatorWrapper::Reset(const IOperator::FResetParams& InParams)
	{
		WrappedOperator.Reset(InParams);
		FadeState = InitialFadeState;
	}

	void FAudioFadeOperatorWrapper::InitInputBuffers(const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData)
	{
		for (const FVertexName& VertexName : InputVerticesToFade)
		{
			FadedInputAudioBuffers.Add(TDataWriteReference<FAudioBuffer>::CreateNew(InOperatorSettings));
			if (const FAnyDataReference* DataRef = InInputData.FindDataReference(VertexName))
			{
				WrappedInputAudioBuffers.Add(DataRef->GetDataReadReference<FAudioBuffer>());
			}
			else
			{
				WrappedInputAudioBuffers.Add(TDataReadReference<FAudioBuffer>::CreateNew(InOperatorSettings));
			}
		}
	}

	void FAudioFadeOperatorWrapper::InitOutputBuffers(const FOperatorSettings& InOperatorSettings)
	{
		FOutputVertexInterfaceData OutputData;
		WrappedOperator.BindOutputs(OutputData);

		for (const FVertexName& VertexName : OutputVerticesToFade)
		{
			FadedOutputAudioBuffers.Add(TDataWriteReference<FAudioBuffer>::CreateNew(InOperatorSettings));
			if (const FAnyDataReference* DataRef = OutputData.FindDataReference(VertexName))
			{
				WrappedOutputAudioBuffers.Add(DataRef->GetDataReadReference<FAudioBuffer>());
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed to find output audio buffer '%ls' when attempting to fade audio buffer.", *VertexName.ToString());
				WrappedOutputAudioBuffers.Add(TDataReadReference<FAudioBuffer>::CreateNew(InOperatorSettings));
			}
		}
	}

	void FAudioFadeOperatorWrapper::FadeBuffers(const TArray<TDataReadReference<FAudioBuffer>>& InSrc, const TArray<TDataWriteReference<FAudioBuffer>>& InDst)
	{
		if (FadeState == EFadeState::FadingIn)
		{
			for (int32 i = 0; i < InSrc.Num(); i++)
			{
				const FAudioBuffer& WrappedBuffer = *InSrc[i];
				FAudioBuffer& OutputBuffer = *InDst[i];

				FMemory::Memcpy(OutputBuffer.GetData(), WrappedBuffer.GetData(), WrappedBuffer.Num() * sizeof(float));
				Audio::ArrayFade(OutputBuffer, 0.f, 1.f);
			}
		}
		else if (FadeState == EFadeState::FadingOut)
		{
			for (int32 i = 0; i < InSrc.Num(); i++)
			{
				const FAudioBuffer& WrappedBuffer = *InSrc[i];
				FAudioBuffer& OutputBuffer = *InDst[i];

				FMemory::Memcpy(OutputBuffer.GetData(), WrappedBuffer.GetData(), WrappedBuffer.Num() * sizeof(float));
				Audio::ArrayFade(OutputBuffer, 1.f, 0.f);
			}
		}
		else if (FadeState == EFadeState::FullVolume)
		{
			for (int32 i = 0; i < InSrc.Num(); i++)
			{
				const FAudioBuffer& WrappedBuffer = *InSrc[i];
				FAudioBuffer& OutputBuffer = *InDst[i];

				FMemory::Memcpy(OutputBuffer.GetData(), WrappedBuffer.GetData(), WrappedBuffer.Num() * sizeof(float));
			}
		}
	}

	void FAudioFadeOperatorWrapper::SetWrappedOperatorInputs(const FInputVertexInterfaceData& InInputData)
	{
		// Create an internal copy so that wrapped data references don't escape 
		// this operator. 
		FInputVertexInterfaceData InputDataCopy = InInputData;

		if (InputVerticesToFade.Num() > 0)
		{
			for (int i = 0; i < InputVerticesToFade.Num(); i++)
			{
				const FVertexName& VertexName = InputVerticesToFade[i];
				InputDataCopy.SetVertex(VertexName, FadedInputAudioBuffers[i]);
			}
		}

		WrappedOperator.BindInputs(InputDataCopy);
	}
}

