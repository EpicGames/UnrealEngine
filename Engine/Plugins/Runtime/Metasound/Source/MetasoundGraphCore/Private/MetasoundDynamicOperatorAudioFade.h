// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"

namespace Metasound::DynamicGraph
{
	// FAudioFadeOperatorWrapper wraps an existing operator and applies a fade in/out to 
	// any designated audio buffers on the next execution. The fade will only occur
	// over a single audio buffer and will be finalized during calls to PostExecute. 
	//
	// When this operator wraps another operator, it will also provide different 
	// output data references for any faded output audio buffers. Users of this operator
	// should be wary to rebind downstream operators in any graph using this operator
	// wrapper. 
	//
	// The wrapped operator can be release if desired, though it leaves this operator
	// in a state where it will have no effect on any audio buffers.
	class FAudioFadeOperatorWrapper : public TExecutableOperator<FAudioFadeOperatorWrapper>
	{
	public:
		enum class EFadeState : uint8
		{
			Silence,
			FadingOut,
			FadingIn,
			FullVolume
		};

		FAudioFadeOperatorWrapper(EFadeState InInitialFadeState, const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData, TUniquePtr<IOperator> InOperator, TArrayView<const FVertexName> InInputVerticesToFade, TArrayView<const FVertexName> InOutputVerticesToFade);

		TUniquePtr<IOperator> ReleaseOperator();

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Execute();
		void PostExecute();
		void Reset(const IOperator::FResetParams& InParams);

	private:

		void InitInputBuffers(const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData);
		void InitOutputBuffers(const FOperatorSettings& InOperatorSettings);

		void FadeBuffers(const TArray<TDataReadReference<FAudioBuffer>>& InSrc, const TArray<TDataWriteReference<FAudioBuffer>>& InDst);
		void SetWrappedOperatorInputs(const FInputVertexInterfaceData& InInputData);

		FExecuter WrappedOperator;

		TArray<FVertexName> InputVerticesToFade;
		TArray<FVertexName> OutputVerticesToFade;

		TArray<TDataReadReference<FAudioBuffer>> WrappedInputAudioBuffers;
		TArray<TDataWriteReference<FAudioBuffer>> FadedInputAudioBuffers;
		TArray<TDataReadReference<FAudioBuffer>> WrappedOutputAudioBuffers;
		TArray<TDataWriteReference<FAudioBuffer>> FadedOutputAudioBuffers;

		EFadeState InitialFadeState;
		EFadeState FadeState;
	};
}
