// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDynamicOperatorTransactor.h"

#include "Algo/Transform.h"
#include "AutoRTFM.h"
#include "Containers/SpscQueue.h"
#include "Containers/UnrealString.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReference.h"
#include "MetasoundDynamicGraphAlgo.h"
#include "MetasoundDynamicOperator.h"
#include "MetasoundGraph.h"
#include "MetasoundGraphAlgo.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorBuilderSettings.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundTrace.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	namespace DynamicGraph
	{
		namespace DynamicOperatorTransactorPrivate
		{
			// Return operator builder settings appropriate for building subgraphs
			// of a dynamic operaotr.
			FOperatorBuilderSettings GetOperatorBuilderSettings()
			{
				FOperatorBuilderSettings Settings = FOperatorBuilderSettings::GetDefaultSettings();

				// Subgraphs must be rebindable to support connecting and disconnecting
				// data references to subgraphs. 
				Settings.bEnableOperatorRebind = true;

				return Settings;
			}

			// Literal nodes always have output vertex with this name. 
			static const FLazyName LiteralNodeOutputVertexName("Value");
			
			FString GetDebugNodeNameString(const INode& InNode)
			{
				return FString::Printf(TEXT("%s_v%d.%d"), *InNode.GetMetadata().ClassName.ToString(), InNode.GetMetadata().MajorVersion, InNode.GetMetadata().MinorVersion);
			}

			FString GetDebugNodeNameString(const FGuid& InNodeID, const INode& InNode)
			{
				return FString::Printf(TEXT("%s:%s"), *InNodeID.ToString(), *GetDebugNodeNameString(InNode));
			}
			


		}

#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
		namespace Debug
		{
			// Represents an edge between two operators. Also supports sorting
			// and special logic for when we do not know the vertex name.
			// 
			// This struct allows for easy comparison between two arrays of operator edges.
			struct FOperatorEdge
			{
				FOperatorID FromOperator;
				FOperatorID ToOperator;
				FVertexName FromVertex;
				FVertexName ToVertex;

				friend bool operator<(const FOperatorEdge& InLHS, const FOperatorEdge& InRHS)
				{
					if (InLHS.FromOperator == InRHS.FromOperator)
					{
						if (InLHS.ToOperator == InRHS.ToOperator)
						{
							if (InLHS.FromVertex.IsNone() || InRHS.ToVertex.IsNone())
							{
								// For scenarios where we don't have a vertex name, 
								// assume that they are equivalent vertex names
								return false;
							} 
							else
							{
								if (InLHS.FromVertex == InRHS.FromVertex)
								{
									return InLHS.ToVertex.FastLess(InRHS.ToVertex);
								}
								else
								{
									return InLHS.FromVertex.FastLess(InRHS.FromVertex);
								}
							}
						}
						else
						{
							return InLHS.ToOperator < InRHS.ToOperator;
						}
					}
					else
					{
						return InLHS.FromOperator < InRHS.FromOperator;
					}
				}
			};

			/** This debugger compares the multiple graph representations that exist to ensure that they
			  * are all the same. 
			  * 
			  * Note: These could be free functions, but it is easy to implement friendship on a class level
			  *        as opposed to making multiple friend functions.
			  */
			class FDynamicOperatorDebugger
			{
			public:
				// Get operator edges from an IGraph
				static TArray<FOperatorEdge> FindGraphOperatorEdges(const IGraph& InGraph)
				{
					auto DataEdgeToOperatorEdge = [](const FDataEdge& InDataEdge) -> FOperatorEdge
					{
						return FOperatorEdge 
						{
							DirectedGraphAlgo::GetOperatorID(InDataEdge.From.Node),
							DirectedGraphAlgo::GetOperatorID(InDataEdge.To.Node),
							InDataEdge.From.Vertex.VertexName,
							InDataEdge.To.Vertex.VertexName
						};
					};

					auto IsValidEdge = [](const FDataEdge& InDataEdge) -> bool
					{
						return (InDataEdge.From.Node != nullptr) && (InDataEdge.To.Node != nullptr);
					};

					TArray<FOperatorEdge> OperatorEdges;
					Algo::TransformIf(InGraph.GetDataEdges(), OperatorEdges, IsValidEdge, DataEdgeToOperatorEdge);
					return OperatorEdges;
				}
				
				// Get operator edges from an incremental graph sorter
				static TArray<FOperatorEdge> FindGraphOperatorEdges(const FDynamicGraphIncrementalSorter& InGraphSorter)
				{
					TArray<FOperatorEdge> OperatorEdges;

					for (const TPair<FOperatorID, FDynamicGraphIncrementalSorter::FIncrementalSortOperatorInfo>& Pair : InGraphSorter.OperatorMap)
					{
						for (const FOperatorID& Output : Pair.Value.Outputs)
						{
							// We do not know what the vertex names are within the sorter
							// so we leave them as NONE.
							OperatorEdges.Add(FOperatorEdge{Pair.Key, Output}); 
						}
					}

					return OperatorEdges;
				}

				// Get operator eges from Graph Operator Data
				static TArray<FOperatorEdge> FindGraphOperatorEdges(const DirectedGraphAlgo::FGraphOperatorData& InGraphData)
				{
					using namespace DirectedGraphAlgo;

					TArray<FOperatorEdge> OperatorEdges;

					for (const TPair<FOperatorID, FGraphOperatorData::FOperatorInfo>& OpInfoPair : InGraphData.OperatorMap)
					{
						for (const TPair<FVertexName, TArray<FGraphOperatorData::FVertexDestination>>& OutConnectPair : OpInfoPair.Value.OutputConnections)
						{
							for (const FGraphOperatorData::FVertexDestination& Destination : OutConnectPair.Value)
							{
								OperatorEdges.Add(FOperatorEdge{OpInfoPair.Key, Destination.OperatorID, OutConnectPair.Key, Destination.VertexName});
							}
						}
					}

					return OperatorEdges;
				}

				static void LogMissingEdge(const TCHAR* InWhereIsMissing, const FOperatorEdge& InEdge)
				{
					UE_LOG(LogMetaSound, Display, TEXT("%s Missing Edge %" UPTRINT_FMT " %s -> %" UPTRINT_FMT " %s"), InWhereIsMissing, InEdge.FromOperator, *InEdge.FromVertex.ToString(), InEdge.ToOperator, *InEdge.ToVertex.ToString());
				}

				// Returns true if arrays are equivalent, false otherwise. 
				// Differences are logged.
				// Input arrays must be sorted. 
				static bool CompareAndLogEdgeArrays(const TCHAR* InSourceA, const TArray<FOperatorEdge>& InEdgesA, const TCHAR* InSourceB, const TArray<FOperatorEdge>& InEdgesB)
				{
					bool bEqual = true;

					const FOperatorEdge* EdgeA = InEdgesA.GetData();
					const FOperatorEdge* const EdgeAEnd = EdgeA + InEdgesA.Num();
					const FOperatorEdge* EdgeB = InEdgesB.GetData();
					const FOperatorEdge* const EdgeBEnd = EdgeB + InEdgesB.Num();

					// Increment through both arrays simultaneously.
					while ((EdgeA != EdgeAEnd) && (EdgeB != EdgeBEnd))
					{
						if (*EdgeA < *EdgeB)
						{
							LogMissingEdge(InSourceB, *EdgeA);
							EdgeA++;
							bEqual = false;
						}
						else if (*EdgeB < *EdgeA)
						{
							LogMissingEdge(InSourceA, *EdgeB);
							EdgeB++;
							bEqual = false;
						}
						else
						{
							// Matching
							EdgeA++;
							EdgeB++;
						}
					}

					// Any left over edges are unmatched.
					while (EdgeA != EdgeAEnd)
					{
						LogMissingEdge(InSourceB, *EdgeA);
						EdgeA++;
						bEqual = false;
					}

					while (EdgeB != EdgeBEnd)
					{
						LogMissingEdge(InSourceA, *EdgeB);
						EdgeB++;
						bEqual = false;
					}

					return bEqual;
				}

				// Returns true if the graph representations are equivalent, false otherwise.
				// Differences are logged.
				static bool CompareAndLogGraphRepresentationDiscrepancies(const FDynamicOperatorTransactor& InTransactor, const FDynamicOperator& InDynamicOperator)
				{
					using namespace Debug;

					// Gather edges from various graph representations
					TArray<FOperatorEdge> GraphEdges = FindGraphOperatorEdges(InTransactor.Graph);
					TArray<FOperatorEdge> GraphSorterEdges = FindGraphOperatorEdges(InTransactor.GraphSorter);
					TArray<FOperatorEdge> RuntimeEdges = FindGraphOperatorEdges(InDynamicOperator.DynamicOperatorData);

					// Edges must be sorted before performing comparison
					GraphEdges.Sort();
					GraphSorterEdges.Sort();
					RuntimeEdges.Sort();

					// Run comparison
					bool bEqualGraphAndSorter = CompareAndLogEdgeArrays(TEXT("Graph"), GraphEdges, TEXT("GraphSorter"), GraphSorterEdges);
					bool bEqualGraphAndRuntime = CompareAndLogEdgeArrays(TEXT("Graph"), GraphEdges, TEXT("DynamicRuntime"), RuntimeEdges);
					bool bEqualRuntimeAndSorter = CompareAndLogEdgeArrays(TEXT("DynamicRuntime"), RuntimeEdges, TEXT("GraphSorter"), GraphSorterEdges);

					return bEqualGraphAndSorter && bEqualGraphAndRuntime && bEqualRuntimeAndSorter;
				}
			};

			// Returns true if the graph representations are equivalent, false otherwise.
			// Differences are logged.
			bool CompareAndLogGraphRepresentationDiscrepancies(const FDynamicOperatorTransactor& InTransactor, const FDynamicOperator& InDynamicOperator)
			{
				return FDynamicOperatorDebugger::CompareAndLogGraphRepresentationDiscrepancies(InTransactor, InDynamicOperator);
			}
		}
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR

		bool operator<(const FDynamicOperatorTransactor::FLiteralNodeID& InLHS, const FDynamicOperatorTransactor::FLiteralNodeID& InRHS)
		{
			if (InLHS.ToNode < InRHS.ToNode)
			{
				return true;
			}
			else if (InRHS.ToNode < InLHS.ToNode)
			{
				return false;
			}
			else
			{
				return InLHS.ToVertex.FastLess(InRHS.ToVertex);
			}
		}

		FDynamicGraphIncrementalSorter::FDynamicGraphIncrementalSorter()
		{
		}

		FDynamicGraphIncrementalSorter::FDynamicGraphIncrementalSorter(const FGraph& InGraph)
		{
			Init(InGraph);
		}

		/** Add a node to the graph. */
		int32 FDynamicGraphIncrementalSorter::InsertOperator(FOperatorID InOperator, FDynamicGraphIncrementalSorter::EInsertLocation InLocation)
		{
			int32 NewOrdinal = ORDINAL_NONE;

			switch (InLocation)
			{
				case EInsertLocation::First:
					MinOrdinal--;
					NewOrdinal = MinOrdinal;
					break;

				case EInsertLocation::Last:
					MaxOrdinal++;
					NewOrdinal = MaxOrdinal;
					break;
			}

			check(NewOrdinal != ORDINAL_NONE);

			if (FIncrementalSortOperatorInfo* Info = OperatorMap.Find(InOperator))
			{
				UE_LOG(LogMetaSound, Warning, TEXT("Attempt to add operator %" UPTRINT_FMT " when operator already exists"), InOperator);
				Info->Ordinal = NewOrdinal;
			}
			else
			{	
				OperatorMap.Add(InOperator, { NewOrdinal });
			}

			return NewOrdinal;
		}

		/** Remove a node from the graph. */
		int32 FDynamicGraphIncrementalSorter::RemoveOperator(FOperatorID InOperatorID)
		{
			int32 RemovedOrdinal = ORDINAL_NONE;

			if (FIncrementalSortOperatorInfo* Info = OperatorMap.Find(InOperatorID))
			{
				RemovedOrdinal = Info->Ordinal;

				// Remove any remaining connections
				for (FOperatorID InputOperatorID : Info->Inputs)
				{
					if (FIncrementalSortOperatorInfo* InputInfo = OperatorMap.Find(InputOperatorID))
					{
						InputInfo->Outputs.RemoveSwap(InOperatorID);
					}
				}

				for (FOperatorID OutputOperatorID : Info->Outputs)
				{
					if (FIncrementalSortOperatorInfo* OutputInfo = OperatorMap.Find(OutputOperatorID))
					{
						OutputInfo->Inputs.RemoveSwap(InOperatorID);
					}
				}
				
				// Remove operator
				OperatorMap.Remove(InOperatorID);
			}

			return RemovedOrdinal;
		}

		void FDynamicGraphIncrementalSorter::GenerateOrdinals(TMap<FOperatorID, int32>& OutOrdinals) const
		{
			for (const TPair<FOperatorID, FIncrementalSortOperatorInfo>& Entry : OperatorMap)
			{
				OutOrdinals.Emplace(Entry.Key, Entry.Value.Ordinal);
			}
		}

		/** Add an edge to the graph, connecting two vertices from two 
		 * nodes. 
		 *
		 * @param InFromOperatorID - Operator which contains the output vertex.
		 * @param InToOperatorID - Operator which contains the input vertex.
		 * @param OutOrdinalUpdates - Array to populate with ordinal updates to maintain topological sort of graph.
		 */
		void FDynamicGraphIncrementalSorter::AddDataEdge(FOperatorID InFromOperatorID, FOperatorID InToOperatorID, TArray<FOrdinalSwap>& OutOrdinalUpdates)
		{
			// Vertex names are not stored here, but if there are multiple edges 
			// conneting two operators, then there will be multiple entries in the 
			// FIncrementalSortOperatorInfo::Outputs & FIncrementalSortOperatorInfo::Inputs arrays. 	
			FIncrementalSortOperatorInfo* FromInfo = OperatorMap.Find(InFromOperatorID);
			if (!FromInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find source operator ID %" UPTRINT_FMT " for adding edge. Dynamic MetaSound may not render properly."), InFromOperatorID);
				return;
			}

			FIncrementalSortOperatorInfo* ToInfo = OperatorMap.Find(InToOperatorID);
			if (!ToInfo)
			{
				UE_LOG(LogMetaSound, Error, TEXT("Could not find destination operator ID %" UPTRINT_FMT " for adding edge. Dynamic MetaSound may not render properly."), InToOperatorID);
				return;
				
			}

			// Add edge to operator info structs
			FromInfo->Outputs.Add(InToOperatorID);
			ToInfo->Inputs.Add(InFromOperatorID);

			// Only sort if the "From" operator isn't executing before the "To" operator. 
			if (FromInfo->Ordinal > ToInfo->Ordinal)
			{
				// Perform incremental sort
				IncrementalTopologicalSortForNewEdge(InFromOperatorID, FromInfo->Ordinal, InToOperatorID, ToInfo->Ordinal, OutOrdinalUpdates);

				// Apply sort changes internally.
				for (const FOrdinalSwap& Update: OutOrdinalUpdates)
				{
					OperatorMap[Update.OperatorID].Ordinal = Update.NewOrdinal;
				}
			}
		}

		/** Remove the given data edge. */
		void FDynamicGraphIncrementalSorter::RemoveDataEdge(FOperatorID InFromOperatorID, FOperatorID InToOperatorID)
		{
			// Vertex names are not stored here, but if there are multiple edges 
			// conneting two operators, then there will be multiple entries in the 
			// FIncrementalSortOperatorInfo::Outputs & FIncrementalSortOperatorInfo::Inputs arrays. 
			if (FIncrementalSortOperatorInfo* FromInfo = OperatorMap.Find(InFromOperatorID))
			{
				FromInfo->Outputs.RemoveSingleSwap(InToOperatorID);
			}
			else
			{
				UE_LOG(LogMetaSound, Warning, TEXT("Could not find source operator ID %" UPTRINT_FMT " for removing edge. Dynamic MetaSound may not render properly."), InFromOperatorID);
			}

			if (FIncrementalSortOperatorInfo* ToInfo = OperatorMap.Find(InToOperatorID))
			{
				ToInfo->Inputs.RemoveSingleSwap(InFromOperatorID);
			}
			else	
			{
				UE_LOG(LogMetaSound, Warning, TEXT("Could not find destination operator ID %" UPTRINT_FMT " for removing edge. Dynamic MetaSound may not render properly."), InToOperatorID);
			}
		}


		void FDynamicGraphIncrementalSorter::Init(const FGraph& InGraph)
		{
			/* determine new operator order. */
			TArray<const INode*> NodeOrder;

			bool bSuccess = DirectedGraphAlgo::DepthFirstTopologicalSort(InGraph, NodeOrder);
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Error, "Cycles found in dynamic graph");
			}

			// Initialize ordinals in operator map
			const int32 Num = NodeOrder.Num();
			const INode** NodeData = NodeOrder.GetData();
			for (int32 Ordinal = 0; Ordinal < Num; Ordinal++)
			{
				OperatorMap.Add(DirectedGraphAlgo::GetOperatorID(*NodeData[Ordinal]), FIncrementalSortOperatorInfo{ Ordinal });
			}

			// The next available ordinal for any operators added after initialization. 
			MaxOrdinal = Num;

			// Initialize edges in operator map
			for (const FDataEdge& Edge : InGraph.GetDataEdges())
			{
				FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(Edge.From.Node);
				FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(Edge.To.Node);

				if (FIncrementalSortOperatorInfo* FromInfo = OperatorMap.Find(FromOperatorID))
				{
					if (FIncrementalSortOperatorInfo* ToInfo = OperatorMap.Find(ToOperatorID))
					{
						FromInfo->Outputs.Add(ToOperatorID);
						ToInfo->Inputs.Add(FromOperatorID);
					}
				}

			}

			// Unconnected nodes are intentionally skipped by the operator builder
			// in order to reduce perf. In a dynamic operator, these nodes may be
			// connected in the future. We add them here so that they exist on in
			// incremental sorter in the case they are connected at a later time.
			TArray<TPair<FGuid, const INode*>> UnconnectedNodes;
			if (InGraph.FindUnconnectedNodes(UnconnectedNodes) > 0)
			{
				for (const TPair<FGuid, const INode*>& GuidAndNode : UnconnectedNodes)
				{
					InsertOperator(DirectedGraphAlgo::GetOperatorID(GuidAndNode.Value), FDynamicGraphIncrementalSorter::EInsertLocation::Last);
				}
			}
		}

		void FDynamicGraphIncrementalSorter::IncrementalTopologicalSortForNewEdge(FOperatorID InFromOperatorID, int32 InFromOrdinal, FOperatorID InToOperatorID, int32 InToOrdinal, TArray<FOrdinalSwap>& OutUpdates)
		{
			UE_CLOGF(InToOrdinal > InFromOrdinal, LogMetaSound, Warning, "Operators are already in order. Only perform incremental sort if necessary.");


			// Incremental topological sort maintains that the "FromOperator"
			// is sorted before the "ToOperator", but does so in an incremental way 
			// to avoid resorting the entire graph. To achieve this it:
			
			// 1. Searches for all the operators and their ordinals which need to be 
			//    resorted..
			// 2. Sorts the operators appropriately, but only using the set of ordinals 
			//    already associated with the operators of interest.
			
			TArray<FOperatorID> SearchStack;
			
			// Find operators which need to be move before the "ToOperator"
			// 
			// Starting with the "FromOperator", find all operators which are:
			//  - Directly or indirectly connected to the input of the "FromOperator"
			//  AND
			//  - Are set to execute after the "ToOperator". 
			SearchStack.Add(InFromOperatorID);
			while (SearchStack.Num())
			{
				FOperatorID CandidateOperatorID = SearchStack.Pop();
				if (!OutUpdates.Contains(CandidateOperatorID))
				{
					const FIncrementalSortOperatorInfo& CandidateOperatorInfo = OperatorMap[CandidateOperatorID];
					if (CandidateOperatorInfo.Ordinal > InToOrdinal)
					{
						OutUpdates.Add({CandidateOperatorID, CandidateOperatorInfo.Ordinal, CandidateOperatorInfo.Ordinal});
						for (const FOperatorID& Connection : CandidateOperatorInfo.Inputs)
						{
							SearchStack.AddUnique(Connection);
						}
					}
				}
			}


			const int32 NumOutputsLessThanOrdinal = OutUpdates.Num();
			OutUpdates.Sort(FOrdinalSwap::OriginalOrdinalLessThan); // Sort by ascending original ordinal to maintain relative order

			// Find operators which need to be moved after the "FromOperator"
			// 
			// Starting with the "ToOperator", find all operators which are:
			//  - Directly or indirectly connected to the output of the "ToOperator"
			//  AND
			//  - Are set to execute before the "FromOperator". 
			SearchStack.Add(InToOperatorID);
			while (SearchStack.Num())
			{
				FOperatorID CandidateOperatorID = SearchStack.Pop();
				if (!OutUpdates.Contains(CandidateOperatorID))
				{
					const FIncrementalSortOperatorInfo& CandidateOperatorInfo = OperatorMap[CandidateOperatorID];
					if (CandidateOperatorInfo.Ordinal < InFromOrdinal)
					{
						OutUpdates.Add({CandidateOperatorID, CandidateOperatorInfo.Ordinal, CandidateOperatorInfo.Ordinal});
						for (const FOperatorID& Connection : CandidateOperatorInfo.Outputs)
						{
							SearchStack.AddUnique(Connection);
						}
					}
				}
			}
			const int32 NumInputsGreaterThanOrdinal = OutUpdates.Num() - NumOutputsLessThanOrdinal;

			// Sort 2nd half of operators in ascending order w/o modifying order of 1st set of operators. 
			// After this sort, the OutUpdates array will contain operators in the desired
			// order but with incorrect "ordinal" values. 
			Algo::Sort(TArrayView<FOrdinalSwap>(&OutUpdates[NumOutputsLessThanOrdinal], NumInputsGreaterThanOrdinal), FOrdinalSwap::OriginalOrdinalLessThan);

			// Gather the available ordinals and sort them in ascending order
			TArray<int32> AvailableOrdinals;
			Algo::Transform(OutUpdates, AvailableOrdinals, [](const FOrdinalSwap& InOperatorOrdinal) { return InOperatorOrdinal.OriginalOrdinal; });
			AvailableOrdinals.Sort();
			
			// Assign the sorted ordinals to the sorted operators. 
			const int32 Num = OutUpdates.Num();
			const int32* AvailableOrdinalData = AvailableOrdinals.GetData();
			FOrdinalSwap* UpdateData = OutUpdates.GetData();

			for (int32 i = 0; i < Num; i++)
			{
				UpdateData[i].NewOrdinal = AvailableOrdinalData[i];
			}

			// Sort all updates by original ordinal to support sorting algorithm
			OutUpdates.Sort(FOrdinalSwap::OriginalOrdinalLessThan);
		}

		FDynamicOperatorTransactor::FDynamicOperatorTransactor(const FGraph& InGraph)
		: OperatorBuilder(DynamicOperatorTransactorPrivate::GetOperatorBuilderSettings())
		, Graph(InGraph)
		, GraphSorter(InGraph)
		{
		}

		FDynamicOperatorTransactor::FDynamicOperatorTransactor()
		: OperatorBuilder(DynamicOperatorTransactorPrivate::GetOperatorBuilderSettings())
		, Graph(FTopLevelAssetPath(), FGuid())
		{
		}

		TSharedRef<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> FDynamicOperatorTransactor::CreateTransformQueue(const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment)
		{
			TSharedPtr<FGraphRenderCost> GraphRenderCost;
			return CreateTransformQueue(InOperatorSettings, InEnvironment, GraphRenderCost);
		}

		TSharedRef<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> FDynamicOperatorTransactor::CreateTransformQueue(const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment, const TSharedPtr<FGraphRenderCost>& InGraphRenderCost)
		{
			TSharedRef<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> Queue = MakeShared<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>>();
			OperatorInfos.Add(FDynamicOperatorInfo{InOperatorSettings, InEnvironment, InGraphRenderCost, Queue});

			TMap<FOperatorID, int32> OperatorOrdinals;
			GraphSorter.GenerateOrdinals(OperatorOrdinals);

			// All of these initial operations have to happen in one fell swoop for the 
			// dynamic metasound to be setup correctly. We use an atomic transform
			// to ensure they are all applied before generating audio.
			TArray<TUniquePtr<IDynamicOperatorTransform>> AtomicTransforms;

			// Unconnected nodes are intentionally skipped by the operator builder
			// in order to reduce perf. In a dynamic operator, these nodes may be
			// connected in the future. We queue them up to be added here so that
			// they exist on the dynamic operator in the case they are connected at
			// a later time.
			TArray<TPair<FGuid, const INode*>> UnconnectedNodes;
			if (Graph.FindUnconnectedNodes(UnconnectedNodes) > 0)
			{
				for (const TPair<FGuid, const INode*>& GuidAndNode : UnconnectedNodes)
				{
					if (const int32* Ordinal = OperatorOrdinals.Find(DirectedGraphAlgo::GetOperatorID(GuidAndNode.Value)))
					{
						AtomicTransforms.Add(CreateInsertOperatorTransform(*GuidAndNode.Value, *Ordinal, InOperatorSettings, InEnvironment, InGraphRenderCost.Get()));
					}
				}
			}

			// When we create a new Dynamic Operator, the first thing that needs
			// to synchronize is the initial set of operator ordinals. The ordinals
			// in the DynamicOperator and the FDynamicGraphIncrementalSorter must
			// be exactly equal so that subsequent modifications to the ordinals
			// result in equal ordinals between the two objects.
			//
			// Note: The ordinals in FDynamicGraphIncrementalSorter are NOT 
			// expected to be equal to the ordinals set in the FOperatorBuilder
			// because they use different algorithms to determine order.
			AtomicTransforms.Add(MakeUnique<FSetOperatorOrdinalsAndSort>(OperatorOrdinals));

			// Only add to THIS queue because we do not know at what point
			// the other queues and dynamic operators were created.
			Queue->Enqueue(MakeUnique<FAtomicTransform>(MoveTemp(AtomicTransforms)));

			return Queue;
		}

		bool FDynamicOperatorTransactor::AddNode(const FGuid& InNodeID, TUniquePtr<INode> InNode)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::AddNode);

			if (!InNode)
			{
				return false;
			}

			// Cache pointer because InNode gets moved. 
			const INode* NodePtr = InNode.Get();

			Graph.AddNode(InNodeID, MoveTemp(InNode));
			int32 OperatorOrdinal = GraphSorter.InsertOperator(GetOperatorID(NodePtr), FDynamicGraphIncrementalSorter::EInsertLocation::Last);

			EnqueueInsertOperatorTransform(*NodePtr, OperatorOrdinal);
			return true;
		}

		bool FDynamicOperatorTransactor::RemoveNode(const FGuid& InNodeID)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::RemoveNode);

			if (const INode* Node = Graph.FindNode(InNodeID))
			{
				TArray<FVertexName> OutputsToFade;
				const FOutputVertexInterface& OutputInterface = Node->GetVertexInterface().GetOutputInterface();
				for (const FOutputDataVertex& OutputVertex : OutputInterface)
				{
					if (OutputVertex.DataTypeName == GetMetasoundDataTypeName<FAudioBuffer>())
					{
						OutputsToFade.Add(OutputVertex.VertexName);
					}
				}

				constexpr bool bRemoveDataEdgesWithNode = true;
				if (OutputsToFade.Num())
				{
					FadeAndRemoveNodeInternal(*Node, OutputsToFade, bRemoveDataEdgesWithNode);
				}
				else
				{
					RemoveNodeInternal(*Node, bRemoveDataEdgesWithNode);
				}

				return true;
			}

			UE_LOGF(LogMetaSound, Error, "No node found in dynamic transactor graph with ID %ls", *InNodeID.ToString());
			return false;
		}

		/** Add an edge to the graph. */
		void FDynamicOperatorTransactor::AddDataEdge(const FGuid& InFromNodeID, const FVertexName& InFromVertex, const FGuid& InToNodeID, const FVertexName& InToVertex)
		{
			using namespace DirectedGraphAlgo; 

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::AddDataEdge);

			const INode* FromNode = Graph.FindNode(InFromNodeID);
			const INode* ToNode = Graph.FindNode(InToNodeID);

			if ((nullptr == ToNode) || (nullptr == FromNode))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add edge from %ls:%ls to %ls:%ls because of missing node", *InFromNodeID.ToString(), *InFromVertex.ToString(), *InToNodeID.ToString(), *InToVertex.ToString());
				return;
			}

			AddDataEdgeInternal(*FromNode, InFromVertex, InToNodeID, *ToNode, InToVertex);
		}

		void FDynamicOperatorTransactor::RemoveDataEdge(const FGuid& InFromNodeID, const FVertexName& InFromVertex, const FGuid& InToNodeID, const FVertexName& InToVertex, TUniquePtr<INode> InReplacementLiteralNode)
		{
			using namespace DirectedGraphAlgo; 

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::RemoveDataEdge);

			const INode* FromNode = Graph.FindNode(InFromNodeID);
			const INode* ToNode = Graph.FindNode(InToNodeID);
			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(FromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(ToNode);
			const FOperatorID LiteralOperatorID = DirectedGraphAlgo::GetOperatorID(InReplacementLiteralNode.Get());

			if ((nullptr == ToNode) || (nullptr == FromNode))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of missing node", *InFromNodeID.ToString(), *InFromVertex.ToString(), *InToNodeID.ToString(), *InToVertex.ToString());
				return;
			}

			if (!ToNode->GetVertexInterface().ContainsInputVertex(InToVertex))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of destination node does not contain vertex %ls.", *InFromNodeID.ToString(), *InFromVertex.ToString(), *InToNodeID.ToString(), *InToVertex.ToString(), *InToVertex.ToString());
				return;
			}

			if (!InReplacementLiteralNode.IsValid())
			{
				UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of invalid pointer to replacement literal node.", *InFromNodeID.ToString(), *InFromVertex.ToString(), *InToNodeID.ToString(), *InToVertex.ToString());
				return;
			}

			/* remove edge from internal graph. */
			bool bSuccess = Graph.RemoveDataEdge(*FromNode, InFromVertex, *ToNode, InToVertex);
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Error, "Failed to remove edge from %ls:%ls to %ls:%ls on internal graph.", *InFromNodeID.ToString(), *InFromVertex.ToString(), *InToNodeID.ToString(), *InToVertex.ToString());
				return;
			}
			GraphSorter.RemoveDataEdge(FromOperatorID, ToOperatorID);


			bSuccess = Graph.AddDataEdge(*InReplacementLiteralNode, DynamicOperatorTransactorPrivate::LiteralNodeOutputVertexName, *ToNode, InToVertex);
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Error, "Failed to add literal for %ls:%ls on internal graph.", *InToNodeID.ToString(), *InToVertex.ToString());
				return;
			}

			INode& LiteralNodeRef = *InReplacementLiteralNode; // Store ref since pointer will be moved. 
			// Store literal node associated with the target of the literal value.
			LiteralNodeMap.Add(FLiteralNodeID{ InToNodeID, InToVertex }, MoveTemp(InReplacementLiteralNode));

			// Put literals in the front of the execution stack to simplify updating 
			// runtime instances. No need to sort the entire graph if we are just
			// inserting something at the beginning of the execution stack. 
			int32 LiteralOrdinal = GraphSorter.InsertOperator(LiteralOperatorID, FDynamicGraphIncrementalSorter::EInsertLocation::First);
			TArray<FOrdinalSwap> OrdinalSwaps;
			GraphSorter.AddDataEdge(LiteralOperatorID, ToOperatorID, OrdinalSwaps);

			// There should not be any ordinal swaps because the literal operator was inserted before any 
			// other operator and so will always have it's output data ready before the ToOperator executes. 
			check(OrdinalSwaps.Num() == 0); 

			// Immediately disconnect non-audio edges. 
			if (ToNode->GetVertexInterface().GetInputVertex(InToVertex).DataTypeName == GetMetasoundDataTypeName<FAudioBuffer>())
			{
				// Handle audio edge removal with a fade out.
				EnqueueFadeAndRemoveEdgeOperatorTransform_Deprecated(*FromNode, InFromVertex, *ToNode, InToVertex, LiteralNodeRef, LiteralOrdinal);
			}
			else
			{
				EnqueueRemoveEdgeOperatorTransform_Deprecated(*FromNode, InFromVertex, *ToNode, InToVertex, LiteralNodeRef, LiteralOrdinal);
			}
		}

		void FDynamicOperatorTransactor::RemoveDataEdge(const FGuid& InFromNodeID, const FVertexName& InFromVertexName, const FGuid& InToNodeID, const FVertexName& InToVertexName, FLiteral InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc)
		{
			using namespace DirectedGraphAlgo;

			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::RemoveDataEdge);

			const INode* FromNode = Graph.FindNode(InFromNodeID);
			const INode* ToNode = Graph.FindNode(InToNodeID);
			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(FromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(ToNode);

			if ((nullptr == ToNode) || (nullptr == FromNode))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of missing node", *InFromNodeID.ToString(), *InFromVertexName.ToString(), *InToNodeID.ToString(), *InToVertexName.ToString());
				return;
			}

			if (!ToNode->GetVertexInterface().ContainsInputVertex(InToVertexName))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of destination node does not contain vertex %ls.", *InFromNodeID.ToString(), *InFromVertexName.ToString(), *InToNodeID.ToString(), *InToVertexName.ToString(), *InToVertexName.ToString());
				return;
			}

			bool bSuccess = Graph.RemoveDataEdge(*FromNode, InFromVertexName, *ToNode, InToVertexName);
			if (!bSuccess)
			{
				UE_LOGF(LogMetaSound, Error, "Failed to remove edge from %ls:%ls to %ls:%ls on internal graph.", *InFromNodeID.ToString(), *InFromVertexName.ToString(), *InToNodeID.ToString(), *InToVertexName.ToString());
				return;
			}
			GraphSorter.RemoveDataEdge(FromOperatorID, ToOperatorID);

			// Update default stored on the node
			Graph.SetNodeDefaultInput(InToNodeID, InToVertexName, InReplacementLiteral);

			// Immediately disconnect non-audio edges. 
			if (ToNode->GetVertexInterface().GetInputVertex(InToVertexName).DataTypeName == GetMetasoundDataTypeName<FAudioBuffer>())
			{
				// Handle audio edge removal with a fade out.
				EnqueueFadeAndRemoveEdgeOperatorTransform(*FromNode, InFromVertexName, *ToNode, InToVertexName, InReplacementLiteral, InReferenceCreateFunc);
			}
			else
			{
				EnqueueRemoveEdgeOperatorTransform(*FromNode, InFromVertexName, *ToNode, InToVertexName, InReplacementLiteral, InReferenceCreateFunc);
			}
		}

		void FDynamicOperatorTransactor::SetValue(const FGuid& InNodeID, const FVertexName& InVertex, TUniquePtr<INode> InLiteralNode)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::SetValue);

			const INode* Node = Graph.FindNode(InNodeID);
			const INode* LiteralNode = InLiteralNode.Get();

			if (nullptr == Node)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot set node value of %ls:%ls because of missing node", *InNodeID.ToString(), *InVertex.ToString());
				return;
			}

			if (!InLiteralNode.IsValid())
			{
				UE_LOGF(LogMetaSound, Error, "Cannot set value on %ls:%ls because of invalid pointer to literal node.", *InNodeID.ToString(), *InVertex.ToString());
				return;
			}

			// Always insert new literal nodes first in execution order.
			int32 LiteralOrdinal = GraphSorter.InsertOperator(DirectedGraphAlgo::GetOperatorID(LiteralNode), FDynamicGraphIncrementalSorter::EInsertLocation::First);

			auto CreateAddNodeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return CreateInsertOperatorTransform(*LiteralNode, LiteralOrdinal, InInfo.OperatorSettings, InInfo.Environment, InInfo.GraphRenderCost.Get());
			};

			EnqueueTransformOnOperatorQueues(CreateAddNodeTransform);

			AddDataEdgeInternal(*LiteralNode, DynamicOperatorTransactorPrivate::LiteralNodeOutputVertexName, InNodeID, *Node, InVertex);

			// Add literal node after calling "AddDataEdgeInternal" so that AddDataEdgeInternal can check if there is a prior existing literal node.
			LiteralNodeMap.Add(FLiteralNodeID{InNodeID, InVertex }, MoveTemp(InLiteralNode));
		}

		void FDynamicOperatorTransactor::SetValue(const FGuid& InNodeID, const FVertexName& InVertexName, const FLiteral& InLiteral, const FReferenceCreationFunction InReferenceCreateFunc)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::SetValue);

			const INode* Node = Graph.FindNode(InNodeID);
			if (nullptr == Node)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot set node value of %ls:%ls because of missing node", *InNodeID.ToString(), *InVertexName.ToString());
				return;
			}

			Graph.SetNodeDefaultInput(InNodeID, InVertexName, InLiteral);

			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(Node);
			const FInputDataVertex& InputVertex = Node->GetVertexInterface().GetInputVertex(InVertexName);
			EDataReferenceAccessType ReferenceAccessType = InputVertex.AccessType == EVertexAccessType::Value ? EDataReferenceAccessType::Value : EDataReferenceAccessType::Write;

			auto CreateSetValueTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TOptional<FAnyDataReference> NewDataReference = InReferenceCreateFunc(InInfo.OperatorSettings, InputVertex.DataTypeName, InLiteral, ReferenceAccessType);

				if (NewDataReference.IsSet())
				{
					return MakeUnique<FSetOperatorInput>(FSetOperatorInput(
						OperatorID, InVertexName, MoveTemp(*NewDataReference)
					));
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot Set Value %ls:%ls because of failure to create data reference for transform.", *InNodeID.ToString(), *InVertexName.ToString());
					return TUniquePtr<IDynamicOperatorTransform>(nullptr);
				}
			};

			EnqueueTransformOnOperatorQueues(CreateSetValueTransform);
		}

		/** Add an input data destination to describe how data provided 
		 * outside this graph should be routed internally.
		 *
		 * @param InNode - Node which receives the data.
		 * @param InVertexName - Key for input vertex on InNode.
		 *
		 */
		void FDynamicOperatorTransactor::AddInputDataDestination(const FGuid& InNodeID, const FVertexName& InVertexName, const FLiteral& InDefaultLiteral, FReferenceCreationFunction InFunc)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::AddInputDataDestination);

			const INode* Node = Graph.FindNode(InNodeID);
			if (nullptr == Node)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add Input Data Destination %ls:%ls because of missing node", *InNodeID.ToString(), *InVertexName.ToString());
				return;
			}

			if (!Node->GetVertexInterface().ContainsInputVertex(InVertexName))
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add Input Data Destination %ls:%ls because of node does not contain input vertex with name %ls.", *InNodeID.ToString(), *InVertexName.ToString(), *InVertexName.ToString());
				return;
			}

			const FInputDataVertex& InputVertex = Node->GetVertexInterface().GetInputVertex(InVertexName);
			EDataReferenceAccessType ReferenceAccessType = InputVertex.AccessType == EVertexAccessType::Value ? EDataReferenceAccessType::Value : EDataReferenceAccessType::Write;
			FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(Node);

			Graph.AddInputDataDestination(*Node, InVertexName);

			auto CreateAddInputTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TOptional<FAnyDataReference> NewDataReference = InFunc(InInfo.OperatorSettings, InputVertex.DataTypeName, InDefaultLiteral, ReferenceAccessType);
				if (NewDataReference.IsSet())
				{
					return MakeUnique<FAddInput>(OperatorID, InVertexName, MoveTemp(*NewDataReference));
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot add Input Data Destination %ls:%ls because of failure to create data reference.", *InNodeID.ToString(), *InVertexName.ToString());
					return TUniquePtr<IDynamicOperatorTransform>(nullptr);
				}
			};

			EnqueueTransformOnOperatorQueues(CreateAddInputTransform);
		}

		void FDynamicOperatorTransactor::RemoveInputDataDestination(const FVertexName& InVertexName)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::RemoveInputDataDestination);

			Graph.RemoveInputDataDestination(InVertexName);

			auto CreateRemoveInputTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return MakeUnique<FRemoveInput>(InVertexName);
			};

			EnqueueTransformOnOperatorQueues(CreateRemoveInputTransform);
		}

		/** Add an output data source which describes routing of data which is 
		 * owned this graph and exposed externally.
		 *
		 * @param InNode - Node which produces the data.
		 * @param InVertexName - Key for output vertex on InNode.
		 */
		void FDynamicOperatorTransactor::AddOutputDataSource(const FGuid& InNodeID, const FVertexName& InVertexName)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::AddOutputDataSource);

			const INode* Node = Graph.FindNode(InNodeID);
			if (nullptr == Node)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot add Output Data Source %ls:%ls because of missing node", *InNodeID.ToString(), *InVertexName.ToString());
				return;
			}

			Graph.AddOutputDataSource(*Node, InVertexName);
			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(Node);

			auto CreateAddOutputTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return MakeUnique<FAddOutput>(OperatorID, InVertexName);
			};

			EnqueueTransformOnOperatorQueues(CreateAddOutputTransform);
		}

		void FDynamicOperatorTransactor::RemoveOutputDataSource(const FVertexName& InVertexName)
		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FDynamicOperatorTransactor::RemoveOutputDataSource);

			Graph.RemoveOutputDataSource(InVertexName);

			auto CreateRemoveOutputTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return MakeUnique<FRemoveOutput>(InVertexName);
			};

			EnqueueTransformOnOperatorQueues(CreateRemoveOutputTransform);
		}

		const FGraph& FDynamicOperatorTransactor::GetGraph() const
		{
			return Graph;
		}

		void FDynamicOperatorTransactor::AddDataEdgeInternal(const INode& InFromNode, const FVertexName& InFromVertex, const FGuid& InToNodeID, const INode& InToNode, const FVertexName& InToVertex)
		{
			using namespace DirectedGraphAlgo;
			using namespace DynamicOperatorTransactorPrivate;

			const FInputDataVertex* InputVertex = InToNode.GetVertexInterface().GetInputInterface().Find(InToVertex);
			if (nullptr == InputVertex)
			{
				UE_LOGF(LogMetaSound, Error, "Cannot connect nodes because destination node %ls does not contain input vertex %ls", *GetDebugNodeNameString(InToNodeID, InToNode), *InToVertex.ToString());
				return;
			}

			/* Determine if there is an existing literal node connected to the node. 
			 * Literal nodes are stored on the FDynamicOperatorTransactor and need to be 
			 * disconnected and removed if they are no longer being used. 
			 */
			TUniquePtr<INode> PriorLiteralNode;
			bool bPriorLiteralNodeExists = LiteralNodeMap.RemoveAndCopyValue(FLiteralNodeID{InToNodeID, InToVertex}, PriorLiteralNode);

			// Get relevant operator IDs.
			const FOperatorID PriorLiteralOperatorID = DirectedGraphAlgo::GetOperatorID(PriorLiteralNode.Get());
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(InToNode);
			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(InFromNode);

			// Update edges on internal graph
			if (bPriorLiteralNodeExists)
			{
				Graph.RemoveDataEdge(*PriorLiteralNode, DynamicOperatorTransactorPrivate::LiteralNodeOutputVertexName, InToNode, InToVertex);
				GraphSorter.RemoveDataEdge(PriorLiteralOperatorID, ToOperatorID);
			}
			Graph.AddDataEdge(InFromNode, InFromVertex, InToNode, InToVertex);
			TArray<FOrdinalSwap> OrdinalUpdates;
			GraphSorter.AddDataEdge(FromOperatorID, ToOperatorID, OrdinalUpdates);
		
			if (bPriorLiteralNodeExists)
			{
				// The Graph does not maintain literal nodes so there is no need to remove the PriorLiteralNode from the Graph. Those are
				// managed in the LiteralNodeMap. But, the GraphSorter DOES maintain literal operators, and so we have to still remove
				// the literal operator from the GraphSorter.
				GraphSorter.RemoveOperator(PriorLiteralOperatorID);
			}

			if (InputVertex->DataTypeName == GetMetasoundDataTypeName<FAudioBuffer>())
			{
				// If edge is audio, then the connection needs to be faded
				EnqueueFadeAndAddEdgeOperatorTransform(InFromNode, InFromVertex, InToNode, InToVertex, OrdinalUpdates);
			}
			else
			{
				// If the edge is not audio, then no fading is performed. 
				EnqueueAddEdgeOperatorTransform(InFromNode, InFromVertex, InToNode, InToVertex, OrdinalUpdates);
			}
		}
		
		void FDynamicOperatorTransactor::EnqueueInsertOperatorTransform(const INode& InNode, int32 InOrdinal)
		{
			auto CreateAddNodeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return CreateInsertOperatorTransform(InNode, InOrdinal, InInfo.OperatorSettings, InInfo.Environment, InInfo.GraphRenderCost.Get());
			};

			EnqueueTransformOnOperatorQueues(CreateAddNodeTransform);
		}

		void FDynamicOperatorTransactor::RemoveNodeInternal(const INode& InNode, bool bInRemoveDataEdgesWithNode)
		{
			// Find any existing edges and remove them from the graph
			TArray<FOperatorID> OperatorsConnectedToInput;
			if (bInRemoveDataEdgesWithNode)
			{
				for (const FDataEdge& Edge : Graph.GetDataEdges())
				{
					if ((Edge.From.Node == &InNode) || (Edge.To.Node == &InNode))
					{
						if ((Edge.From.Node != nullptr) && (Edge.To.Node != nullptr))
						{
							// Remove edge from graph sorter
							GraphSorter.RemoveDataEdge(DirectedGraphAlgo::GetOperatorID(Edge.From.Node), DirectedGraphAlgo::GetOperatorID(Edge.To.Node));

							if (Edge.To.Node == &InNode)
							{
								// Track all incoming edges to remove them from dynamic operator's model.
								OperatorsConnectedToInput.AddUnique(DirectedGraphAlgo::GetOperatorID(Edge.From.Node));
							}
						}
					}
				}
			}

			
			GraphSorter.RemoveOperator(DirectedGraphAlgo::GetOperatorID(InNode));

			// that do not exist will stay on the runtime model. 
			// There's an open question about whether we should fade disconnections. We probably should, 
			// but we lack the literal node's to do so. 
			EnqueueRemoveOperatorTransform(InNode, OperatorsConnectedToInput);

			bool bRemovedNodeFromGraph = Graph.RemoveNode(InNode.GetInstanceID(), bInRemoveDataEdgesWithNode);
			check(bRemovedNodeFromGraph); //< Should always be true because callers of RemoveNodeInternal() ensure that the node exists in graph.
		}

		void FDynamicOperatorTransactor::FadeAndRemoveNodeInternal(const INode& InNode, TArrayView<const FVertexName> InOutputsToFade, bool bInRemoveDataEdgesWithNode)
		{
			TArrayView<const FVertexName> InputsToFade; // We do not need to fade any inputs when removing a node.

			EnqueueBeginFadeOperatorTransform(InNode, EAudioFadeType::FadeOut, InputsToFade, InOutputsToFade);

			// We can skip the FEndAudioFadeTransform as an optimization here.
			// FEndAudioFadeTransform removes the fading wrapper around the node,
			// but since the node is being removed, we can remove the wrapper
			// and the node with a single FRemoveOperator transform.
			RemoveNodeInternal(InNode, bInRemoveDataEdgesWithNode);
		}

		void FDynamicOperatorTransactor::EnqueueAddEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertex, const INode& InToNode, const FVertexName& InToVertex, const TArray<FOrdinalSwap>& InOrdinalUpdates)
		{
			/* enqueue an update. */
			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(InFromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(InToNode);

			// Create transforms for runtime
			auto CreateAddEdgeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TArray<TUniquePtr<IDynamicOperatorTransform>> AtomicTransforms;

				if (InOrdinalUpdates.Num() > 0)
				{
					AtomicTransforms.Add(MakeUnique<FSwapOperatorOrdinalsAndSort>(InOrdinalUpdates));
				}

				AtomicTransforms.Add(MakeUnique<FConnectOperators>(FromOperatorID, InFromVertex, ToOperatorID, InToVertex));

				return MakeUnique<FAtomicTransform>(MoveTemp(AtomicTransforms));
			};

			EnqueueTransformOnOperatorQueues(CreateAddEdgeTransform);
		}

		void FDynamicOperatorTransactor::EnqueueFadeAndAddEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertex, const INode& InToNode, const FVertexName& InToVertex, const TArray<FOrdinalSwap>& InOrdinalUpdates)
		{
			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(InFromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(InToNode);

			// Fade inputs on the receiving node when adding an edge. We don't fade the source node's outputs
			// because those outputs could also be connected to other nodes which we do not want to fade. 
			TArrayView<const FVertexName> InputsToFade(&InToVertex, 1);
			TArrayView<const FVertexName> OutputsToFade;

			auto CreateBeginFadeAndAddEdgeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TArray<TUniquePtr<IDynamicOperatorTransform>> AtomicTransforms;

				
				if (InOrdinalUpdates.Num() > 0)
				{
					AtomicTransforms.Add(MakeUnique<FSwapOperatorOrdinalsAndSort>(InOrdinalUpdates));
				}
				
				AtomicTransforms.Add(MakeUnique<FConnectOperators>(FromOperatorID, InFromVertex, ToOperatorID, InToVertex));

				AtomicTransforms.Add(MakeUnique<FBeginAudioFadeTransform>(ToOperatorID, EAudioFadeType::FadeIn, InputsToFade, OutputsToFade));
				
				// Fence must be the last transform since the fade must be performed
				// before anything else happens in the graph. To apply a fade, the 
				// graph must execute. This FExecuteFence transform ensures that the graph
				// is executed before any additional transforms are applied. 
				AtomicTransforms.Add(MakeUnique<FExecuteFence>());

				return MakeUnique<FAtomicTransform>(MoveTemp(AtomicTransforms));
			};

			EnqueueTransformOnOperatorQueues(CreateBeginFadeAndAddEdgeTransform);

			EnqueueEndFadeOperatorTransform(InToNode);
		}

		void FDynamicOperatorTransactor::EnqueueBeginFadeOperatorTransform(const INode& InNode, EAudioFadeType InFadeType, TArrayView<const FVertexName> InInputsToFade, TArrayView<const FVertexName> InOutputsToFade)
		{
			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(InNode);

			auto CreateBeginAudioFadeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TArray<TUniquePtr<IDynamicOperatorTransform>> BeginAudioFadeAtomicTransforms;

				BeginAudioFadeAtomicTransforms.Add(MakeUnique<FBeginAudioFadeTransform>(OperatorID, InFadeType, InInputsToFade, InOutputsToFade));
				// Fence must be the last transform since the fade must be performed
				// before anything else happens in the graph. To apply a fade, the 
				// graph must execute. This FExecuteFence transform ensures that the graph
				// is executed before any additional transforms are applied. 
				BeginAudioFadeAtomicTransforms.Add(MakeUnique<FExecuteFence>());

				return MakeUnique<FAtomicTransform>(MoveTemp(BeginAudioFadeAtomicTransforms));
			};

			EnqueueTransformOnOperatorQueues(CreateBeginAudioFadeTransform);
		}

		void FDynamicOperatorTransactor::EnqueueEndFadeOperatorTransform(const INode& InNode)
		{
			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(InNode);

			auto CreateEndAudioFadeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return MakeUnique<FEndAudioFadeTransform>(OperatorID);
			};

			EnqueueTransformOnOperatorQueues(CreateEndAudioFadeTransform);
		}

		void FDynamicOperatorTransactor::EnqueueRemoveOperatorTransform(const INode& InNode, const TArray<FOperatorID>& InOperatorsConnectedToInput)
		{
			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(InNode);

			auto CreateRemoveNodeTransform = [&OperatorID, &InOperatorsConnectedToInput](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				return MakeUnique<FRemoveOperator>(OperatorID, InOperatorsConnectedToInput);
			};
			EnqueueTransformOnOperatorQueues(CreateRemoveNodeTransform);
		}

		void FDynamicOperatorTransactor::EnqueueRemoveEdgeOperatorTransform_Deprecated(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const INode& InReplacementLiteralNode, int32 InLiteralOrdinal)
		{
			using namespace DynamicOperatorTransactorPrivate;

			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(InFromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(InToNode);
			const FOperatorID LiteralOperatorID = DirectedGraphAlgo::GetOperatorID(InReplacementLiteralNode);

			auto CreateRemoveEdgeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				// Add the literal node.
				TUniquePtr<IDynamicOperatorTransform> AddNodeTransform = CreateInsertOperatorTransform(InReplacementLiteralNode, InLiteralOrdinal, InInfo.OperatorSettings, InInfo.Environment, InInfo.GraphRenderCost.Get());

				// Swap prior connection with new connections.
				TUniquePtr<IDynamicOperatorTransform> ConnectOperatorsTransform = MakeUnique<FSwapOperatorConnection>(FromOperatorID, InFromVertexName, LiteralOperatorID, DynamicOperatorTransactorPrivate::LiteralNodeOutputVertexName, ToOperatorID, InToVertexName);

				if (AddNodeTransform.IsValid() && ConnectOperatorsTransform.IsValid())
				{
					// Create an atomic transform so all sub-transforms happen before next execution.
					TArray<TUniquePtr<IDynamicOperatorTransform>> AtomicTransforms;

					AtomicTransforms.Emplace(MoveTemp(AddNodeTransform));
					AtomicTransforms.Emplace(MoveTemp(ConnectOperatorsTransform));

					return MakeUnique<FAtomicTransform>(MoveTemp(AtomicTransforms));
				}
				else
				{

					UE_LOGF(LogMetaSound, Error, "Cannot remove edge from %ls:%ls to %ls:%ls because of failure to create all transforms needed to perform operatorn.", *GetDebugNodeNameString(InFromNode), *InFromVertexName.ToString(), *GetDebugNodeNameString(InToNode), *InToVertexName.ToString());
					return TUniquePtr<IDynamicOperatorTransform>(nullptr);
				}
			};

			EnqueueTransformOnOperatorQueues(CreateRemoveEdgeTransform);
		}
		
		void FDynamicOperatorTransactor::EnqueueRemoveEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const FLiteral& InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc)
		{
			using namespace DynamicOperatorTransactorPrivate;

			const FOperatorID FromOperatorID = DirectedGraphAlgo::GetOperatorID(InFromNode);
			const FOperatorID ToOperatorID = DirectedGraphAlgo::GetOperatorID(InToNode);
			const FInputDataVertex& InputVertex = InToNode.GetVertexInterface().GetInputVertex(InToVertexName);
			EDataReferenceAccessType ReferenceAccessType = InputVertex.AccessType == EVertexAccessType::Value ? EDataReferenceAccessType::Value : EDataReferenceAccessType::Write;

			auto CreateRemoveEdgeTransform = [&](const FDynamicOperatorInfo& InInfo) -> TUniquePtr<IDynamicOperatorTransform>
			{
				TOptional<FAnyDataReference> NewDataReference = InReferenceCreateFunc(InInfo.OperatorSettings, InputVertex.DataTypeName, InReplacementLiteral, ReferenceAccessType);
				if (NewDataReference.IsSet())
				{
					return MakeUnique<FRemoveOperatorConnection>(FromOperatorID, InFromVertexName, ToOperatorID, InToVertexName, MoveTemp(*NewDataReference));
				}
				else
				{
					UE_LOGF(LogMetaSound, Error, "Cannot add Remove Edge %ls:%ls because of failure to create data reference for replacement literal.", *GetDebugNodeNameString(InToNode), *InToVertexName.ToString());
					return TUniquePtr<IDynamicOperatorTransform>(nullptr);
				}
			};

			EnqueueTransformOnOperatorQueues(CreateRemoveEdgeTransform);
		}
		
		void FDynamicOperatorTransactor::EnqueueFadeAndRemoveEdgeOperatorTransform_Deprecated(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const INode& InReplacementLiteralNode, int32 InLiteralOrdinal)
		{
			// Fade the input to the node getting disconnected rather than the output of the source node. The source node
			// may be connected to other nodes and fading it's output would fade all the other connected nodes' inputs. 
			TArrayView<const FVertexName> InputVerticesToFade(&InToVertexName, 1);
			TArrayView<const FVertexName> OutputVerticesToFade;

			EnqueueBeginFadeOperatorTransform(InToNode, EAudioFadeType::FadeOut, InputVerticesToFade, OutputVerticesToFade);

			// Replace input with literal. This assumes that the replacement audio
			// buffer contains silent audio. The fade transform will get the input
			// audio to silent which will then seamlessly be swapped with a silent
			// audio buffer as a permanent connection. 
			//
			// If we ever find ourselves creating audio buffers with literals which 
			// are anything other than silent buffers, we should rework this operation
			// to do either a cross-fade, or an additional "fade in" to the new value. 
			EnqueueRemoveEdgeOperatorTransform_Deprecated(InFromNode, InFromVertexName, InToNode, InToVertexName, InReplacementLiteralNode, InLiteralOrdinal);

			// Remove fade operation. 
			EnqueueEndFadeOperatorTransform(InToNode);
		}

		void FDynamicOperatorTransactor::EnqueueFadeAndRemoveEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const FLiteral& InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc)
		{
			// Fade the input to the node getting disconnected rather than the output of the source node. The source node
			// may be connected to other nodes and fading it's output would fade all the other connected nodes' inputs. 
			TArrayView<const FVertexName> InputVerticesToFade(&InToVertexName, 1);
			TArrayView<const FVertexName> OutputVerticesToFade;

			EnqueueBeginFadeOperatorTransform(InToNode, EAudioFadeType::FadeOut, InputVerticesToFade, OutputVerticesToFade);

			// Replace input with literal. This assumes that the replacement audio
			// buffer contains silent audio. The fade transform will get the input
			// audio to silent which will then seamlessly be swapped with a silent
			// audio buffer as a permanent connection. 
			//
			// If we ever find ourselves creating audio buffers with literals which 
			// are anything other than silent buffers, we should rework this operation
			// to do either a cross-fade, or an additional "fade in" to the new value. 
			EnqueueRemoveEdgeOperatorTransform(InFromNode, InFromVertexName, InToNode, InToVertexName, InReplacementLiteral, InReferenceCreateFunc);

			// Remove fade operation. 
			EnqueueEndFadeOperatorTransform(InToNode);
		}

		TUniquePtr<IDynamicOperatorTransform> FDynamicOperatorTransactor::CreateInsertOperatorTransform(const INode& InNode, int32 InOrdinal, const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment, FGraphRenderCost* InGraphRenderCost) const
		{
			using namespace DynamicOperatorTransactorPrivate;

			const FOperatorID OperatorID = DirectedGraphAlgo::GetOperatorID(InNode);
			FVertexInterfaceData InterfaceData(InNode.GetVertexInterface());
			FBuildOperatorParams OperatorParams
			{
				InNode,
				InOperatorSettings,
				InterfaceData.GetInputs(),
				InEnvironment,
				&OperatorBuilder, // Supply an operator builder set to build rebindable inputs to ensure that subgraphs have their data references updated. 
				InGraphRenderCost
			};

			FBuildResults Results;
			TUniquePtr<IOperator> Operator = InNode.GetDefaultOperatorFactory()->CreateOperator(OperatorParams, Results);

			for (const TUniquePtr<IOperatorBuildError>& Error : Results.Errors)
			{
				if (Error.IsValid())
				{
					UE_LOGF(LogMetaSound, Warning, "Encountered error while building operator for node %ls. %ls:%ls", *GetDebugNodeNameString(InNode), *Error->GetErrorType().ToString(), *Error->GetErrorDescription().ToString());
				}
			}

			if (Operator.IsValid())
			{
				Operator->BindInputs(InterfaceData.GetInputs());
				Operator->BindOutputs(InterfaceData.GetOutputs());

				FOperatorInfo OpInfo
				{
					InOrdinal,
					MoveTemp(Operator),
					MoveTemp(InterfaceData)
				};

				return MakeUnique<FInsertOperator>(OperatorID, MoveTemp(OpInfo));
			}
			else
			{
				return TUniquePtr<IDynamicOperatorTransform>(nullptr);
			}
		}


		void FDynamicOperatorTransactor::EnqueueTransformOnOperatorQueues(FCreateTransformFunctionRef InFunc)
		{
			TArray<FDynamicOperatorInfo>::TIterator OperatorInfoIterator = OperatorInfos.CreateIterator();
			while (OperatorInfoIterator)
			{
				TSharedPtr<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> OperatorQueue = OperatorInfoIterator->Queue.Pin();
				if (OperatorQueue.IsValid())
				{
					TUniquePtr<IDynamicOperatorTransform> Transform = InFunc(*OperatorInfoIterator);
					if (Transform.IsValid())
					{
						IDynamicOperatorTransform* InnerTransform = Transform.Release();
						UE_AUTORTFM_ONCOMMIT(OperatorQueue, InnerTransform)
						{
							OperatorQueue->Enqueue(TUniquePtr<IDynamicOperatorTransform>(InnerTransform));
						};
					}
				}
				else
				{
					OperatorInfoIterator.RemoveCurrent();
				}
				OperatorInfoIterator++;
			}
		}
	}
}
