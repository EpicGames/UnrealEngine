// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnvironment.h"


DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(void);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(bool);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(int8);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(uint8);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(int16);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(uint16);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(int32);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(uint32);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(int64);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(uint64);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(float);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(double);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(FString);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(FName);
DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(TArray<FGuid>);

namespace Metasound
{
	FMetasoundEnvironment::FMetasoundEnvironment(const FMetasoundEnvironment& InOther)
	{
		*this = InOther;
	}

	FMetasoundEnvironment& FMetasoundEnvironment::operator=(const FMetasoundEnvironment& InOther)
	{
		Variables.Reset();

		for (auto& VarTuple : InOther.Variables)
		{
			if (ensure(VarTuple.Value.IsValid()))
			{
				Variables.Add(VarTuple.Key, VarTuple.Value->Clone());
			}
		}

		return *this;
	}

	namespace CoreInterface
	{
		namespace Environment
		{
			// The InstanceID acts as an external ID for communicating and in and out of MetaSounds. Each MetaSound
			// has a unique InstanceID
			const FLazyName InstanceID = "InstanceID";
			
			// An array representing the graph hierarchy.
			const FLazyName GraphHierarchy = "GraphHierarhcy";
		}
	}
}
