// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFacade.h"

namespace Metasound
{
	FNodeFacade::FFactory::FFactory(FCreateOperatorFunction InCreateFunc)
	:	CreateFunc(InCreateFunc)
	{
	}

	TUniquePtr<IOperator> FNodeFacade::FFactory::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		return CreateFunc(InParams, OutResults);
	}

	/** Return a reference to the default operator factory. */
	FOperatorFactorySharedRef FNodeFacade::GetDefaultOperatorFactory() const
	{
		return Factory;
	}
}
