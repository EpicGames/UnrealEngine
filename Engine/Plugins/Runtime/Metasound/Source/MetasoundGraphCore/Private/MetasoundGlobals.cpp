// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGlobals.h"

#include "HAL/IConsoleManager.h"


namespace Metasound
{
	namespace GlobalsPrivate
	{
		int32 EnableCommandletExecution = 0;
		FAutoConsoleVariableRef CVarEnableCommandletExecution(
			TEXT("au.MetaSound.EnableCommandletExecution"),
			EnableCommandletExecution,
			TEXT("If application is a commandlet, enables execution of MetaSounds irrespective of whether sound is rendered to hardware or not. (Ignored if cooking)")
			TEXT("Default: 0"),
			ECVF_Default);
	} // namespace GlobalsPrivate

	bool CanEverExecuteGraph(bool bIsCooking)
	{
		if (bIsCooking || IsRunningCookCommandlet())
		{
			return false;
		}

		// TODO: Test builds need au.MetaSound.EnableCommandletExecution set to true, otherwise they fail.
		// if (IsRunningCommandlet())
		// {
		// 	return GlobalsPrivate::EnableCommandletExecution != 0;
		// }

		return true;
	}
} // namespace Metasound
