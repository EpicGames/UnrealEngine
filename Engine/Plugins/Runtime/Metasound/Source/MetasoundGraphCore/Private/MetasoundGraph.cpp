// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundGraph.h"

#include "Algo/BinarySearch.h"
#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundPolymorphic.h"
#include "Templates/SharedPointer.h"

namespace Metasound
{
	const INode* FGraph::FDeprecationNodeStorageAdapter::GetConstNode() const
	{
		return Storage.Get();
	}

	INode* FGraph::FDeprecationNodeStorageAdapter::GetMutableNode() const
	{
		return MutableNode;
	}

	FGraph::FGraph(const FString& InInstanceName, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InOperatorData)
		: AssetPath(FTopLevelAssetPath(InInstanceName))
		, InstanceName(AssetPath.GetPackageName().IsNone() ? AssetPath.GetAssetName() : AssetPath.GetPackageName())
		, InstanceID(InInstanceID)
		, Metadata(FNodeClassMetadata::GetEmpty())
		, OperatorData(MoveTemp(InOperatorData))
	{
		// FGraph is a graph boundary, not a configurable node — it must not
		// have sub-interfaces or variants on its DefaultInterface.
		ensure(!Metadata.DefaultInterface.ContainsSubInterfaces() && !Metadata.DefaultInterface.ContainsVariants());
	}

	FGraph::FGraph(const FName& InInstanceName, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InOperatorData)
		: AssetPath(FTopLevelAssetPath({ }, InInstanceName))
		, InstanceName(InInstanceName)
		, InstanceID(InInstanceID)
		, Metadata(FNodeClassMetadata::GetEmpty())
		, OperatorData(MoveTemp(InOperatorData))
	{
		ensure(!Metadata.DefaultInterface.ContainsSubInterfaces() && !Metadata.DefaultInterface.ContainsVariants());
	}

	FGraph::FGraph(const FTopLevelAssetPath& InAssetPath, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InOperatorData)
	: AssetPath(InAssetPath)
	, InstanceName(InAssetPath.GetPackageName().IsNone() ? InAssetPath.GetAssetName() : InAssetPath.GetPackageName())
	, InstanceID(InInstanceID)
	, Metadata(FNodeClassMetadata::GetEmpty())
	, OperatorData(MoveTemp(InOperatorData))
	{
		ensure(!Metadata.DefaultInterface.ContainsSubInterfaces() && !Metadata.DefaultInterface.ContainsVariants());
	}

	const FTopLevelAssetPath& FGraph::GetAssetPath() const
	{
		return AssetPath;
	}

	const FVertexName& FGraph::GetInstanceName() const
	{
		return InstanceName;
	}

	const FGuid& FGraph::GetInstanceID() const
	{
		return InstanceID;
	}

	const FNodeClassMetadata& FGraph::GetMetadata() const
	{
		return Metadata;
	}

	bool FGraph::AddInputDataDestination(const INode& InNode, const FVertexName& InVertexName)
	{
		if (!InNode.GetVertexInterface().ContainsInputVertex(InVertexName))
		{
			return false;
		}

		FInputDataDestination Destination(InNode, InNode.GetVertexInterface().GetInputVertex(InVertexName));

		AddInputDataDestination(Destination);
		
		return true;
	}

	void FGraph::AddInputDataDestination(const FInputDataDestination& InDestination)
	{
		GraphInterface.GetInputInterface().Add(InDestination.Vertex);
		InputDestinations.Add(MakeDestinationDataVertexKey(InDestination), InDestination);
		Metadata.DefaultInterface = GraphInterface;
	}

	bool FGraph::RemoveInputDataDestination(const FVertexName& InVertexName)
	{
		bool bSuccessRemoveFromInterface = GraphInterface.GetInputInterface().Remove(InVertexName);
		Metadata.DefaultInterface = GraphInterface;

		FInputDataDestinationCollection::TIterator Iter = InputDestinations.CreateIterator();
		bool bSuccessRemoveFromDestinations = false;
		do
		{
			if (Iter.Value().Vertex.VertexName == InVertexName)
			{
				Iter.RemoveCurrent();
				bSuccessRemoveFromDestinations = true;
			}
		} while(++Iter);

		return bSuccessRemoveFromInterface && bSuccessRemoveFromDestinations;
	}

	const FInputDataDestinationCollection& FGraph::GetInputDataDestinations() const
	{
		return InputDestinations;
	}

	bool FGraph::AddOutputDataSource(const INode& InNode, const FVertexName& InVertexName)
	{
		if (!InNode.GetVertexInterface().ContainsOutputVertex(InVertexName))
		{
			return false;
		}

		FOutputDataSource Source(InNode, InNode.GetVertexInterface().GetOutputVertex(InVertexName));

		AddOutputDataSource(Source);

		return true;
	}

	void FGraph::AddOutputDataSource(const FOutputDataSource& InSource)
	{
		GraphInterface.GetOutputInterface().Add(InSource.Vertex);
		OutputSources.Add(MakeSourceDataVertexKey(InSource), InSource);
		Metadata.DefaultInterface = GraphInterface;
	}

	bool FGraph::RemoveOutputDataSource(const FVertexName& InVertexName)
	{
		bool bSuccessRemoveFromInterface = GraphInterface.GetOutputInterface().Remove(InVertexName);
		Metadata.DefaultInterface = GraphInterface;

		FOutputDataSourceCollection::TIterator Iter = OutputSources.CreateIterator();
		bool bSuccessRemoveFromSources = false;
		do
		{
			if (Iter.Value().Vertex.VertexName == InVertexName)
			{
				Iter.RemoveCurrent();
				bSuccessRemoveFromSources = true;
			}
		} while(++Iter);

		return bSuccessRemoveFromInterface && bSuccessRemoveFromSources;
	}

	const FOutputDataSourceCollection& FGraph::GetOutputDataSources() const
	{
		return OutputSources;
	}

	void FGraph::AddDataEdge(const FDataEdge& InEdge)
	{
		Edges.Add(InEdge);
	}

	bool FGraph::AddDataEdge(const INode& FromNode, const FVertexName& FromKey, const INode& ToNode, const FVertexName& ToKey)
	{
		const FVertexInterface& FromVertexInterface = FromNode.GetVertexInterface();
		const FVertexInterface& ToVertexInterface = ToNode.GetVertexInterface();

		if (!FromVertexInterface.ContainsOutputVertex(FromKey))
		{
			return false;
		}

		if (!ToVertexInterface.ContainsInputVertex(ToKey))
		{
			return false;
		}

		const FOutputDataVertex& FromVertex = FromVertexInterface.GetOutputVertex(FromKey);
		const FInputDataVertex& ToVertex = ToVertexInterface.GetInputVertex(ToKey);


		if (!IsCastable(FromVertex.DataTypeName, ToVertex.DataTypeName))
		{
			return false;
		}

		Edges.Add(FDataEdge{FOutputDataSource(FromNode, FromVertex), FInputDataDestination(ToNode, ToVertex)});

		return true;
	}

	bool FGraph::RemoveDataEdge(const INode& FromNode, const FVertexName& FromVertexKey, const INode& ToNode, const FVertexName& ToVertexKey)
	{
		auto IsTargetDataEdge = [&](const FDataEdge& InEdge)
		{
			return (InEdge.To.Node == &ToNode) && (InEdge.To.Vertex.VertexName == ToVertexKey) && (InEdge.From.Node == &FromNode) && (InEdge.From.Vertex.VertexName == FromVertexKey);
		};
		int32 NumRemoved = Edges.RemoveAllSwap(IsTargetDataEdge);

		return (NumRemoved > 0);
	}

	void FGraph::RemoveDataEdgesWithNode(const INode& InNode)
	{
		auto ContainsNode = [&](const FDataEdge& InEdge)
		{
			return (InEdge.To.Node == &InNode) || (InEdge.From.Node == &InNode);
		};
		Edges.RemoveAllSwap(ContainsNode);
	}

	void FGraph::AddNode(const FGuid& InNodeID, TSharedPtr<const INode> InNode)
	{
		Nodes.Add(InNodeID, InNode);
	}

	void FGraph::AddNode(const FGuid& InNodeID, TUniquePtr<INode> InNode)
	{
		Nodes.Add(InNodeID, MoveTemp(InNode));
	}

	void FGraph::SetNodeDefaultInput(const FGuid& InNodeID, const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		if (INode* Node = FindMutableNode(InNodeID))
		{
			Node->SetDefaultInput(InVertexName, InLiteral);
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Could not set input vertex literal on vertex %ls. No node with ID %ls found in graph %ls", *InVertexName.ToString(), *InNodeID.ToString(), *GetInstanceName().ToString())
		}
	}

	void FGraph::AddInputNode(FGuid InNodeId, const FVertexName& InVertexName, TSharedPtr<const INode> InNode)
	{
		if (InNode.IsValid())
		{
			AddInputDataDestination(*InNode, InVertexName);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AddNode(InNodeId, MoveTemp(InNode));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		}
	}

	void FGraph::AddInputNode(FGuid InNodeId, const FVertexName& InVertexName, TUniquePtr<INode> InNode)
	{
		if (InNode.IsValid())
		{
			AddInputDataDestination(*InNode, InVertexName);
			AddNode(InNodeId, MoveTemp(InNode));
		}
	}

	void FGraph::AddOutputNode(FGuid InNodeID, const FVertexName& InVertexName, TSharedPtr<const INode> InNode)
	{
		if (InNode.IsValid())
		{
			AddOutputDataSource(*InNode, InVertexName);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AddNode(InNodeID, InNode);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	void FGraph::AddOutputNode(FGuid InNodeID, const FVertexName& InVertexName, TUniquePtr<INode> InNode)
	{
		if (InNode.IsValid())
		{
			AddOutputDataSource(*InNode, InVertexName);
			AddNode(InNodeID, MoveTemp(InNode));
		}
	}

	const INode* FGraph::FindNode(const FGuid& InNodeID) const
	{
		if (const FDeprecationNodeStorageAdapter* NodeStorage = Nodes.Find(InNodeID))
		{
			return NodeStorage->GetConstNode();
		}
		return nullptr;
	}

	INode* FGraph::FindMutableNode(const FGuid& InNodeID)
	{
		if (const FDeprecationNodeStorageAdapter* NodeStorage = Nodes.Find(InNodeID))
		{
			return NodeStorage->GetMutableNode();
		}
		return nullptr;
	}

	int32 FGraph::FindUnconnectedNodes(TArray<TPair<FGuid, const INode*>>& OutUnconnectedNodes) const
	{
		TArray<const INode*> ConnectedNodes;

		auto AddNodeIfUnique = [&](const INode* InNode)
		{
			// Find index >= InNode
			int32 Index = Algo::LowerBound(ConnectedNodes, InNode);

			if (Index < ConnectedNodes.Num())
			{
				if (ConnectedNodes[Index] != InNode)
				{
					ConnectedNodes.Insert(InNode, Index);
				}
			}
			else
			{
				ConnectedNodes.Add(InNode);
			}
		};

		// Find nodes with an edge.
		for (const FDataEdge& Edge : Edges)
		{
			AddNodeIfUnique(Edge.From.Node);
			AddNodeIfUnique(Edge.To.Node);
		}

		// Find input and output nodes.
		for (const TPair<FNodeDataVertexKey, FInputDataDestination>& Pair : InputDestinations)
		{
			AddNodeIfUnique(Pair.Value.Node);
		}
		for (const TPair<FNodeDataVertexKey, FOutputDataSource>& Pair : OutputSources)
		{
			AddNodeIfUnique(Pair.Value.Node);
		}

		// Filter all nodes by the connected nodes to get the unconnected nodes.
		auto IsNodeUnconnected = [&](const TPair<FGuid, FDeprecationNodeStorageAdapter>& InGuidAndNode)
		{
			const INode* NodePtr = InGuidAndNode.Value.GetConstNode();
			return (nullptr != NodePtr) && (INDEX_NONE == Algo::BinarySearch(ConnectedNodes, NodePtr));
		};

		auto RemoveSharedPtr = [&](const TPair<FGuid, FDeprecationNodeStorageAdapter>& InGuidAndNode) -> TPair<FGuid, const INode*>
		{
			return TPair<FGuid, const INode*>(InGuidAndNode.Key, InGuidAndNode.Value.GetConstNode());
		};

		Algo::TransformIf(Nodes, OutUnconnectedNodes, IsNodeUnconnected, RemoveSharedPtr);

		return OutUnconnectedNodes.Num();
	}

	bool FGraph::RemoveNode(const FGuid& InNodeID, bool bInRemoveDataEdgesWithNode)
	{
		if (bInRemoveDataEdgesWithNode)
		{
			FDeprecationNodeStorageAdapter NodePtr;
			if (Nodes.RemoveAndCopyValue(InNodeID, NodePtr))
			{
				RemoveDataEdgesWithNode(*NodePtr.GetConstNode());
				return true;
			}
			return false;
		}
		else
		{
			return Nodes.Remove(InNodeID) != 0;
		}
	}

	const TArray<FDataEdge>& FGraph::GetDataEdges() const
	{
		return Edges;
	}

	const FVertexInterface& FGraph::GetVertexInterface() const
	{
		return GraphInterface;
	}

	void FGraph::SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		// There's some significant redundancy when storing default literals on a
		// graph's input. We update them all. 
		if (FInputDataVertex* InputVertex = GraphInterface.GetInputInterface().Find(InVertexName))
		{
			// Update literal on exposed vertex
			InputVertex->SetDefaultLiteral(InLiteral);

			// Update literal on input data destination
			const INode* InputNode = nullptr;
			for (auto& Pair : InputDestinations)
			{
				if (Pair.Key.Get<1>() == InVertexName)
				{
					Pair.Value.Vertex.SetDefaultLiteral(InLiteral);
					InputNode = Pair.Key.Get<0>();
					break;
				}
			}

			// Update literal on input node
			if (InputNode)
			{
				// This check for mutable nodes will not be necessary once we remove
				// storage for TSharedPtr<const INodes>. At that point we can assume
				// that all internal nodes can be mutated safely.
				if (INode* MutableInputNode = FindMutableNode(InputNode->GetInstanceID()))
				{
					MutableInputNode->SetDefaultInput(InVertexName, InLiteral);
				}
			}

			Metadata.DefaultInterface = GraphInterface;
		}
	}

	TSharedPtr<const IOperatorData> FGraph::GetOperatorData() const
	{
		return OperatorData;
	}

	FOperatorFactorySharedRef FGraph::GetDefaultOperatorFactory() const 
	{
		return MakeShared<FGraph::FFactory>();
	}

	TUniquePtr<IOperator> FGraph::FFactory::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FGraph& Graph = static_cast<const FGraph&>(InParams.Node);

		FBuildGraphOperatorParams BuildParams = FBuildGraphOperatorParams::FromBuildOperatorParams(Graph, InParams);

		if (nullptr != InParams.Builder)
		{
			// Use the provided builder if it exists.
			return InParams.Builder->BuildGraphOperator(BuildParams, OutResults);
		}
		else
		{
			return FOperatorBuilder(FOperatorBuilderSettings::GetDefaultSettings()).BuildGraphOperator(BuildParams, OutResults);
		}
	}
}
