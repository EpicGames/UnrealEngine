// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGraphAlgo.h"
#include "MetasoundGraphAlgoPrivate.h"

#include "Graph/DirectedGraphUtils.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "MetasoundLog.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertexData.h"
#include "Templates/Tuple.h"

namespace Metasound
{
	// The core graph algorithms utilize integers to represent vertices
	// and pairs of vertices to represent edges. This class provides 
	// convenience functions for converting back and forth between 
	// MetaSound types and the core graph algorithm types.
	class FDirectedGraphAlgoAdapter
	{
		using FNodePair = TTuple<const INode*, const INode*>;
		using FDirectedEdge = UE::MathCore::Graph::FDirectedEdge;

		// The TLazyObject holds the object and a flag to denote whether the 
		// object has been initialized.
		template<typename T>
		struct TLazyObject
		{
			bool bIsInitialized = false;

			T& operator*() { return Object; }
			const T& operator*() const { return Object; }

			T* operator->() { return &Object; }
			const T* operator->() const { return &Object; }

			private:
			T Object;
		};

		public:
			FDirectedGraphAlgoAdapter(const IGraph& InGraph)
			{
				// Gather all nodes references in graph edges.
				for (const FDataEdge& Edge : InGraph.GetDataEdges())
				{
					if (nullptr != Edge.From.Node) 
					{
						MetasoundNodeSet.Add(Edge.From.Node);	
					}
					
					if (nullptr != Edge.To.Node)
					{
						MetasoundNodeSet.Add(Edge.To.Node);	
					}
				}

				// Gather all nodes referenced as input destinations
				for (const FInputDataDestinationCollection::ElementType& Dest : InGraph.GetInputDataDestinations())
				{
					MetasoundNodeSet.Add(Dest.Value.Node);
				}

				// Gather all nodes referenced as output sources.
				for (const FOutputDataSourceCollection::ElementType& Source : InGraph.GetOutputDataSources())
				{
					MetasoundNodeSet.Add(Source.Value.Node);
				}

				// Convert to array so we can use index of node to 
				// represent the int32 needed for the core graph algorithms. 
				MetasoundUniqueNodes = MetasoundNodeSet.Array();

				for (int32 i = 0; i < MetasoundUniqueNodes.Num(); i++)
				{
					MetasoundNodeToInt.Add(MetasoundUniqueNodes[i], i);
				}

				for (const FDataEdge& Edge : InGraph.GetDataEdges())
				{
					if ((nullptr == Edge.From.Node) || (nullptr == Edge.To.Node))
					{
						// skip edges with invalid null nodes. 
						continue;
					}

					int32 FromVertex = MetasoundNodeToInt[Edge.From.Node];
					int32 ToVertex = MetasoundNodeToInt[Edge.To.Node];

					EdgeSet.Add(FDirectedEdge(FromVertex, ToVertex));

					MetasoundNodePairEdges.Emplace(FNodePair(Edge.From.Node, Edge.To.Node), Edge);
				}

				// Gather all vertices referenced as input destinations
				for (const FInputDataDestinationCollection::ElementType& Dest : InGraph.GetInputDataDestinations())
				{
					InputVertices.AddUnique(MetasoundNodeToInt[Dest.Value.Node]);
				}

				// Gather all vertices referenced as output sources.
				for (const FOutputDataSourceCollection::ElementType& Source : InGraph.GetOutputDataSources())
				{
					OutputVertices.AddUnique(MetasoundNodeToInt[Source.Value.Node]);
				}
			}

			// All unique vertices in the graph.
			const TArray<int32>& GetUniqueVertices() const
			{
				if (!Lazy.UniqueVertices.bIsInitialized)
				{
					Lazy.UniqueVertices->Reserve(MetasoundUniqueNodes.Num());

					for (int32 i = 0; i < MetasoundUniqueNodes.Num(); i++)
					{
						Lazy.UniqueVertices->Add(i);
					}

					Lazy.UniqueVertices.bIsInitialized = true;
				}

				return *Lazy.UniqueVertices;
			}

			// All unique input vertices in the graph.
			const TArray<int32>& GetUniqueInputVertices() const
			{
				return InputVertices;
			}

			// All unique output vertices in the graph.
			const TArray<int32>& GetUniqueOutputVertices() const
			{
				return OutputVertices;
			}

			// Get the vertex representing the node.
			int32 GetVertex(const INode* InNode) const
			{
				return MetasoundNodeToInt[InNode];
			}

			// Get all unique edges in the graph.
			const TArray<FDirectedEdge>& GetUniqueEdges() const
			{
				if (!Lazy.UniqueEdges.bIsInitialized)
				{
					*Lazy.UniqueEdges = EdgeSet.Array();
					Lazy.UniqueEdges.bIsInitialized = true;
				}

				return *Lazy.UniqueEdges;
			}

			// Get all unique edges in the graph.
			const TSet<FDirectedEdge>& GetEdgeSet() const
			{
				return EdgeSet;
			}

			// Get node for vertex.
			const INode* GetNode(int32 InVertex) const
			{
				return MetasoundUniqueNodes[InVertex];
			}

			// Get a set of nodes from a set of vertices.
			void GetNodeSet(const TSet<int32>& InVertexSet, TSet<const INode*>& OutNodeSet) const
			{
				for (int32 Vertex : InVertexSet)
				{
					OutNodeSet.Add(MetasoundUniqueNodes[Vertex]);
				}
			}

			// Get all the metasound edges represented by an edge.
			void GetDataEdges(const FDirectedEdge& InEdge, TArray<FDataEdge>& OutMetasoundDataEdges) const
			{
				const INode* FromNode = MetasoundUniqueNodes[InEdge.Get<0>()];
				const INode* ToNode = MetasoundUniqueNodes[InEdge.Get<1>()];

				MetasoundNodePairEdges.MultiFind(FNodePair(FromNode, ToNode), OutMetasoundDataEdges);
			}

			// Get in the form of a directed tree
			const UE::MathCore::Graph::FDirectedTree& GetTree() const
			{
				if (!Lazy.Tree.bIsInitialized)
				{
					UE::MathCore::Graph::BuildDirectedTree(GetUniqueEdges(), *Lazy.Tree);
					Lazy.Tree.bIsInitialized = true;
				}

				return *Lazy.Tree;
			}

			// Get in the form of a transpose directed tree
			const UE::MathCore::Graph::FDirectedTree& GetTransposeTree() const
			{
				if (!Lazy.TransposeTree.bIsInitialized)
				{
					UE::MathCore::Graph::BuildTransposeDirectedTree(GetUniqueEdges(), *Lazy.TransposeTree);
					Lazy.TransposeTree.bIsInitialized = true;
				}

				return *Lazy.TransposeTree;
			}
			
		private:
			// Everything in the lazy cache is mutable.
			struct FLazyCache
			{
				TLazyObject<TArray<int32>> UniqueVertices;
				TLazyObject<TArray<UE::MathCore::Graph::FDirectedEdge>> UniqueEdges;
				TLazyObject<UE::MathCore::Graph::FDirectedTree> Tree;
				TLazyObject<UE::MathCore::Graph::FDirectedTree> TransposeTree;
			};

			TSet<FDirectedEdge> EdgeSet;

			TMap<const INode*, int32> MetasoundNodeToInt;
			TArray<int32> InputVertices;
			TArray<int32> OutputVertices;


			TArray<const INode*> MetasoundUniqueNodes;
			TSet<const INode*> MetasoundNodeSet;

			TMultiMap<FNodePair, FDataEdge> MetasoundNodePairEdges;

			// Everything in the lazy cache is mutable.
			mutable FLazyCache Lazy;
	};

	
	TPimplPtr<FDirectedGraphAlgoAdapter> FDirectedGraphAlgo::CreateDirectedGraphAlgoAdapter(const IGraph& InGraph)
	{
		return DirectedGraphAlgo::CreateDirectedGraphAlgoAdapter(InGraph);
	}

	bool FDirectedGraphAlgo::DepthFirstTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder)
	{
		return DirectedGraphAlgo::DepthFirstTopologicalSort(InGraph, OutNodeOrder);
	}

	bool FDirectedGraphAlgo::DepthFirstTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder)
	{
		return DirectedGraphAlgo::DepthFirstTopologicalSort(InAdapter, OutNodeOrder);
	}

	bool FDirectedGraphAlgo::KahnTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder)
	{
		return DirectedGraphAlgo::KahnTopologicalSort(InGraph, OutNodeOrder);
	}

	bool FDirectedGraphAlgo::KahnTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder)
	{
		return DirectedGraphAlgo::KahnTopologicalSort(InAdapter, OutNodeOrder);
	}

	bool FDirectedGraphAlgo::TarjanStronglyConnectedComponents(const IGraph& InGraph, TArray<DirectedGraphAlgo::FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex)
	{
		return DirectedGraphAlgo::TarjanStronglyConnectedComponents(InGraph, OutComponents, bExcludeSingleVertex);
	}

	bool FDirectedGraphAlgo::TarjanStronglyConnectedComponents(const FDirectedGraphAlgoAdapter& InAdapter, TArray<DirectedGraphAlgo::FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex)
	{
		return DirectedGraphAlgo::TarjanStronglyConnectedComponents(InAdapter, OutComponents, bExcludeSingleVertex);
	}


	void FDirectedGraphAlgo::FindReachableNodesFromInput(const IGraph& InGraph, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodesFromInput(InGraph, OutNodes);
	}

	void FDirectedGraphAlgo::FindReachableNodesFromInput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodesFromInput(InAdapter, OutNodes);
	}

	void FDirectedGraphAlgo::FindReachableNodesFromOutput(const IGraph& InGraph, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodesFromOutput(InGraph, OutNodes);
	}

	void FDirectedGraphAlgo::FindReachableNodesFromOutput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodesFromOutput(InAdapter, OutNodes);
	}

	void FDirectedGraphAlgo::FindReachableNodes(const IGraph& InGraph, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodes(InGraph, OutNodes);
	}

	void FDirectedGraphAlgo::FindReachableNodes(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
	{
		DirectedGraphAlgo::FindReachableNodes(InAdapter, OutNodes);
	}

	namespace DirectedGraphAlgo
	{
		TPimplPtr<FDirectedGraphAlgoAdapter> CreateDirectedGraphAlgoAdapter(const IGraph& InGraph)
		{
			return MakePimpl<FDirectedGraphAlgoAdapter>(InGraph);
		}

		bool DepthFirstTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder)
		{
			return DepthFirstTopologicalSort(FDirectedGraphAlgoAdapter(InGraph), OutNodeOrder);
		}

		bool DepthFirstTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder)
		{
			using namespace Audio;

			TArray<int32> VertexOrder;

			// Call algo implementation
			bool bResult = UE::MathCore::Graph::DepthFirstTopologicalSort(InAdapter.GetUniqueVertices(), InAdapter.GetUniqueEdges(), VertexOrder);

			if (!bResult)
			{
				return false;
			}

			// Convert back to Metasound types.
			OutNodeOrder.Reserve(OutNodeOrder.Num() + VertexOrder.Num());

			for (int32 Vertex : VertexOrder)
			{
				OutNodeOrder.Add(InAdapter.GetNode(Vertex));
			}

			return bResult;
		}

		bool KahnTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder)
		{
			return KahnTopologicalSort(FDirectedGraphAlgoAdapter(InGraph), OutNodeOrder);
		}

		bool KahnTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder)
		{
			using namespace Audio;

			TArray<int32> VertexOrder;

			// Call algo implementation
			bool bResult = UE::MathCore::Graph::KahnTopologicalSort(InAdapter.GetUniqueVertices(), InAdapter.GetUniqueEdges(), VertexOrder);

			if (!bResult)
			{
				return false;
			}

			// Convert back to Metasound types.
			OutNodeOrder.Reserve(OutNodeOrder.Num() + VertexOrder.Num());

			for (int32 Vertex : VertexOrder)
			{
				OutNodeOrder.Add(InAdapter.GetNode(Vertex));
			}

			return bResult;
		}

		bool TarjanStronglyConnectedComponents(const IGraph& InGraph, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex)
		{
			return TarjanStronglyConnectedComponents(FDirectedGraphAlgoAdapter(InGraph), OutComponents, bExcludeSingleVertex);
		}

		bool TarjanStronglyConnectedComponents(const FDirectedGraphAlgoAdapter& InAdapter, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex)
		{
			TArray<UE::MathCore::Graph::FStronglyConnectedComponent> StronglyConnectedComponents;

			// Run tarjan on metasound derived graph edges 
			if (UE::MathCore::Graph::TarjanStronglyConnectedComponents(InAdapter.GetEdgeSet(), StronglyConnectedComponents, bExcludeSingleVertex))
			{
				// If strongly connected components are found, they must be converted
				// back into metasound types. 
				for (const UE::MathCore::Graph::FStronglyConnectedComponent& Component : StronglyConnectedComponents)
				{
					FStronglyConnectedComponent& MetasoundGraphComponent = OutComponents.AddDefaulted_GetRef();

					for (int32 Vertex : Component.Vertices)
					{
						MetasoundGraphComponent.Nodes.Add(InAdapter.GetNode(Vertex));
					}

					for (const UE::MathCore::Graph::FDirectedEdge& Edge : Component.Edges)
					{
						TArray<FDataEdge> MetasoundComponentEdges;

						InAdapter.GetDataEdges(Edge, MetasoundComponentEdges);

						MetasoundGraphComponent.Edges.Append(MetasoundComponentEdges);
					}
				}

				return true;
			}

			// No strongly connected components found.
			return false;
		}


		void FindReachableNodesFromInput(const IGraph& InGraph, TSet<const INode*>& OutNodes)
		{
			FindReachableNodesFromInput(FDirectedGraphAlgoAdapter(InGraph), OutNodes);
		}

		void FindReachableNodesFromInput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
		{
			TSet<int32> VisitedVertices;

			for (int32 Vertex : InAdapter.GetUniqueInputVertices())
			{
				UE::MathCore::Graph::DepthFirstNodeTraversal(Vertex, InAdapter.GetTree(), [&](int32 InVertexBeingVisited) -> bool
					{
						bool bIsAlreadyInSet = false;
						VisitedVertices.Add(InVertexBeingVisited, &bIsAlreadyInSet);

						return !bIsAlreadyInSet;
					}
				);
			}

			InAdapter.GetNodeSet(VisitedVertices, OutNodes);
		}

		void FindReachableNodesFromOutput(const IGraph& InGraph, TSet<const INode*>& OutNodes)
		{
			FindReachableNodesFromOutput(FDirectedGraphAlgoAdapter(InGraph), OutNodes);
		}

		void FindReachableNodesFromOutput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
		{
			TSet<int32> VisitedVertices;

			for (int32 Vertex : InAdapter.GetUniqueOutputVertices())
			{ 
				UE::MathCore::Graph::DepthFirstNodeTraversal(Vertex, InAdapter.GetTransposeTree(), [&](int32 InVertexBeingVisited) -> bool
					{
						bool bIsAlreadyInSet = false;
						VisitedVertices.Add(InVertexBeingVisited, &bIsAlreadyInSet);

						return !bIsAlreadyInSet;
					}
				);
			}

			InAdapter.GetNodeSet(VisitedVertices, OutNodes);
		}

		void FindReachableNodes(const IGraph& InGraph, TSet<const INode*>& OutNodes)
		{
			FindReachableNodes(FDirectedGraphAlgoAdapter(InGraph), OutNodes);
		}

		void FindReachableNodes(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes)
		{
			TSet<const INode*> NodesFromInput;
			FindReachableNodesFromInput(InAdapter, NodesFromInput);

			TSet<const INode*> NodesFromOutput;
			FindReachableNodesFromOutput(InAdapter, NodesFromOutput);

			OutNodes = NodesFromInput.Union(NodesFromOutput);
		}

		FOperatorID GetOperatorID(const INode& InNode)
		{
			return GetOperatorID(&InNode);
		}

		FOperatorID GetOperatorID(const INode* InNode)
		{
			return reinterpret_cast<FOperatorID>(InNode);
		}
		
		FGraphOperatorData::FGraphOperatorData(const FOperatorSettings& InOperatorSettings)
		: OperatorSettings(InOperatorSettings)
		{
		}
	}
}
