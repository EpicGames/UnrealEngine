// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	// Forward Declare
	class INode;

	namespace DirectedGraphAlgo
	{
		// ID used to lookup operators.
		using FOperatorID = uintptr_t;

		// Retrieve an operator ID from a node. 
		FOperatorID GetOperatorID(const INode& InNode);
		FOperatorID GetOperatorID(const INode* InNode);

		/** FGraphOoperatorData contains all the objects needed to implement a runtime instance of a MetaSound graph.  */
		struct FGraphOperatorData
		{
			// Represents an input of an operator
			struct FVertexDestination
			{
				FOperatorID OperatorID;
				FVertexName VertexName;
			};

			struct FOperatorInfo
			{
				// The ordinal associated with the operator. Ordinals can be used
				// to determine ordering of operators, but shouldn't be considered
				// as indexes.
				int32 Ordinal;

				TUniquePtr<IOperator> Operator;

				// Vertex Data bound to operator.
				FVertexInterfaceData VertexData;

				// Map where the operators output vertex is the key, and an array of output
				// connections is the value.
				TSortedVertexNameMap<TArray<FVertexDestination>> OutputConnections;
			};

			FGraphOperatorData(const FOperatorSettings& InOperatorSettings);

			FOperatorSettings OperatorSettings;

			// Vertex Data bound to the graph
			FVertexInterfaceData VertexData;
			
			// Map of all operators in graph
			TSortedMap<FOperatorID, FOperatorInfo> OperatorMap;

			// Map with input vertex name as key, and OperatorID of input node as value.
			TSortedVertexNameMap<FOperatorID> InputVertexMap;

			// Map with output vertex name as key, and OperatorID of output node as value.
			TSortedVertexNameMap<FOperatorID> OutputVertexMap;
		};

		/** FStaticGraphOperatorData contains FGraphOperatorData plus a static ordering
		 * of nodes. This structure allows the FOpeatorBuilder to more efficiently inform
		 * the FGraphOperator how the operators should order execution tables. 
		 */
		struct FStaticGraphOperatorData : FGraphOperatorData
		{
			using FGraphOperatorData::FGraphOperatorData; // Inherit constructor

			// Order of node execution
			TArray<const INode*> NodeOrder;
		};
	}
}
