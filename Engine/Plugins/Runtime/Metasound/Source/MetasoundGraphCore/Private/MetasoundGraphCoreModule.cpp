// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGraphCoreModule.h"

#include "CoreMinimal.h"
#include "MetasoundLog.h"
#include "MetasoundProfilingOperator.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogMetaSound);

namespace Metasound 
{
	class FMetasoundGraphCoreModule : public IMetasoundGraphCoreModule
	{
	public:
		virtual void StartupModule() override
		{
			Profiling::Init();
		}
	};
}

IMPLEMENT_MODULE(Metasound::FMetasoundGraphCoreModule, MetasoundGraphCore);

