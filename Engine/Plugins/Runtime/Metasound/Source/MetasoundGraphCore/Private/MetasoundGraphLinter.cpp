// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGraphLinter.h"

#include "Algo/Count.h"
#include "Algo/Transform.h"
#include "CoreMinimal.h"

#include "MetasoundBuildError.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundGraphAlgo.h"

namespace Metasound
{
	namespace GraphLinterPrivate
	{
		using FBuildErrorPtr = TUniquePtr<IOperatorBuildError>;

		// run validation on input destination
		bool IsInputDestinationValid(const FInputDataDestination& InDest)
		{
			if (InDest.Node)
			{
				const FInputVertexInterface& Interface = InDest.Node->GetVertexInterface().GetInputInterface();
				if (const FInputDataVertex* Vertex = Interface.Find(InDest.Vertex.VertexName))
				{
					return *Vertex == InDest.Vertex;
				}
			}

			return false;
		}
		
		// run validation on output source
		bool IsOutputSourceValid(const FOutputDataSource& InSource)
		{
			if (InSource.Node)
			{
				const FOutputVertexInterface& Interface = InSource.Node->GetVertexInterface().GetOutputInterface();
				if (const FOutputDataVertex* Vertex = Interface.Find(InSource.Vertex.VertexName))
				{
					return *Vertex == InSource.Vertex;
				}
			}

			return false;
		};

		const FDataEdge* GetEdgePointer(const FDataEdge& InEdge) 
		{ 
			return &InEdge; 
		};

		// For sorting edges by destination.
		bool CompareEdgeDestination(const FDataEdge& InLHS, const FDataEdge& InRHS)
		{
			return InLHS.To < InRHS.To;
		}
	}

	bool FGraphLinter::ValidateEdgeDataTypesMatch(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors)
	{
		using namespace GraphLinterPrivate;

		bool bIsValid = true;

		for (const FDataEdge& Edge : InGraph.GetDataEdges())
		{
			if (!IsCastable(Edge.From.Vertex.DataTypeName, Edge.To.Vertex.DataTypeName))
			{
				AddBuildError<FInvalidConnectionDataTypeError>(OutErrors, Edge);

				bIsValid = false;
			}
		}

		return bIsValid;
	}

	bool FGraphLinter::ValidateVerticesExist(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors) 
	{
		using namespace GraphLinterPrivate;

		bool bAllVerticesAreValid = true;

		for (const FDataEdge& Edge : InGraph.GetDataEdges())
		{
			if (nullptr == Edge.From.Node)
			{
				bAllVerticesAreValid = false;
				AddBuildError<FDanglingVertexError>(OutErrors, Edge.From);
			}

			if (!IsOutputSourceValid(Edge.From))
			{
				bAllVerticesAreValid = false;
				AddBuildError<FMissingVertexError>(OutErrors, Edge.From);
			}

			if (nullptr == Edge.To.Node)
			{
				bAllVerticesAreValid = false;
				AddBuildError<FDanglingVertexError>(OutErrors, Edge.To);
			}

			if (!IsInputDestinationValid(Edge.To))
			{
				bAllVerticesAreValid = false;
				AddBuildError<FMissingVertexError>(OutErrors, Edge.To);
			}
		}

		for (auto Element : InGraph.GetInputDataDestinations())
		{
			const FInputDataDestination& Dest = Element.Value;
			if (!IsInputDestinationValid(Dest))
			{
				bAllVerticesAreValid = false;
				AddBuildError<FMissingVertexError>(OutErrors, Dest);
			}
		}

		for (auto Element : InGraph.GetOutputDataSources())
		{
			const FOutputDataSource& Source = Element.Value;
			if (!IsOutputSourceValid(Source))
			{
				bAllVerticesAreValid = false;
				AddBuildError<FMissingVertexError>(OutErrors, Source);
			}
		}
		
		return bAllVerticesAreValid;
	}

	bool FGraphLinter::ValidateNoDuplicateInputs(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors)
	{
		using namespace GraphLinterPrivate;

		bool bIsValid = true;

		TArray<const FDataEdge*> Edges;

		Algo::Transform(InGraph.GetDataEdges(), Edges, GetEdgePointer);

		Edges.Sort(CompareEdgeDestination);

		if (Edges.Num() < 1)
		{
			return bIsValid;
		}

		// Loop through edges sorted by destinations. Look for duplicate destinations.
		int32 GroupStartIndex = 0;
		int32 GroupEndIndex = 0;
		const FInputDataDestination* CurrentDestination = &Edges[GroupStartIndex]->To;

		auto AddErrorsIfGroupHasDuplicates = [&]()
		{
			if (GroupEndIndex > GroupStartIndex)
			{
				// This condition is hit when more than one edge refers to the
				// same destination.
				bIsValid = false;

				// Collected edges with same destination.
				TArray<FDataEdge> Duplicates;
				for (int32 j = GroupStartIndex; j <= GroupEndIndex; j++)
				{
					Duplicates.Add(*Edges[j]);
				}

				// Add error.
				AddBuildError<FDuplicateInputError>(OutErrors, Duplicates);
			}
		};

		for (int32 i = 1; i < Edges.Num(); i++)
		{
			const FInputDataDestination* Destination = &Edges[i]->To;

			if (*Destination == *CurrentDestination)
			{
				GroupEndIndex = i;
			}
			else
			{
				AddErrorsIfGroupHasDuplicates();

				GroupStartIndex = GroupEndIndex = i;
				CurrentDestination = Destination;
			}
		}

		AddErrorsIfGroupHasDuplicates();

		return bIsValid;
	}

	bool FGraphLinter::ValidateNoCyclesInGraph(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors) 
	{
		TPimplPtr<FDirectedGraphAlgoAdapter> Adapter = DirectedGraphAlgo::CreateDirectedGraphAlgoAdapter(InGraph);

		check(Adapter.IsValid());

		return ValidateNoCyclesInGraph(*Adapter, OutErrors);
	}

	bool FGraphLinter::ValidateNoCyclesInGraph(const FDirectedGraphAlgoAdapter& InAdapter, TArray<FBuildErrorPtr>& OutErrors) 
	{
		using namespace GraphLinterPrivate;

		bool bIsValid = true;

		TArray<DirectedGraphAlgo::FStronglyConnectedComponent> Cycles;

		// In graph theory, a single vertex is technically a strongly connected
		// component. The graph linter is only interested in strongly connected
		// components of more than one vertex since this denotes a cycle.
		bool bExcludeSingleVertex = true;

		if(DirectedGraphAlgo::TarjanStronglyConnectedComponents(InAdapter, Cycles, bExcludeSingleVertex))
		{
			bIsValid = false;

			for (const DirectedGraphAlgo::FStronglyConnectedComponent& Cycle : Cycles)
			{
				AddBuildError<FGraphCycleError>(OutErrors, Cycle.Nodes, Cycle.Edges);
			}
		}

		return bIsValid;
	}
}
