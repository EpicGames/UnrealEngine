// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGraphOperator.h"

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundOperatorInterface.h"
#include "Misc/ReverseIterate.h"

namespace Metasound
{
	namespace MetasoundGraphPrivate
	{
		// FGraphOperator does not support rebinding with new inputs or outputs. This checks
		// that underlying data pointers were not updated when bind is called on the graph
		// operator.
		//
		// In order for FGraphOperator to support rebinding with new inputs, it would need
		// to maintain an internal map of all connections in the graph in order to update
		// internal operators appropriately. It does not hold onto this data for 
		// performance reasons. 
		template<typename InterfaceDataType>
		bool IsSupportedVertexData(const InterfaceDataType& InCurrentData, const InterfaceDataType& InNewData)
		{
#if DO_CHECK
			TArray<FVertexDataState> CurrentState;
			GetVertexInterfaceDataState(InCurrentData, CurrentState);

			TArray<FVertexDataState> NewState;
			GetVertexInterfaceDataState(InNewData, NewState);

			CurrentState.Sort();
			NewState.Sort();

			TArray<FVertexDataState>::TConstIterator CurrentIter = CurrentState.CreateConstIterator();
			TArray<FVertexDataState>::TConstIterator NewIter = NewState.CreateConstIterator();

			while (CurrentIter && NewIter)
			{
				if (NewIter->VertexName == CurrentIter->VertexName)
				{
					if ((NewIter->ID != nullptr) && (NewIter->ID != CurrentIter->ID))
					{
						UE_LOGF(LogMetaSound, Warning, "Cannot bind to FGraphOperator because vertex %ls has mismatched data", *(NewIter->VertexName.ToString()));
						return false;
					}
					else
					{
						NewIter++;
						CurrentIter++;
					}
				}
				else if (*NewIter < *CurrentIter)
				{
					UE_LOGF(LogMetaSound, Warning, "Cannot bind to FGraphOperator because vertex %ls does not exist in current vertex data", *(NewIter->VertexName.ToString()));
					return false;
				}
				else 
				{
					// It's ok if we have an entry in the current vertex data that does not exist in the new vertex data. 
					CurrentIter++;
				}
			}

			if (NewIter)
			{
				UE_LOGF(LogMetaSound, Warning, "Cannot bind to FGraphOperator because vertex %ls does not exist in current vertex data", *(NewIter->VertexName.ToString()));
				return false;
			}

			return true;
#else
			return true;
#endif // DO_CHECK
		}
	}

	FGraphOperator::FGraphOperator(TUniquePtr<DirectedGraphAlgo::FStaticGraphOperatorData> InOperatorState)
	{
		using namespace DirectedGraphAlgo;

		const int32 NumNodes = InOperatorState->NodeOrder.Num();
		ExecuteStack.Reserve(NumNodes);
		PostExecuteStack.Reserve(NumNodes);
		ResetStack.Reserve(NumNodes);

		// Append operators in order.
		for (const INode* Node: InOperatorState->NodeOrder)
		{
			// The Node pointer may not point to a valid memory because there is
			// nothing ensuring that the node is still alive. GetOperatorID(...)
			// simply uses the pointer address as the ID and does not access the 
			// actual underlying INode. 
			const FOperatorID OperatorID = GetOperatorID(Node);

			FGraphOperatorData::FOperatorInfo* OperatorInfo = InOperatorState->OperatorMap.Find(OperatorID);
			if (ensureMsgf(OperatorInfo, TEXT("Encountered possible corrupt operator data")))
			{
				AppendOperator(MoveTemp(OperatorInfo->Operator));
			}
		}

		ExecuteStack.Shrink();
		PostExecuteStack.Shrink();
		ResetStack.Shrink();

		// Copy over vertex data
		SetVertexInterfaceData(MoveTemp(InOperatorState->VertexData));
	}

	void FGraphOperator::AppendOperator(FOperatorPtr InOperator)
	{
		if (InOperator.IsValid())
		{
			bool bIsOperatorInAnyStack = false;

			if (FExecuteFunction ExecuteFunc = InOperator->GetExecuteFunction())
			{
				ExecuteStack.Emplace(*InOperator, ExecuteFunc);
				bIsOperatorInAnyStack = true;
			}

			if (FPostExecuteFunction PostExecuteFunc = InOperator->GetPostExecuteFunction())
			{
				PostExecuteStack.Emplace(*InOperator, PostExecuteFunc);
				bIsOperatorInAnyStack = true;
			}

			if (FResetFunction ResetFunc = InOperator->GetResetFunction())
			{
				ResetStack.Emplace(*InOperator, ResetFunc);
				bIsOperatorInAnyStack = true;
			}

			if (bIsOperatorInAnyStack)
			{
				ActiveOperators.Add(MoveTemp(InOperator));
			}
		}
	}

	void FGraphOperator::SetVertexInterfaceData(FVertexInterfaceData&& InVertexData)
	{
		VertexData = MoveTemp(InVertexData);
	}

	void FGraphOperator::BindInputs(FInputVertexInterfaceData& InInputVertexData)
	{
		if (!MetasoundGraphPrivate::IsSupportedVertexData(VertexData.GetInputs(), InInputVertexData))
		{
			UE_LOGF(LogMetaSound, Error, "FGraphOperator does not support rebinding with new data");
		}

		InInputVertexData = VertexData.GetInputs();
	}

	void FGraphOperator::BindOutputs(FOutputVertexInterfaceData& InOutputVertexData)
	{
		InOutputVertexData = VertexData.GetOutputs();
	}

	IOperator::FPostExecuteFunction FGraphOperator::GetPostExecuteFunction()
	{
		return &StaticPostExecute;
	}

	void FGraphOperator::StaticPostExecute(IOperator* InOperator)
	{
		FGraphOperator* GraphOperator = static_cast<FGraphOperator*>(InOperator);
		check(GraphOperator);

		GraphOperator->PostExecute();
	}

	void FGraphOperator::Execute()
	{
		for (FExecuteEntry& Entry : ExecuteStack)
		{
			Entry.Execute();
		}
	}

	void FGraphOperator::PostExecute()
	{
		// Reverse iterate over post execute so that inputs to operators do
		// not change from last execute
		for (FPostExecuteEntry& Entry : ReverseIterate(PostExecuteStack))
		{
			Entry.PostExecute();
		}
	}

	void FGraphOperator::Reset(const FGraphOperator::FResetParams& InParams)
	{
		for (FResetEntry& Entry : ResetStack)
		{
			Entry.Reset(InParams);
		}
	}

	FGraphOperator::FExecuteEntry::FExecuteEntry(IOperator& InOperator, FExecuteFunction InFunc)
	: Operator(&InOperator)
	, Function(InFunc)
	{
		check(Function);
	}

	void FGraphOperator::FExecuteEntry::Execute()
	{
		Function(Operator);
	}

	FGraphOperator::FPostExecuteEntry::FPostExecuteEntry(IOperator& InOperator, FPostExecuteFunction InFunc)
	: Operator(&InOperator)
	, Function(InFunc)
	{
		check(Function);
	}

	void FGraphOperator::FPostExecuteEntry::PostExecute()
	{
		Function(Operator);
	}

	FGraphOperator::FResetEntry::FResetEntry(IOperator& InOperator, FResetFunction InFunc)
	: Operator(&InOperator)
	, Function(InFunc)
	{
		check(Function);
	}

	void FGraphOperator::FResetEntry::Reset(const FGraphOperator::FResetParams& InParams)
	{
		Function(Operator, InParams);
	}


}
