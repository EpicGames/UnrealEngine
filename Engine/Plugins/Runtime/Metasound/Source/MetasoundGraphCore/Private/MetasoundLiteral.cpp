// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundLiteral.h"

#include "IAudioProxyInitializer.h"
#include "Misc/TVariant.h"
#include <type_traits>

namespace Metasound
{
#if METASOUND_DEBUG_LITERALS
	void FLiteral::InitDebugString() const
	{
		DebugString = LexToString(*this);
	}
#endif // METASOUND_DEBUG_LITERALS


	// builds an invalid FLiteral.
	FLiteral FLiteral::CreateInvalid()
	{
		return FLiteral(FInvalid());
	}

	FLiteral FLiteral::GetDefaultForType(ELiteralType InType)
	{
		switch (InType)
		{
			case ELiteralType::None:
				return FLiteral(TLiteralTypeInfo<FLiteral::FNone>::GetDefaultValue());

			case ELiteralType::Boolean:
				return FLiteral(TLiteralTypeInfo<bool>::GetDefaultValue());

			case ELiteralType::Integer:
				return FLiteral(TLiteralTypeInfo<int32>::GetDefaultValue());

			case ELiteralType::Float:
				return FLiteral(TLiteralTypeInfo<float>::GetDefaultValue());

			case ELiteralType::String:
				return FLiteral(TLiteralTypeInfo<FString>::GetDefaultValue());

			case ELiteralType::UObjectProxy:
				return FLiteral(TLiteralTypeInfo<TSharedPtr<Audio::IProxyData>>::GetDefaultValue());
			
			case ELiteralType::NoneArray:
				return FLiteral(TLiteralTypeInfo<TArray<FLiteral::FNone>>::GetDefaultValue());
			
			case ELiteralType::BooleanArray:
				return FLiteral(TLiteralTypeInfo<TArray<bool>>::GetDefaultValue());
			
			case ELiteralType::IntegerArray:
				return FLiteral(TLiteralTypeInfo<TArray<int32>>::GetDefaultValue());
			
			case ELiteralType::FloatArray:
				return FLiteral(TLiteralTypeInfo<TArray<float>>::GetDefaultValue());

			case ELiteralType::StringArray:
				return FLiteral(TLiteralTypeInfo<TArray<FString>>::GetDefaultValue());

			case ELiteralType::UObjectProxyArray:
				return FLiteral(TLiteralTypeInfo<TArray<TSharedPtr<Audio::IProxyData>>>::GetDefaultValue());

			case ELiteralType::Invalid:
			default:
				static_assert(static_cast<int32>(ELiteralType::Invalid) == 12, "Possible missing ELiteralType case coverage");
				return FLiteral::CreateInvalid();

		}
	}

	bool FLiteral::IsValid() const
	{
		return GetType() != ELiteralType::Invalid;
	}

	ELiteralType FLiteral::GetType() const
	{
		switch (Value.GetIndex())
		{
			case 0:
				return ELiteralType::None;

			case 1:
				return ELiteralType::Boolean;

			case 2:
				return ELiteralType::Integer;

			case 3:
				return ELiteralType::Float;

			case 4:
				return ELiteralType::String;

			case 5:
				return ELiteralType::UObjectProxy;

			case 6:
				return ELiteralType::NoneArray;

			case 7:
				return ELiteralType::BooleanArray;

			case 8:
				return ELiteralType::IntegerArray;

			case 9:
				return ELiteralType::FloatArray;

			case 10:
				return ELiteralType::StringArray;

			case 11:
				return ELiteralType::UObjectProxyArray;

			case 12:
			default:
				static_assert(TVariantSize_V<FVariantType> == 13, "Possible missing FVariantType case coverage");
				return ELiteralType::Invalid;
		}
	}
}

FString LexToString(const Metasound::FLiteral& InLiteral)
{
	using namespace Metasound;

	switch (InLiteral.GetType())
	{
		case ELiteralType::None:
			return TEXT("NONE");

		case ELiteralType::Boolean:
			return FString::Printf(TEXT("Boolean: %s"), InLiteral.Value.Get<bool>() ? TEXT("true") : TEXT("false"));

		case ELiteralType::Integer:
			return FString::Printf(TEXT("Int32: %d"), InLiteral.Value.Get<int32>());

		case ELiteralType::Float:
			return FString::Printf(TEXT("Float: %f"), InLiteral.Value.Get<float>());

		case ELiteralType::String:
			return FString::Printf(TEXT("String: %s"), *InLiteral.Value.Get<FString>());

		case ELiteralType::UObjectProxy:
		{
			FString ProxyType = TEXT("nullptr");
			if (InLiteral.Value.Get<TSharedPtr<Audio::IProxyData>>().IsValid())
			{
				ProxyType = InLiteral.Value.Get<TSharedPtr<Audio::IProxyData>>()->GetProxyTypeName().ToString();
			}
			return FString::Printf(TEXT("TSharedPtr<Audio::IProxyData>: %s"), *ProxyType);
		}

		case ELiteralType::NoneArray:
			return TEXT("TArray<NONE>");

		case ELiteralType::BooleanArray:
			return TEXT("TArray<Boolean>");

		case ELiteralType::IntegerArray:
			return TEXT("TArray<int32>");

		case ELiteralType::FloatArray:
			return TEXT("TArray<float>");

		case ELiteralType::StringArray:
			return TEXT("TArray<FString>");

		case ELiteralType::UObjectProxyArray:
			return TEXT("TArray<TSharedPtr<Audio::IProxyData>>");

		case ELiteralType::Invalid:
			return TEXT("INVALID");

		default:
			static_assert(static_cast<int32>(ELiteralType::Invalid) == 12, "Possible missing ELiteralType case coverage");
			checkNoEntry();
			return TEXT("INVALID");
	}
}

