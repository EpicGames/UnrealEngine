// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundNode.h"
#include "CoreMinimal.h"

namespace Metasound
{
	FNode::FNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FNodeClassMetadata& InInfo, TSharedPtr<const IOperatorData> InOperatorData)
	: InstanceName(InInstanceName)
	, InstanceID(InInstanceID)
	, Info(InInfo)
	, VertexInterface(InInfo.DefaultInterface)
	, OperatorData(MoveTemp(InOperatorData))
	{
	}

	/** Return the name of this specific instance of the node class. */
	const FVertexName& FNode::GetInstanceName() const
	{
		return InstanceName;
	}

	/** Return the ID of this specific instance of the node class. */
	const FGuid& FNode::GetInstanceID() const
	{
		return InstanceID;
	}

	/** Return the type name of this node. */
	const FNodeClassMetadata& FNode::GetMetadata() const
	{
		return Info;
	}

	const FVertexInterface& FNode::GetVertexInterface() const
	{
		return VertexInterface;
	}

	void FNode::SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		if (FInputDataVertex* Vertex = VertexInterface.GetInputInterface().Find(InVertexName))
		{
			Vertex->SetDefaultLiteral(InLiteral);
		}
		else
		{
			UE_LOGF(LogMetaSound, Error, "Cannot set default input. Vertex %ls does not exist on node %ls", *InVertexName.ToString(), *InstanceName.ToString());
		}
	}

	TSharedPtr<const IOperatorData> FNode::GetOperatorData() const 
	{
		return OperatorData;
	}
}
