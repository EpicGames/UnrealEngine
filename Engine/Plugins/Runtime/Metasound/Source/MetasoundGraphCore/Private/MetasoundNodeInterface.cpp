// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeInterface.h"

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"

namespace Metasound
{
	FNodeData::FNodeData() = default;

	FNodeData::FNodeData(const FNodeData&) = default;

	FNodeData::FNodeData(const FName& InName, const FGuid& InID, FVertexInterface InInterface)
	: FNodeData(InName, InID, MoveTemp(InInterface), TSharedPtr<const IOperatorData>())
	{
	}

	FNodeData::FNodeData(const FName& InName, const FGuid& InID, FVertexInterface InInterface, TSharedPtr<const IOperatorData> InOperatorData)
	: Name(InName)
	, ID(InID)
	, Interface(MoveTemp(InInterface))
	, OperatorData(MoveTemp(InOperatorData))
	{
	}

	FNodeData::~FNodeData() = default;

	const FString PluginAuthor = TEXT("Epic Games, Inc.");
#if WITH_EDITOR
	const FText PluginNodeMissingPrompt = NSLOCTEXT("MetasoundGraphCore", "Metasound_DefaultMissingNodePrompt", "The node was likely removed, renamed, or the Metasound plugin is not loaded.");
#else 
	const FText PluginNodeMissingPrompt = FText::GetEmpty();
#endif // WITH_EDITOR

	const FNodeClassName FNodeClassName::InvalidNodeClassName;

	FNodeClassName::FNodeClassName()
	{
	}

	FNodeClassName::FNodeClassName(const FName& InNamespace, const FName& InName, const FName& InVariant)
	: Namespace(InNamespace)
	, Name(InName)
	, Variant(InVariant)
	{
	}

	/** Namespace of node class. */
	const FName& FNodeClassName::GetNamespace() const
	{
		return Namespace;
	}

	/** Name of node class. */
	const FName& FNodeClassName::GetName() const
	{
		return Name;
	}

	/** Variant of node class. */
	const FName& FNodeClassName::GetVariant() const
	{
		return Variant;
	}

	/** The full name of the Node formatted Namespace.Name[.Variant] */
	const FString FNodeClassName::ToString() const
	{
		FNameBuilder Builder;
		FormatFullName(Builder, Namespace, Name, Variant);
		return *Builder;
	}

	FName FNodeClassName::FormatFullName(const FName& InNamespace, const FName& InName, const FName& InVariant)
	{
		FNameBuilder Builder;
		FormatFullName(Builder, InNamespace, InName, InVariant);
		return *Builder;
	}

	FName FNodeClassName::FormatScopedName(const FName& InNamespace, const FName& InName)
	{
		FNameBuilder Builder;
		FormatScopedName(Builder, InNamespace, InName);
		return *Builder;
	}

	void FNodeClassName::FormatFullName(FNameBuilder& InBuilder, const FName& InNamespace, const FName& InName, const FName& InVariant)
	{
		FormatScopedName(InBuilder, InNamespace, InName);

		if (InVariant != NAME_None)
		{
			InBuilder.Append(".");
			InVariant.AppendString(InBuilder);
		}
	}

	void FNodeClassName::FormatScopedName(FNameBuilder& InBuilder, const FName& InNamespace, const FName& InName)
	{
		InNamespace.AppendString(InBuilder);
		InBuilder.Append(".");
		InName.AppendString(InBuilder);
	}

	bool FNodeClassName::IsValid() const
	{
		return *this != InvalidNodeClassName;
	}

	FNodeClassMetadata::FNodeClassMetadata()
		: ClassName()
		, MajorVersion(-1)
		, MinorVersion(-1)
		, DisplayName()
		, Description()
		, Author()
		, PromptIfMissing()
		, DefaultInterface()
		, CategoryHierarchy()
		, Keywords()
		, DisplayStyle()
		, AccessFlags(ENodeClassAccessFlags::Default)
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bDeprecated(false)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	{
	}

	FNodeClassMetadata::FNodeClassMetadata(const FNodeClassMetadata& InOther)
		: ClassName(InOther.ClassName)
		, MajorVersion(InOther.MajorVersion)
		, MinorVersion(InOther.MinorVersion)
		, DisplayName(InOther.DisplayName)
		, Description(InOther.Description)
		, Author(InOther.Author)
		, PromptIfMissing(InOther.PromptIfMissing)
		, DefaultInterface(InOther.DefaultInterface)
		, CategoryHierarchy(InOther.CategoryHierarchy)
		, Keywords(InOther.Keywords)
		, DisplayStyle(InOther.DisplayStyle)
		, AccessFlags(InOther.AccessFlags)
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bDeprecated(InOther.bDeprecated)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	{
	}

	FNodeClassMetadata::FNodeClassMetadata(
		FNodeClassName InClassName,
		int32 InMajorVersion,
		int32 InMinorVersion,
		FText InDisplayName,
		FText InDescription,
		FString InAuthor,
		FText InPromptIfMissing,
		FClassInterface InDefaultInterface,
		TArray<FText> InCategoryHierarchy,
		TArray<FText> InKeywords,
		FNodeDisplayStyle InDisplayStyle)
		: ClassName(MoveTemp(InClassName))
		, MajorVersion(InMajorVersion)
		, MinorVersion(InMinorVersion)
		, DisplayName(MoveTemp(InDisplayName))
		, Description(MoveTemp(InDescription))
		, Author(MoveTemp(InAuthor))
		, PromptIfMissing(MoveTemp(InPromptIfMissing))
		, DefaultInterface(MoveTemp(InDefaultInterface))
		, CategoryHierarchy(MoveTemp(InCategoryHierarchy))
		, Keywords(MoveTemp(InKeywords))
		, DisplayStyle(MoveTemp(InDisplayStyle))
		, AccessFlags(ENodeClassAccessFlags::Default)
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bDeprecated(false)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	{
	}

	FNodeClassMetadata& FNodeClassMetadata::operator=(const FNodeClassMetadata& InOther)
	{
		ClassName = InOther.ClassName;
		MajorVersion = InOther.MajorVersion;
		MinorVersion = InOther.MinorVersion;
		DisplayName = InOther.DisplayName;
		Description = InOther.Description;
		Author = InOther.Author;
		PromptIfMissing = InOther.PromptIfMissing;
		DefaultInterface = InOther.DefaultInterface;
		CategoryHierarchy = InOther.CategoryHierarchy;
		Keywords = InOther.Keywords;
		DisplayStyle = InOther.DisplayStyle;
		AccessFlags = InOther.AccessFlags;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bDeprecated = InOther.bDeprecated;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

		return *this;
	}

	FNodeClassMetadata& FNodeClassMetadata::operator=(FNodeClassMetadata&& InOther)
	{
		ClassName = MoveTemp(InOther.ClassName);
		MajorVersion = MoveTemp(InOther.MajorVersion);
		MinorVersion = MoveTemp(InOther.MinorVersion);
		DisplayName = MoveTemp(InOther.DisplayName);
		Description = MoveTemp(InOther.Description);
		Author = MoveTemp(InOther.Author);
		PromptIfMissing = MoveTemp(InOther.PromptIfMissing);
		DefaultInterface = MoveTemp(InOther.DefaultInterface);
		CategoryHierarchy = MoveTemp(InOther.CategoryHierarchy);
		Keywords = MoveTemp(InOther.Keywords);
		DisplayStyle = MoveTemp(InOther.DisplayStyle);
		AccessFlags = MoveTemp(InOther.AccessFlags);

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bDeprecated = InOther.bDeprecated;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

		return *this;
	}

	FNodeClassMetadata::~FNodeClassMetadata() = default;

	const FNodeClassMetadata& FNodeClassMetadata::GetEmpty()
	{
		static const FNodeClassMetadata EmptyInfo;
		return EmptyInfo;
	}

	bool operator==(const FOutputDataSource& InLeft, const FOutputDataSource& InRight)
	{
		return (InLeft.Node == InRight.Node) && (InLeft.Vertex == InRight.Vertex);
	}

	bool operator!=(const FOutputDataSource& InLeft, const FOutputDataSource& InRight)
	{
		return !(InLeft == InRight);
	}

	bool operator<(const FOutputDataSource& InLeft, const FOutputDataSource& InRight)
	{
		if (InLeft.Node == InRight.Node)
		{
			return InLeft.Vertex < InRight.Vertex;
		}
		else
		{
			return InLeft.Node < InRight.Node;
		}
	}

	/** Check if two FInputDataDestinations are equal. */
	bool operator==(const FInputDataDestination& InLeft, const FInputDataDestination& InRight)
	{
		return (InLeft.Node == InRight.Node) && (InLeft.Vertex == InRight.Vertex);
	}

	bool operator!=(const FInputDataDestination& InLeft, const FInputDataDestination& InRight)
	{
		return !(InLeft == InRight);
	}

	bool operator<(const FInputDataDestination& InLeft, const FInputDataDestination& InRight)
	{
		if (InLeft.Node == InRight.Node)
		{
			return InLeft.Vertex < InRight.Vertex;
		}
		else
		{
			return InLeft.Node < InRight.Node;
		}
	}

	/** Check if two FDataEdges are equal. */
	bool operator==(const FDataEdge& InLeft, const FDataEdge& InRight)
	{
		return (InLeft.From == InRight.From) && (InLeft.To == InRight.To);
	}

	bool operator!=(const FDataEdge& InLeft, const FDataEdge& InRight)
	{
		return !(InLeft == InRight);
	}

	bool operator<(const FDataEdge& InLeft, const FDataEdge& InRight)
	{
		if (InLeft.From == InRight.From)
		{
			return InLeft.To < InRight.To;	
		}
		else
		{
			return InLeft.From < InRight.From;
		}
	}

#if !UE_METASOUND_PURE_VIRTUAL_SET_DEFAULT_INPUT
	void INodeBase::SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		static bool bDidWarn = false;
		if (!bDidWarn)
		{
			UE_LOGF(LogMetaSound, Warning, "Ignoring input default for vertex %ls. Please implement INodeInterface::SetDefaultInput(...) for the class representing node %ls. This method will become pure virtual in future releases. Define UE_METASOUND_PURE_VIRTUAL_SET_DEFAULT_INPUT in order to build with this method as a pure virtual on the interface.", *InVertexName.ToString(), *GetInstanceName().ToString());
			bDidWarn = true;
		}
	}
#endif

#if !UE_METASOUND_PURE_VIRTUAL_GET_OPERATOR_DATA
	TSharedPtr<const IOperatorData> INodeBase::GetOperatorData() const
	{
		static bool bDidWarn = false;
		if (!bDidWarn)
		{
			UE_LOGF(LogMetaSound, Warning, "Please implement INodeInterface::GetOperatorData(...) for the class representing node %ls. This method will become pure virtual in future releases. Define UE_METASOUND_PURE_VIRTUAL_GET_OPERATOR_DATA in order to build with this method as a pure virtual on the interface.", *GetInstanceName().ToString());
			bDidWarn = true;
		}
		return {};
	}
#endif
}
