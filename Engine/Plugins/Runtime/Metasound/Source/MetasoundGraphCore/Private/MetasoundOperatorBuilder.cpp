// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorBuilder.h"

#include "Algo/Count.h"
#include "Algo/Transform.h"
#include "HAL/IConsoleManager.h"
#include "MetasoundBuildError.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundBuildError.h"
#include "MetasoundDynamicOperator.h"
#include "MetasoundDynamicOperatorTransactor.h"
#include "MetasoundGraphAlgo.h"
#include "MetasoundGraphAlgoPrivate.h"
#include "MetasoundGraphLinter.h"
#include "MetasoundGraphOperator.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundProfilingOperator.h"
#include "MetasoundRebindableGraphOperator.h"
#include "MetasoundRenderCost.h"
#include "MetasoundThreadLocalDebug.h"
#include "MetasoundTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Templates/PimplPtr.h"


namespace Metasound
{
	static bool bVerboseTraceEvents = false;
	FAutoConsoleVariableRef CVarVerboseTraceEvents(
		TEXT("au.metasound.VerboseBuildGraphTraceEvents"),
		bVerboseTraceEvents,
		TEXT("Enable additional trace scopes for operator building")
	);

	namespace OperatorBuilder
	{
		// Shared context used in the builder to maintain state of current build.
		struct FBuildContext
		{
			const IGraph& Graph;
			const FDirectedGraphAlgoAdapter& AlgoAdapter;
			const FOperatorSettings& Settings;
			const FMetasoundEnvironment& Environment;
			FGraphRenderCost* GraphRenderCost = nullptr;
			
			DirectedGraphAlgo::FGraphOperatorData& GraphOperatorData;
			FBuildResults& Results;

			FBuildContext(
				const IGraph& InGraph,
				const FDirectedGraphAlgoAdapter& InAlgoAdapter,
				const FOperatorSettings& InSettings,
				const FMetasoundEnvironment& InEnvironment,
				FGraphRenderCost* InGraphRenderCost,
				DirectedGraphAlgo::FGraphOperatorData& InGraphOperatorData,
				FBuildResults& OutResults)
			: Graph(InGraph)
			, AlgoAdapter(InAlgoAdapter)
			, Settings(InSettings)
			, Environment(InEnvironment)
			, GraphRenderCost(InGraphRenderCost)
			, GraphOperatorData(InGraphOperatorData)
			, Results(OutResults)
			{
			}
		};
	}

	namespace OperatorBuilderPrivate
	{
		using FBuildErrorPtr = TUniquePtr<IOperatorBuildError>;

		// Convenience function for adding graph cycle build errors
		void AddBuildErrorsForCycles(const FDirectedGraphAlgoAdapter& InAdapter, TArray<FBuildErrorPtr>& OutErrors)
		{
			if (FGraphLinter::ValidateNoCyclesInGraph(InAdapter, OutErrors))
			{
				AddBuildError<FInternalError>(OutErrors, __FILE__, __LINE__);
			}
		}
	}
	
	FOperatorBuilder::FOperatorBuilder(const FOperatorBuilderSettings& InBuilderSettings)
	: BuilderSettings(InBuilderSettings)
	{
	}

	FOperatorBuilder::~FOperatorBuilder()
	{
	}

	TUniquePtr<IOperator> FOperatorBuilder::BuildGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::BuildGraphOperator, bVerboseTraceEvents);

		if (BuilderSettings.bEnableOperatorRebind)
		{
			return BuildRebindableGraphOperator(InParams, OutResults);
		}
		else
		{
			return BuildStaticGraphOperator(InParams, OutResults);
		}
	}

	TUniquePtr<IOperator> FOperatorBuilder::BuildDynamicGraphOperator(const FBuildDynamicGraphOperatorParams& InParams, FBuildResults& OutResults)
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::BuildDynamicGraphOperator);
		using namespace DynamicGraph;

		TArray<const INode*> NodeOrder;
		TUniquePtr<FDynamicOperator> GraphOperator = MakeUnique<FDynamicOperator>(InParams.OperatorSettings, InParams.TransformQueue, InParams.OperatorUpdateCallbacks);
		bool bSuccess = BuildGraphOperatorData(InParams, GetDynamicGraphOperatorData(*GraphOperator), NodeOrder, OutResults);

		if (bSuccess)
		{
			GetDynamicGraphOperatorData(*GraphOperator).InitTables();
			return GraphOperator;
		}

		return TUniquePtr<IOperator>(nullptr);
	}

	TUniquePtr<IOperator> FOperatorBuilder::BuildRebindableGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::BuildRebindableGraphOperator);

		TArray<const INode*> NodeOrder;
		TUniquePtr<FRebindableGraphOperator> GraphOperator = MakeUnique<FRebindableGraphOperator>(InParams.OperatorSettings);
		bool bSuccess =  BuildGraphOperatorData(InParams, GetDynamicGraphOperatorData(*GraphOperator), NodeOrder, OutResults);

		if (bSuccess)
		{
			GetDynamicGraphOperatorData(*GraphOperator).InitTables();
			return GraphOperator;
		}

		return TUniquePtr<IOperator>(nullptr);
	}

	TUniquePtr<IOperator> FOperatorBuilder::BuildStaticGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::BuildStaticGraphOperator, bVerboseTraceEvents);

		TUniquePtr<DirectedGraphAlgo::FStaticGraphOperatorData> GraphOperatorData = MakeUnique<DirectedGraphAlgo::FStaticGraphOperatorData>(InParams.OperatorSettings);

		bool bSuccess =  BuildGraphOperatorData(InParams, *GraphOperatorData, GraphOperatorData->NodeOrder, OutResults);

		if (bSuccess)
		{
			// Create graph operator from collection of node operators.
			return MakeUnique<FGraphOperator>(MoveTemp(GraphOperatorData));
		}

		return TUniquePtr<IOperator>(nullptr);
	}

	DynamicGraph::FDynamicGraphOperatorData& FOperatorBuilder::GetDynamicGraphOperatorData(DynamicGraph::IDynamicGraphInPlaceBuildable& InBuildable) const
	{
		// This function exists as a convenience to avoid needing to do `static_cast<IDynamicGraphInPlaceBuildable&>(GraphOperator)->GetDynamicGraphOperatorData()` 
		// all over the place. 
		return InBuildable.GetDynamicGraphOperatorData();
	}

	bool FOperatorBuilder::BuildGraphOperatorData(const FBuildGraphOperatorParams& InParams, DirectedGraphAlgo::FGraphOperatorData& OutGraphOperatorData, TArray<const INode*>& OutNodeOrder, FBuildResults& OutResults) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::BuildGraphOperator);

		using namespace OperatorBuilderPrivate;
		using namespace DirectedGraphAlgo;

		FBuildStatus BuildStatus = FBuildStatus::NoError;

		// Validate that the sources and destinations declared in an edge actually
		// exist in the node.
		if (BuilderSettings.bValidateVerticesExist)
		{
			if (!FGraphLinter::ValidateVerticesExist(InParams.Graph, OutResults.Errors))
			{
				BuildStatus |= FBuildStatus::FatalError;
			}
		}

		// Validate that the data types for a source and destination match.
		if (BuilderSettings.bValidateEdgeDataTypesMatch)
		{
			if (!FGraphLinter::ValidateEdgeDataTypesMatch(InParams.Graph, OutResults.Errors))
			{
				BuildStatus |= FBuildStatus::FatalError;
			}
		}
		
		// Validate that node inputs only have one source
		if (BuilderSettings.bValidateNoDuplicateInputs)
		{
			if (!FGraphLinter::ValidateNoDuplicateInputs(InParams.Graph, OutResults.Errors))
			{
				BuildStatus |= FBuildStatus::FatalError;
			}
		}

		// Possible early exit if edge validation fails.
		if (BuildStatus > GetMaxErrorLevel())
		{
			return false;
		}

		// Create algo adapter view of graph to cache graph operations.
		TPimplPtr<FDirectedGraphAlgoAdapter> AlgoAdapter = DirectedGraphAlgo::CreateDirectedGraphAlgoAdapter(InParams.Graph);
		
		if (!AlgoAdapter.IsValid())
		{
			AddBuildError<FInternalError>(OutResults.Errors, __FILE__, __LINE__);
			return false;
		}

		// Update environment with current graph hierarchy
		FMetasoundEnvironment NewEnvironment;
		if (!InParams.Environment.Contains<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy))
		{
			// Copy old environment and add new environment variable
			NewEnvironment = InParams.Environment;
			TArray<FGuid> GraphHierarchy;
			GraphHierarchy.Emplace(InParams.Graph.GetInstanceID());
			NewEnvironment.SetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy, GraphHierarchy);
		}
		else
		{
			// Copy and append to existing environment variable (environment variables aren't modifiable in place)
			for (const auto& Iter : InParams.Environment)
			{
				if (Iter.Key == CoreInterface::Environment::GraphHierarchy)
				{
					TArray<FGuid> GraphHierarchy;
					for (const FGuid Id : InParams.Environment.GetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy))
					{
						GraphHierarchy.Emplace(Id);
					}
					GraphHierarchy.Emplace(InParams.Graph.GetInstanceID());

					NewEnvironment.SetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy, GraphHierarchy);
				}
				else
				{
					// Copy other environment variables
					NewEnvironment.SetValue(Iter.Value->Clone());
				}
			}
		}

		OperatorBuilder::FBuildContext BuildContext(InParams.Graph, *AlgoAdapter, InParams.OperatorSettings, NewEnvironment, InParams.GraphRenderCost, OutGraphOperatorData, OutResults);

		// Sort the nodes in a valid execution order
		BuildStatus |= DepthFirstTopologicalSort(BuildContext, OutNodeOrder);

		// TODO: Add FindReachableNodesFromVariables in Prune.
		// TODO: will need to prune edges as well.
		// Otherwise, subgraphs incorrectly get pruned.
		// BuildStatus |= PruneNodges(BuildContext, OutNodeOrder);

		// Check build status in case build routine should be exited early.
		if (BuildStatus > GetMaxErrorLevel())
		{
			return false;
		}

		InitializeVertexInterfaceData(InParams.Graph, BuildContext.GraphOperatorData);

		InitializeOperatorInfo(InParams.Graph, OutNodeOrder, BuildContext.GraphOperatorData);

		// Assign external inputs to various vertex interfaces.
		BuildStatus |= FOperatorBuilder::GatherExternalInputDataReferences(BuildContext, InParams.InputData);

		// Check build status in case build routine should be exited early.
		if (BuildStatus > GetMaxErrorLevel())
		{
			return false;
		}

		// Create node operators from factories.
		BuildStatus |= CreateOperators(BuildContext, OutNodeOrder, InParams.InputData);

		if (BuildStatus > GetMaxErrorLevel())
		{
			return false;
		}

		if (BuilderSettings.bPopulateInternalDataReferences)
		{
			GatherInternalGraphDataReferences(BuildContext, OutNodeOrder, BuildContext.Results.InternalDataReferences);
		}

		// Gather the inputs for the graph data. 
		BuildStatus |= GatherGraphDataReferences(BuildContext, BuildContext.GraphOperatorData.VertexData);

		if (BuildStatus > GetMaxErrorLevel())
		{
			return false;
		}

		return true;
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::DepthFirstTopologicalSort(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& OutNodes) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::DepthFirstTopologicalSort, bVerboseTraceEvents);
		using namespace OperatorBuilderPrivate;

		bool bSuccess = DirectedGraphAlgo::DepthFirstTopologicalSort(InOutContext.AlgoAdapter, OutNodes);

		if (!bSuccess)
		{
			// If there was an error, there is likely a cycle in the graph.
			AddBuildErrorsForCycles(InOutContext.AlgoAdapter, InOutContext.Results.Errors);

			return FBuildStatus::FatalError;
		}

		return FBuildStatus::NoError;
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::KahnsTopologicalSort(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& OutNodes) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::KahnsTopologicalSort, bVerboseTraceEvents);
		using namespace OperatorBuilderPrivate;

		bool bSuccess = DirectedGraphAlgo::KahnTopologicalSort(InOutContext.AlgoAdapter, OutNodes);

		if (!bSuccess)
		{
			// If there was an error, there is likely a cycle in the graph.
			AddBuildErrorsForCycles(InOutContext.AlgoAdapter, InOutContext.Results.Errors);

			return FBuildStatus::FatalError;
		}

		return FBuildStatus::NoError;
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::PruneNodes(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& InOutNodes) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::PruneNodes);
		using namespace OperatorBuilderPrivate;

		FBuildStatus BuildStatus = FBuildStatus::NoError;

		TSet<const INode*> ReachableNodes;

		switch (BuilderSettings.PruningMode)
		{
			case EOperatorBuilderNodePruning::PruneNodesWithoutExternalDependency:
				DirectedGraphAlgo::FindReachableNodes(InOutContext.AlgoAdapter, ReachableNodes);
				break;

			case EOperatorBuilderNodePruning::PruneNodesWithoutOutputDependency:
				DirectedGraphAlgo::FindReachableNodesFromOutput(InOutContext.AlgoAdapter, ReachableNodes);
				break;

			case EOperatorBuilderNodePruning::PruneNodesWithoutInputDependency:
				DirectedGraphAlgo::FindReachableNodesFromInput(InOutContext.AlgoAdapter, ReachableNodes);
				break;

			case EOperatorBuilderNodePruning::None:
			default:
				return FBuildStatus::NoError;
		}

		if (InOutNodes.Num() == ReachableNodes.Num())
		{
			// Nothing to remove since all nodes are reachable. It's assumed that
			// InOutNodes has a unique set of nodes. 
			return FBuildStatus::NoError;
		}

		if (0 == ReachableNodes.Num())
		{
			// Pruning all nodes. 
			for (const INode* Node : InOutNodes)
			{
				AddBuildError<FNodePrunedError>(InOutContext.Results.Errors, Node);
			}

			InOutNodes.Reset();

			// This is non fatal, but results in an IOperator which is a No-op.
			return FBuildStatus::NonFatalError;
		}

		// Split the nodes-to-keep and the nodes-to-prune into two arrays. Need 
		// to ensure that kept nodes are still in same relative order. 
		TArray<const INode*> SortedNodesToKeep;

		SortedNodesToKeep.Reserve(ReachableNodes.Num());

		for (const INode* Node : InOutNodes)
		{
			if (ReachableNodes.Contains(Node))
			{
				SortedNodesToKeep.Add(Node);
			}
			else
			{
				AddBuildError<FNodePrunedError>(InOutContext.Results.Errors, Node);

				// Denote a pruned node as a non-fatal error. In the future this
				// may be simply a warning as some nodes are required to conform
				// to metasound interfaces even if they are unused.
				BuildStatus |= FBuildStatus::NonFatalError;
			}
		}

		InOutNodes = SortedNodesToKeep;

		return BuildStatus;
	}

	void FOperatorBuilder::InitializeVertexInterfaceData(const IGraph& InGraph, DirectedGraphAlgo::FGraphOperatorData& InOutGraphOperatorData) const
	{
		InOutGraphOperatorData.VertexData = FVertexInterfaceData(InGraph.GetVertexInterface());
	}

	void FOperatorBuilder::InitializeOperatorInfo(const IGraph& InGraph, TArray<const INode*>& InSortedNodes, DirectedGraphAlgo::FGraphOperatorData& InOutGraphOperatorData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::InitializeOperatorInfo, bVerboseTraceEvents);
		using namespace DirectedGraphAlgo;

		// Create FOperatorInfos from Nodes
		TSortedMap<FOperatorID, FGraphOperatorData::FOperatorInfo>& OperatorMap = InOutGraphOperatorData.OperatorMap;

		const int32 NumNodes = InSortedNodes.Num();
		OperatorMap.Reserve(OperatorMap.Num() + NumNodes);

		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::InitializeOperatorInfo::Nodes, bVerboseTraceEvents);
			int32 Ordinal = 0;
			for (const INode* Node : InSortedNodes)
			{
				FOperatorID OperatorID = GetOperatorID(Node);
				OperatorMap.Add(OperatorID, FGraphOperatorData::FOperatorInfo{Ordinal, nullptr, Node->GetVertexInterface()});
				Ordinal++;
			}
		}

		{
			METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::InitializeOperatorInfo::OutputDestinations, bVerboseTraceEvents);
			// Set the output destinations on operator infos
			for (const FDataEdge& Edge : InGraph.GetDataEdges())
			{
				if ((Edge.From.Node != nullptr) && (Edge.To.Node != nullptr))
				{
					const FOperatorID FromOperatorID = GetOperatorID(Edge.From.Node);
					FGraphOperatorData::FOperatorInfo& OperatorInfo = OperatorMap.FindChecked(FromOperatorID);

					OperatorInfo.OutputConnections.FindOrAdd(Edge.From.Vertex.VertexName).Add(FGraphOperatorData::FVertexDestination{GetOperatorID(Edge.To.Node), Edge.To.Vertex.VertexName});
				}
			}
		}
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::GatherExternalInputDataReferences(OperatorBuilder::FBuildContext& InOutContext, const FInputVertexInterfaceData& InExternalInputData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::GatherExternalInputDataReferences, bVerboseTraceEvents);
		using namespace DirectedGraphAlgo;

		FBuildStatus BuildStatus;
		// Gather external input data to graph destinations
		for (const TPair<FNodeDataVertexKey, FInputDataDestination>& InputDestinationKV : InOutContext.Graph.GetInputDataDestinations())
		{
			const FInputDataDestination& Destination = InputDestinationKV.Value;

			FOperatorID OperatorID = GetOperatorID(Destination.Node);
			FGraphOperatorData::FOperatorInfo& OperatorInfo = InOutContext.GraphOperatorData.OperatorMap.FindChecked(OperatorID);

			if (const FAnyDataReference* DataReference = InExternalInputData.FindDataReference(Destination.Vertex.VertexName))
			{
				if (IsCastable(DataReference->GetDataTypeName(), Destination.Vertex.DataTypeName))
				{
					OperatorInfo.VertexData.GetInputs().SetVertex(Destination.Vertex.VertexName, *DataReference);
				}
				else
				{
					// Mismatch in datatypes. This likely corresponds to a corrupt 
					// Metasound Graph. The graph's inputs should route directly
					// to TInputNode<>s with matching data types. 
					
					// Create source for reporting connection since external inputs do not have nodes. 
					const FInputDataVertex& GraphVertex = InExternalInputData.GetVertex(Destination.Vertex.VertexName);

					FOutputDataSource Source;
					Source.Vertex.VertexName = GraphVertex.VertexName; 
					Source.Vertex.DataTypeName = GraphVertex.DataTypeName;
#if WITH_EDITORONLY_DATA
					Source.Vertex.Metadata = GraphVertex.Metadata;
#endif

					AddBuildError<FInvalidConnectionDataTypeError>(InOutContext.Results.Errors, FDataEdge{Source, Destination});
				}
			}
			else
			{
				OperatorInfo.VertexData.GetInputs().SetDefaultLiteral(Destination.Vertex.VertexName, Destination.Vertex.GetDefaultLiteral());
			}
		}

		return BuildStatus;
	}

	void FOperatorBuilder::GatherInternalGraphDataReferences(OperatorBuilder::FBuildContext& InOutContext, const TArray<const INode*>& InNodes, TMap<FGuid, FDataReferenceCollection>& OutNodeVertexData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::GatherInternalGraphDataReferences, bVerboseTraceEvents);
		using namespace DirectedGraphAlgo;

		for (const INode* NodePtr : InNodes)
		{
			check(NodePtr);
			if (const FGraphOperatorData::FOperatorInfo* OpInfo = InOutContext.GraphOperatorData.OperatorMap.Find(GetOperatorID(NodePtr)))
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				OutNodeVertexData.Emplace(NodePtr->GetInstanceID(), OpInfo->VertexData.GetOutputs().ToDataReferenceCollection());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::ValidateOperatorOutputsAreBound(const INode& InNode, const FOutputVertexInterfaceData& InVertexData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::ValidateOperatorOutputsAreBound, bVerboseTraceEvents);
		using FOutputVertexBinding = VertexDataPrivate::FOutputBinding;

		FBuildStatus BuildStatus = FBuildStatus::NoError;
		bool bFoundUnboundVertex = false;
		for (const FOutputVertexBinding& OutputBinding : InVertexData)
		{
			if (!OutputBinding.IsBound())
			{
				bFoundUnboundVertex = true;
				const FNodeClassMetadata& Metadata = InNode.GetMetadata();
				UE_LOGF(LogMetaSound, Warning, "Operator for node %ls v%d.%d contains unbound output vertex %ls", *Metadata.ClassName.ToString(), Metadata.MajorVersion, Metadata.MinorVersion, *OutputBinding.GetVertex().VertexName.ToString());
			}
		}

		if (bFoundUnboundVertex)
		{
			BuildStatus |= FBuildStatus::NonFatalError;
		}

		return BuildStatus;
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::CreateOperators(OperatorBuilder::FBuildContext& InOutContext, const TArray<const INode*>& InSortedNodes, const FInputVertexInterfaceData& InExternalInputData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::CreateOperators);
		METASOUND_DEBUG_DECLARE_SCOPE;

		using namespace DirectedGraphAlgo;

		bool ProfileOperators = BuilderSettings.bProfileOperators || Profiling::ProfileAllGraphs();

		FBuildStatus BuildStatus;

		// Create FOperatorInfos from Nodes
		TSortedMap<FOperatorID, FGraphOperatorData::FOperatorInfo>& OperatorMap = InOutContext.GraphOperatorData.OperatorMap;

		// Call operator factory for each node.
		for (const INode* Node : InSortedNodes)
		{
			METASOUND_DEBUG_SET_ACTIVE_NODE_SCOPE(Node);
			FOperatorID OperatorID = GetOperatorID(Node);
			FGraphOperatorData::FOperatorInfo& OperatorInfo = OperatorMap.FindChecked(OperatorID);

			{
#if METASOUND_CPUPROFILERTRACE_ENABLED
				// Use node class name if valid, otherwise (for example graph nodes) use instance name 
				TStringBuilder<256> TraceNamePtr;
				const FNodeClassName& NodeClassName = Node->GetMetadata().ClassName;
				const FString NodeTraceName = NodeClassName.IsValid() ? NodeClassName.ToString() : Node->GetInstanceName().ToString();
				TraceNamePtr << "Metasound::FOperatorBuilder::CreateOperators::CreateAndBind " << NodeTraceName;
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_COND(*TraceNamePtr, bVerboseTraceEvents);
#endif // METASOUND_CPUPROFILERTRACE_ENABLED

				FBuildOperatorParams CreateParams{*Node, InOutContext.Settings, OperatorInfo.VertexData.GetInputs(), InOutContext.Environment, this, InOutContext.GraphRenderCost};
				FOperatorFactorySharedRef Factory = Node->GetDefaultOperatorFactory();
				if (ProfileOperators && Profiling::OperatorShouldBeProfiled(Node->GetMetadata()))
				{
					METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::CreateOperators::CreateOperator, bVerboseTraceEvents);

					TUniquePtr<FProfilingOperator> ProfilingOperatorPtr = MakeUnique<FProfilingOperator>(Factory->CreateOperator(CreateParams, InOutContext.Results), Node);
					check(ProfilingOperatorPtr);
#if UE_METASOUND_DEBUG_ENABLED
					ProfilingOperatorPtr->SetAssetMetaData(METASOUND_DEBUG_ACTIVE_ASSET_SCOPE);
#endif // #if UE_METASOUND_DEBUG_ENABLED
					
					OperatorInfo.Operator = MoveTemp(ProfilingOperatorPtr);
				}
				else
				{
					METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::CreateOperators::CreateOperator, bVerboseTraceEvents);
					OperatorInfo.Operator = Factory->CreateOperator(CreateParams, InOutContext.Results);
				}

				if (!OperatorInfo.Operator.IsValid())
				{
					return FBuildStatus::FatalError;
				}

				// Bind vertex to operator data
				{
					METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::CreateOperators::BindInputsAndOutputs, bVerboseTraceEvents);
					// Inputs don't need to be bound for all nodes unless they are dynamic 
					// Inputs for input nodes will be bound separately in GatherGraphDataReferences
					if (BuilderSettings.bEnableOperatorRebind)
					{
						OperatorInfo.Operator->BindInputs(OperatorInfo.VertexData.GetInputs());
					}
					OperatorInfo.Operator->BindOutputs(OperatorInfo.VertexData.GetOutputs());
				}
				
				// Check if outputs are bound correctly.
				if (BuilderSettings.bValidateOperatorOutputsAreBound)
				{
					METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Metasound::FOperatorBuilder::CreateOperators::ValidateOperatorOutputsAreBound);

					BuildStatus |= ValidateOperatorOutputsAreBound(*Node, OperatorInfo.VertexData.GetOutputs());
				}
			}


			// Route outputs of operator to downstream operators' VertexData
			{
				METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::CreateOperators::RouteOutputs, bVerboseTraceEvents);
				for (const TPair<FVertexName, TArray<FGraphOperatorData::FVertexDestination>>& OutputRouting : OperatorInfo.OutputConnections)
				{
					const FVertexName& VertexName = OutputRouting.Key;

					if (const FAnyDataReference* Ref = OperatorInfo.VertexData.GetOutputs().FindDataReference(VertexName))
					{
						for (const FGraphOperatorData::FVertexDestination& Destination : OutputRouting.Value)
						{
							OperatorMap.FindChecked(Destination.OperatorID).VertexData.GetInputs().SetVertex(Destination.VertexName, *Ref);
						}
					}
					else
					{
						// Does not contain any reference
						// This is likely a node programming error where the edges reported by the INode interface
						// did not match the readable parameter refs created by the operators outputs. Or, the edge description is invalid.

						AddBuildError<FMissingOutputDataReferenceError>(InOutContext.Results.Errors, FOutputDataSource{*Node, Node->GetVertexInterface().GetOutputVertex(VertexName)});

						BuildStatus |= FBuildStatus::NonFatalError;
					}
				}
			}
		}

		return BuildStatus;
	}

	FOperatorBuilder::FBuildStatus FOperatorBuilder::GatherGraphDataReferences(OperatorBuilder::FBuildContext& InOutContext, FVertexInterfaceData& OutVertexData) const
	{
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Metasound::FOperatorBuilder::GatherGraphDataReferences, bVerboseTraceEvents);
		using namespace OperatorBuilderPrivate;
		using namespace DirectedGraphAlgo;
		using FDestinationElement = FInputDataDestinationCollection::ElementType;
		using FSourceElement = FOutputDataSourceCollection::ElementType;

		FBuildStatus BuildStatus;
		// Gather graph inputs
		for (const FDestinationElement& Element : InOutContext.Graph.GetInputDataDestinations())
		{
			bool bFoundDataReference = false;
			const FInputDataDestination& InputDestination = Element.Value;

			const FOperatorID OperatorID = GetOperatorID(InputDestination.Node);
			if (FGraphOperatorData::FOperatorInfo* OperatorInfo = InOutContext.GraphOperatorData.OperatorMap.Find(OperatorID))
			{
				FInputVertexInterfaceData& NodeInputData = OperatorInfo->VertexData.GetInputs();
				OperatorInfo->Operator->BindInputs(NodeInputData);

				if (const FAnyDataReference* DataReference = NodeInputData.FindDataReference(InputDestination.Vertex.VertexName))
				{
					if (IsCastable(DataReference->GetDataTypeName(), InputDestination.Vertex.DataTypeName))
					{
						bFoundDataReference = true;
						OutVertexData.GetInputs().SetVertex(InputDestination.Vertex.VertexName, *DataReference);
					}
				}
				InOutContext.GraphOperatorData.InputVertexMap.Emplace(InputDestination.Vertex.VertexName, OperatorID);
			}

			if (!bFoundDataReference)
			{
				AddBuildError<FMissingInputDataReferenceError>(InOutContext.Results.Errors, InputDestination);
				BuildStatus |= FBuildStatus::NonFatalError;
			}
		}

		// Gather graph outputs.
		for (const FSourceElement& Element : InOutContext.Graph.GetOutputDataSources())
		{
			bool bFoundDataReference = false;
			const FOutputDataSource& OutputSource = Element.Value;

			const FOperatorID OperatorID = GetOperatorID(OutputSource.Node);
			if (const FGraphOperatorData::FOperatorInfo* OperatorInfo = InOutContext.GraphOperatorData.OperatorMap.Find(OperatorID))
			{
				const FOutputVertexInterfaceData& NodeOutputData = OperatorInfo->VertexData.GetOutputs();
				if (const FAnyDataReference* DataReference = NodeOutputData.FindDataReference(OutputSource.Vertex.VertexName))
				{
					if (IsCastable(DataReference->GetDataTypeName() , OutputSource.Vertex.DataTypeName))
					{
						bFoundDataReference = true;
						OutVertexData.GetOutputs().SetVertex(OutputSource.Vertex.VertexName, FAnyDataReference{*DataReference});
					}
				}
				InOutContext.GraphOperatorData.OutputVertexMap.Emplace(OutputSource.Vertex.VertexName, OperatorID);
			}

			if (!bFoundDataReference)
			{
				AddBuildError<FMissingOutputDataReferenceError>(InOutContext.Results.Errors, OutputSource);
				BuildStatus |= FBuildStatus::NonFatalError;
			}
		}
	
		return BuildStatus;
	}

	FOperatorBuilder::FBuildStatus::EStatus FOperatorBuilder::GetMaxErrorLevel() const
	{
		return BuilderSettings.bFailOnAnyError ? FBuildStatus::NoError : FBuildStatus::NonFatalError;
	}
}
