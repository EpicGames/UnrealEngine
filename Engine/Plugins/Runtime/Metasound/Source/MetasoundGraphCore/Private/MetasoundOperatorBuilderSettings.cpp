// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorBuilderSettings.h"

#include "Misc/Build.h"


namespace Metasound
{
	const FOperatorBuilderSettings& FOperatorBuilderSettings::GetDefaultSettings()
	{
#if UE_BUILD_SHIPPING
		return GetDefaultShippingSettings();
#elif UE_BUILD_TEST
		return GetDefaultTestSettings();
#elif UE_BUILD_DEVELOPMENT
		return GetDefaultDevelopementSettings();
#elif UE_BUILD_DEBUG
		return GetDefaultDebugSettings();
#else
		return GetDefaultShippingSettings();
#endif
	}

	const FOperatorBuilderSettings& FOperatorBuilderSettings::GetDefaultDebugSettings()
	{
		auto CreateSettings = [&]()
		{
			FOperatorBuilderSettings Settings;
			Settings.PruningMode = EOperatorBuilderNodePruning::None;
			Settings.bValidateNoCyclesInGraph = true;
			Settings.bValidateNoDuplicateInputs = true;
			Settings.bValidateVerticesExist = true;
			Settings.bValidateEdgeDataTypesMatch = true;
			Settings.bValidateOperatorOutputsAreBound = true;
			Settings.bFailOnAnyError = false;
			return Settings;
		};

		static const FOperatorBuilderSettings DebugSettings = CreateSettings();
		return DebugSettings;
	}

	const FOperatorBuilderSettings& FOperatorBuilderSettings::GetDefaultDevelopementSettings()
	{
		auto CreateSettings = [&]()
		{
			FOperatorBuilderSettings Settings;
			Settings.PruningMode = EOperatorBuilderNodePruning::PruneNodesWithoutExternalDependency;
			Settings.bValidateNoCyclesInGraph = true;
			Settings.bValidateNoDuplicateInputs = true;
			Settings.bValidateVerticesExist = true;
			Settings.bValidateEdgeDataTypesMatch = true;
			Settings.bValidateOperatorOutputsAreBound = true;
			Settings.bFailOnAnyError = false;
			return Settings;
		};

		static const FOperatorBuilderSettings DevSettings = CreateSettings();
		return DevSettings;
	}

	const FOperatorBuilderSettings& FOperatorBuilderSettings::GetDefaultTestSettings()
	{
		auto CreateSettings = [&]()
		{
			FOperatorBuilderSettings Settings;
			Settings.PruningMode = EOperatorBuilderNodePruning::PruneNodesWithoutExternalDependency;
			Settings.bValidateNoCyclesInGraph = false;
			Settings.bValidateNoDuplicateInputs = false;
			Settings.bValidateVerticesExist = false;
			Settings.bValidateEdgeDataTypesMatch = false;
			Settings.bValidateOperatorOutputsAreBound = false;
			Settings.bFailOnAnyError = false;
			return Settings;
		};

		static const FOperatorBuilderSettings TestSettings = CreateSettings();
		return TestSettings;
	}

	const FOperatorBuilderSettings& FOperatorBuilderSettings::GetDefaultShippingSettings()
	{
		auto CreateSettings = [&]()
		{
			FOperatorBuilderSettings Settings;
			Settings.PruningMode = EOperatorBuilderNodePruning::PruneNodesWithoutExternalDependency;
			Settings.bValidateNoCyclesInGraph = false;
			Settings.bValidateNoDuplicateInputs = false;
			Settings.bValidateVerticesExist = false;
			Settings.bValidateEdgeDataTypesMatch = false;
			Settings.bValidateOperatorOutputsAreBound = false;
			Settings.bFailOnAnyError = false;
			return Settings;
		};

		static const FOperatorBuilderSettings ShippingSettings = CreateSettings();
		return ShippingSettings;
	}
} // namespace Metasound
