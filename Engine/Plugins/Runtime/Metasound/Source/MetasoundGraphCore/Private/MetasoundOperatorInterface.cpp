// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorInterface.h"

#include "HAL/IConsoleManager.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundLog.h"
#include "MetasoundThreadLocalDebug.h"

#include <atomic>

namespace Metasound
{
	namespace MetasoundOperatorInterfacePrivate
	{
		static bool bEnableMetaSoundOperatorMissingOverrideLog = false;
	}
}

FAutoConsoleVariableRef CVarMetaSoundEnableOperatorMissingOverrideLog(
	TEXT("au.MetaSound.Debug.EnableOperatorMissingOverrideLog"),
	Metasound::MetasoundOperatorInterfacePrivate::bEnableMetaSoundOperatorMissingOverrideLog,
	TEXT("Enables additional logging on operators with missing overrides\n")
	TEXT("Default: false"),
	ECVF_Default);

namespace Metasound
{
	namespace MetasoundOperatorInterfacePrivate
	{
		static std::atomic<bool> bHasWarnedMissingBindImplementation = false;

		void WarnMissingBindImplementation()
		{
			if (!bHasWarnedMissingBindImplementation)
			{
				bHasWarnedMissingBindImplementation = true;
				UE_LOGF(LogMetaSound, Warning, "One or more MetaSound IOperator derived classes have not overridden the BindInputs(...) and/or BindOutputs(...) methods. Please override these methods and ensure that the IOperator derived class conforms to the most recent IOperator API. Enable console variable au.MetaSound.Debug.EnableOperatorMissingOverrideLog for additional information in logs.");
			}

			if (bEnableMetaSoundOperatorMissingOverrideLog)
			{
				UE_LOGF(LogMetaSound, Warning, "MetaSound Node %ls produced IOperator which does not override BindInputs(...) or BindOutputs(...). Please override these methods and ensure that the IOperator derived class conforms to the most recent IOperator API.", METASOUND_DEBUG_ACTIVE_NODE_NAME);
			}
		}
	}

	FDataReferenceCollection IOperator::GetInputs() const 
	{
		return FDataReferenceCollection();
	}

	FDataReferenceCollection IOperator::GetOutputs() const
	{
		return FDataReferenceCollection();
	}

	void IOperator::Bind(FVertexInterfaceData& InVertexData) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InVertexData.GetInputs().Set(GetInputs());
		InVertexData.GetOutputs().Set(GetOutputs());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	
	void IOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
	{
		MetasoundOperatorInterfacePrivate::WarnMissingBindImplementation();

		FVertexInterfaceData Data;
		Data.GetInputs() = InVertexData;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Bind(Data);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		InVertexData = Data.GetInputs();
	}

	void IOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData) 
	{
		MetasoundOperatorInterfacePrivate::WarnMissingBindImplementation();

		FVertexInterfaceData Data;
		Data.GetOutputs() = InVertexData;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Bind(Data);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		InVertexData = Data.GetOutputs();
	}
}
