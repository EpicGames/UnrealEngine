// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundOperatorSettings.h"

#include "DSP/BufferVectorOperations.h"
#include "CoreMinimal.h"

namespace Metasound
{
	namespace MetasoundOperatorSettingsPrivate
	{
		// Fall back to this sample rate if an invalid sample rate is given.
		static constexpr float DefaultSampleRate = 48000.f;
		static constexpr float MinimumTargetBlockRate = 1.0f;

		// Using same alignment as in BufferVectorOperations.h
		static constexpr int32 FloatAlignment = AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;

		static constexpr int32 MinimumNumFramesPerBlock = FloatAlignment;
		static constexpr int32 MaximumNumFramesPerBlock = 16384;
	}

	FOperatorSettings::FOperatorSettings(FSampleRate InSampleRate, float InTargetBlockRate)
	{
		SetSampleRate(InSampleRate);
		SetTargetBlockRate(InTargetBlockRate);
	}

	void FOperatorSettings::SetSampleRate(FSampleRate InSampleRate)
	{
		using namespace MetasoundOperatorSettingsPrivate;

		if (ensureMsgf(InSampleRate > 0, TEXT("Invalid sample rate %d. Defaulting to %f"), InSampleRate, DefaultSampleRate))
		{
			SampleRate = InSampleRate;
		}
		else
		{
			SampleRate = DefaultSampleRate;
		}

		// Update frames per block and actual block rate.
		Update(); 
	}

	void FOperatorSettings::SetTargetBlockRate(float InTargetBlockRate)
	{
		using namespace MetasoundOperatorSettingsPrivate;

		if (ensureMsgf(InTargetBlockRate >= MinimumTargetBlockRate, TEXT("Invalid block rate %f. Defaulting to %f"), InTargetBlockRate, MinimumTargetBlockRate))
		{
			TargetBlockRate = InTargetBlockRate;
		}
		else
		{
			TargetBlockRate = MinimumTargetBlockRate;
		}

		// Update frames per block and actual block rate.
		Update();
	}

	float FOperatorSettings::GetTargetBlockRate() const
	{
		return TargetBlockRate;
	}

	float FOperatorSettings::GetActualBlockRate() const
	{
		return ActualBlockRate;
	}

	void FOperatorSettings::Update()
	{
		using namespace MetasoundOperatorSettingsPrivate;

		// Protects for divide by zero.
		check(MinimumNumFramesPerBlock > 0);


		float TargetNumFramesPerBlock = SampleRate / FMath::Max(TargetBlockRate, MinimumTargetBlockRate);

		NumFramesPerBlock = RoundToAligned(FloatAlignment, FMath::RoundToInt(TargetNumFramesPerBlock));

		// The minimum and maximum number of frames per a block must also satisfy alignment 
		// requirements since we clamp to them after enforcing alignment.
		checkf(RoundToAligned(FloatAlignment, MinimumNumFramesPerBlock) == MinimumNumFramesPerBlock, TEXT("MinimumNumFramesPerBlock (%d) is not aligned to required buffer alignment (%d)"), MinimumNumFramesPerBlock, FloatAlignment);
		checkf(RoundToAligned(FloatAlignment, MaximumNumFramesPerBlock) == MaximumNumFramesPerBlock, TEXT("MaximumNumFramesPerBlock (%d) is not aligned to required buffer alignment (%d)"), MaximumNumFramesPerBlock, FloatAlignment);

		NumFramesPerBlock = FMath::Clamp(NumFramesPerBlock, MinimumNumFramesPerBlock, MaximumNumFramesPerBlock);

		ActualBlockRate = SampleRate / static_cast<float>(NumFramesPerBlock);
	}

	int32 FOperatorSettings::RoundToAligned(int32 Alignment, int32 InNum) const
	{
		check(Alignment > 0);

		// Need to add (Alignment / 2) in order to perform rounding operation.
		// Otherwise, this function would be more like "FloorToAligned(...)"
		int32 AlignedNum = InNum + (Alignment / 2);

		AlignedNum = AlignedNum - (AlignedNum % Alignment);

		return AlignedNum;
	}

	bool operator<(const FOperatorSettings& InLHS, const FOperatorSettings& InRHS)
	{
		if (InLHS.SampleRate < InRHS.SampleRate)
		{
			return true;
		}
		else if (InRHS.SampleRate < InLHS.SampleRate)
		{
			return false;
		}

		// Ignore TargetBlockRate in comparison because it is not used in any external calculations
		// Ignore ActualBlockRate because it is derived from NumFramesPerBlock

		return InLHS.NumFramesPerBlock < InRHS.NumFramesPerBlock;
	}

	FString FOperatorSettings::ToString() const
	{
		return FString::Printf(TEXT("SampleRate:%d BlockSize:%d"), SampleRate, NumFramesPerBlock);
	}

	bool operator==(const FOperatorSettings& InLHS, const FOperatorSettings& InRHS)
	{
		// Ignore TargetBlockRate in comparison because it is not used in any external calculations
		// Ignore ActualBlockRate because it is derived from NumFramesPerBlock
		return (InLHS.SampleRate == InRHS.SampleRate) && (InLHS.NumFramesPerBlock == InRHS.NumFramesPerBlock);
	}
}

