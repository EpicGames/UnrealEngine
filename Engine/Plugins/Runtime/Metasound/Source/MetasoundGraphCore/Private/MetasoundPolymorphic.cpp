// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundPolymorphic.h"

namespace Metasound
{
	FPolyRegistry& FPolyRegistry::Get()
	{
		static FPolyRegistry Instance;
		return Instance;
	}

	void FPolyRegistry::Register(const FPolyTypeInfo& Info)
	{
		Map.Add(Info.TypeName, Info);
	}

	void FPolyRegistry::Unregister(const FPolyTypeInfo& Info)
	{
		Map.Remove(Info.TypeName);
	}

	const FPolyTypeInfo* FPolyRegistry::Find(const FName Type) const
	{
		return Map.Find(Type);
	}

	void FPolyRegistry::GetAllRegisteredTypes(TArray<FPolyTypeInfo>& Out) const
	{
		Map.GenerateValueArray(Out);
	}

	bool IsChildOfByName(const FName InDataType, const FName InPotentialBase)
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS	
		const FPolyRegistry& Reg = FPolyRegistry::Get();
		for (FName i = InDataType; i != NAME_None; )
		{
			if (i == InPotentialBase)
			{
				return true;
			}
			const FPolyTypeInfo* Info = Reg.Find(i);
			if (!Info || !Info->bIsPolymorphic)
			{
				return false;
			}
			i = Info->BaseTypeName;
		}
		return false;
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}

	bool IsCastable(const FName InType, const FName InPotentialBase)
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS	
		return IsChildOfByName(InType, InPotentialBase);
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}
}