// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundProfilingOperator.h"

#include "MetasoundTrace.h"
#include "HAL/IConsoleManager.h"

#include "ProfilingDebugging/AssetMetadataTrace.h"

DEFINE_LOG_CATEGORY_STATIC(LogMetasoundProfiling, Log, All);

namespace Metasound
{
	namespace Profiling
	{
		int32 ProfileAllEnabled = 0;
		FAutoConsoleVariableRef CVarMetaSoundProfileAllEnabled(
			TEXT("au.MetaSound.ProfileAllGraphs"),
			ProfileAllEnabled,
			TEXT("Enable profiling of all MetaSound graphs. NOTE: If the node filter is set it will still apply (see au.Metasound.AddProfileNode)\n")
			TEXT("0: Disabled (default), !0: Enabled"),
			ECVF_Default);

		TArray<FString> ProfilingNodeFilter;

		void Init()
		{
			IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("au.Metasound.Profiling.AddNodes"),
				TEXT("Adds the specified node class name(s) to the list of metasound nodes that will be profiled and visible in Insights."),
				FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
					{
						for (const FString& NodeName : Args)
						{
							ProfilingNodeFilter.AddUnique(NodeName);
						}
					}),
				ECVF_Default
			);
			IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("au.Metasound.Profiling.ListNodes"),
				TEXT("Lists the node class names that will be profiled and visible in Insights."),
				FConsoleCommandDelegate::CreateLambda([]()
					{
						UE_LOGF(LogMetasoundProfiling, Display, "Metasound profiling limited to nodes with these names:");
						for (const FString& NodeName : ProfilingNodeFilter)
						{
							UE_LOGF(LogMetasoundProfiling, Display, "    %ls", *NodeName);
						}
					}),
				ECVF_Default
			);
			IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("au.Metasound.Profiling.RemoveNodes"),
				TEXT("Removes the specified node class name(s) (or ALL if no names are provided) from the list of node types that will be profiled and visible in Insights."),
				FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
					{
						if (Args.Num() == 0)
						{
							ProfilingNodeFilter.Empty();
						}
						else
						{
							for (const FString& NodeName : Args)
							{
								ProfilingNodeFilter.Remove(NodeName);
							}
						}
					}),
				ECVF_Default
			);

		}

		bool OperatorShouldBeProfiled(const FNodeClassMetadata& NodeMetadata)
		{
			return ProfilingNodeFilter.IsEmpty() || ProfilingNodeFilter.Contains(NodeMetadata.ClassName.GetName().ToString());
		}

		bool ProfileAllGraphs()
		{
			return ProfileAllEnabled != 0;
		}
		
		
	}

#if UE_METASOUND_DEBUG_ENABLED
	void FProfilingOperator::SetAssetMetaData(const ThreadLocalDebug::FAssetMetaData& InAssetMetaData)
	{
		AssetMetaData = InAssetMetaData;
	}
#endif // #if UE_METASOUND_DEBUG_ENABLED

	void FProfilingOperator::StaticReset(IOperator* InOperator, const IOperator::FResetParams& InParams)
	{
		FProfilingOperator* DerivedOperator = static_cast<FProfilingOperator*>(InOperator);
		check(nullptr != DerivedOperator && DerivedOperator->ResetFunction);
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_FAST(DerivedOperator->InsightsResetEventSpecId, *DerivedOperator->InsightsResetEventName);
		DerivedOperator->ResetFunction(DerivedOperator->Operator.Get(), InParams);
	}

	void FProfilingOperator::StaticExecute(IOperator* InOperator)
	{
		FProfilingOperator* DerivedOperator = static_cast<FProfilingOperator*>(InOperator);
		check(nullptr != DerivedOperator && DerivedOperator->ExecuteFunction); 
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_FAST(DerivedOperator->InsightsExecuteEventSpecId, *DerivedOperator->InsightsExecuteEventName);

#if UE_METASOUND_DEBUG_ENABLED
		const ThreadLocalDebug::FAssetMetaData& AssetMetaData = DerivedOperator->AssetMetaData;
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(AssetMetaData.ClassName, AssetMetaData.AssetName, AssetMetaData.AssetPath);
#endif // #if UE_METASOUND_DEBUG_ENABLED
		
		DerivedOperator->ExecuteFunction(DerivedOperator->Operator.Get());
	}

	void FProfilingOperator::StaticPostExecute(IOperator* InOperator)
	{
		FProfilingOperator* DerivedOperator = static_cast<FProfilingOperator*>(InOperator);
		check(nullptr != DerivedOperator && DerivedOperator->PostExecuteFunction);
		METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_FAST(DerivedOperator->InsightsPostExecuteEventSpecId, *DerivedOperator->InsightsPostExecuteEventName);
		DerivedOperator->PostExecuteFunction(DerivedOperator->Operator.Get());
	}

}