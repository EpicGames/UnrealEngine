// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundThreadLocalDebug.h"

namespace Metasound
{

	namespace Profiling
	{
		void Init();
		bool OperatorShouldBeProfiled(const FNodeClassMetadata& NodeMetadata);
		bool ProfileAllGraphs();
	}

	// This is a wrapper around any IOperator that causes its functions to be timed for Insights
	class FProfilingOperator : public IOperator
	{
	public:
		FProfilingOperator(
			TUniquePtr<IOperator>&& WrappedOperator, const INode* Node)
				: Operator(MoveTemp(WrappedOperator))
				, ResetFunction(Operator->GetResetFunction())
				, ExecuteFunction(Operator->GetExecuteFunction())
				, PostExecuteFunction(Operator->GetPostExecuteFunction())
				, InsightsResetEventSpecId(0)
				, InsightsExecuteEventSpecId(0)
				, InsightsPostExecuteEventSpecId(0)
		{
			check(Operator);
			const FNodeClassMetadata& NodeMetadata = Node->GetMetadata();
			FName BaseEventName = NodeMetadata.ClassName.GetName();
			if (BaseEventName.IsNone())
			{
				BaseEventName = Node->GetInstanceName();
			}
			
			BaseEventName.AppendString(InsightsResetEventName);
			InsightsExecuteEventName = InsightsResetEventName;
			InsightsPostExecuteEventName = InsightsResetEventName;

			InsightsResetEventName.Append(TEXT("_RESET"));
			InsightsExecuteEventName.Append(TEXT("_EXECUTE"));
			InsightsPostExecuteEventName.Append(TEXT("_POSTEXECUTE"));
		}

#if UE_METASOUND_DEBUG_ENABLED
		void SetAssetMetaData(const ThreadLocalDebug::FAssetMetaData& InAssetMetaData);
#endif // #if UE_METASOUND_DEBUG_ENABLED

		virtual ~FProfilingOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
		{
			Operator->BindInputs(InVertexData);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData)
		{
			Operator->BindOutputs(InVertexData);
		}

		virtual FResetFunction GetResetFunction() override
		{
			if (!ResetFunction)
			{
				return nullptr;
			}
			return &StaticReset;
		}
		virtual FExecuteFunction GetExecuteFunction() override
		{
			if (!ExecuteFunction)
			{
				return nullptr;
			}
			return &StaticExecute;
		}
		virtual FPostExecuteFunction GetPostExecuteFunction() override
		{
			if (!PostExecuteFunction)
			{
				return nullptr;
			}
			return &StaticPostExecute;
		}

		static void StaticReset(IOperator* InOperator, const IOperator::FResetParams& InParams);
		static void StaticExecute(IOperator* InOperator);
		static void StaticPostExecute(IOperator* InOperator);

	private:
#if UE_METASOUND_DEBUG_ENABLED
		ThreadLocalDebug::FAssetMetaData AssetMetaData;
#endif // #if UE_METASOUND_DEBUG_ENABLED
		
		TUniquePtr<IOperator>			 Operator;
		FResetFunction        			 ResetFunction;
		FExecuteFunction      			 ExecuteFunction;
		FPostExecuteFunction  			 PostExecuteFunction;
		FString               			 InsightsResetEventName;
		FString               			 InsightsExecuteEventName;
		FString               			 InsightsPostExecuteEventName;
		uint32                			 InsightsResetEventSpecId;
		uint32                			 InsightsExecuteEventSpecId;
		uint32                			 InsightsPostExecuteEventSpecId;
	};
}