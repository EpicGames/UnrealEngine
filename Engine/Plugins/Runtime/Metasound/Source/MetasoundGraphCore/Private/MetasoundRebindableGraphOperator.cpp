// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundRebindableGraphOperator.h"

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "MetasoundDynamicGraphAlgo.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexData.h"
#include "Misc/Guid.h"
#include "Misc/ReverseIterate.h"
#include "Templates/UniquePtr.h"

namespace Metasound
{
	FRebindableGraphOperator::FRebindableGraphOperator(const FOperatorSettings& InOperatorSettings)
	: GraphOperatorData(InOperatorSettings)
	{
	}

	// Bind the graph's interface data references to FVertexInterfaceData.
	void FRebindableGraphOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace DynamicGraph;

		RebindGraphInputs(InOutVertexData, GraphOperatorData);
	}

	void FRebindableGraphOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace DynamicGraph;

		RebindGraphOutputs(InOutVertexData, GraphOperatorData);
	}

	void FRebindableGraphOperator::Execute()
	{
		using namespace DynamicGraph;

		for (FExecuteEntry& Entry : GraphOperatorData.ExecuteTable)
		{
			Entry.Execute();
		}
	}

	void FRebindableGraphOperator::PostExecute()
	{
		using namespace DynamicGraph;

		// Reverse iterate over post execute to keep inputs to operators unchanged
		// between calls to Execute() and PostExecute()
		for (FPostExecuteEntry& Entry : ReverseIterate(GraphOperatorData.PostExecuteTable))
		{
			Entry.PostExecute();
		}
	}

	void FRebindableGraphOperator::Reset(const IOperator::FResetParams& InParams)
	{
		using namespace DynamicGraph;

		for (FResetEntry& Entry : GraphOperatorData.ResetTable)
		{
			Entry.Reset(InParams);
		}
	}

	IOperator::FPostExecuteFunction FRebindableGraphOperator::GetPostExecuteFunction() 
	{
		return &StaticPostExecute;
	}

	void FRebindableGraphOperator::StaticPostExecute(IOperator* InOperator)
	{
		check(InOperator);
		static_cast<FRebindableGraphOperator*>(InOperator)->PostExecute();
	}

	DynamicGraph::FDynamicGraphOperatorData& FRebindableGraphOperator::GetDynamicGraphOperatorData()
	{
		return GraphOperatorData;
	}
}


