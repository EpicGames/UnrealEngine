// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundExecutableOperator.h"
#include "MetasoundDynamicGraphAlgo.h"
#include "MetasoundOperatorInterface.h"

namespace Metasound
{
	class FDataReferenceCollection;
	class FInputVertexInterfaceData;
	class FOutputVertexInterfaceData;


	class FRebindableGraphOperator : public TExecutableOperator<FRebindableGraphOperator>, public DynamicGraph::IDynamicGraphInPlaceBuildable
	{
	public:

		FRebindableGraphOperator(const FOperatorSettings& InOperatorSettigns);

		// Bind the graph's interface data references to FVertexInterfaceData.
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Execute();
		void PostExecute();
		void Reset(const IOperator::FResetParams& InParams);

		virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override;

	private:
		static void StaticPostExecute(IOperator* InOperator);

		// Interface for FOperatorBuilder to access internal data structure.
		virtual DynamicGraph::FDynamicGraphOperatorData& GetDynamicGraphOperatorData() override;

		DynamicGraph::FDynamicGraphOperatorData GraphOperatorData;
	};
}

