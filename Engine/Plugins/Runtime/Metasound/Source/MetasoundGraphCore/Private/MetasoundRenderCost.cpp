// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundRenderCost.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundEnvironment.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

namespace Metasound
{
	FGraphRenderCost::FGraphRenderCost(EPrivateToken InToken)
	{
	}

	TSharedRef<FGraphRenderCost> FGraphRenderCost::MakeGraphRenderCost()
	{
		return MakeShared<FGraphRenderCost>(PrivateToken);
	}

	FNodeRenderCost FGraphRenderCost::AddNode(const FGuid& InNodeInstanceID, const FMetasoundEnvironment& InEnv)
	{
		int32 NodeIndex = NodeCosts.Num();
		NodeCosts.Add(0.f);

		AddNodeHierarchy(InNodeInstanceID, InEnv);

		return FNodeRenderCost(NodeIndex, AsShared());
	}

	void FGraphRenderCost::ResetNodeRenderCosts()
	{
		if (NodeCosts.Num() > 0)
		{
			FMemory::Memzero(NodeCosts.GetData(), sizeof(float) * NodeCosts.Num());
		}
	}

	void FGraphRenderCost::SetNodeRenderCost(int32 InNodeIndex, float InRenderCost)
	{
		NodeCosts[InNodeIndex] = InRenderCost;
	}

	float FGraphRenderCost::ComputeGraphRenderCost() const
	{
		float Cost = 0.f;
		Audio::ArraySum(NodeCosts, Cost);
		return Cost;
	};

#if UE_METASOUNDRENDERCOST_TRACK_NODE_HIERARCHY
	void FGraphRenderCost::AddNodeHierarchy(const FGuid& InNodeInstanceID, const FMetasoundEnvironment& InEnv)
	{
		// Node hierarchies provide a convenient path for finding costly nodes by looking at their graph hierarchy. 
		TArray<FGuid> NodeHierarchy;
		if (InEnv.Contains<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy))
		{
			NodeHierarchy = InEnv.GetValue<TArray<FGuid>>(CoreInterface::Environment::GraphHierarchy);
		}
		NodeHierarchy.Add(InNodeInstanceID);
		NodeHierarchies.Add(MoveTemp(NodeHierarchy));
	}
#else
	void FGraphRenderCost::AddNodeHierarchy(const FGuid& InNodeInstanceID, const FMetasoundEnvironment& InEnv)
	{
	}
#endif // UE_METASOUNDRENDERCOST_TRACK_NODE_HIERARCHY

	FNodeRenderCost::FNodeRenderCost(int32 InNodeIndex, TSharedRef<FGraphRenderCost> InGraphRenderCost)
	: NodeIndex(InNodeIndex)
	, GraphRenderCost(MoveTemp(InGraphRenderCost))
	{
	}

	void FNodeRenderCost::SetRenderCost(float InCost)
	{
		if (GraphRenderCost.IsValid())
		{
			// Render cost values are stored on an array in the FGraphRenderCost. 
			GraphRenderCost->SetNodeRenderCost(NodeIndex, InCost);
		}
	}
}

