// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundThreadLocalDebug.h"

#if UE_METASOUND_DEBUG_ENABLED

#include "MetasoundNodeInterface.h"
#include "Containers/UnrealString.h"

namespace Metasound
{
	namespace ThreadLocalDebug
	{
		class FDebugInfo
		{
		public:
			// Set the active node for the debug info.
			void SetActiveNode(const INode* InNode);

			// set the current asset scope
			void SetCurrentAsset(FName Class, FName Name, FName Path);
			const FAssetMetaData& GetCurrentAsset() const;

			// Return the active node.
			const INode* GetActiveNode() const;

			// Returns the class name and version string for the active node in the current thread.
			const TCHAR* GetActiveNodeClassNameAndVersion() const;

			FAssetMetaData GetFullNodeAssetMetaData() const;

		private:

			const INode* ActiveNode = nullptr;
			FString NodeClassNameAndVersion;

			FAssetMetaData AssetMetaData;
		};

		namespace ThreadLocalDebugPrivate
		{
			static thread_local FDebugInfo DebugInfoOnThisThread;
		}

		void FDebugInfo::SetActiveNode(const INode* InNode)
		{
			ActiveNode = InNode;
			if (ActiveNode)
			{
				const FNodeClassMetadata& Metadata = ActiveNode->GetMetadata();
				NodeClassNameAndVersion = FString::Format(TEXT("{0} v{1}.{2}"), {Metadata.ClassName.ToString(), Metadata.MajorVersion, Metadata.MinorVersion});
			}
			else
			{
				NodeClassNameAndVersion = TEXT("[No Active Debug Node Set]");
			}
		}

		void FDebugInfo::SetCurrentAsset(FName Class, FName Name, FName Path)
		{
			AssetMetaData.ClassName = Class;
			AssetMetaData.AssetName = Name;
			AssetMetaData.AssetPath = Path;
		}

		const FAssetMetaData& FDebugInfo::GetCurrentAsset() const
		{
			return AssetMetaData;
		}

		const INode* FDebugInfo::GetActiveNode() const
		{
			return ActiveNode;
		}

		const TCHAR* FDebugInfo::GetActiveNodeClassNameAndVersion() const
		{
			if (ActiveNode)
			{
				return *NodeClassNameAndVersion;
			}
			return TEXT("");
		}

		FAssetMetaData FDebugInfo::GetFullNodeAssetMetaData() const
		{
			if (nullptr == ActiveNode) 
			{
				return {
					.ClassName = AssetMetaData.ClassName,
					.AssetName = AssetMetaData.AssetName,
					.AssetPath = AssetMetaData.AssetName
				};
			}
				
			return {
				.ClassName = AssetMetaData.ClassName,
				.AssetName = AssetMetaData.AssetName,
				.AssetPath = FName(AssetMetaData.AssetName.ToString() + '/' + ActiveNode->GetMetadata().ClassName.ToString())
			};
		}

		const TCHAR* GetActiveNodeClassNameAndVersionOnThisThread()
		{
			return ThreadLocalDebugPrivate::DebugInfoOnThisThread.GetActiveNodeClassNameAndVersion();
		}

		void SetAssetScopeForThisThread(FName Class, FName Name, FName Path)
		{
			ThreadLocalDebugPrivate::DebugInfoOnThisThread.SetCurrentAsset(Class, Name, Path);
		}

		FDebugInfo* GetDebugInfoOnThisThread()
		{
			return &ThreadLocalDebugPrivate::DebugInfoOnThisThread;
		}

		FScopeDebugActiveNode::FScopeDebugActiveNode(FDebugInfo* InDebugInfo, const INode* InNode)
		: DebugInfo(InDebugInfo)
		{
			if (DebugInfo)
			{
				PriorNode = DebugInfo->GetActiveNode();
				DebugInfo->SetActiveNode(InNode);
				UpdateAssetMetaData();
			}
		}

		FScopeDebugActiveNode::~FScopeDebugActiveNode()
		{
			if (DebugInfo)
			{
				DebugInfo->SetActiveNode(PriorNode);
			}
		}

		void FScopeDebugActiveNode::UpdateAssetMetaData()
		{
			if (DebugInfo)
			{
				AssetMetaData = DebugInfo->GetFullNodeAssetMetaData();
			}
		}
	}
}

#endif
