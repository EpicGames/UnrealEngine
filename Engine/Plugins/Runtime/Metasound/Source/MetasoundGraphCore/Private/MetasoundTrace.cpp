// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundTrace.h"

#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"

#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_LLM_MEMORY_STAT(TEXT("MetaSound"), STAT_MetaSoundLLM, STATGROUP_LLMFULL);

LLM_DEFINE_TAG(Audio_MetaSound, TEXT("MetaSound"), TEXT("Audio"), GET_STATFNAME(STAT_MetaSoundLLM), GET_STATFNAME(STAT_AudioSummaryLLM));

// Defines the "Audio_Metasound" category in the CSV profiler.
// This should only be defined here. Modules who wish to use this category should contain the line
// 		CSV_DECLARE_CATEGORY_MODULE_EXTERN(METASOUNDGRAPHCORE_API, Audio_Metasound);
//
CSV_DEFINE_CATEGORY_MODULE(METASOUNDGRAPHCORE_API, Audio_Metasound, false);
