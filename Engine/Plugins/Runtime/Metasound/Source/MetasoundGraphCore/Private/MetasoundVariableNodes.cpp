// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundVariableNodes.h"

#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundGraphCore"

namespace Metasound
{
	namespace VariableNames
	{
		DEFINE_METASOUND_PARAM(InputData, "Value", "Value")
		DEFINE_METASOUND_PARAM(OutputData, "Value", "Value")
		DEFINE_METASOUND_PARAM(InputVariable, "Variable", "Variable")
		DEFINE_METASOUND_PARAM(OutputVariable, "Variable", "Variable")

		const FDataVertexMetadata InputDataMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(InputData) // display name
		};

		const FDataVertexMetadata OutputDataMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(OutputData) // display name
		};

		const FDataVertexMetadata InputVariableMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(InputVariable) // display name
		};

		const FDataVertexMetadata OutputVariableMetadata
		{
			  FText::GetEmpty() // description
			, METASOUND_GET_PARAM_DISPLAYNAME(OutputVariable) // display name
		};

		FNodeClassName GetVariableNodeClassName(const FName& InDataTypeName)
		{
			static const FName NodeNamespace = "InitVariable";
			return FNodeClassName{NodeNamespace, InDataTypeName, ""};
		}

		FNodeClassName GetVariableMutatorNodeClassName(const FName& InDataTypeName)
		{
			static const FName NodeNamespace = "VariableMutator";
			return FNodeClassName{NodeNamespace, InDataTypeName, ""};
		}

		FNodeClassName GetVariableDeferredAccessorNodeClassName(const FName& InDataTypeName)
		{
			static const FName NodeNamespace = "VariableDeferredAccessor";
			return FNodeClassName{NodeNamespace, InDataTypeName, ""};
		}

		FNodeClassName GetVariableAccessorNodeClassName(const FName& InDataTypeName)
		{
			static const FName NodeNamespace = "VariableAccessor";
			return FNodeClassName{NodeNamespace, InDataTypeName, ""};
		}
	}

	
	namespace MetasoundVariableNodesPrivate
	{
		bool IsSupportedVertexData(const IDataReference& InCurrentVariableRef, const FInputVertexInterfaceData& InNew)
		{
			using namespace VariableNames;
#if DO_CHECK
			// Variable nodes are internal to the graph and their underlying TVariable<> Object should 
			// never change. Changing the underlying variable would break the state of the TVariable<>
			// object which is initialized during CreateOperator(...) calls of various TVariable*Operators.
			FDataReferenceID CurrentID = GetDataReferenceID(InCurrentVariableRef);
			FDataReferenceID NewID = nullptr;

			if (const FAnyDataReference* InNewVar = InNew.FindDataReference(METASOUND_GET_PARAM_NAME(InputVariable)))
			{
				NewID = GetDataReferenceID(*InNewVar);
			}

			// The new data reference must be missing or equal to the current data reference.
			return  (NewID == nullptr) || (NewID == CurrentID);
#else
			return true;
#endif // #if DO_CHECK

		}
	}
}

#undef LOCTEXT_NAMESPACE // MetasoundGraphCore
