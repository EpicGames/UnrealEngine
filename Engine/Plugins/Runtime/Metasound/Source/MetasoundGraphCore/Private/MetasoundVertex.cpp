// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundVertex.h"

#include "MetasoundLog.h"
#include "MetasoundVertexPrivate.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Compare.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Containers/Array.h"

namespace Metasound
{
	namespace VertexPrivate
	{
		template<typename ArgType>
		uint16 CastToSubInterfaceInstancePositionChecked(ArgType InArg)
		{
			static_assert(std::is_integral_v<ArgType>, "ArgType must be integral type");
			checkf((InArg <= SubInterface::MaxInstancePosition) && (InArg >= SubInterface::MinInstancePosition), TEXT("SubInterface instance position (%d) out of range [%d, %d]"), (int32)InArg, SubInterface::MinInstancePosition, SubInterface::MaxInstancePosition);
			return static_cast<uint16>(InArg);
		}

		// Functor used for finding FSubInterfaceLayouts by name. 
		struct FEqualSubInterfaceName
		{
			const FName& NameRef;

			FEqualSubInterfaceName(const FName& InName)
			: NameRef(InName)
			{
			}

			bool operator()(const FSubInterfaceLayout& InOther) const
			{
				return NameRef == InOther.SubInterfaceName;
			}
		};

		FInputVertexInterfaceDeclarationBuilder::FInputVertexInterfaceDeclarationBuilder(TArray<FInputDataVertex>& OutVertices)
		: Vertices(OutVertices)
		{
		}


		void FInputVertexInterfaceDeclarationBuilder::Add(FInputDataVertex&& InVertex)
		{
			using FEqualVertexName = VertexPrivate::TEqualVertexName<FInputDataVertex>;

			checkf(!Vertices.ContainsByPredicate(FEqualVertexName(InVertex.VertexName)), TEXT("Duplicate vertex name %s. Vertex names must be unique"), *InVertex.VertexName.ToString());

			Vertices.Emplace(MoveTemp(InVertex));
		}

		FOutputVertexInterfaceDeclarationBuilder::FOutputVertexInterfaceDeclarationBuilder(TArray<FOutputDataVertex>& OutVertices)
		: Vertices(OutVertices)
		{
		}

		void FOutputVertexInterfaceDeclarationBuilder::Add(FOutputDataVertex&& InVertex)
		{
			using FEqualVertexName = VertexPrivate::TEqualVertexName<FOutputDataVertex>;

			checkf(!Vertices.ContainsByPredicate(FEqualVertexName(InVertex.VertexName)), TEXT("Duplicate vertex name %s. Vertex names must be unique"), *InVertex.VertexName.ToString());

			Vertices.Emplace(MoveTemp(InVertex));
		}

		FEnvironmentDeclarationBuilder::FEnvironmentDeclarationBuilder(TArray<FEnvironmentVertex>& OutVertices)
		: Vertices(OutVertices)
		{
		}

		/** Builds an interface from a sub-interface configuration */
		template<typename VertexType>
		class TInterfaceConfigurationBuilder
		{
		public:

			TInterfaceConfigurationBuilder(TArray<VertexType>& OutVertices, TArray<FSubInterfaceLayout>& OutSubInterfaceLayouts)
			: Vertices(OutVertices)
			, SubInterfaces(OutSubInterfaceLayouts)
			{
			}

			virtual ~TInterfaceConfigurationBuilder() = default;

			struct FBuildParams
			{
				EVertexDirection VertexDirection;
				TConstArrayView<VertexType> ClassVertices;

				const FVariantSchema* VariantSchema = nullptr;
				const TMap<FName, FName>* VariantSelectionsMap = nullptr;

				const FSubInterfaceSchema* SubInterfaceSchema = nullptr;
				const TMap<FName, uint32>* SubInterfaceCountsMap = nullptr;
			};

			/** In order to create the vertex interface, the subinterface and variant
			 * configurations need to be incorporated. 
			 *
			 * Variant configurations are applied first since instances of the same 
			 * subinterface are expected to share variant configurations. If there 
			 * is an subinterface containing a variant, then all the instances of 
			 * variant have to be the same type.
			 *
			 * For example
			 * FClassInterface
			 * {
			 * 		FSubInterface
			 * 		{
			 * 			"SubInterface1",
			 * 			TVariantInputVertex<bool, int32>("In1", ...)
			 * 		}
			 * 	}
			 * 	If there are 5 instances of "SubInterface1", all 5 must have the same
			 * 	variant configuration for "In1". All 5 vertices must be bool or int32. 
			 */
			void Build(const FBuildParams& InBuildParams)
			{
				// Initialize with a copy of the class vertexes
				Vertices = InBuildParams.ClassVertices;

				// If there are subinterfaces, initialize the data structures for
				// tracking the locations of subinterfaces
				if (InBuildParams.SubInterfaceSchema)
				{
					InitializeSubInterfaceLayouts(InBuildParams.VertexDirection, *InBuildParams.SubInterfaceSchema);
				}

				// If there are variants, initialize variants before expanding
				// subinterfaces instances.
				if (InBuildParams.VariantSchema && InBuildParams.VariantSelectionsMap)
				{
					ApplyVariantConfigurations(InBuildParams.VertexDirection, *InBuildParams.VariantSchema, *InBuildParams.VariantSelectionsMap);
				}

				// Build sub interfaces
				if (InBuildParams.SubInterfaceSchema)
				{
					
					BuildSubInterfaces(*InBuildParams.SubInterfaceSchema, InBuildParams.SubInterfaceCountsMap);
				}
			}

		private:

			void InitializeSubInterfaceLayouts(EVertexDirection InVertexDirection, const FSubInterfaceSchema& InSubInterfaceSchema)
			{
				TConstArrayView<FSubInterfaceClassLayout> ClassLayouts;
				switch (InVertexDirection)
				{
					case EVertexDirection::Input:
						ClassLayouts = InSubInterfaceSchema.InputSubInterfaceLayouts;
						break;
					case EVertexDirection::Output:
						ClassLayouts = InSubInterfaceSchema.OutputSubInterfaceLayouts;
						break;
					default:
						{
							// Unhandled vertex type
							checkNoEntry();
						}

				}

				/** Initialize subinterface layouts to 1 instance of each subinterface 
				 * since we always have 1 instance of each subinterface in the 
				 * class definition. */
				for (const FSubInterfaceClassLayout& ClassLayout : ClassLayouts)
				{
					SubInterfaces.Add(FSubInterfaceLayout{ClassLayout.SubInterfaceName, {FSubInterfaceLayout::FInstance{ClassLayout.Begin, ClassLayout.End}}});
				}
			}

			void ApplyVariantConfigurations(EVertexDirection InVertexDirection, const FVariantSchema& InVariantSchema, const TMap<FName, FName>& InVariantSelections)
			{
				for (int32 VariantIndex = 0; VariantIndex < InVariantSchema.Variants.Num(); VariantIndex++)
				{
					const FVariantDescription& VariantDescription = InVariantSchema.Variants[VariantIndex];

					const FName* SelectedDataType = InVariantSelections.Find(VariantDescription.VariantName);
					if (!SelectedDataType)
					{
						// No selection provided for this variant — leave unresolved.
						continue;
					}

					// Make sure the configured data type is actually part of the
					// supported set of datatypes
					const int32 DataTypeIndex = VariantDescription.DataTypes.IndexOfByKey(*SelectedDataType);
					if (INDEX_NONE == DataTypeIndex)
					{
						UE_LOGF(LogMetaSound, Error, "Cannot configure variant %ls with data type %ls because it is not a supported data type of the variant", *VariantDescription.VariantName.ToString(), *SelectedDataType->ToString());
						continue;
					}

					// Iterate through all variant vertices and apply configuration
					// to the ones that are part of this particular variant.
					for (const auto& VertexKeyIndexPair : InVariantSchema.VertexKeyToVariantIndex)
					{
						if (VertexKeyIndexPair.Value == VariantIndex)
						{
							if (VertexKeyIndexPair.Key.Direction == InVertexDirection)
							{
								auto HasSameVertexName = [&VertexKeyIndexPair](const VertexType& InVertex)
								{
									return InVertex.VertexName == VertexKeyIndexPair.Key.Name;
								};
								VertexType* Vertex = Vertices.FindByPredicate(HasSameVertexName);

								if (nullptr == Vertex)
								{
									// Vertices in the VariantSchema should always exist by now.
									UE_LOGF(LogMetaSound, Error, "Failed to find vertex %ls when configuring variant %ls", *VertexKeyIndexPair.Key.Name.ToString(), *VariantDescription.VariantName.ToString());
									continue;
								}
								else if (Vertex->DataTypeName != GVariantDataTypeName)
								{
									// We shouldn't be configuration a variant vertex twice.
									UE_LOGF(LogMetaSound, Error, "While attempting to configure variant vertex %ls, the vertex is already configured to datatype %ls when configuring variant %ls", *Vertex->VertexName.ToString(), *Vertex->DataTypeName.ToString(), *VariantDescription.VariantName.ToString());
									continue;
								}

								// Set the configured data type.
								Vertex->DataTypeName = *SelectedDataType;

								// Input vertices need to also have their default
								// literals updated along with the data type.
								if constexpr (std::is_base_of_v<FInputDataVertex, VertexType>)
								{
									const TArray<FLiteral, TInlineAllocator<2>>* DefaultLiterals = InVariantSchema.VariantLiterals.Find(Vertex->VertexName);
									if (nullptr != DefaultLiterals)
									{
										if (DefaultLiterals->IsValidIndex(DataTypeIndex))
										{
											Vertex->SetDefaultLiteral((*DefaultLiterals)[DataTypeIndex]);
										}
									}
								}
							}
						}
					}
				}
			}

			void BuildSubInterfaces(const FSubInterfaceSchema& InSubInterfaceSchema, const TMap<FName, uint32>* InSubInterfaceCounts)
			{
				if (SubInterfaces.Num() == 0)
				{
					return;
				}

				for (int32 LayoutIndex = 0; LayoutIndex < SubInterfaces.Num(); LayoutIndex++)
				{
					FSubInterfaceLayout& Layout = SubInterfaces[LayoutIndex];

					// Find the description associated with the layout
					const FSubInterfaceDescription* Description = Algo::FindBy(
							InSubInterfaceSchema.SubInterfaces,
							Layout.SubInterfaceName,
							[](const FSubInterfaceDescription& InDesc) { return InDesc.SubInterfaceName; });

					if (ensureMsgf(Description, TEXT("Missing sub interface description %s"), *Layout.SubInterfaceName.ToString()))
					{
						// Determine number of sub interface instances.
						uint32 Num = Description->NumDefault;

						if (InSubInterfaceCounts)
						{
							if (const uint32* ConfiguredNum = InSubInterfaceCounts->Find(Layout.SubInterfaceName))
							{
								Num = FMath::Clamp(*ConfiguredNum, Description->Min, Description->Max);
							}
						}

						// Construct sub interface instances
						BuildSubInterfaceInstances(Layout, LayoutIndex, Num);
					}
				}
				
#if DO_CHECK
				// Check that there are no duplicate names in the interface
				for (int32 IndexA = 0; IndexA < Vertices.Num(); IndexA++)
				{
					const FVertexName& VertexNameA = Vertices[IndexA].VertexName;
					for (int32 IndexB = IndexA + 1; IndexB < Vertices.Num(); IndexB++)
					{
						checkf(VertexNameA != Vertices[IndexB].VertexName, TEXT("Found duplicate names (%s) in interface"), *VertexNameA.ToString());
					}
				}
#endif
			}


			void BuildSubInterfaceInstances(FSubInterfaceLayout& InLayout, uint32 InLayoutIndex, uint32 InNum)
			{
				// We should always be beginning from a declaration of a sub interface
				// which enforces that there is only one instance of the sub interface. 
				check(InLayout.Instances.Num() == 1);

				if (InNum == 1)
				{
					// We already have 1 instance from the declaration. Nothing to do. 
					return;
				}

				if (InNum == 0)
				{
					RemoveSubInterface(InLayout, InLayoutIndex);
				}
				else
				{
					SetNumSubInterfaces(InLayout, InLayoutIndex, InNum);
				}
			}

			void SetNumSubInterfaces(FSubInterfaceLayout& InLayout, uint32 InLayoutIndex, uint32 InNum)
			{
				// Assume we are starting with a sub interface declaration. 
				check(InLayout.Instances.Num() == 1);
				
				// Get the position of the prototype for the sub interface. 
				const int32 ProtoBegin = InLayout.Instances[0].Begin;
				const int32 ProtoEnd = InLayout.Instances[0].End;
				const int32 ProtoNum = ProtoEnd - ProtoBegin;

				// Shift the location of sub interfaces that occur after this one 
				// to account for the about-to-be added vertices. 
				int32 NumToAdd = (InNum - 1) * ProtoNum;
				for (FSubInterfaceLayout& Layout : SubInterfaces)
				{
					for (FSubInterfaceLayout::FInstance& Instance : Layout.Instances)
					{
						if (Instance.Begin > ProtoBegin)
						{
							Instance.Begin += NumToAdd;
							Instance.End += NumToAdd;
						}
					}
				}

				// Add instances
				TArray<FSubInterfaceLayout::FInstance>& Instances = InLayout.Instances;
				Instances.Reserve(InNum);
				Instances.SetNum(InNum);

				if (InNum > 1)
				{
					check((ProtoBegin + ProtoNum) <= Vertices.Num());
					const TArray<VertexType> Prototype{Vertices.GetData() + ProtoBegin, ProtoNum};

					for (uint32 Cntr = 1; Cntr < InNum; Cntr++)
					{
						// Initialize the position of the sub interface instance
						const uint32 InsertIndex = ProtoBegin + (Cntr * ProtoNum);
						Instances[Cntr].Begin = InsertIndex;
						Instances[Cntr].End = InsertIndex + ProtoNum;

						// Create the vertices of the sub interface instance. 
						InsertSubInterfaceInstance(Prototype, InsertIndex, Cntr);
					}
				}
			}

			void RemoveSubInterface(FSubInterfaceLayout& InLayout, uint32 InLayoutIndex)
			{
				// We should always be beginning from a declaration of a sub interface
				// which enforces that there is only one instance of the sub interface. 
				check(InLayout.Instances.Num() == 1);

				// Get the position of the prototype for the sub interface. 
				const int32 ProtoBegin = InLayout.Instances[0].Begin;
				const int32 ProtoEnd = InLayout.Instances[0].End;
				const int32 ProtoNum = ProtoEnd - ProtoBegin;

				// Remove all instances from this subinterface
				InLayout.Instances.Empty();

				if (ProtoNum > 0)
				{
					// Remove actual vertices from interface
					RemoveVerticesAt(ProtoBegin, ProtoNum);

					// Shift positions of other instances of sub interfaces
					for (FSubInterfaceLayout& Layout : SubInterfaces)
					{
						for (FSubInterfaceLayout::FInstance& Instance : Layout.Instances)
						{
							if (Instance.Begin > ProtoBegin)
							{
								Instance.Begin -= ProtoNum;
								Instance.End -= ProtoNum;
							}
						}
					}
				}
			}


			void InsertSubInterfaceInstance(const TArray<VertexType>& InPrototype, uint32 InInsertPos, uint32 InSubInterfaceInstanceIndex)
			{
				if (InPrototype.Num() > 0)
				{
					// check that the insert position of the sub interface instance is valid. 
					check(InInsertPos <= static_cast<uint32>(Vertices.Num()));

					// Copy vertices from prototype vertices
					Vertices.Insert(InPrototype.GetData(), InPrototype.Num(), InInsertPos);

					// Rename vertices
					const uint32 NewEnd = InInsertPos + InPrototype.Num();
					for (uint32 VertexIndex = InInsertPos;  VertexIndex < NewEnd; VertexIndex++)
					{
						checkf(
							Vertices[VertexIndex].VertexName.GetNumber() == NAME_NO_NUMBER_INTERNAL, 
							TEXT("Prototype vertex %s in sub interface cannot have a trailing number because it is in a sub interface."), 
							*Vertices[VertexIndex].VertexName.ToString()
						);

						Vertices[VertexIndex].VertexName.SetNumber(1 + InSubInterfaceInstanceIndex);
					}
				}
			}

			void RemoveVerticesAt(uint32 InVertexIndexBegin, uint32 InNum) 
			{
				check((InVertexIndexBegin + InNum) <= static_cast<unsigned>(Vertices.Num()));
				Vertices.RemoveAt(InVertexIndexBegin, InNum);
			}


			TArray<VertexType>& Vertices;
			TArray<FSubInterfaceLayout>& SubInterfaces;
		};


		class FInputInterfaceConfigurationBuilder : public TInterfaceConfigurationBuilder<FInputDataVertex>
		{
		public:
			FInputInterfaceConfigurationBuilder(FInputVertexInterface& InProto)
			: TInterfaceConfigurationBuilder<FInputDataVertex>(InProto.Vertices, InProto.SubInterfaces)
			{
			}
		};

		class FOutputInterfaceConfigurationBuilder : public TInterfaceConfigurationBuilder<FOutputDataVertex>
		{
		public:
			FOutputInterfaceConfigurationBuilder(FOutputVertexInterface& InProto)
			: TInterfaceConfigurationBuilder<FOutputDataVertex>(InProto.Vertices, InProto.SubInterfaces)
			{
			}
		};

		bool ContainsSubInterfaceDescription(const FClassInterfaceData& InClassInterfaceData, const FName& InName)
		{
			return Algo::AnyOf(InClassInterfaceData.SubInterfaces, [&InName](const FSubInterfaceDescription& InDesc) { return InDesc.SubInterfaceName == InName; });
		}

		void AssertSubInterfacesAreWellFormed(const FClassInterfaceData& InClassInterfaceData)
		{
#if DO_CHECK
			for (const FSubInterfaceClassLayout& Layout : InClassInterfaceData.InputSubInterfaceLayouts)
			{
				checkf(ContainsSubInterfaceDescription(InClassInterfaceData, Layout.SubInterfaceName), TEXT("Missing sub interface description for sub interface (%s)"), *Layout.SubInterfaceName.ToString());
				checkf(1 == Algo::CountBy(InClassInterfaceData.InputSubInterfaceLayouts, Layout.SubInterfaceName, [](const FSubInterfaceClassLayout& OtherLayout) { return OtherLayout.SubInterfaceName; }), TEXT("Sub interface names must be unique. Found duplicate (%s)"), *Layout.SubInterfaceName.ToString());
			}

			for (const FSubInterfaceClassLayout& Layout : InClassInterfaceData.OutputSubInterfaceLayouts)
			{
				checkf(ContainsSubInterfaceDescription(InClassInterfaceData, Layout.SubInterfaceName), TEXT("Missing sub interface description for sub interface (%s)"), *Layout.SubInterfaceName.ToString());
				checkf(1 == Algo::CountBy(InClassInterfaceData.OutputSubInterfaceLayouts, Layout.SubInterfaceName, [](const FSubInterfaceClassLayout& OtherLayout) { return OtherLayout.SubInterfaceName; }), TEXT("Sub interface names must be unique. Found duplicate (%s)"), *Layout.SubInterfaceName.ToString());
			}
#endif // if DO_CHECK
		}

		void AssertVariantsAreWellFormed(const FClassInterfaceData& InClassInterfaceData)
		{
#if DO_CHECK
			for (const FVariantDescription& Variant : InClassInterfaceData.Variants)
			{
				checkf(1 == Algo::CountBy(InClassInterfaceData.Variants, Variant.VariantName, [](const FVariantDescription& OtherVariant) { return OtherVariant.VariantName; }), TEXT("Variant names must be unique. Found duplicate (%s)"), *Variant.VariantName.ToString());
			}

			for (const auto& Pair : InClassInterfaceData.VertexKeyToVariantIndex)
			{
				checkf(Pair.Value < InClassInterfaceData.Variants.Num(), TEXT("Missing Variant for VariantVertex (%s)"), *Pair.Key.Name.ToString());
			}

			for (const FInputDataVertex& Input : InClassInterfaceData.Inputs)
			{
				if (Input.DataTypeName == GVariantDataTypeName)
				{
					checkf(InClassInterfaceData.VariantLiterals.Contains(Input.VertexName), TEXT("Missing input defaults for input variant vertex (%s)"), *Input.VertexName.ToString());
					checkf(InClassInterfaceData.VertexKeyToVariantIndex.Contains(FVertexKey{Input.VertexName, EVertexDirection::Input}), TEXT("Missing variant for input variant vertex (%s)"), *Input.VertexName.ToString());
				}
			}

			for (const FOutputDataVertex& Output : InClassInterfaceData.Outputs)
			{
				if (Output.DataTypeName == GVariantDataTypeName)
				{
					checkf(InClassInterfaceData.VertexKeyToVariantIndex.Contains(FVertexKey{Output.VertexName, EVertexDirection::Output}), TEXT("Missing variant for output variant vertex (%s)"), *Output.VertexName.ToString());
				}
			}
#endif // if DO_CHECK
		}

		const VertexPrivate::FClassInterfaceDataBuilder& FBuildableInterface::GetClassInterfaceDataBuilder() const
		{
			return Builder;
		}
	} // namespace VertexPrivate

	FDataVertexMetadata FDataVertexMetadata::EmptyBasic { FText{}, FText{}, false };
	FDataVertexMetadata FDataVertexMetadata::EmptyAdvanced { FText{}, FText{}, true };

	bool operator==(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS)
	{
		return (InLHS.VertexName == InRHS.VertexName) && (InLHS.DataTypeName == InRHS.DataTypeName);
	}

	bool operator!=(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS)
	{
		return !(InLHS == InRHS);
	}

	bool operator<(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS)
	{
		if (InLHS.VertexName == InRHS.VertexName)
		{
			return InLHS.DataTypeName.FastLess(InRHS.DataTypeName);
		}
		else
		{
			return InLHS.VertexName.FastLess(InRHS.VertexName);
		}
	}

	bool operator==(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS)
	{
		return (InLHS.VertexName == InRHS.VertexName) && (InLHS.DataTypeName == InRHS.DataTypeName);
	}

	bool operator!=(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS)
	{
		return !(InLHS == InRHS);
	}

	bool operator<(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS)
	{
		if (InLHS.VertexName == InRHS.VertexName)
		{
			return InLHS.DataTypeName.FastLess(InRHS.DataTypeName);
		}
		else
		{
			return InLHS.VertexName.FastLess(InRHS.VertexName);
		}
	}

	bool operator==(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS)
	{
		return (InLHS.VertexName == InRHS.VertexName);
	}

	bool operator!=(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS)
	{
		return !(InLHS == InRHS);
	}

	bool operator<(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS)
	{
		return InLHS.VertexName.FastLess(InRHS.VertexName);
	}

	const FName FSubInterface::GetName() const
	{
		return SubInterfaceName;
	}

	const FLazyName GVariantDataTypeName{FName()}; // NAME_None

	FVariantVertex::FVariantVertex(const FVertexName& InVertexName, const FVertexName& InVariantName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType)
	: VertexName(InVertexName)
	, VariantName(InVariantName)
#if WITH_EDITORONLY_DATA
	, Metadata(InMetadata)
#endif // WITH_EDITORONLY_DATA
	, AccessType(InAccessType)
	{
	}

	FVariantVertex::FVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const TArray<FName>& InDataTypeNames, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType)
	: VertexName(InVertexName)
	, VariantName(InVariantName)
	, DataTypeNames(InDataTypeNames)
#if WITH_EDITORONLY_DATA
	, Metadata(InMetadata)
#endif // WITH_EDITORONLY_DATA
	, AccessType(InAccessType)
	{
	}

	FInputVariantVertex::FInputVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType)
	: FVariantVertex(InVertexName, InVariantName, InMetadata, InAccessType)
	{
	}

	FInputVariantVertex::FInputVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const TArray<FName>& InDataTypeNames, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType, const TArray<FLiteral>& InLiterals)
	: FVariantVertex(InVertexName, InVariantName, InDataTypeNames, InMetadata, InAccessType)
	, Literals(InLiterals)
	{
		checkf(Literals.Num() == DataTypeNames.Num(), TEXT("Mismatch in number of default literals (%d) vs data types (%d). These must be equal"), Literals.Num(), DataTypeNames.Num());
	}


	namespace VertexPrivate
	{
		FVariantSchema FVariantSchema::FromClassInterfaceData(const VertexPrivate::FClassInterfaceData& InClassInterfaceData)
		{
			FVariantSchema Schema;
			Schema.VertexKeyToVariantIndex = InClassInterfaceData.VertexKeyToVariantIndex;
			Schema.Variants = InClassInterfaceData.Variants;
			// This custom copy is here because of the different allocators used in the TSortedMap<FVertexName, TArray<FLiteral, ALLOCATOR>>. 
			Schema.VariantLiterals.Empty(InClassInterfaceData.VariantLiterals.Num());
			for (const auto& Pair : InClassInterfaceData.VariantLiterals)
			{
				Schema.VariantLiterals.Add(Pair.Key, TArray<FLiteral, TInlineAllocator<2>>(Pair.Value));
			}
			return Schema;
		}

		FSubInterfaceSchema FSubInterfaceSchema::FromClassInterfaceData(const VertexPrivate::FClassInterfaceData& InClassInterfaceData)
		{
			FSubInterfaceSchema Schema;
			Schema.SubInterfaces = InClassInterfaceData.SubInterfaces;
			Schema.InputSubInterfaceLayouts = InClassInterfaceData.InputSubInterfaceLayouts;
			Schema.OutputSubInterfaceLayouts = InClassInterfaceData.OutputSubInterfaceLayouts;
			return Schema;
		}
	}

	FInputVertexInterface::FInputVertexInterface() = default;

	FInputVertexInterface::FInputVertexInterface(TArray<FInputDataVertex> InVertices, TArray<VertexPrivate::FSubInterfaceLayout> InSubInterfaces)
	: TVertexInterfaceImpl<FInputDataVertex>(MoveTemp(InVertices))
	, SubInterfaces(MoveTemp(InSubInterfaces))
	{
	}

	void FInputVertexInterface::ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TConstArrayView<FInputDataVertex>)> Callable) const
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Instances = FindSubInterfaceLayout(InSubInterfaceName))
		{
			for (const FSubInterfaceLayout::FInstance& Instance : Instances->Instances)
			{
				check(Vertices.IsValidIndex(Instance.Begin) && Vertices.IsValidIndex(Instance.End - 1));
				Callable(MakeConstArrayView<FInputDataVertex>(Vertices.GetData() + Instance.Begin, Instance.End - Instance.Begin));
			}
		}
	}

	void FInputVertexInterface::ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TArrayView<FInputDataVertex>)> Callable)
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Instances = FindSubInterfaceLayout(InSubInterfaceName))
		{
			for (const FSubInterfaceLayout::FInstance& Instance : Instances->Instances)
			{
				check(Vertices.IsValidIndex(Instance.Begin) && Vertices.IsValidIndex(Instance.End - 1));
				Callable(MakeArrayView<FInputDataVertex>(Vertices.GetData() + Instance.Begin, Instance.End - Instance.Begin));
			}
		}
	}

	TConstArrayView<VertexPrivate::FSubInterfaceLayout> FInputVertexInterface::GetSubInterfaces(const VertexPrivate::FPrivateAccessTag& InTag) const
	{
		return SubInterfaces;
	}

	const VertexPrivate::FSubInterfaceLayout* FInputVertexInterface::FindSubInterfaceLayout(const FName& InName) const
	{
		using namespace VertexPrivate;

		return SubInterfaces.FindByPredicate(VertexPrivate::FEqualSubInterfaceName(InName));
	}

	FOutputVertexInterface::FOutputVertexInterface() = default;

	FOutputVertexInterface::FOutputVertexInterface(TArray<FOutputDataVertex> InVertices, TArray<VertexPrivate::FSubInterfaceLayout> InSubInterfaces)
	: TVertexInterfaceImpl<FOutputDataVertex>(MoveTemp(InVertices))
	, SubInterfaces(MoveTemp(InSubInterfaces))
	{
	}

	void FOutputVertexInterface::ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TConstArrayView<FOutputDataVertex>)> Callable) const
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Instances = FindSubInterfaceLayout(InSubInterfaceName))
		{
			for (const FSubInterfaceLayout::FInstance& Instance : Instances->Instances)
			{
				check(Vertices.IsValidIndex(Instance.Begin) && Vertices.IsValidIndex(Instance.End - 1));
				Callable(MakeConstArrayView<FOutputDataVertex>(Vertices.GetData() + Instance.Begin, Instance.End - Instance.Begin));
			}
		}
	}

	void FOutputVertexInterface::ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TArrayView<FOutputDataVertex>)> Callable)
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Instances = FindSubInterfaceLayout(InSubInterfaceName))
		{
			for (const FSubInterfaceLayout::FInstance& Instance : Instances->Instances)
			{
				check(Vertices.IsValidIndex(Instance.Begin) && Vertices.IsValidIndex(Instance.End - 1));
				Callable(MakeArrayView<FOutputDataVertex>(Vertices.GetData() + Instance.Begin, Instance.End - Instance.Begin));
			}
		}
	}

	TConstArrayView<VertexPrivate::FSubInterfaceLayout> FOutputVertexInterface::GetSubInterfaces(const VertexPrivate::FPrivateAccessTag& InTag) const
	{
		return SubInterfaces;
	}

	const VertexPrivate::FSubInterfaceLayout* FOutputVertexInterface::FindSubInterfaceLayout(const FName& InName) const
	{
		return SubInterfaces.FindByPredicate(VertexPrivate::FEqualSubInterfaceName(InName));
	}

	FEnvironmentVertexInterface::FEnvironmentVertexInterface(TArray<FEnvironmentVertex> InVertices)
	: TVertexInterfaceImpl<FEnvironmentVertex>(MoveTemp(InVertices))
	{
	}

	FVertexInterface::FVertexInterface() = default;

	FVertexInterface::FVertexInterface(const FInputVertexInterface& InInputs, const FOutputVertexInterface& InOutputs)
	:	InputInterface(InInputs)
	,	OutputInterface(InOutputs)
	{
	}

	FVertexInterface::FVertexInterface(FInputVertexInterface InInputs, FOutputVertexInterface InOutputs, FEnvironmentVertexInterface InEnvironmentVariables)
	:	InputInterface(MoveTemp(InInputs))
	,	OutputInterface(MoveTemp(InOutputs))
	,	EnvironmentInterface(MoveTemp(InEnvironmentVariables))
	{
	}

	FVertexInterface::~FVertexInterface() = default;

	const FInputVertexInterface& FVertexInterface::GetInputInterface() const
	{
		return InputInterface;
	}

	FInputVertexInterface& FVertexInterface::GetInputInterface()
	{
		return InputInterface;
	}

	const FInputDataVertex& FVertexInterface::GetInputVertex(const FVertexName& InKey) const
	{
		return InputInterface[InKey];
	}

	bool FVertexInterface::ContainsInputVertex(const FVertexName& InKey) const
	{
		return InputInterface.Contains(InKey);
	}

	const FOutputVertexInterface& FVertexInterface::GetOutputInterface() const
	{
		return OutputInterface;
	}

	FOutputVertexInterface& FVertexInterface::GetOutputInterface()
	{
		return OutputInterface;
	}

	const FOutputDataVertex& FVertexInterface::GetOutputVertex(const FVertexName& InName) const
	{
		return OutputInterface[InName];
	}

	bool FVertexInterface::ContainsOutputVertex(const FVertexName& InName) const
	{
		return OutputInterface.Contains(InName);
	}

	const FEnvironmentVertexInterface& FVertexInterface::GetEnvironmentInterface() const
	{
		return EnvironmentInterface;
	}

	FEnvironmentVertexInterface& FVertexInterface::GetEnvironmentInterface()
	{
		return EnvironmentInterface;
	}

	const FEnvironmentVertex& FVertexInterface::GetEnvironmentVertex(const FVertexName& InKey) const
	{
		return EnvironmentInterface[InKey];
	}

	bool FVertexInterface::ContainsEnvironmentVertex(const FVertexName& InKey) const
	{
		return EnvironmentInterface.Contains(InKey);
	}

	bool operator==(const FVertexInterface& InLHS, const FVertexInterface& InRHS)
	{
		const bool bIsEqual = (InLHS.InputInterface == InRHS.InputInterface) && 
			(InLHS.OutputInterface == InRHS.OutputInterface) && 
			(InLHS.EnvironmentInterface == InRHS.EnvironmentInterface);

		return bIsEqual;
	}

	bool operator!=(const FVertexInterface& InLHS, const FVertexInterface& InRHS)
	{
		return !(InLHS == InRHS);
	}



	FClassInterface::FClassInterface(const FVertexInterface& InVertexInterface)
	{
		for (const FInputDataVertex& Vertex : InVertexInterface.GetInputInterface())
		{
			Inputs.Add(Vertex);
		}
		for (const FOutputDataVertex& Vertex : InVertexInterface.GetOutputInterface())
		{
			Outputs.Add(Vertex);
		}
		for (const FEnvironmentVertex& Vertex : InVertexInterface.GetEnvironmentInterface())
		{
			EnvironmentVariables.Add(Vertex);
		}
	}

	FClassInterface::FClassInterface(const FClassInterface& Other)
	: VariantSchema(Other.VariantSchema ? MakeUnique<VertexPrivate::FVariantSchema>(*Other.VariantSchema) : nullptr)
	, SubInterfaceSchema(Other.SubInterfaceSchema ? MakeUnique<VertexPrivate::FSubInterfaceSchema>(*Other.SubInterfaceSchema) : nullptr)
	, Inputs(Other.Inputs)
	, Outputs(Other.Outputs)
	, EnvironmentVariables(Other.EnvironmentVariables)
	{
	}

	FClassInterface& FClassInterface::operator=(const FClassInterface& Other)
	{
		if (this != &Other)
		{
			VariantSchema = Other.VariantSchema ? MakeUnique<VertexPrivate::FVariantSchema>(*Other.VariantSchema) : nullptr;
			SubInterfaceSchema = Other.SubInterfaceSchema ? MakeUnique<VertexPrivate::FSubInterfaceSchema>(*Other.SubInterfaceSchema) : nullptr;
			Inputs = Other.Inputs;
			Outputs = Other.Outputs;
			EnvironmentVariables = Other.EnvironmentVariables;
		}
		return *this;
	}

	FClassInterface::operator FVertexInterface() const
	{
		return CreateVertexInterface();
	}

	FVertexInterface FClassInterface::CreateVertexInterface() const
	{
		using namespace VertexPrivate;

		FVertexInterface Interface{
			FInputVertexInterface{},
			FOutputVertexInterface{},
			FEnvironmentVertexInterface{EnvironmentVariables}};

		{
			FInputInterfaceConfigurationBuilder::FBuildParams Params
			{
				.VertexDirection = EVertexDirection::Input,
				.ClassVertices  = Inputs,
				.VariantSchema = VariantSchema.Get(),
				.SubInterfaceSchema = SubInterfaceSchema.Get(),
			};
			FInputInterfaceConfigurationBuilder(Interface.GetInputInterface()).Build(Params);
		}

		{
			FOutputInterfaceConfigurationBuilder::FBuildParams Params
			{
				.VertexDirection = EVertexDirection::Output,
				.ClassVertices = Outputs,
				.VariantSchema = VariantSchema.Get(),
				.SubInterfaceSchema = SubInterfaceSchema.Get(),
			};
			FOutputInterfaceConfigurationBuilder(Interface.GetOutputInterface()).Build(Params);
		}

		return Interface;
	}

	FClassInterface::FClassInterface(const VertexPrivate::FClassInterfaceData& InClassInterfaceData)
	: Inputs(InClassInterfaceData.Inputs)
	, Outputs(InClassInterfaceData.Outputs)
	, EnvironmentVariables(InClassInterfaceData.EnvironmentVariables)
	{
		using namespace VertexPrivate;

		AssertSubInterfacesAreWellFormed(InClassInterfaceData);
		AssertVariantsAreWellFormed(InClassInterfaceData);

		if (InClassInterfaceData.SubInterfaces.Num())
		{
			SubInterfaceSchema = MakeUnique<FSubInterfaceSchema>(FSubInterfaceSchema::FromClassInterfaceData(InClassInterfaceData));
		}

		if (InClassInterfaceData.Variants.Num())
		{
			VariantSchema = MakeUnique<FVariantSchema>(FVariantSchema::FromClassInterfaceData(InClassInterfaceData));
		}
	}

	FVertexInterface FClassInterface::CreateVertexInterface(const TMap<FName, uint32>& InSubInterfaceCounts, const TMap<FName, FName>& InVariantSelections) const
	{
		using namespace VertexPrivate;

		FVertexInterface Interface{
			FInputVertexInterface{},
			FOutputVertexInterface{},
			FEnvironmentVertexInterface{EnvironmentVariables}};

		{
			// Build the inputs
			FInputInterfaceConfigurationBuilder::FBuildParams Params
			{
				.VertexDirection = EVertexDirection::Input,
				.ClassVertices  = Inputs,
				.VariantSchema = VariantSchema.Get(),
				.VariantSelectionsMap = &InVariantSelections,
				.SubInterfaceSchema = SubInterfaceSchema.Get(),
				.SubInterfaceCountsMap = &InSubInterfaceCounts
			};

			// Build the interface in-place
			FInputInterfaceConfigurationBuilder(Interface.GetInputInterface()).Build(Params);
		}

		{
			// Build the outputs
			FOutputInterfaceConfigurationBuilder::FBuildParams Params
			{
				.VertexDirection = EVertexDirection::Output,
				.ClassVertices = Outputs,
				.VariantSchema = VariantSchema.Get(),
				.VariantSelectionsMap = &InVariantSelections,
				.SubInterfaceSchema = SubInterfaceSchema.Get(),
				.SubInterfaceCountsMap = &InSubInterfaceCounts
			};

			// Build the interface in-place
			FOutputInterfaceConfigurationBuilder(Interface.GetOutputInterface()).Build(Params);
		}

		return Interface;
	}

	TConstArrayView<FSubInterfaceDescription> FClassInterface::GetSubInterfaceDescriptions() const
	{
		if (SubInterfaceSchema)
		{
			return SubInterfaceSchema->SubInterfaces;
		}
		return {};
	}

	TConstArrayView<FVariantDescription> FClassInterface::GetVariantDescriptions() const
	{
		if (VariantSchema)
		{
			return VariantSchema->Variants;
		}
		return {};
	}

	const FVariantDescription* FClassInterface::FindVariantDescriptionForInput(const FVertexName& InVertexName) const
	{
		if (VariantSchema)
		{
			if (const uint8* Idx = VariantSchema->VertexKeyToVariantIndex.Find(VertexPrivate::FVertexKey{InVertexName, VertexPrivate::EVertexDirection::Input}))
			{
				return &VariantSchema->Variants[*Idx];
			}
		}
		return nullptr;
	}

	const FVariantDescription* FClassInterface::FindVariantDescriptionForOutput(const FVertexName& InVertexName) const
	{
		if (VariantSchema)
		{
			if (const uint8* Idx = VariantSchema->VertexKeyToVariantIndex.Find(VertexPrivate::FVertexKey{InVertexName, VertexPrivate::EVertexDirection::Output}))
			{
				return &VariantSchema->Variants[*Idx];
			}
		}
		return nullptr;
	}

	namespace VertexPrivate
	{
		template<typename ElementType, typename ArrayType>
		struct TAssertOnDuplicateName
		{
			TAssertOnDuplicateName(const ElementType& InNewElement, const ArrayType& InArray, const TCHAR* InArrayTypeName)
			{
				auto HasSameName = [&InNewElement](const ArrayType::ElementType& InElement) -> bool 
				{ 
					return InElement.VertexName == InNewElement.VertexName; 
				};
				checkf(Algo::NoneOf<ArrayType>(InArray, HasSameName), TEXT("Cannot have duplicate %s with same name (%s)"), InArrayTypeName, *InNewElement.VertexName.ToString());
			}
		};

		const FClassInterfaceData& FClassInterfaceDataBuilder::GetClassInterfaceData() const
		{
			return ClassInterfaceData;
		}

		void FClassInterfaceDataBuilder::Add(const FInputDataVertex* InInputVertex)
		{
			check(InInputVertex);
			TAssertOnDuplicateName{*InInputVertex, ClassInterfaceData.Inputs, TEXT("inputs")};
			ClassInterfaceData.Inputs.Add(*InInputVertex);
		}

		void FClassInterfaceDataBuilder::Add(const FOutputDataVertex* InOutputVertex)
		{
			check(InOutputVertex);
			TAssertOnDuplicateName{*InOutputVertex, ClassInterfaceData.Outputs, TEXT("outputs")};
			ClassInterfaceData.Outputs.Add(*InOutputVertex);
		}

		void FClassInterfaceDataBuilder::Add(const FEnvironmentVertex* InEnvironmentVertex)
		{
			check(InEnvironmentVertex);
			TAssertOnDuplicateName{*InEnvironmentVertex, ClassInterfaceData.EnvironmentVariables, TEXT("environment variables")};
			ClassInterfaceData.EnvironmentVariables.Add(*InEnvironmentVertex);
		}

		void FClassInterfaceDataBuilder::Add(const FSubInterface* InSubInterface)
		{
			check(InSubInterface);

			const FClassInterfaceDataBuilder& SubInterfaceBuilder = InSubInterface->GetClassInterfaceDataBuilder();

			checkf(!SubInterfaceBuilder.ContainsAnySubInterfaces(), TEXT("Cannot have a sub interface nested in a sub interface (%s)"), *InSubInterface->GetName().ToString());

			// Add all the vertices associated with the sub interface
			AddSubInterfaceVertices(InSubInterface->GetName(), SubInterfaceBuilder.GetClassInterfaceData());

			// Add any variants from the subinterface to this interface
			MergeSubInterfaceVariantData(SubInterfaceBuilder.GetClassInterfaceData());
		}

		void FClassInterfaceDataBuilder::Add(const FSubInterfaceDescription* InSubInterfaceDescription)
		{
			check(InSubInterfaceDescription);
			checkf(!ContainsSubInterfaceDescription(ClassInterfaceData, InSubInterfaceDescription->SubInterfaceName), TEXT("Sub interface descriptions must have unique names. Found duplicate name (%s)"), *InSubInterfaceDescription->SubInterfaceName.ToString());

			ClassInterfaceData.SubInterfaces.Add(*InSubInterfaceDescription);
		}

		void FClassInterfaceDataBuilder::Add(const FInputVariantVertex* InInputVariantVertex)
		{
			check(InInputVariantVertex);
			// Add to Inputs array but give it a special data type to denote that it's
			// a variant input
			TAssertOnDuplicateName{*InInputVariantVertex, ClassInterfaceData.Inputs, TEXT("inputs")};

			ClassInterfaceData.Inputs.Add(FInputDataVertex{
					InInputVariantVertex->VertexName, 
					GVariantDataTypeName, 
#if WITH_EDITORONLY_DATA
					InInputVariantVertex->Metadata, 
#else
					FDataVertexMetadata{},
#endif
					InInputVariantVertex->AccessType
				});
			
			// Add to VertexKeyToVariantIndex and VariantLiterals to store extra data associated with variant.
			ClassInterfaceData.VariantLiterals.Emplace(InInputVariantVertex->VertexName, InInputVariantVertex->Literals);
			uint8 VariantIndex = FindOrAddVariant(*InInputVariantVertex);
			ClassInterfaceData.VertexKeyToVariantIndex.Add(FVertexKey{InInputVariantVertex->VertexName, EVertexDirection::Input}, VariantIndex);
		}

		void FClassInterfaceDataBuilder::Add(const FOutputVariantVertex* InOutputVariantVertex)
		{
			check(InOutputVariantVertex);
			// Add to Outputs array but give it a special data type to denote that it's
			// a variant input
			TAssertOnDuplicateName{*InOutputVariantVertex, ClassInterfaceData.Outputs, TEXT("outputs")};
			ClassInterfaceData.Outputs.Add(FOutputDataVertex{
					InOutputVariantVertex->VertexName, 
					GVariantDataTypeName, 
#if WITH_EDITORONLY_DATA
					InOutputVariantVertex->Metadata, 
#else
					FDataVertexMetadata{},
#endif
					InOutputVariantVertex->AccessType});
			
			// Add to VertexKeyToVariantIndex to store extra data associated with variant.
			uint8 VariantIndex = FindOrAddVariant(*InOutputVariantVertex);
			ClassInterfaceData.VertexKeyToVariantIndex.Add(FVertexKey{InOutputVariantVertex->VertexName, EVertexDirection::Output}, VariantIndex);
		}

		void FClassInterfaceDataBuilder::AddSubInterfaceVertices(const FName& InSubInterfaceName, const FClassInterfaceData& InSubInterfaceData)
		{
			if (InSubInterfaceData.Inputs.Num())
			{
				ClassInterfaceData.InputSubInterfaceLayouts.Add(FSubInterfaceClassLayout
				{ 
					.SubInterfaceName = InSubInterfaceName,
					.Begin = CastToSubInterfaceInstancePositionChecked(ClassInterfaceData.Inputs.Num()),
					.End = CastToSubInterfaceInstancePositionChecked(ClassInterfaceData.Inputs.Num() + InSubInterfaceData.Inputs.Num())
				});

				ClassInterfaceData.Inputs.Append(InSubInterfaceData.Inputs);


				// This should not be possible because FSubInterface objects should only be nested inside of class inputs or outputs and therefore only contain inputs OR outputs
				checkf(InSubInterfaceData.Outputs.Num() == 0, TEXT("FSubInterface declarations must only contain inputs Or outputs. Inputs AND outputs found in sub interface (%s)"), *InSubInterfaceName.ToString());

			}
			else if (InSubInterfaceData.Outputs.Num())
			{
				ClassInterfaceData.OutputSubInterfaceLayouts.Add(FSubInterfaceClassLayout
				{ 
					.SubInterfaceName = InSubInterfaceName, 
					.Begin = CastToSubInterfaceInstancePositionChecked(ClassInterfaceData.Outputs.Num()), 
					.End = CastToSubInterfaceInstancePositionChecked(ClassInterfaceData.Outputs.Num() + InSubInterfaceData.Outputs.Num())
				});

				ClassInterfaceData.Outputs.Append(InSubInterfaceData.Outputs);

				// This should not be possible because FSubInterface objects should only be nested inside of class inputs or outputs and therefore only contain inputs OR outputs
				checkf(InSubInterfaceData.Inputs.Num() == 0, TEXT("FSubInterface declarations must only contain inputs Or outputs. Inputs AND outputs found in sub interface (%s)"), *InSubInterfaceName.ToString());
			}
		}

		void FClassInterfaceDataBuilder::MergeSubInterfaceVariantData(const FClassInterfaceData& InSubInterfaceData)
		{
			// Add in any variant data gathered from the sub interface
			TSortedMap<uint8, uint8> SubInterfaceVariantRemap;
			for (int32 VGIndex = 0; VGIndex < InSubInterfaceData.Variants.Num(); VGIndex++)
			{
				const FVariantDescription& Variant = InSubInterfaceData.Variants[VGIndex];

				// Check to see if variant already exists. 
				int32 NewIndex = IndexOfVariant(Variant.VariantName);
				if (INDEX_NONE == NewIndex)
				{
					NewIndex = ClassInterfaceData.Variants.Num();
					checkf((NewIndex >= TNumericLimits<uint8>::Min()) && (NewIndex <= TNumericLimits<uint8>::Max()), TEXT("There are too many variants (%d). Last variant (%s)"), NewIndex, *Variant.VariantName.ToString());

					ClassInterfaceData.Variants.Add(Variant);

				}
				else
				{
					// Merge a variant 
					FVariantDescription& ExistingVariant = ClassInterfaceData.Variants[NewIndex];
					checkf(ExistingVariant.DataTypes == Variant.DataTypes, TEXT("Cannot have inconsistent data types in variant (%s)"), *ExistingVariant.VariantName.ToString());
				}

				// hold onto a remaping between the two variant indices 
				SubInterfaceVariantRemap.Add(static_cast<uint8>(VGIndex), static_cast<uint8>(NewIndex));
			}

			for (const TPair<FVertexKey, uint8>& KeyToVariant : InSubInterfaceData.VertexKeyToVariantIndex)
			{
				ClassInterfaceData.VertexKeyToVariantIndex.Add(KeyToVariant.Key, SubInterfaceVariantRemap[KeyToVariant.Value]);
			}

			ClassInterfaceData.VariantLiterals.Append(InSubInterfaceData.VariantLiterals);
		}

		uint8 FClassInterfaceDataBuilder::FindOrAddVariant(const FVariantVertex& InVariantVertex)
		{
			int32 Index = IndexOfVariant(InVariantVertex.VariantName);
			if (INDEX_NONE == Index)
			{
				Index = ClassInterfaceData.Variants.Num();
				ClassInterfaceData.Variants.Add(
					FVariantDescription
					{
						.VariantName=InVariantVertex.VariantName,
						.DataTypes=InVariantVertex.DataTypeNames
					}
				);
			}

			FVariantDescription& Variant = ClassInterfaceData.Variants[Index];

			checkf(Algo::Compare(Variant.DataTypes, InVariantVertex.DataTypeNames), TEXT("Cannot have inconsistent data types in variant (%s) on vertex (%s)"), *InVariantVertex.VariantName.ToString(), *InVariantVertex.VertexName.ToString());
			
			checkf((Index >= TNumericLimits<uint8>::Min()) && (Index <= TNumericLimits<uint8>::Max()), TEXT("There are too many variants (%d). Last variant (%s)"), Index, *InVariantVertex.VariantName.ToString());

			return  static_cast<uint8>(Index);
		}

		bool FClassInterfaceDataBuilder::ContainsAnySubInterfaces() const
		{
			return !(ClassInterfaceData.SubInterfaces.IsEmpty() &&
					ClassInterfaceData.InputSubInterfaceLayouts.IsEmpty() &&
					ClassInterfaceData.OutputSubInterfaceLayouts.IsEmpty());
		}

		int32 FClassInterfaceDataBuilder::IndexOfVariant(const FName& InName) const
		{
			if (InName.IsNone())
			{
				return INDEX_NONE;
			}
			return ClassInterfaceData.Variants.IndexOfByPredicate([&InName](const FVariantDescription& InVariant) { return InVariant.VariantName == InName; });
		}
	}
}

FString LexToString(Metasound::EVertexAccessType InAccessType)
{
	using namespace Metasound;

	switch (InAccessType)
	{
		case EVertexAccessType::Value:
			return TEXT("Value");

		case EVertexAccessType::Reference:
		default:
			return TEXT("Reference");
	}
}
