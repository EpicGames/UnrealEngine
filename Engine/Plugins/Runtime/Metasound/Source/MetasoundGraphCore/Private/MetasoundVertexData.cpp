// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundVertexData.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "MetasoundDataReference.h"
#include "MetasoundLog.h"
#include "MetasoundThreadLocalDebug.h"
#include "MetasoundVertex.h"
#include "MetasoundVertexPrivate.h"
#include "Templates/Function.h"

#include <type_traits>

namespace Metasound
{
	namespace VertexDataPrivate
	{
		template<typename VertexType>
		using TBindingType = std::conditional_t<std::is_same_v<VertexType, FInputDataVertex>, FInputBinding, FOutputBinding>;

#if ENABLE_METASOUND_ACCESS_TYPE_COMPATIBILITY_TEST
		void CheckAccessTypeCompatibility(const FDataVertex& InDataVertex, const FAnyDataReference& InDataReference)
		{
			bool bIsCompatible = false;
			const EDataReferenceAccessType ReferenceAccessType = InDataReference.GetAccessType();

			switch (InDataVertex.AccessType)
			{
				case EVertexAccessType::Reference:
					// Reference vertices can accept read, write and value types
					bIsCompatible = (EDataReferenceAccessType::Read == ReferenceAccessType) || (EDataReferenceAccessType::Write == ReferenceAccessType) || (EDataReferenceAccessType::Value == ReferenceAccessType);
					break;

				case EVertexAccessType::Value:
					// value vertices require a value data reference type
					bIsCompatible = EDataReferenceAccessType::Value == ReferenceAccessType;
					break;

				default:
					{
						checkNoEntry();
					}
			}

			checkf(bIsCompatible, TEXT("Vertex access type \"%s\" is incompatible with data access type \"%s\" on vertex \"%s\" on node \"%s\""), *LexToString(InDataVertex.AccessType), *LexToString(ReferenceAccessType), *InDataVertex.VertexName.ToString(), METASOUND_DEBUG_ACTIVE_NODE_NAME);
		}
#endif // #if ENABLE_METASOUND_ACCESS_TYPE_COMPATIBILITY_TEST
		EVertexAccessType DataReferenceAccessTypeToVertexAccessType(EDataReferenceAccessType InReferenceAccessType)
		{
			switch (InReferenceAccessType)
			{
				case EDataReferenceAccessType::Read:
				case EDataReferenceAccessType::Write:
					return EVertexAccessType::Reference;

				case EDataReferenceAccessType::Value:
					return EVertexAccessType::Value;

				default:
					return EVertexAccessType::Reference;
			}
		}

		FInputBinding::FInputBinding(FInputDataVertex&& InVertex)
		: Vertex(MoveTemp(InVertex))
		{
		}

		FInputBinding::FInputBinding(const FInputDataVertex& InVertex)
		: Vertex(InVertex)
		{
		}

		FInputBinding::FInputBinding(const FVertexName& InVertexName, FAnyDataReference&& InReference)
		: Vertex(InVertexName, InReference.GetDataTypeName(), FDataVertexMetadata{}, DataReferenceAccessTypeToVertexAccessType(InReference.GetAccessType()))
		{
			Set(MoveTemp(InReference));
		}

		void FInputBinding::Set(FAnyDataReference&& InAnyDataReference)
		{
			check(IsCastable(InAnyDataReference.GetDataTypeName(),Vertex.DataTypeName));
			Data.Emplace(MoveTemp(InAnyDataReference));
			CheckAccessTypeCompatibility(Vertex, *Data);
		}
		
		const FInputDataVertex& FInputBinding::GetVertex() const
		{
			return Vertex;
		}

		void FInputBinding::SetDefaultLiteral(const FLiteral& InLiteral)
		{
			Vertex.SetDefaultLiteral(InLiteral);
		}

		bool FInputBinding::IsBound() const
		{
			return Data.IsSet();
		}

		EDataReferenceAccessType FInputBinding::GetAccessType() const
		{
			if (Data.IsSet())
			{
				return Data->GetAccessType();
			}
			return EDataReferenceAccessType::None;
		}

		// Get data reference 
		const FAnyDataReference* FInputBinding::GetDataReference() const
		{
			return Data.GetPtrOrNull();
		}

		FDataReferenceID FInputBinding::GetDataReferenceID() const
		{
			if (Data.IsSet())
			{
				return Metasound::GetDataReferenceID(*Data);
			}
			return nullptr;
		}


		void FInputBinding::Bind(FInputBinding& InBinding)
		{
			check(IsCastable(InBinding.GetVertex().DataTypeName,Vertex.DataTypeName));

			if (Data.IsSet()) 
			{ 
				if (InBinding.Data.IsSet())
				{
					*InBinding.Data = *Data;
				}
				else
				{
					InBinding.Data = Data;
				}
			}
			else if (InBinding.Data.IsSet())
			{
				Data = InBinding.Data;
				CheckAccessTypeCompatibility(Vertex, *Data);
			}
		}

		FOutputBinding::FOutputBinding(FOutputDataVertex&& InVertex)
		: Vertex(MoveTemp(InVertex))
		{
		}

		FOutputBinding::FOutputBinding(const FOutputDataVertex& InVertex)
		: Vertex(InVertex)
		{
		}

		FOutputBinding::FOutputBinding(const FVertexName& InVertexName, FAnyDataReference&& InReference)
		: Vertex(InVertexName, InReference.GetDataTypeName(), FDataVertexMetadata{}, DataReferenceAccessTypeToVertexAccessType(InReference.GetAccessType()))
		{
			Set(MoveTemp(InReference));
		}

		void FOutputBinding::Set(FAnyDataReference&& InAnyDataReference)
		{
			check(IsCastable(InAnyDataReference.GetDataTypeName(),Vertex.DataTypeName));
			Data.Emplace(MoveTemp(InAnyDataReference));
			CheckAccessTypeCompatibility(Vertex, *Data);
		}
		
		const FOutputDataVertex& FOutputBinding::GetVertex() const
		{
			return Vertex;
		}

		bool FOutputBinding::IsBound() const
		{
			return Data.IsSet();
		}

		EDataReferenceAccessType FOutputBinding::GetAccessType() const
		{
			if (Data.IsSet())
			{
				return Data->GetAccessType();
			}
			return EDataReferenceAccessType::None;
		}

		// Get data reference 
		const FAnyDataReference* FOutputBinding::GetDataReference() const
		{
			return Data.GetPtrOrNull();
		}

		FDataReferenceID FOutputBinding::GetDataReferenceID() const
		{
			if (Data.IsSet())
			{
				return Metasound::GetDataReferenceID(*Data);
			}
			return nullptr;
		}


		void FOutputBinding::Bind(FOutputBinding& InBinding)
		{
			check(Vertex.DataTypeName == InBinding.GetVertex().DataTypeName);

			if (InBinding.Data.IsSet())
			{
				Data = InBinding.Data;
				CheckAccessTypeCompatibility(Vertex, *Data);
			}
		}

		template<typename VertexType>
		void EmplaceBindings(TArray<TBindingType<VertexType>>& InArray, const TVertexInterfaceImpl<VertexType>& InVertexInterface)
		{
			for (const VertexType& DataVertex : InVertexInterface)
			{
				InArray.Emplace(DataVertex);
			}
		}

		template<typename BindingType>
		BindingType* Find(TArray<BindingType>& InBindings, const FVertexName& InVertexName)
		{
			auto IsVertexWithName = [&InVertexName](const BindingType& InBinding)
			{
				return InBinding.GetVertex().VertexName == InVertexName;
			};
			return InBindings.FindByPredicate(IsVertexWithName);
		}

		template<typename BindingType>
		const BindingType* Find(const TArray<BindingType>& InBindings, const FVertexName& InVertexName)
		{
			auto IsVertexWithName = [&InVertexName](const BindingType& InBinding)
			{
				return InBinding.GetVertex().VertexName == InVertexName;
			};
			return InBindings.FindByPredicate(IsVertexWithName);
		}

		template<typename BindingType>
		void SetVertex(bool bIsVertexInterfaceFrozen, TArray<BindingType>& InBindings, const FVertexName& InVertexName, FAnyDataReference&& InDataReference)
		{
			if (BindingType* Binding = Find(InBindings, InVertexName))
			{
				if (IsCastable(
					InDataReference.GetDataTypeName(),						// Incoming ref. (dynamic reference).
					Binding->GetVertex().DataTypeName						// Type of this pin. (must be derived from this or equal).
					))
				{
					Binding->Set(MoveTemp(InDataReference));
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed bind vertex with name '%ls'. Supplied data type (%ls) does not match vertex data type (%ls)", *InVertexName.ToString(), *InDataReference.GetDataTypeName().ToString(), *Binding->GetVertex().DataTypeName.ToString());
				}
			}
			else if (!bIsVertexInterfaceFrozen)
			{
				BindingType NewBinding(InVertexName, MoveTemp(InDataReference));
				InBindings.Add(MoveTemp(NewBinding));
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Failed to bind vertex data", *InVertexName.ToString());
			}
		}

		template<typename BindingType>
		void SetVertex(bool bIsVertexInterfaceFrozen, TArray<BindingType>& InBindings, const FVertexName& InVertexName, const FAnyDataReference& InDataReference)
		{
			SetVertex<BindingType>(bIsVertexInterfaceFrozen, InBindings, InVertexName, FAnyDataReference{InDataReference});
		}

		template<typename BindingType>
		void Bind(bool bIsVertexInterfaceFrozen, TArray<BindingType>& InThisBindings, TArray<BindingType>& InOtherBindings)
		{
			for (BindingType& OtherBinding : InOtherBindings)
			{
				const FVertexName& OtherVertexName = OtherBinding.GetVertex().VertexName;
				if (BindingType* ThisBinding = Find(InThisBindings, OtherVertexName))
				{
					const FName& OtherDataTypeName= OtherBinding.GetVertex().DataTypeName;
					if (IsCastable(OtherDataTypeName,ThisBinding->GetVertex().DataTypeName))
					{
						ThisBinding->Bind(OtherBinding);
					}
					else
					{
						UE_LOGF(LogMetaSound, Warning, "Failed bind vertex with name '%ls'. Supplied data type (%ls) does not match vertex data type (%ls)", *OtherVertexName.ToString(), *OtherDataTypeName.ToString(), *ThisBinding->GetVertex().DataTypeName.ToString());
					}
				}
				else if (!bIsVertexInterfaceFrozen)
				{
					InThisBindings.Add(OtherBinding);
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Failed to bind vertex data", *OtherVertexName.ToString());
				}
			}
		}

		template<typename BindingType>
		void Set(TArray<BindingType>& InBindings, const FDataReferenceCollection& InCollection)
		{
			for (BindingType& Binding : InBindings)
			{
				const FDataVertex& Vertex = Binding.GetVertex();

				if (const FAnyDataReference* DataRef = InCollection.FindDataReference(Vertex.VertexName))
				{
					Binding.Set(*DataRef);
				}
			}
		}

		template<typename BindingType>
		bool IsVertexBound(const TArray<BindingType>& InBindings, const FVertexName& InVertexName)
		{
			if (const BindingType* Binding = Find(InBindings, InVertexName))
			{
				return Binding->IsBound();
			}
			return false;
		}

		template<typename BindingType>
		bool AreAllVerticesBound(const TArray<BindingType>& InBindings)
		{
			return Algo::AllOf(InBindings, [](const BindingType& Binding) { return Binding.IsBound(); });
		}

		template<typename BindingType>
		EDataReferenceAccessType GetVertexDataAccessType(const TArray<BindingType>& InBindings, const FVertexName& InVertexName)
		{
			if (const BindingType* Binding = Find(InBindings, InVertexName))
			{
				return Binding->GetAccessType();
			}
			return EDataReferenceAccessType::None;
		}

		template<typename BindingType>
		FDataReferenceCollection ToDataReferenceCollection(const TArray<BindingType>& InBindings)
		{
			FDataReferenceCollection Collection;
			for (const BindingType& Binding : InBindings)
			{
				if (const FAnyDataReference* Ref = Binding.GetDataReference())
				{
					Collection.AddDataReference(Binding.GetVertex().VertexName, FAnyDataReference{*Ref});
				}
			}
			return Collection;
		}

		template<typename BindingType>
		const FAnyDataReference* FindDataReference(const TArray<BindingType>& InBindings, const FVertexName& InVertexName)
		{
			if (const BindingType* Binding = Find<BindingType>(InBindings, InVertexName))
			{
				return Binding->GetDataReference();
			}
			return nullptr;
		}

		template<typename BindingType>
		void GetVertexInterfaceDataState(const TArray<BindingType>& InBindings, TArray<FVertexDataState>& OutState)
		{
			OutState.Reset();

			for (const BindingType& InBinding : InBindings)
			{
				OutState.Add(FVertexDataState{InBinding.GetVertex().VertexName, InBinding.GetDataReferenceID()});
			}
		}

		template<typename BindingType>
		void CompareVertexInterfaceDataToPriorState(const TArray<BindingType>& InBindings, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates)
		{
			for (const BindingType& Binding : InBindings)
			{
				if (const FAnyDataReference* CurrentReference = Binding.GetDataReference())
				{
					const FVertexDataState* OtherState = nullptr;
					const FName& VertexName = Binding.GetVertex().VertexName;

					for (const FVertexDataState& State : InPriorState)
					{
						if (State.VertexName == VertexName)
						{
							OtherState = &State;
							break;
						}
					}

					if ((nullptr == OtherState) || (OtherState->ID != Binding.GetDataReferenceID()))
					{
						OutUpdates.Add(Binding.GetVertex().VertexName, *CurrentReference);
					}
				}
			}
		}

		// Implementation for iterating over all sub interfaces on a FVertexInterfaceData
		template<typename BindingType>
		void ForEachSubInterfaceBindingImpl(const VertexPrivate::FSubInterfaceLayout& InSubInterfaceLayout, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, BindingType&)> InApplyFunc, TArrayView<BindingType> InOutBindings)
		{
			using namespace VertexPrivate;

			// In some scenarios, we need to determine if the number of vertices match the expected num. For instance if binding an array of data references,
			// we need to check that the array size matches the number of sub interfaces.  
			if ((InExpectedNumSubInterfaceInstances != INDEX_NONE) && (InSubInterfaceLayout.Instances.Num() != InExpectedNumSubInterfaceInstances))
			{
				UE_LOGF(LogMetaSound, Verbose, "Number of instances of sub interface '%ls' is %d, expected %d. This may result in unbound vertex data.", *InSubInterfaceLayout.SubInterfaceName.ToString(), InSubInterfaceLayout.Instances.Num(), InExpectedNumSubInterfaceInstances);
			}

			if (InSubInterfaceLayout.Instances.Num() > 0)
			{
				// The first sub interface will contain the original vertex name. 
				// Find the offset of the original vertex name so it can be located
				// in subsequent sub interface instances. 
				const FSubInterfaceLayout::FInstance& FirstInstance = InSubInterfaceLayout.Instances[0];
				TArrayView<BindingType> FirstSubInterface = InOutBindings.Slice(FirstInstance.Begin, FirstInstance.End - FirstInstance.Begin);
				const uint32 Offset = FirstSubInterface.IndexOfByPredicate([&InVertexName](const BindingType& Binding) { return Binding.GetVertex().VertexName == InVertexName; });

				if (Offset != INDEX_NONE)
				{
					const TArray<FSubInterfaceLayout::FInstance>& Instances = InSubInterfaceLayout.Instances;

					uint32 MaxSubInterfaceIndex = static_cast<uint32>(Instances.Num());
					if (INDEX_NONE != InExpectedNumSubInterfaceInstances)
					{
						MaxSubInterfaceIndex = FMath::Min(MaxSubInterfaceIndex, InExpectedNumSubInterfaceInstances);
					}

					for (uint32 SubInterfaceIndex = 0; SubInterfaceIndex < MaxSubInterfaceIndex; SubInterfaceIndex++)
					{
						const FSubInterfaceLayout::FInstance& Instance = Instances[SubInterfaceIndex];
						uint32 Pos = Instance.Begin + Offset;
						check(Pos < static_cast<uint32>(InOutBindings.Num()));
						InApplyFunc(SubInterfaceIndex, InOutBindings[Pos]);
					}
				}
				else
				{
					UE_LOGF(LogMetaSound, Warning, "Failed to find vertex with name '%ls' in sub interface with name '%ls'.", *InVertexName.ToString(), *InSubInterfaceLayout.SubInterfaceName.ToString());
				}
			}
		}
	}

	FInputVertexInterfaceData::FInputVertexInterfaceData()
	: bIsVertexInterfaceFrozen(false)
	{
	}

	FInputVertexInterfaceData::FInputVertexInterfaceData(const FInputVertexInterface& InVertexInterface)
	: bIsVertexInterfaceFrozen(true)
	, SubInterfaces(InVertexInterface.GetSubInterfaces(VertexPrivate::FPrivateAccessTag{})) 
	{
		VertexDataPrivate::EmplaceBindings(Bindings, InVertexInterface);
	}

	bool FInputVertexInterfaceData::IsVertexInterfaceFrozen() const
	{
		return bIsVertexInterfaceFrozen;
	}

	void FInputVertexInterfaceData::SetIsVertexInterfaceFrozen(bool bInFreezeVertices)
	{
		bIsVertexInterfaceFrozen = bInFreezeVertices;
		checkf(bIsVertexInterfaceFrozen || SubInterfaces.Num() == 0, TEXT("Interfaces containing sub interfaces cannot be unfrozen"));
	}

	bool FInputVertexInterfaceData::Contains(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::Find(Bindings, InVertexName) != nullptr;
	}

	void FInputVertexInterfaceData::BindVertex(const FVertexName& InVertexName, const FAnyDataReference& InOutDataReference)
	{
		BindVertex(InVertexName, const_cast<FAnyDataReference&>(InOutDataReference));
	}

	void FInputVertexInterfaceData::BindVertex(const FVertexName& InVertexName, FAnyDataReference& InOutDataReference)
	{
		using namespace VertexDataPrivate;

		auto CreateBinding = [&]()
		{
			FInputDataVertex Vertex(InVertexName, InOutDataReference.GetDataTypeName(), FDataVertexMetadata{}, DataReferenceAccessTypeToVertexAccessType(InOutDataReference.GetAccessType()));

			return FInputBinding(Vertex);
		};

		auto BindData = [&](FInputBinding& Binding) { Binding.Bind(InOutDataReference); };
		
		Apply(InVertexName, CreateBinding, BindData);
	}

	uint32 FInputVertexInterfaceData::GetNumSubInterfaceInstances(const FName& InSubInterfaceName) const
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			return Layout->Instances.Num();
		}
		return 0;
	}

	void FInputVertexInterfaceData::Bind(const FInputVertexInterfaceData& InVertexData)
	{
		Bind(const_cast<FInputVertexInterfaceData&>(InVertexData));
	}

	void FInputVertexInterfaceData::Bind(FInputVertexInterfaceData& InVertexData)
	{
		VertexDataPrivate::Bind(IsVertexInterfaceFrozen(), Bindings, InVertexData.Bindings);
	}

	void FInputVertexInterfaceData::SetVertex(const FVertexName& InVertexName, FAnyDataReference&& InDataReference)
	{
		VertexDataPrivate::SetVertex(IsVertexInterfaceFrozen(), Bindings, InVertexName, MoveTemp(InDataReference));
	}

	void FInputVertexInterfaceData::SetVertex(const FVertexName& InVertexName, const FAnyDataReference& InDataReference)
	{
		VertexDataPrivate::SetVertex(IsVertexInterfaceFrozen(), Bindings, InVertexName, InDataReference);
	}

	void FInputVertexInterfaceData::Set(const FDataReferenceCollection& InCollection)
	{
		VertexDataPrivate::Set(Bindings, InCollection);
	}

	FDataReferenceCollection FInputVertexInterfaceData::ToDataReferenceCollection() const
	{
		return VertexDataPrivate::ToDataReferenceCollection(Bindings);
	}

	bool FInputVertexInterfaceData::IsVertexBound(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::IsVertexBound(Bindings, InVertexName);
	}

	const FInputDataVertex& FInputVertexInterfaceData::GetVertex(const FVertexName& InVertexName) const
	{
		return FindChecked(InVertexName)->GetVertex();
	}

	void FInputVertexInterfaceData::AddVertex(const FInputDataVertex& InVertex)
	{
		if (IsVertexInterfaceFrozen())
		{
			UE_LOGF(LogMetaSound, Warning, "Cannot add vertex with name '%ls'. Vertex interface is frozen", *InVertex.VertexName.ToString());
		}
		else
		{
			Bindings.Emplace(InVertex);
		}
	}

	void FInputVertexInterfaceData::RemoveVertex(const FVertexName& InVertexName)
	{
		if (IsVertexInterfaceFrozen())
		{
			UE_LOGF(LogMetaSound, Warning, "Cannot remove vertex with name '%ls'. Vertex interface is frozen", *InVertexName.ToString());
		}
		else
		{
			Bindings.RemoveAll([&](const FInputBinding& Binding) { return InVertexName == Binding.GetVertex().VertexName; });
		}
	}

	void FInputVertexInterfaceData::SetDefaultLiteral(const FVertexName& InVertexName, const FLiteral& InLiteral)
	{
		if (FInputBinding* Binding = Find(InVertexName))
		{
			Binding->SetDefaultLiteral(InLiteral);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Cannot set default literal. Failed to find input vertex with name '%ls'.", *InVertexName.ToString());
		}
	}

	EDataReferenceAccessType FInputVertexInterfaceData::GetVertexDataAccessType(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::GetVertexDataAccessType(Bindings, InVertexName);
	}

	bool FInputVertexInterfaceData::AreAllVerticesBound() const
	{
		return VertexDataPrivate::AreAllVerticesBound(Bindings);
	}

	const FAnyDataReference* FInputVertexInterfaceData::FindDataReference(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::FindDataReference(Bindings, InVertexName);
	}

	TArray<const FAnyDataReference*> FInputVertexInterfaceData::FindSubInterfaceReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName) const
	{
		TArray<const FAnyDataReference*> References;
		auto GetRef = [&References](uint32 InIndex, const FInputBinding& InBinding)
		{
			check(References.Num() == InIndex); // This check will be false if the references array 
												// is not being built in the correct order. 
			References.Add(InBinding.GetDataReference());
		};
		ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, INDEX_NONE, GetRef);
		return References;
	}

	void FInputVertexInterfaceData::Apply(const FVertexName& InVertexName, TFunctionRef<FInputBinding ()> InCreateFunc, TFunctionRef<void (FInputBinding&)> InBindFunc)
	{
		if (FInputBinding* Binding = Find(InVertexName))
		{
			InBindFunc(*Binding);
		}
		else if (IsVertexInterfaceFrozen())
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find input vertex with name '%ls'.", *InVertexName.ToString());
		}
		else
		{
			FInputBinding NewBinding = InCreateFunc();
			InBindFunc(NewBinding);
			Bindings.Add(MoveTemp(NewBinding));
		}
	}

	void FInputVertexInterfaceData::ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, FInputBinding&)> InApplyFunc)
	{
		using namespace VertexPrivate;
		using namespace VertexDataPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			ForEachSubInterfaceBindingImpl<FInputBinding>(*Layout, InVertexName, InExpectedNumSubInterfaceInstances, InApplyFunc, Bindings);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find sub interface with name '%ls'.", *InSubInterfaceName.ToString());
		}
	}

	void FInputVertexInterfaceData::ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, const FInputBinding&)> InApplyFunc) const
	{
		using namespace VertexPrivate;
		using namespace VertexDataPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			ForEachSubInterfaceBindingImpl<const FInputBinding>(*Layout, InVertexName, InExpectedNumSubInterfaceInstances, InApplyFunc, Bindings);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find sub interface with name '%ls'.", *InSubInterfaceName.ToString());
		}
	}

	FInputVertexInterfaceData::FInputBinding* FInputVertexInterfaceData::Find(const FVertexName& InVertexName)
	{
		return VertexDataPrivate::Find(Bindings, InVertexName);
	}

	const FInputVertexInterfaceData::FInputBinding* FInputVertexInterfaceData::Find(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::Find(Bindings, InVertexName);
	}

	FInputVertexInterfaceData::FInputBinding* FInputVertexInterfaceData::FindChecked(const FVertexName& InVertexName)
	{
		FInputBinding* Binding = Find(InVertexName);
		checkf(nullptr != Binding, TEXT("Attempt to access vertex \"%s\" which does not exist on interface."), *InVertexName.ToString());
		return Binding;
	}

	const FInputVertexInterfaceData::FInputBinding* FInputVertexInterfaceData::FindChecked(const FVertexName& InVertexName) const
	{
		const FInputBinding* Binding = Find(InVertexName);
		checkf(nullptr != Binding, TEXT("Attempt to access vertex \"%s\" which does not exist on interface."), *InVertexName.ToString());
		return Binding;
	}

	const VertexPrivate::FSubInterfaceLayout* FInputVertexInterfaceData::FindSubInterface(const FName& InSubInterfaceName) const
	{
		return SubInterfaces.FindByPredicate([&InSubInterfaceName](const VertexPrivate::FSubInterfaceLayout& InLayout) { return InLayout.SubInterfaceName == InSubInterfaceName; });
	}

	FOutputVertexInterfaceData::FOutputVertexInterfaceData()
	: bIsVertexInterfaceFrozen(false)
	{
	}

	FOutputVertexInterfaceData::FOutputVertexInterfaceData(const FOutputVertexInterface& InVertexInterface)
	: bIsVertexInterfaceFrozen(true)
	, SubInterfaces(InVertexInterface.GetSubInterfaces(VertexPrivate::FPrivateAccessTag{})) 
	{
		VertexDataPrivate::EmplaceBindings(Bindings, InVertexInterface);
	}

	bool FOutputVertexInterfaceData::IsVertexInterfaceFrozen() const
	{
		return bIsVertexInterfaceFrozen;
	}

	void FOutputVertexInterfaceData::SetIsVertexInterfaceFrozen(bool bInFreezeVertices)
	{
		bIsVertexInterfaceFrozen = bInFreezeVertices;
		checkf(bIsVertexInterfaceFrozen || SubInterfaces.Num() == 0, TEXT("Interfaces containing sub interfaces cannot be unfrozen"));
	}

	bool FOutputVertexInterfaceData::Contains(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::Find(Bindings, InVertexName) != nullptr;
	}

	void FOutputVertexInterfaceData::BindVertex(const FVertexName& InVertexName, const FAnyDataReference& InOutDataReference)
	{
		BindVertex(InVertexName, const_cast<FAnyDataReference&>(InOutDataReference));
	}

	void FOutputVertexInterfaceData::BindVertex(const FVertexName& InVertexName, FAnyDataReference& InOutDataReference)
	{
		using namespace VertexDataPrivate;
		auto CreateBinding = [&]()
		{
			FOutputDataVertex Vertex(InVertexName, InOutDataReference.GetDataTypeName(), FDataVertexMetadata{}, DataReferenceAccessTypeToVertexAccessType(InOutDataReference.GetAccessType()));

			return FOutputBinding(Vertex);
		};

		auto BindData = [&](FOutputBinding& Binding) { Binding.Bind(InOutDataReference); };
		
		Apply(InVertexName, CreateBinding, BindData);
	}

	uint32 FOutputVertexInterfaceData::GetNumSubInterfaceInstances(const FName& InSubInterfaceName) const
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			return Layout->Instances.Num();
		}
		return 0;
	}

	void FOutputVertexInterfaceData::Bind(const FOutputVertexInterfaceData& InVertexData)
	{
		Bind(const_cast<FOutputVertexInterfaceData&>(InVertexData));
	}

	void FOutputVertexInterfaceData::Bind(FOutputVertexInterfaceData& InVertexData)
	{
		VertexDataPrivate::Bind(IsVertexInterfaceFrozen(), Bindings, InVertexData.Bindings);
	}

	void FOutputVertexInterfaceData::SetVertex(const FVertexName& InVertexName, FAnyDataReference&& InDataReference)
	{
		VertexDataPrivate::SetVertex(IsVertexInterfaceFrozen(), Bindings, InVertexName, MoveTemp(InDataReference));
	}

	void FOutputVertexInterfaceData::SetVertex(const FVertexName& InVertexName, const FAnyDataReference& InDataReference)
	{
		VertexDataPrivate::SetVertex(IsVertexInterfaceFrozen(), Bindings, InVertexName, InDataReference);
	}

	void FOutputVertexInterfaceData::Set(const FDataReferenceCollection& InCollection)
	{
		VertexDataPrivate::Set(Bindings, InCollection);
	}

	FDataReferenceCollection FOutputVertexInterfaceData::ToDataReferenceCollection() const
	{
		return VertexDataPrivate::ToDataReferenceCollection(Bindings);
	}

	bool FOutputVertexInterfaceData::IsVertexBound(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::IsVertexBound(Bindings, InVertexName);
	}

	const FOutputDataVertex& FOutputVertexInterfaceData::GetVertex(const FVertexName& InVertexName) const
	{
		return FindChecked(InVertexName)->GetVertex();
	}

	void FOutputVertexInterfaceData::AddVertex(const FOutputDataVertex& InVertex)
	{
		if (IsVertexInterfaceFrozen())
		{
			UE_LOGF(LogMetaSound, Warning, "Cannot add vertex with name '%ls'. Vertex interface is frozen", *InVertex.VertexName.ToString());
		}
		else
		{
			Bindings.Emplace(InVertex);
		}
	}

	void FOutputVertexInterfaceData::RemoveVertex(const FVertexName& InVertexName)
	{
		if (IsVertexInterfaceFrozen())
		{
			UE_LOGF(LogMetaSound, Warning, "Cannot remove vertex with name '%ls'. Vertex interface is frozen", *InVertexName.ToString());
		}
		else
		{
			Bindings.RemoveAll([&](const FOutputBinding& Binding) { return InVertexName == Binding.GetVertex().VertexName; });
		}
	}

	EDataReferenceAccessType FOutputVertexInterfaceData::GetVertexDataAccessType(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::GetVertexDataAccessType(Bindings, InVertexName);
	}

	bool FOutputVertexInterfaceData::AreAllVerticesBound() const
	{
		return VertexDataPrivate::AreAllVerticesBound(Bindings);
	}

	const FAnyDataReference* FOutputVertexInterfaceData::FindDataReference(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::FindDataReference(Bindings, InVertexName);
	}

	TArray<const FAnyDataReference*> FOutputVertexInterfaceData::FindSubInterfaceReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName) const
	{
		TArray<const FAnyDataReference*> References;
		auto GetRef = [&References](uint32 InIndex, const FOutputBinding& InBinding)
		{
			check(References.Num() == InIndex); // This check will be false if the references array 
												// is not being built in the correct order. 
			References.Add(InBinding.GetDataReference());
		};
		ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, INDEX_NONE, GetRef);
		return References;
	}

	void FOutputVertexInterfaceData::Apply(const FVertexName& InVertexName, TFunctionRef<FOutputBinding ()> InCreateFunc, TFunctionRef<void (FOutputBinding&)> InBindFunc)
	{
		if (FOutputBinding* Binding = Find(InVertexName))
		{
			InBindFunc(*Binding);
		}
		else if (!IsVertexInterfaceFrozen())
		{
			FOutputBinding NewBinding = InCreateFunc();
			InBindFunc(NewBinding);
			Bindings.Add(MoveTemp(NewBinding));
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find output vertex with name '%ls'.", *InVertexName.ToString());
		}
	}

	void FOutputVertexInterfaceData::ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, FOutputBinding&)> InApplyFunc)
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			ForEachSubInterfaceBindingImpl<FOutputBinding>(*Layout, InVertexName, InExpectedNumSubInterfaceInstances, InApplyFunc, Bindings);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find sub interface with name '%ls'.", *InSubInterfaceName.ToString());
		}
	}

	void FOutputVertexInterfaceData::ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, const FOutputBinding&)> InApplyFunc) const
	{
		using namespace VertexPrivate;

		if (const FSubInterfaceLayout* Layout = FindSubInterface(InSubInterfaceName))
		{
			ForEachSubInterfaceBindingImpl<const FOutputBinding>(*Layout, InVertexName, InExpectedNumSubInterfaceInstances, InApplyFunc, Bindings);
		}
		else
		{
			UE_LOGF(LogMetaSound, Warning, "Failed to find sub interface with name '%ls'.", *InSubInterfaceName.ToString());
		}
	}

	FOutputVertexInterfaceData::FOutputBinding* FOutputVertexInterfaceData::Find(const FVertexName& InVertexName)
	{
		return VertexDataPrivate::Find(Bindings, InVertexName);
	}

	const FOutputVertexInterfaceData::FOutputBinding* FOutputVertexInterfaceData::Find(const FVertexName& InVertexName) const
	{
		return VertexDataPrivate::Find(Bindings, InVertexName);
	}

	FOutputVertexInterfaceData::FOutputBinding* FOutputVertexInterfaceData::FindChecked(const FVertexName& InVertexName)
	{
		FOutputBinding* Binding = Find(InVertexName);
		checkf(nullptr != Binding, TEXT("Attempt to access vertex \"%s\" which does not exist on interface."), *InVertexName.ToString());
		return Binding;
	}

	const FOutputVertexInterfaceData::FOutputBinding* FOutputVertexInterfaceData::FindChecked(const FVertexName& InVertexName) const
	{
		const FOutputBinding* Binding = Find(InVertexName);
		checkf(nullptr != Binding, TEXT("Attempt to access vertex \"%s\" which does not exist on interface."), *InVertexName.ToString());
		return Binding;
	}

	const VertexPrivate::FSubInterfaceLayout* FOutputVertexInterfaceData::FindSubInterface(const FName& InSubInterfaceName) const
	{
		return SubInterfaces.FindByPredicate([&InSubInterfaceName](const VertexPrivate::FSubInterfaceLayout& InLayout) { return InLayout.SubInterfaceName == InSubInterfaceName; });
	}

	FVertexInterfaceData::FVertexInterfaceData(const FVertexInterface& InVertexInterface)
	: InputVertexInterfaceData(InVertexInterface.GetInputInterface())
	, OutputVertexInterfaceData(InVertexInterface.GetOutputInterface())
	{
	}

	void FVertexInterfaceData::Bind(const FVertexInterfaceData& InVertexData)
	{
		Bind(const_cast<FVertexInterfaceData&>(InVertexData));
	}

	void FVertexInterfaceData::Bind(FVertexInterfaceData& InVertexData)
	{
		InputVertexInterfaceData.Bind(InVertexData.GetInputs());
		OutputVertexInterfaceData.Bind(InVertexData.GetOutputs());
	}

	bool operator<(const FVertexDataState& InLHS, const FVertexDataState& InRHS)
	{
		if (InLHS.VertexName.FastLess(InRHS.VertexName))
		{
			return true;
		}
		else if (InRHS.VertexName.FastLess(InLHS.VertexName))
		{
			return false;
		}
		else
		{
			return InLHS.ID < InRHS.ID;
		}
	}

	bool operator==(const FVertexDataState& InLHS, const FVertexDataState& InRHS)
	{
		return (InLHS.ID == InRHS.ID) && (InLHS.VertexName == InRHS.VertexName);
	}

	void GetVertexInterfaceDataState(const FInputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState)
	{
		VertexDataPrivate::GetVertexInterfaceDataState(InVertexInterface.Bindings, OutState);
	}

	void GetVertexInterfaceDataState(const FOutputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState)
	{
		VertexDataPrivate::GetVertexInterfaceDataState(InVertexInterface.Bindings, OutState);
	}

	void CompareVertexInterfaceDataToPriorState(const FInputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates)
	{
		VertexDataPrivate::CompareVertexInterfaceDataToPriorState(InCurrentInterface.Bindings, InPriorState, OutUpdates);
	}

	void CompareVertexInterfaceDataToPriorState(const FOutputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates)
	{
		VertexDataPrivate::CompareVertexInterfaceDataToPriorState(InCurrentInterface.Bindings, InPriorState, OutUpdates);
	}
}
