// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace Metasound::VertexPrivate
{
	// Used to control API access when a method must be public but we do not want
	// it to be used outside of this module. 
	struct FPrivateAccessTag
	{
	};
}
