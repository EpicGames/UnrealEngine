// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"
#include "MetasoundGraphCoreModule.h"
#include "MetasoundDataReferenceMacro.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorSettings.h"

/** Define which determines whether to check that the size of the audio buffer has not changed since initialization */
#define METASOUNDGRAPHCORE_CHECKAUDIONUM !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

namespace Metasound
{
	/**  FAudioBuffer
	 *
	 * FAudioBuffer is the default buffer for passing audio data between nodes.  It should not be resized.
	 * It is compatible with functions accepting const references to FAlignedFloatBuffer (const FAlignedFloatBuffer&)
	 * arguments via an implicit conversion operator which exposes the underlying FAlignedFloatBuffer container.
	 *
	 */
	class FAudioBuffer
	{
		public:
			/** Create an FAudioBuffer with a specific number of samples.
			 *
			 * @param InNumSamples - Number of samples in buffer.
			 */
			explicit FAudioBuffer(int32 InNumSamples)
			{
				Buffer.AddZeroed(InNumSamples);

#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				// InitialNum should not change during the life of an FAudioBuffer
				InitialNum = Buffer.Num();
#endif
			}

			/**
			 * This is the constructor used by the frontend.
			 */
			FAudioBuffer(const FOperatorSettings& InSettings)
			{
				Buffer.AddZeroed(InSettings.GetNumFramesPerBlock());

#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				// InitialNum should not change during the life of an FAudioBuffer
				InitialNum = Buffer.Num();
#endif	
			}

			FAudioBuffer()
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				// InitialNum should not change during the life of an FAudioBuffer
				InitialNum = Buffer.Num();
#endif
			}

			/** Return a pointer to the audio float data. */
			inline const float* GetData() const
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif				
				return Buffer.GetData();
			}

			/** Return a pointer to the audio float data. */
			inline float* GetData()
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif
				return Buffer.GetData();
			}

			/** Return the number of samples in the audio buffer. */
			inline int32 Num() const
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif
				return Buffer.Num();
			}

			/** Implicit conversion to Audio::FAlignedFloatBuffer */
			inline operator const Audio::FAlignedFloatBuffer& () const
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif
return Buffer;
			}

			/** Implicit conversion to Audio::FAlignedFloatBuffer 
			 *
			 * WARNING: if the buffer is resized, it will cause errors.
			 */
			inline operator Audio::FAlignedFloatBuffer& ()
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif				
				return Buffer;
			}

			/** Implicit conversion to Audio::FAlignedFloatBuffer */
			inline operator TArrayView<const float> () const
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif
return Buffer;
			}

			/** Implicit conversion to Audio::FAlignedFloatBuffer 
			 *
			 * WARNING: if the buffer is resized, it will cause errors.
			 */
			inline operator TArrayView<float> ()
			{
#if METASOUNDGRAPHCORE_CHECKAUDIONUM
				UE_CLOGF(InitialNum != Buffer.Num(), LogMetaSound, Error, "MetaSound audio buffer size change detected.  Audio Buffers should not be resized.");
#endif				
				return Buffer;
			}

			inline void Zero()
			{
				if (Buffer.Num() > 0)
				{
					FMemory::Memset(Buffer.GetData(), 0, sizeof(float) * Buffer.Num());
				}
			}

		private:

			Audio::FAlignedFloatBuffer Buffer;

#if METASOUNDGRAPHCORE_CHECKAUDIONUM
			int32 InitialNum;
#endif
	};

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FAudioBuffer, METASOUNDGRAPHCORE_API, FAudioBufferTypeInfo, FAudioBufferReadRef, FAudioBufferWriteRef);


	// This empty base class is used so that we can specialize various nodes (Send, Receive, etc.) for subclasses of IAudioDatatype.
	class IAudioDataType
	{
		/**
		 * Audio datatypes require the following member functions:
		 * int32 GetNumChannels() const { return NumChannels; }
		 *
		 * int32 GetMaxNumChannels() const { return MaxNumChannels; }

		 * const TArrayView<const FAudioBufferReadRef> GetBuffers() const { return ReadableBuffers; }

		 * const TArrayView<const FAudioBufferWriteRef> GetBuffers() { return WritableBuffers; }
		 */
	};
};
