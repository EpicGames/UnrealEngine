// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundNodeInterface.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	class FBasicNode : public INode
	{
		public:
			/** Construct a basic node.
			 *
			 * @param InNodeData - Information unique to this node instance. 
			 * @param InClassMetadata - Shared object representing the node class.
			 */
			UE_API FBasicNode(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata);

			virtual ~FBasicNode() = default;

			/** Return the name of this specific instance of the node class. */
			UE_API virtual const FName& GetInstanceName() const override;

			/** Return the ID of this specific instance of the node class. */
			UE_API virtual const FGuid& GetInstanceID() const override;

			/** Return metadata associated with this node. */
			UE_API virtual const FNodeClassMetadata& GetMetadata() const override;

			/** Return the interface associated with this node instance. */
			UE_API virtual const FVertexInterface& GetVertexInterface() const override;

			/** Set the default literal for an input vertex. */
			UE_API virtual void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral) override;

			/** Return the configuration for this node. */
			UE_API virtual TSharedPtr<const IOperatorData> GetOperatorData() const override;

		private:

			FNodeData NodeData;
			TSharedRef<const FNodeClassMetadata> ClassMetadata;
	};
}

#undef UE_API
