// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	/** FBuildErrorBase
	 *
	 * A general build error which contains a error type and human readable 
	 * error description.
	 */
	class FBuildErrorBase : public IOperatorBuildError
	{
		public:

			UE_API FBuildErrorBase(const FName& InErrorType, const FText& InErrorDescription);

			virtual ~FBuildErrorBase() = default;

			/** Returns the type of error. */
			UE_API virtual const FName& GetErrorType() const override;

			/** Returns a human readable error description. */
			UE_API virtual const FText& GetErrorDescription() const override;

			/** Returns an array of destinations associated with the error. */
			UE_API virtual const TArray<FInputDataDestination>& GetInputDataDestinations() const override;
			
			/** Returns an array of sources associated with the error. */
			UE_API virtual const TArray<FOutputDataSource>& GetOutputDataSources() const override;

			/** Returns an array of edges associated with the error. */
			UE_API virtual const TArray<FDataEdge>& GetDataEdges() const override;

			/** Returns an array of Nodes associated with the error. */
			UE_API virtual const TArray<const INode*>& GetNodes() const override;


		protected:
			// Add input destinations to be associated with error.
			UE_API void AddInputDataDestination(const FInputDataDestination& InInputDataDestination);
			UE_API void AddInputDataDestinations(TArrayView<const FInputDataDestination> InInputDataDestinations);

			// Add input destinations to be associated with error.
			UE_API void AddOutputDataSource(const FOutputDataSource& InOutputDataSource);
			UE_API void AddOutputDataSources(TArrayView<const FOutputDataSource> InOutputDataSources);

			// Add edges to be associated with error.
			UE_API void AddDataEdge(const FDataEdge& InEdge);
			UE_API void AddDataEdges(TArrayView<const FDataEdge> InEdges);

			// Add nodes to be associated with error.
			UE_API void AddNode(const INode& InNode);
			UE_API void AddNodes(TArrayView<INode const* const> InNodes);

		private:

			FName ErrorType;
			FText ErrorDescription;

			TArray<const INode*> Nodes;
			TArray<FDataEdge> Edges;
			TArray<FInputDataDestination> Destinations;
			TArray<FOutputDataSource> Sources;
	};

	/** FDanglingVertexError
	 *
	 * Caused by FDataEdges, FInputDataDestinations or FOutputDataSources 
	 * pointing to null nodes.
	 */
	class FDanglingVertexError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FDanglingVertexError(const FInputDataDestination& InDest);
			UE_API FDanglingVertexError(const FOutputDataSource& InSource);
			UE_API FDanglingVertexError(const FDataEdge& InEdge);

			virtual ~FDanglingVertexError() = default;
		protected:

			UE_API FDanglingVertexError();

		private:
	};

	/** FMissingVertexError
	 *
	 * Caused by a referenced FDataVertex which does not exist on a node.
	 */
	class FMissingVertexError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FMissingVertexError(const FInputDataDestination& InDestination);
			UE_API FMissingVertexError(const FOutputDataSource& InSource);

			virtual ~FMissingVertexError() = default;

		private:
	};


	/** FDuplicateInputError
	 *
	 * Caused by multiple FDataEdges pointing same FInputDataDestination
	 */
	class FDuplicateInputError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FDuplicateInputError(const TArrayView<FDataEdge> InEdges);

			virtual ~FDuplicateInputError() = default;

		private:
	};

	/** FGraphCycleError
	 *
	 * Caused by circular paths in graph.
	 */
	class FGraphCycleError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FGraphCycleError(TArrayView<INode const* const> InNodes, const TArray<FDataEdge>& InEdges);

			virtual ~FGraphCycleError() = default;

		private:
	};

	/** FNodePrunedError
	 *
	 * Caused by nodes which are in the graph but unreachable from the graph's
	 * inputs and/or outputs.
	 */
	class FNodePrunedError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FNodePrunedError(const INode* InNode);

			virtual ~FNodePrunedError() = default;

		private:
	};

	/** FInternalError
	 *
	 * Caused by internal state or logic errors. 
	 */
	class FInternalError : public FBuildErrorBase
	{
		public:
			static UE_API const FName ErrorType;

			UE_API FInternalError(const FString& InFileName, int32 InLineNumber);

			virtual ~FInternalError() = default;

			UE_API const FString& GetFileName() const;
			UE_API int32 GetLineNumber() const;

		private:
			FString FileName;
			int32 LineNumber;
	};

	/** FMissingInputDataReferenceError
	 *
	 * Caused by IOperators not exposing expected IDataReferences in their input
	 * FDataReferenceCollection
	 */
	class FMissingInputDataReferenceError : public FBuildErrorBase
	{
		public:

			static UE_API const FName ErrorType;
			
			UE_API FMissingInputDataReferenceError(const FInputDataDestination& InInputDataDestination);

			virtual ~FMissingInputDataReferenceError() = default;
	};

	/** FMissingOutputDataReferenceError
	 *
	 * Caused by IOperators not exposing expected IDataReferences in their output
	 * FDataReferenceCollection
	 */
	class FMissingOutputDataReferenceError : public FBuildErrorBase
	{
		public:

			static UE_API const FName ErrorType;
			
			UE_API FMissingOutputDataReferenceError(const FOutputDataSource& InOutputDataSource);

			virtual ~FMissingOutputDataReferenceError() = default;

	};

	/** FInvalidConnectionDataTypeError
	 *
	 * Caused when edges describe a connection between vertices with different
	 * data types.
	 */
	class FInvalidConnectionDataTypeError  : public FBuildErrorBase
	{
		public:

			static UE_API const FName ErrorType;

			UE_API FInvalidConnectionDataTypeError(const FDataEdge& InEdge);

			virtual ~FInvalidConnectionDataTypeError() = default;
	};

	/** FInputReceiverInitializationError
	 *
	 * Caused by Inputs that are set to enable transmission fail to create a receiver.
	 */
	class FInputReceiverInitializationError : public FBuildErrorBase
	{
		public:

			static UE_API const FName ErrorType;
			
			UE_API FInputReceiverInitializationError(const INode& InInputNode, const FName& InVertexKey, const FName& InDataType);

			virtual ~FInputReceiverInitializationError() = default;
	};
} // namespace Metasound

#undef UE_API
