// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundDataReference.h"
#include "MetasoundEnvironment.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundRenderCost.h"
#include "Templates/UniquePtr.h"

class FName;
class FText;

namespace Metasound
{
	// Forward Declare
	class IOperatorBuilder;

	/** IOperatorBuildError
	 *
	 * This interface is intended for errors encountered when building an GraphOperator.
	 */
	class IOperatorBuildError
	{
		public:
			virtual ~IOperatorBuildError() = default;

			/** Returns the type of error. */
			virtual const FName& GetErrorType() const = 0;

			/** Returns a human readable error description. */
			virtual const FText& GetErrorDescription() const = 0;

			/** Returns an array of destinations associated with the error. */
			virtual const TArray<FInputDataDestination>& GetInputDataDestinations() const = 0;
			
			/** Returns an array of sources associated with the error. */
			virtual const TArray<FOutputDataSource>& GetOutputDataSources() const = 0;

			/** Returns an array of Nodes associated with the error. */
			virtual const TArray<const INode*>& GetNodes() const = 0;

			/** Returns an array of edges associated with the error. */
			virtual const TArray<FDataEdge>& GetDataEdges() const = 0;
	};

	/** Array of build errors. */
	using FBuildErrorArray = TArray<TUniquePtr<IOperatorBuildError>>;

	/** Structure of all resulting data generated during graph operator build. */
	struct FBuildResults
	{
		/** An array of errors. Errors can be added if issues occur while creating an IOperator. */
		FBuildErrorArray Errors;

		/** Internal data references if enabled by build settings (not populated if disabled). */
		TMap<FGuid, FDataReferenceCollection> InternalDataReferences;
	};

	/** FBuildOperatorParams holds the parameters provided to operator factories
	 * during the creation of an IOperator
	 */
	struct FBuildOperatorParams
	{
		FBuildOperatorParams(const INode& InNode, const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData, const FMetasoundEnvironment& InEnvironment, const IOperatorBuilder* InBuilder=nullptr, FGraphRenderCost* InGraphRenderCost=nullptr)
		: Node(InNode)
		, OperatorSettings(InOperatorSettings)
		, InputData(InInputData)
		, Environment(InEnvironment)
		, Builder(InBuilder)
		, GraphRenderCost(InGraphRenderCost)
		{
		}

		/** The node associated with this factory and the desired IOperator. */
		const INode& Node;

		/** General operator settings for the graph. */
		const FOperatorSettings& OperatorSettings;

		/** Input data references for an IOperator */
		const FInputVertexInterfaceData& InputData;

		/** Environment settings available. */
		const FMetasoundEnvironment& Environment;

		/** Pointer to builder actively building graph. */
		const IOperatorBuilder* Builder = nullptr;

		/** Runtime render cost tracker. */
		FGraphRenderCost* GraphRenderCost = nullptr;

		/** Implicit conversion to FResetParams for convenience. */
		operator IOperator::FResetParams() const
		{
			return IOperator::FResetParams{OperatorSettings, Environment, GraphRenderCost};
		}
	};

	struct UE_DEPRECATED(5.6, "Use FBuildGraphOperatorParams instead") FBuildGraphParams;
	/** Parameters for building an operator from a graph.  */
	struct FBuildGraphParams
	{
		/** Reference to graph being built. */
		const IGraph& Graph;

		/** General operator settings for the graph. */
		const FOperatorSettings& OperatorSettings;

		/** Collection of input parameters available for an IOperator. */
		const FDataReferenceCollection& InputDataReferences;

		/** Environment settings available. */
		const FMetasoundEnvironment& Environment;
	};

	/** Parameters for building an operator from a graph. */
	struct FBuildGraphOperatorParams
	{
		FBuildGraphOperatorParams(const IGraph& InGraph, const FOperatorSettings& InOperatorSettings, const FInputVertexInterfaceData& InInputData, const FMetasoundEnvironment& InEnvironment, FGraphRenderCost* InGraphRenderCost=nullptr)
		: Graph(InGraph)
		, OperatorSettings(InOperatorSettings)
		, InputData(InInputData)
		, Environment(InEnvironment)
		, GraphRenderCost(InGraphRenderCost)
		{
		}

		/** Reference to graph being built. */
		const IGraph& Graph;

		/** General operator settings for the graph. */
		const FOperatorSettings& OperatorSettings;

		/** Bound input data available for an IOperator. */
		const FInputVertexInterfaceData& InputData;

		/** Environment settings available. */
		const FMetasoundEnvironment& Environment;

		/** Runtime render cost tracker. */
		FGraphRenderCost* GraphRenderCost = nullptr;

		FName ClassName;
		FName AssetName;
		FName AssetPath;
		
		/** Convert operator building params to graph building params */
		static FBuildGraphOperatorParams FromBuildOperatorParams(const IGraph& InGraph, const FBuildOperatorParams& InParams)
		{
			return FBuildGraphOperatorParams{InGraph, InParams.OperatorSettings, InParams.InputData, InParams.Environment, InParams.GraphRenderCost};
		}
	};

	/** Convenience template for adding build errors.
	 *
	 * The function can be used in the following way:
	 * 
	 * FBuildErrorArray MyErrorArray;
	 * AddBuildError<FMyBuildErrorType>(MyErrorArray, MyBuildErrorConstructorArgs...);
	 *
	 * @param OutErrors - Array which holds the errors.
	 * @param Args - Constructor arguments for the error.
	 */
	template<typename ErrorType, typename... ArgTypes>
	void AddBuildError(FBuildErrorArray& OutErrors, ArgTypes&&... Args)
	{
		OutErrors.Add(MakeUnique<ErrorType>(Forward<ArgTypes>(Args)...));
	}


	/** IOperatorFactory
	 *
	 * IOperatorFactory defines an interface for building an IOperator from an INode.  In practice,
	 * each INode returns its own IOperatorFactory through the INode::GetDefaultOperatorFactory() 
	 * member function.
	 */
	class IOperatorFactory
	{
		public:
			virtual ~IOperatorFactory() = default;

			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) = 0;
	};


	/** IOperatorBuilder
	 *
	 * Defines an interface for building a graph of operators from a graph of nodes. 
	 */
	class IOperatorBuilder 
	{
		public:
			/** A TUniquePtr of an IOperatorBuildError */
			using FBuildErrorPtr = TUniquePtr<IOperatorBuildError>;

			virtual ~IOperatorBuilder() = default;

			/** Build a graph operator from a graph.
			 *
			 * @params InParams - Input parameters for building a graph.
			 * @param OutResults - Results data pertaining to the given build operator result.
			 *
			 * @return A unique pointer to the built IOperator. Null if build failed.
			 */
			virtual TUniquePtr<IOperator> BuildGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const = 0;
	};
}
