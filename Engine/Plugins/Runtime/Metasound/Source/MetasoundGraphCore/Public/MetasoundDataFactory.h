// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundLiteral.h"
#include "MetasoundOperatorSettings.h"
#include "Misc/Variant.h"
#include <type_traits>

namespace Metasound
{
	namespace DataFactoryPrivate
	{
		// Deprecated in 5.2. Use TSharedPtr<Audio::IProxyData> instead of TUniquePtr<Audio::IProxyData>.
		using ProxyDataPtrType_DEPRECATED = TUniquePtr<Audio::IProxyData>;

		/** Description of available constructors for a registered Metasound Data Type. 
		 *
		 * @tparam DataType - The registered Metasound Data Type.
		 * @tparam ArgTypes - A parameter pack of arguments to be passed to the data type's constructor.
		 */
		template<typename DataType, typename... ArgTypes>
		struct TDataTypeConstructorTraits
		{
			// Array construction for TArray types. 
			static constexpr bool bIsArrayDefaultConstructible = false;
			static constexpr bool bIsArrayConstructibleWithSettings = false;
			static constexpr bool bIsArrayConstructibleWithArgs = false;
			static constexpr bool bIsArrayConstructibleWithSettingsAndArgs = false;

			// DataType Construction
			static constexpr bool bIsDefaultConstructible = std::is_constructible<DataType>::value || CHasFactoryFunctionDefaultConstruction<DataType>;
			static constexpr bool bIsConstructibleWithSettings = std::is_constructible<DataType, const FOperatorSettings&>::value || CHasFactoryFunctionWithSettings<DataType>;
			static constexpr bool bIsConstructibleWithArgs = std::is_constructible<DataType, ArgTypes...>::value || CHasFactoryFunctionWithArgs<DataType, ArgTypes...>;
			static constexpr bool bIsConstructibleWithSettingsAndArgs = std::is_constructible<DataType, const FOperatorSettings&, ArgTypes...>::value || CHasFactoryFunctionWithSettingsAndArgs<DataType, ArgTypes...>;
		};

		/** Partial specialization for TArray<> types. A TArray can be constructed elementwise with
		 * an array containing constructor args.
		 */
		template<typename ElementType, typename OtherElementType>
		struct TDataTypeConstructorTraits<TArray<ElementType>, TArray<OtherElementType>>
		{
		private:
			using DataType = TArray<ElementType>;
			using ElementConstructorTraits = TDataTypeConstructorTraits<ElementType, OtherElementType>;

			// FLiteral::FNone is a special case where an array of TArray<FLiteral::FNone> only determines 
			// the number of elements that should be default constructed. 
			static constexpr bool bIsOtherElementNone = std::is_same<FLiteral::FNone, OtherElementType>::value;

		public:

			// Array construction of TArray types.
			static constexpr bool bIsArrayDefaultConstructible = ElementConstructorTraits::bIsDefaultConstructible && bIsOtherElementNone;
			static constexpr bool bIsArrayConstructibleWithSettings = ElementConstructorTraits::bIsConstructibleWithSettings && bIsOtherElementNone;
			static constexpr bool bIsArrayConstructibleWithArgs = ElementConstructorTraits::bIsConstructibleWithArgs;
			static constexpr bool bIsArrayConstructibleWithSettingsAndArgs = ElementConstructorTraits::bIsConstructibleWithSettingsAndArgs;

			// TArray construction
			static constexpr bool bIsDefaultConstructible = std::is_constructible<DataType>::value;
			static constexpr bool bIsConstructibleWithSettings = std::is_constructible<DataType, const FOperatorSettings&>::value;
			static constexpr bool bIsConstructibleWithArgs = std::is_constructible<ElementType, OtherElementType>::value;
			// TArray does not take operator settings.
			static constexpr bool bIsConstructibleWithSettingsAndArgs = false;
		};

		/** Constructor traits for a for variant inputs. These traits test each
		 * individual type supported by the variant and determines if *any* of the
		 * types can be used as a single argument to construct the DataType.
		 *
		 * @tparam DataType - The registered Metasound Data Type.
		 * @tparam Types - The types supported by the variant.
		 */
		template<typename DataType, typename... Types>
		struct TDataTypeVariantConstructorTraits;

		// Specialization for unrolling parameter packs. 
		template<typename DataType, typename ArgType, typename... AdditionalTypes>
		struct TDataTypeVariantConstructorTraits<DataType, ArgType, AdditionalTypes...>
		{
			private:

			// Recursive declaration to get constructor traits for the additional types. 
			using FAdditionalConstructorTraits = TDataTypeVariantConstructorTraits<DataType, AdditionalTypes...>;

			// Constructor traits for this ArgType
			using FArgConstructorTraits = TDataTypeConstructorTraits<DataType, ArgType>;
			
			public:

			static constexpr bool bIsArrayDefaultConstructible = FAdditionalConstructorTraits::bIsArrayDefaultConstructible;
			static constexpr bool bIsArrayConstructibleWithSettings = FAdditionalConstructorTraits::bIsArrayConstructibleWithSettings;
			static constexpr bool bIsArrayConstructibleWithArgs = FArgConstructorTraits::bIsArrayConstructibleWithArgs || FAdditionalConstructorTraits::bIsArrayConstructibleWithArgs;
			static constexpr bool bIsArrayConstructibleWithSettingsAndArgs = FArgConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs || FAdditionalConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;

			static constexpr bool bIsDefaultConstructible = FAdditionalConstructorTraits::bIsDefaultConstructible;
			static constexpr bool bIsConstructibleWithSettings = FAdditionalConstructorTraits::bIsConstructibleWithSettings;
			static constexpr bool bIsConstructibleWithArgs = FArgConstructorTraits::bIsConstructibleWithArgs || FAdditionalConstructorTraits::bIsConstructibleWithArgs;
			static constexpr bool bIsConstructibleWithSettingsAndArgs = FArgConstructorTraits::bIsConstructibleWithSettingsAndArgs || FAdditionalConstructorTraits::bIsConstructibleWithSettingsAndArgs;
		};

		// Specialization for single arg. This terminates the variadic parameter unpacking by excluding the recursive declaration of TDataTypeVariantConstructorTraits<>
		template<typename DataType, typename ArgType>
		struct TDataTypeVariantConstructorTraits<DataType, ArgType>
		{
			// Constructor traits for this arg type.
			using FConstructorTraits = TDataTypeConstructorTraits<DataType, ArgType>;

			public:
			static constexpr bool bIsArrayDefaultConstructible = FConstructorTraits::bIsArrayDefaultConstructible;
			static constexpr bool bIsArrayConstructibleWithSettings = FConstructorTraits::bIsArrayConstructibleWithSettings;
			static constexpr bool bIsArrayConstructibleWithArgs = FConstructorTraits::bIsArrayConstructibleWithArgs;
			static constexpr bool bIsArrayConstructibleWithSettingsAndArgs = FConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;

			static constexpr bool bIsDefaultConstructible = FConstructorTraits::bIsDefaultConstructible;
			static constexpr bool bIsConstructibleWithSettings = FConstructorTraits::bIsConstructibleWithSettings;
			static constexpr bool bIsConstructibleWithArgs = FConstructorTraits::bIsConstructibleWithArgs;
			static constexpr bool bIsConstructibleWithSettingsAndArgs = FConstructorTraits::bIsConstructibleWithSettingsAndArgs;
		};

		// Specialization for unpacking the types supported by a TVariant. 
		template<typename DataType, typename FirstVariantType, typename... AdditionalVariantTypes>
		struct TDataTypeVariantConstructorTraits<DataType, TVariant<FirstVariantType, AdditionalVariantTypes...>>
		{
			private:

			using FConstructorTraits = TDataTypeVariantConstructorTraits<DataType, FirstVariantType, AdditionalVariantTypes...>;
			
			public:

			static constexpr bool bIsArrayDefaultConstructible = FConstructorTraits::bIsArrayDefaultConstructible;
			static constexpr bool bIsArrayConstructibleWithSettings = FConstructorTraits::bIsArrayConstructibleWithSettings;
			static constexpr bool bIsArrayConstructibleWithArgs = FConstructorTraits::bIsArrayConstructibleWithArgs;
			static constexpr bool bIsArrayConstructibleWithSettingsAndArgs = FConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;

			static constexpr bool bIsDefaultConstructible = FConstructorTraits::bIsDefaultConstructible;
			static constexpr bool bIsConstructibleWithSettings = FConstructorTraits::bIsConstructibleWithSettings;
			static constexpr bool bIsConstructibleWithArgs = FConstructorTraits::bIsConstructibleWithArgs;
			static constexpr bool bIsConstructibleWithSettingsAndArgs = FConstructorTraits::bIsConstructibleWithSettingsAndArgs;
		};


		/** Denotes that both FOperatorSettings and parameter pack arguments must be
		 * used in the constructor of the Metasound Data Type.
		 *
		 * @tparam DataType - The registered Metasound Data Type.
		 * @tparam ArgTypes - A parameter pack of arguments to be passed to the data type's constructor.
		 */
		template<typename DataType, typename... ArgTypes>
		struct TExplicitConstructorForwarding
		{
			using TConstructorTraits = TDataTypeConstructorTraits<DataType, ArgTypes...>;

			static constexpr bool bForwardSettingsAndArgs = TConstructorTraits::bIsConstructibleWithSettingsAndArgs || CHasFactoryFunctionWithSettingsAndArgs<DataType, ArgTypes...>;
			static constexpr bool bForwardArgs = false;
			static constexpr bool bForwardSettings = false;
			static constexpr bool bForwardNone = false;

			static constexpr bool bCannotForwardToConstructor = !(bForwardSettingsAndArgs || bForwardArgs || bForwardSettings || bForwardNone);
		};

		/** Denotes that parameter pack arguments must be used in the constructor of
		 * the Metasound Data Type. The use of the FOperatorSettings is optional.
		 *
		 * @tparam DataType - The registered Metasound Data Type.
		 * @tparam ArgTypes - A parameter pack of arguments to be passed to the data type's constructor.
		 */
		template<typename DataType, typename... ArgTypes>
		struct TExplicitArgsConstructorForwarding
		{
			using TConstructorTraits = TDataTypeConstructorTraits<DataType, ArgTypes...>;

			static constexpr bool bForwardSettingsAndArgs = TConstructorTraits::bIsConstructibleWithSettingsAndArgs || CHasFactoryFunctionWithSettingsAndArgs<DataType, ArgTypes...>;
			static constexpr bool bForwardArgs = !bForwardSettingsAndArgs && (TConstructorTraits::bIsConstructibleWithArgs || CHasFactoryFunctionWithArgs<DataType, ArgTypes...>); 
			static constexpr bool bForwardSettings = false;
			static constexpr bool bForwardNone = false;

			static constexpr bool bCannotForwardToConstructor = !(bForwardSettingsAndArgs || bForwardArgs || bForwardSettings || bForwardNone);
		};

		/** Denotes that parameter pack arguments and/or FOperatorSettings is optional
		 * when constructing the Metasound Data Type. 
		 *
		 * @tparam DataType - The registered Metasound Data Type.
		 * @tparam ArgTypes - A parameter pack of arguments to be passed to the data type's constructor.
		 */
		template<typename DataType, typename... ArgTypes>
		struct TAnyConstructorForwarding
		{
			using TConstructorTraits = TDataTypeConstructorTraits<DataType, ArgTypes...>;

			static constexpr bool bForwardSettingsAndArgs = TConstructorTraits::bIsConstructibleWithSettingsAndArgs || CHasFactoryFunctionWithSettingsAndArgs<DataType, ArgTypes...>;
			static constexpr bool bForwardArgs = !bForwardSettingsAndArgs && (TConstructorTraits::bIsConstructibleWithArgs || CHasFactoryFunctionWithArgs<DataType, ArgTypes...>);
			static constexpr bool bForwardSettings = !(bForwardArgs || bForwardSettingsAndArgs) && (TConstructorTraits::bIsConstructibleWithSettings || CHasFactoryFunctionWithSettings<DataType>);
			static constexpr bool bForwardNone = !(bForwardArgs || bForwardSettingsAndArgs || bForwardSettings) && (TConstructorTraits::bIsDefaultConstructible || CHasFactoryFunctionDefaultConstruction<DataType>);

			static constexpr bool bCannotForwardToConstructor = !(bForwardSettingsAndArgs || bForwardArgs || bForwardSettings || bForwardNone);
		};

		/** Create a DataType. For use in TDataFactory. */
		template<typename DataType>
		struct TDataTypeCreator
		{
			template<typename... ArgTypes>
			static DataType CreateNew(ArgTypes&&... Args)
			{
				return DataType(Forward<ArgTypes>(Args)...);
			}
		};

	MSVC_PRAGMA(warning(push))
	MSVC_PRAGMA(warning(disable : 4800)) // Disable warning when converting int to bool.

		/** Core factory type for creating objects related to metasound data types. 
		 *
		 * TDataFactoryInternal provides a unified interface for constructing objects
		 * using a FOperatorSettings and a parameter pack of arguments. The ArgForwardType
		 * uses SFINAE to choose whether to forward all arguments, a subset of arguments
		 * or no arguments to the underlying CreatorType.
		 *
		 * @tparam DataType - The registered metasound data type which defines the supported constructors.
		 * @tparam CreatorType - A class providing a static "CreateNew(...)" function which
		 *                       accepts the forwarded arguments and returns an object.
		 */
		template<typename DataType, typename CreatorType>
		struct TDataFactoryInternal
		{
			/** Create a new object using FOperatorSettings and a parameter pack of
			 * arguments. 
			 *
			 * Note: In this implementation, all arguments are forwarded to CreatorType::CreateNew(...)
			 *
			 * @tparam ArgForwardType - A type which describes which settings and arguments
			 *                          should be forwarded to CreateType::CreateNew(...)
			 * @tparam ArgTypes - A parameter pack of types to be forwarded to CreateType::CreateNew(...)
			 */
			template<
				typename ArgForwardType,
				typename... ArgTypes,
				typename std::enable_if< ArgForwardType::bForwardSettingsAndArgs, int>::type = 0
			>
			static auto CreateNew(const FOperatorSettings& InSettings, ArgTypes&&... Args)
			{
				return CreatorType::CreateNew(InSettings, Forward<ArgTypes>(Args)...);
			}

			/** Create a new object using FOperatorSettings and a parameter pack of
			 * arguments. 
			 *
			 * Note: In this implementation, the parameter pack is forwarded to CreateType::CreateNew(...), 
			 * while the FOperatorSettings are ignored.
			 *
			 * @tparam ArgForwardType - A type which describes which settings and arguments
			 *                          should be forwarded to CreateType::CreateNew(...)
			 * @tparam ArgTypes - A parameter pack of types to be forwarded to CreateType::CreateNew(...)
			 */
			template<
				typename ArgForwardType,
				typename... ArgTypes,
				typename std::enable_if< ArgForwardType::bForwardArgs, int>::type = 0
			>
			static auto CreateNew(const FOperatorSettings& InSettings, ArgTypes&&... Args)
			{
				return CreatorType::CreateNew(Forward<ArgTypes>(Args)...);
			}

			/** Create a new object using FOperatorSettings and a parameter pack of
			 * arguments. 
			 *
			 * Note: In this implementation, the FOperatorSettings are forwarded to CreateType::CreateNew(...), 
			 * while the paramter pack is ignored.
			 *
			 * @tparam ArgForwardType - A type which describes which settings and arguments
			 *                          should be forwarded to CreateType::CreateNew(...)
			 * @tparam ArgTypes - A parameter pack of types to be forwarded to CreateType::CreateNew(...)
			 */
			template<
				typename ArgForwardType,
				typename... ArgTypes,
				typename std::enable_if< ArgForwardType::bForwardSettings, int>::type = 0
			>
			static auto CreateNew(const FOperatorSettings& InSettings, ArgTypes&&... Args)
			{
				return CreatorType::CreateNew(InSettings);
			}

			/** Create a new object using FOperatorSettings and a parameter pack of
			 * arguments. 
			 *
			 * Note: In this implementation, the FOperatorSettings and parameter pack
			 * are ignored.
			 *
			 * @tparam ArgForwardType - A type which describes which settings and arguments
			 *                          should be forwarded to CreateType::CreateNew(...)
			 * @tparam ArgTypes - A parameter pack of types to be forwarded to CreateType::CreateNew(...)
			 */
			template<
				typename ArgForwardType,
				typename... ArgTypes,
				typename std::enable_if< ArgForwardType::bForwardNone, int>::type = 0
			>
			static auto CreateNew(const FOperatorSettings& InSettings, ArgTypes&&... Args)
			{
				return CreatorType::CreateNew();
			}

			/** Create a new object using FOperatorSettings and a parameter pack of
			 * arguments. 
			 *
			 * Note: In this implementation, the FOperatorSettings and parameter pack
			 * are ignored.
			 *
			 * @tparam ArgForwardType - A type which describes which settings and arguments
			 *                          should be forwarded to CreateType::CreateNew(...)
			 * @tparam ArgTypes - A parameter pack of types to be forwarded to CreateType::CreateNew(...)
			 */
			template<
				typename ArgForwardType,
				typename... ArgTypes,
				typename std::enable_if< ArgForwardType::bCannotForwardToConstructor, int>::type = 0
			>
			static auto CreateNew(const FOperatorSettings& InSettings, ArgTypes&&... Args)
			{
				static_assert(!ArgForwardType::bCannotForwardToConstructor, "No constructor exists for the DataType which matches the given arguments and argument forwarding type.");
			}
		};

	MSVC_PRAGMA(warning(pop))

		/** Core factory type for creating objects related to Metasound DataTypes. 
		 *
		 * TDataVariantFactoryInternal provides a unified interface for constructing objects
		 * using a FOperatorSettings and TVariant arguments. The VariantParseType
		 * uses SFINAE to choose how to interpret the TVariant argument.
		 *
		 * @tparam DataType - The registered metasound data type which defines the supported constructors.
		 * @tparam CreatorType - A class providing a static "CreateNewFromVariant(...)" function which
		 *                       accepts the forwarded arguments and returns an object.
		 */
		template<typename DataType, typename CreatorType>
		struct TDataVariantFactoryInternal
		{
			using FInternalFactory = TDataFactoryInternal<DataType, CreatorType>;

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, all arguments are forwarded to CreatorType::CreateNew(...)
			 * and the TVariant is parsed as the `VariantType`. FOperatorSettings are
			 * forwarded to CreatorType::CreateNew(...) if it is supported.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateWithArg, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				// The constructor must use the desired variant argument.
				using TExplicitArgsConstructorForwarding = TExplicitArgsConstructorForwarding<DataType, typename VariantParseType::ArgType>;

				return FInternalFactory::template CreateNew<TExplicitArgsConstructorForwarding>(InSettings, InVariant.template Get<typename VariantParseType::ArgType>());
			}

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, the InVariant object is ignored. FOperatorSettings 
			 * are forwarded to CreatorType::CreateNew(...) if it is supported.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateWithoutArg, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				// The constructor must be the default constructor or accept a FOperatorSettings.
				using TExplicitArgsConstructorForwarding = TExplicitArgsConstructorForwarding<DataType>;

				constexpr bool bExpectsNone = std::is_same<typename VariantParseType::ArgType, FLiteral::FNone>::value;
				
				// When constructing an object from a variant, callers expect the variant value
				// to be used in the constructor. It is an error to ignore the variant value.
				// This error can be fixed by changing the InVariant object passed in 
				// or by adding a constructor to the DataType which accepts the VariantType.
				checkf(bExpectsNone, TEXT("The value passed to the constructor is being ignored")); 

				return FInternalFactory::template CreateNew<TExplicitArgsConstructorForwarding>(InSettings);
			}

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, the InVariant object is an array of args
			 * where each array element is the constructor args for one resulting 
			 * array element. Constructor args are forwarded to the element data type and 
			 * the resulting object is placed in the output array.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateArrayElementsWithArg, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				using FConstructorForwarding = TExplicitArgsConstructorForwarding<DataType>;

				using FArgType = typename VariantParseType::ArgType;
				using FArgElementType = typename VariantParseType::ArgElementType;
				using FElementType = typename VariantParseType::ElementType;
				using FElementConstructorForwarding = TExplicitArgsConstructorForwarding<FElementType, FArgElementType>;

				using FElementFactory = TDataFactoryInternal<FElementType, DataFactoryPrivate::TDataTypeCreator<FElementType>>;

				const FArgType& ConstructorValues = InVariant.template Get<FArgType>();

				// Array construct objects
				DataType OutputContainer;
				for (const FArgElementType& ConstructorValue : ConstructorValues)
				{
					OutputContainer.Emplace(FElementFactory::template CreateNew<FElementConstructorForwarding>(InSettings, ConstructorValue));
				}

				// Move construct array.
				return FInternalFactory::template CreateNew<FConstructorForwarding>(InSettings, MoveTemp(OutputContainer));
			}

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, the InVariant object is an array of 
			 * FLiteral::FNone where each array element simply denotes that a corresponding
			 * element should be constructed with FOperatorSettings.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateArrayElementsWithSettings, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				using FConstructorForwarding = TExplicitArgsConstructorForwarding<DataType>;

				using FArgType = typename VariantParseType::ArgType;
				using FArgElementType = typename VariantParseType::ArgElementType;
				using FElementType = typename VariantParseType::ElementType;
				using FElementConstructorForwarding = TExplicitArgsConstructorForwarding<FElementType>;

				using FElementFactory = TDataFactoryInternal<FElementType, DataFactoryPrivate::TDataTypeCreator<FElementType>>;

				const FArgType& ConstructorValues = InVariant.template Get<FArgType>();

				// Array construct objects
				DataType OutputContainer;
				for (const FArgElementType& ConstructorValue : ConstructorValues)
				{
					OutputContainer.Add(FElementFactory::template CreateNew<FElementConstructorForwarding>(InSettings));
				}

				// Move construct array.
				return FInternalFactory::template CreateNew<FConstructorForwarding>(InSettings, MoveTemp(OutputContainer));
			}

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, the InVariant object is an array of 
			 * FLiteral::FNone where each array element simply denotes that a corresponding
			 * element should be default constructed. 
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateArrayElementsWithDefaultConstructor, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				using FConstructorForwarding = TExplicitArgsConstructorForwarding<DataType>;

				using FArgType = typename VariantParseType::ArgType;
				using FArgElementType = typename VariantParseType::ArgElementType;
				using FElementType = typename VariantParseType::ElementType;
				using FElementConstructorForwarding = TExplicitArgsConstructorForwarding<FElementType>;
				using FElementFactory = TDataFactoryInternal<FElementType, DataFactoryPrivate::TDataTypeCreator<FElementType>>;

				constexpr bool bExpectsNone = std::is_same<FArgElementType, FLiteral::FNone>::value;
				
				// When constructing an object from a variant, callers expect the variant value
				// to be used in the constructor. It is an error to ignore the variant value.
				// This error can be fixed by changing the InVariant object passed in 
				// or by adding a constructor to the DataType which accepts the VariantType.
				checkf(bExpectsNone, TEXT("The value passed to the constructor is being ignored")); 


				// Retrieve argument array
				const FArgType& ConstructorValues = InVariant.template Get<FArgType>();

				// Default construct equivalent number of objects
				DataType OutputContainer;
				OutputContainer.AddDefaulted(ConstructorValues.Num());

				// Move construct array.
				return FInternalFactory::template CreateNew<FConstructorForwarding>(InSettings, MoveTemp(OutputContainer));
			}

			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation, the TVariant is parsed as a fallback type.
			 * FOperatorSettings are forwarded to CreatorType::CreateNew(...) if it 
			 * is supported.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCreateWithFallbackArg, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				// The constructor must use the fallback variant argument.
				using TExplicitArgsConstructorForwarding = TExplicitArgsConstructorForwarding<DataType, typename VariantParseType::FallbackArgType>;

				// When constructing an object from a variant, callers expect the variant value
				// to be used in the constructor. It is an error to interpret the variant value
				// as something other than the expected type. Likely, this error is due
				// to the DataType's constructor not supporting the VariantType as an argument.
				// This error can be fixed by changing the InVariant object passed in 
				// or by adding a constructor to the DataType which accepts the VariantType.
				checkf(false, TEXT("The value passed to the constructor is parsed as an unrelated type.")); 

				return FInternalFactory::template CreateNew<TExplicitArgsConstructorForwarding>(InSettings, InVariant.template Get<typename VariantParseType::FallbackArgType>());
			}
			
			/** Create a new object using FOperatorSettings and TVariant.
			 *
			 * Note: In this implementation signals a static error as not possible ways
			 * to construct that underlying object could be found.
			 *
			 * @tparam VariantParseType - A type which describes how to parse the InVariant
			 *                            argument.
			 * @tparam VariantType - The expected underlying type stored in InVariant.
			 */
			template<
				typename VariantParseType,
				typename VariantType,
				typename std::enable_if< VariantParseType::bCannotForwardToConstructor, int>::type = 0
			>
			static auto CreateNewFromVariant(const FOperatorSettings& InSettings, const VariantType& InVariant)
			{
				// There is no DataType constructor which can be used to create the 
				// DataType. This includes no default constructor, nor any constructor
				// which accepts any of the variant types. 
				static_assert(!VariantParseType::bCannotForwardToConstructor, "Constructor has no default constructor and does not support any supplied variant type");
			}
		};

		// Promote which type to use for fallback 
		template<bool ValidType, typename ThisType, typename PreviousType = void>
		struct TTypePromoter
		{
			typedef PreviousType Type;
			static constexpr bool bIsValidType = ValidType;
		};

		// TTypePromoter partial specialization for ValidType=true
		template<typename ThisType, typename PreviousType>
		struct TTypePromoter<true, ThisType, PreviousType>
		{
			typedef PreviousType Type;
			static constexpr bool bIsValidType = true;
		};

		// TTypePromoter partial specialization for ValidType=false
		template<typename ThisType, typename PreviousType>
		struct TTypePromoter<false, ThisType, PreviousType>
		{
			typedef PreviousType Type;
			static constexpr bool bIsValidType = !std::is_same<PreviousType, void>::value;
		};

		// TTypePromoter partial specialization for ValidType=true and no PreviousType.
		template<typename ThisType>
		struct TTypePromoter<true, ThisType>
		{
			typedef ThisType Type;
			static constexpr bool bIsValidType = true;
		};


		// Helper for determine whether constructor accepts the variant type. Supports
		// unrolling the parameter pack via recursive template.
		template<typename DataType, typename ThisArgType, typename... VariantArgTypes>
		struct TDataTypeVariantFallbackHelper
		{
		private:

			static constexpr bool bIsConstructibleWithThisArg = std::is_constructible<DataType, ThisArgType>::value;
			static constexpr bool bIsConstructibleWithSettingsAndThisArg = std::is_constructible<DataType, const FOperatorSettings&, ThisArgType>::value;

			static constexpr bool bPromotThisArgType = bIsConstructibleWithThisArg || bIsConstructibleWithSettingsAndThisArg;
			using FVariantPromoter = TTypePromoter<bPromotThisArgType, ThisArgType, typename TDataTypeVariantFallbackHelper<DataType, VariantArgTypes...>::FallbackType>;

		public:
			
			// True if the DataType is default constructible or constructible with any 
			// of the variant types.
			static constexpr bool bIsConstructibleWithArgs = bIsConstructibleWithThisArg || TDataTypeVariantFallbackHelper<DataType, VariantArgTypes...>::bIsConstructibleWithArgs;

			// True if the DataType is default constructible or constructible with any 
			// of the variant types.
			static constexpr bool bIsConstructibleWithSettingsAndArgs = bIsConstructibleWithSettingsAndThisArg || TDataTypeVariantFallbackHelper<DataType, VariantArgTypes...>::bIsConstructibleWithSettingsAndArgs;

			// Fallback type to use if there is no other way to construct the object.
			typedef typename FVariantPromoter::Type FallbackType;

			// True if the constructor accepts the fallback type.
			static constexpr bool bIsConstructibleWithFallbackArg = FVariantPromoter::bIsValidType; 
		};

		// Specialization of TDataTypeVariantFallbackHelper to handle end of recursion on 
		// VariantArgTypes.
		template<typename DataType, typename ThisArgType>
		struct TDataTypeVariantFallbackHelper<DataType, ThisArgType>
		{
		private:
			static constexpr bool bIsConstructibleWithThisArg = std::is_constructible<DataType, ThisArgType>::value;
			static constexpr bool bIsConstructibleWithSettingsAndThisArg = std::is_constructible<DataType, const FOperatorSettings&, ThisArgType>::value;

			static constexpr bool bPromotThisArgType = bIsConstructibleWithThisArg || bIsConstructibleWithSettingsAndThisArg;
			using FVariantPromoter = TTypePromoter<bPromotThisArgType, ThisArgType>;

		public:
			// True if the DataType is default constructible or constructible with any 
			// of the variant types.
			static constexpr bool bIsConstructibleWithArgs = bIsConstructibleWithThisArg;

			// True if the DataType is default constructible or constructible with any 
			// of the variant types.
			static constexpr bool bIsConstructibleWithSettingsAndArgs = bIsConstructibleWithSettingsAndThisArg;

			// Fallback type to use if there is no other way to construct the object.
			typedef typename FVariantPromoter::Type FallbackType;
			// True if the constructor accepts the fallback type.
			static constexpr bool bIsConstructibleWithFallbackArg = FVariantPromoter::bIsValidType; 
		};

		/** TElementType determines the element type of a container. */
		template<typename ContainterType>
		struct TElementType
		{
			using Type = void;
		};

		/** TElementType specialization for TArray */
		template<typename ElementType>
		struct TElementType<TArray<ElementType>>
		{
			using Type = ElementType;
		};

		/** TDataTypeVariantParsing informs the TDataVariantFactoryInternal on which factory
		 * method to instantiate.
		 */
		template<typename DataType, typename DesiredArgType, typename FirstVariantType, typename... AdditionalVariantTypes>
		struct TDataTypeVariantParsing
		{
		private:
			using FDesiredConstructorTraits = TDataTypeConstructorTraits<DataType, DesiredArgType>; 
			using FFallbackHelper = TDataTypeVariantFallbackHelper<DataType, FirstVariantType, AdditionalVariantTypes...>;

			// Determine which elementwise construction methods are supported
			static constexpr bool bCanCreateArrayElementsWithArg = FDesiredConstructorTraits::bIsArrayConstructibleWithArgs || FDesiredConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;
			static constexpr bool bCanCreateArrayElementsWithSettings = FDesiredConstructorTraits::bIsArrayConstructibleWithSettings; 
			static constexpr bool bCanCreateArrayElementsWithDefaultConstructor = FDesiredConstructorTraits::bIsArrayDefaultConstructible;

			// Determine which construction methods are supported
			static constexpr bool bCanCreateWithArg = FDesiredConstructorTraits::bIsConstructibleWithArgs || FDesiredConstructorTraits::bIsConstructibleWithSettingsAndArgs;
			static constexpr bool bCanCreateWithoutArg = FDesiredConstructorTraits::bIsConstructibleWithSettings || FDesiredConstructorTraits::bIsDefaultConstructible;
			static constexpr bool bCanCreateWithFallbackArg = FFallbackHelper::bIsConstructibleWithFallbackArg;

		public:

			static constexpr bool bCannotForwardToConstructor = !(bCanCreateWithArg || bCanCreateWithoutArg || bCanCreateWithFallbackArg || bCanCreateArrayElementsWithArg || bCanCreateArrayElementsWithDefaultConstructor);

			// Determine which construction method to use.
			static constexpr bool bCreateArrayElementsWithArg = bCanCreateArrayElementsWithArg;
			static constexpr bool bCreateArrayElementsWithSettings = !(bCanCreateArrayElementsWithArg) && bCanCreateArrayElementsWithSettings;
			static constexpr bool bCreateArrayElementsWithDefaultConstructor = !(bCanCreateArrayElementsWithArg || bCreateArrayElementsWithSettings) && bCanCreateArrayElementsWithDefaultConstructor;

			static constexpr bool bCreateWithArg = !(bCreateArrayElementsWithArg || bCreateArrayElementsWithSettings || bCreateArrayElementsWithDefaultConstructor) && bCanCreateWithArg;
			static constexpr bool bCreateWithoutArg = !(bCreateArrayElementsWithArg || bCreateArrayElementsWithSettings || bCreateArrayElementsWithDefaultConstructor || bCreateWithArg) && bCanCreateWithoutArg;
			static constexpr bool bCreateWithFallbackArg = !(bCreateArrayElementsWithArg || bCreateArrayElementsWithSettings || bCreateArrayElementsWithDefaultConstructor || bCreateWithArg || bCreateWithoutArg) && bCanCreateWithFallbackArg;

			// Determine types
			using ElementType = typename TElementType<DataType>::Type;
			using ArgElementType = typename TElementType<DesiredArgType>::Type;
			using ArgType = DesiredArgType;
			using FallbackArgType = typename FFallbackHelper::FallbackType;
		};
	} // End namespace DataFactoryPrivate

	/** Determines whether a DataType supports a constructor which accepts and FOperatorSettings
	 * with ArgTypes and/or just ArgTypes. 
	 *
	 * @tparam DataType - The registered Metasound Data Type.
	 * @tparam ArgTypes - A parameter pack of arguments to be passed to the data type's constructor.
	 */
	template<typename DataType, typename... ArgTypes>
	struct TIsParsable
	{
		using FConstructorTraits = DataFactoryPrivate::TDataTypeConstructorTraits<DataType, ArgTypes...>;

		/* True if the DataType supports a constructor which accepts and FOperatorSettings 
		 * with ArgTypes and/or just ArgTypes. False otherwise.
		 */
		static constexpr bool Value = FConstructorTraits::bIsConstructibleWithArgs || FConstructorTraits::bIsConstructibleWithSettingsAndArgs || FConstructorTraits::bIsArrayConstructibleWithArgs || FConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;
	};

	/* Specialization for FLiteral::FNone arg type. */
	template<typename DataType>
	struct TIsParsable<DataType, FLiteral::FNone>
	{
		using FConstructorTraits = DataFactoryPrivate::TDataTypeConstructorTraits<DataType>;

		/* True if the DataType supports a constructor which accepts and FOperatorSettings or default constructor.  */
		static constexpr bool Value = FConstructorTraits::bIsConstructibleWithArgs || FConstructorTraits::bIsConstructibleWithSettingsAndArgs || FConstructorTraits::bIsArrayConstructibleWithArgs || FConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;
	};

	/* Specialization for TArray data type and TArray<FLiteral::FNone> arg type. */
	template<typename DataElementType>
	struct TIsParsable<TArray<DataElementType>, TArray<FLiteral::FNone>>
	{
		using FElementConstructorTraits = DataFactoryPrivate::TDataTypeConstructorTraits<DataElementType>;

		/* True if the DataElementType supports a default constructor or constructor which accepts and FOperatorSettings. */
		static constexpr bool Value = FElementConstructorTraits::bIsConstructibleWithArgs || FElementConstructorTraits::bIsConstructibleWithSettingsAndArgs ;
	};

	/* Determines whether a DataType is an array type. */
	template<typename DataType>
	struct TIsArrayType
	{
		static constexpr bool Value = false;
	};

	/* Specialization for Array DataTypes. */
	template<typename DataElementType>
	struct TIsArrayType<TArray<DataElementType>>
	{
		static constexpr bool Value = true;
	};

	/** Determines whether a DataType supports construction using the given literal.
	 *
	 * @tparam DataType - The registered Metasound Data Type.
	 */
	template<typename DataType>
	struct TLiteralTraits
	{
		using TVariantConstructorTraits = DataFactoryPrivate::TDataTypeVariantConstructorTraits<DataType, FLiteral::FVariantType>;

		static constexpr bool bIsParseableFromAnyArrayLiteralType =
			TVariantConstructorTraits::bIsArrayDefaultConstructible ||
			TVariantConstructorTraits::bIsArrayConstructibleWithSettings ||
			TVariantConstructorTraits::bIsArrayConstructibleWithArgs ||
			TVariantConstructorTraits::bIsArrayConstructibleWithSettingsAndArgs;

		static constexpr bool bIsParsableFromAnyLiteralType =
			TVariantConstructorTraits::bIsDefaultConstructible ||
			TVariantConstructorTraits::bIsConstructibleWithSettings ||
			TVariantConstructorTraits::bIsConstructibleWithArgs ||
			TVariantConstructorTraits::bIsConstructibleWithSettingsAndArgs ||
			bIsParseableFromAnyArrayLiteralType;

		/** Determines if a constructor for the DataType exists which accepts 
		 * an FOperatorSettings with the literals constructor arg type, and/or one that
		 * accepts the literal constructor arg type. 
		 *
		 * @param InLiteral - The literal containing the constructor argument.
		 *
		 * @return True if a constructor for the DataType exists which accepts 
		 * an FOperatorSettings with the literals constructor arg type, or one that
		 * accepts the literal constructor arg type. It returns False otherwise.
		 */
		static const bool IsParsable(const FLiteral& InLiteral)
		{
			switch (InLiteral.GetType())
			{
				case ELiteralType::None:

					return TIsParsable<DataType>::Value;

				case ELiteralType::Boolean:

					return TIsParsable<DataType, bool>::Value;

				case ELiteralType::Integer:

					return TIsParsable<DataType, int32>::Value;

				case ELiteralType::Float:

					return TIsParsable<DataType, float>::Value;

				case ELiteralType::String:

					return TIsParsable<DataType, FString>::Value;

				case ELiteralType::UObjectProxy:
					// Support for current proxy pointer and deprected proxy pointer. Deprecation of prior proxy pointer occured in release 5.2
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return TIsParsable<DataType, TSharedPtr<Audio::IProxyData>>::Value || TIsParsable<DataType, DataFactoryPrivate::ProxyDataPtrType_DEPRECATED>::Value;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

				case ELiteralType::NoneArray:

					return TIsParsable<DataType, TArray<FLiteral::FNone>>::Value; 

				case ELiteralType::BooleanArray:

					return TIsParsable<DataType, TArray<bool>>::Value;

				case ELiteralType::IntegerArray:

					return TIsParsable<DataType, TArray<int32>>::Value;

				case ELiteralType::FloatArray:

					return TIsParsable<DataType, TArray<float>>::Value;

				case ELiteralType::StringArray:

					return TIsParsable<DataType, TArray<FString>>::Value;

				case ELiteralType::UObjectProxyArray:

					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return TIsParsable<DataType, TArray<TSharedPtr<Audio::IProxyData>>>::Value || TIsParsable<DataType, TArray<DataFactoryPrivate::ProxyDataPtrType_DEPRECATED>>::Value;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

				default:

					checkNoEntry();
					static_assert(static_cast<int32>(ELiteralType::Invalid) == 12, "Possible omitted ELiteralType value.");
					return false;
			}
		}
	};

	/** A base factory type for creating objects related to Metasound DataTypes. 
	 *
	 * TDataFactory provides a unified interface for constructing objects using 
	 * a FOperatorSettings and a parameter pack of arguments. The various factory 
	 * methods determine how strictly the DataType constructors must match the 
	 * arguments to the factory method.
	 *
	 * @tparam DataType - The registered metasound data type which defines the supported constructors.
	 * @tparam CreatorType - A class providing a static "CreateNew(...)" function which
	 *                       accepts the forwarded arguments and returns an object.
	 */
	template<typename DataType, typename CreatorType>
	struct TDataFactory
	{
        using FInternalFactory = DataFactoryPrivate::TDataFactoryInternal<DataType, CreatorType>;

		/** Create the object using any supported constructor. */
		template<typename... ArgTypes>
		static auto CreateAny(const FOperatorSettings& InSettings, ArgTypes&&... Args)
		{
			using TForwarding = DataFactoryPrivate::TAnyConstructorForwarding<DataType, ArgTypes...>;

			// CreateAny(...) tries to find a constructor which accepts the following
			// signatures. This static assert is triggered if none of these constructors
			// exist. 
			//
			// DataType()
			// DataType(const FOperatorSettings&)
			// DataType(ArgTypes...)
			// DataType(const FOperatorSettings&, ArgTypes...);
			//
			static_assert(!TForwarding::bCannotForwardToConstructor, "No constructor exists for the DataType which accepts the forwarded arguments");

			return FInternalFactory::template CreateNew<TForwarding>(InSettings, Forward<ArgTypes>(Args)...);
		}

		/** Create the object using only a constructor which exactly matches the
		 * arguments to this factory method. 
		 */
		template<typename... ArgTypes>
		static auto CreateExplicit(const FOperatorSettings& InSettings, ArgTypes&&... Args)
		{
			using TForwarding = DataFactoryPrivate::TExplicitConstructorForwarding<DataType, ArgTypes...>;

			// CreateExplicit(...) tries to find a constructor which accepts the 
			// exact same signature as this factory method. This static assert is 
			// triggered because the constructor does not exist.
			//
			// DataType(const FOperatorSettings&, ArgTypes...);
			//
			static_assert(!TForwarding::bCannotForwardToConstructor, "No constructor exists for the DataType which accepts the forwarded arguments");

			return FInternalFactory::template CreateNew<TForwarding>(InSettings, Forward<ArgTypes>(Args)...);
		}

		/** Create the object using only constructors which utilize all arguments
		 * in the parameter pack (ArgTypes...).
		 */
		template<typename... ArgTypes>
		static auto CreateExplicitArgs(const FOperatorSettings& InSettings, ArgTypes&&... Args)
		{
			using TForwarding = DataFactoryPrivate::TExplicitArgsConstructorForwarding<DataType, ArgTypes...>;

			// CreateExplicitArgs(...) tries to find a constructor which accepts the following
			// signatures. This static assert is triggered if none of these constructors
			// exist. 
			//
			// DataType(ArgTypes...)
			// DataType(const FOperatorSettings&, ArgTypes...);
			//
			static_assert(!TForwarding::bCannotForwardToConstructor, "No constructor exists for the DataType which accepts the forwarded arguments");

			return FInternalFactory::template CreateNew<TForwarding>(InSettings, Forward<ArgTypes>(Args)...);
		}
	};


	/** TDataTypeFactory creates a DataType.
	 *
	 * TDataTypeFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType.
	 */
	template<typename DataType>
	struct TDataTypeFactory : TDataFactory<DataType, DataFactoryPrivate::TDataTypeCreator<DataType>>
	{
	};

	/** TDataValueReferenceFactory creates TDataValueReferences for the given DataType.
	 *
	 * TDataValueReferenceFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataValueReference.
	 */
	template<typename DataType>
	struct TDataValueReferenceFactory : TDataFactory<DataType, TDataValueReference<DataType>>
	{
	};

	/** TDataReadReferenceFactory creates TDataReadReferences for the given DataType.
	 *
	 * TDataReadReferenceFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataReadReference.
	 */
	template<typename DataType>
	struct TDataReadReferenceFactory : TDataFactory<DataType, TDataReadReference<DataType>>
	{
	};

	/** TDataWriteReferenceFactory creates TDataWriteReferences for the given DataType.
	 *
	 * TDataWriteReferenceFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataWriteReference.
	 */
	template<typename DataType>
	struct TDataWriteReferenceFactory : TDataFactory<DataType, TDataWriteReference<DataType>>
	{
	};

	
	/** A base factory type for creating objects related to Metasound DataTypes. 
	 *
	 * TDataLiteralFactory provides a unified interface for constructing objects using 
	 * a FOperatorSettings and FLiteral. The various factory 
	 * methods determine how strictly the DataType constructors must match the 
	 * arguments to the factory method.
	 *
	 * @tparam DataType - The registered metasound data type which defines the supported constructors.
	 * @tparam CreatorType - A class providing a static "CreateNew(...)" function which
	 *                       accepts the forwarded arguments and returns an object.
	 */
	template<typename DataType, typename CreatorType>
	struct TDataLiteralFactory
	{
		using FInternalFactory = DataFactoryPrivate::TDataVariantFactoryInternal<DataType, CreatorType>;

		/** Create the object using any supported constructor. */
		template<typename FirstVariantType, typename... AdditionalVariantTypes>
		static auto CreateAny(const FOperatorSettings& InSettings, const TVariant<FirstVariantType, AdditionalVariantTypes...>& InVariant)
		{
			using FDataVariantParsing = DataFactoryPrivate::TDataTypeVariantParsing<DataType, void, FirstVariantType, AdditionalVariantTypes...>;

			return FInternalFactory::template CreateNewFromVariant<FDataVariantParsing>(InSettings, InVariant);
		}

		/** Create the object using only constructors which utilize the InVariant. */
		template<typename DesiredArgType, typename FirstVariantType, typename... AdditionalVariantTypes>
		static auto CreateExplicitArgs(const FOperatorSettings& InSettings, const TVariant<FirstVariantType, AdditionalVariantTypes...>& InVariant)
		{
			using FDataVariantParsing = DataFactoryPrivate::TDataTypeVariantParsing<DataType, DesiredArgType, FirstVariantType, AdditionalVariantTypes...>;

			return FInternalFactory::template CreateNewFromVariant<FDataVariantParsing>(InSettings, InVariant);
		}

		/** Create the object using only constructors which utilize the InLiteral. 
		 *
		 * @param InSettings - The FOperatorSettings to be passed to the DataType 
		 *                     constructor if an appropriate constructor exists.
		 * @param InLiteral - The literal to be passed to the constructor.
		 *
		 * @return An object related to the DataType. The exact type determined by
		 *         the CreatorType class template parameter.
		 */
		static auto CreateExplicitArgs(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
		{
			switch (InLiteral.GetType())
			{
				case ELiteralType::Boolean:

					return CreateExplicitArgs<bool>(InSettings, InLiteral.Value);

				case ELiteralType::Integer:

					return CreateExplicitArgs<int32>(InSettings, InLiteral.Value);

				case ELiteralType::Float:

					return CreateExplicitArgs<float>(InSettings, InLiteral.Value);

				case ELiteralType::String:

					return CreateExplicitArgs<FString>(InSettings, InLiteral.Value);

				case ELiteralType::UObjectProxy:
					{
						constexpr bool bIsParsable = TIsParsable<DataType, TSharedPtr<Audio::IProxyData>>::Value;

						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						constexpr bool bIsParsableWithDeprecatedPtr = TIsParsable<DataType, DataFactoryPrivate::ProxyDataPtrType_DEPRECATED>::Value;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS

						// Temporarily convert TSharedPtr to TUniquePtr in scenario where only the deprecated proxy factory exists. 
						if constexpr (!bIsParsable && bIsParsableWithDeprecatedPtr)
						{
							using FFactoryForDeprecatedPtr = TDataFactory<DataType, CreatorType>;

							// The proxy is stored in a TUniquePtr<>, but it is released from the TUniquePtr<> before the TUniquePtr<> destructor is called. 
							// This is done to match the interface for MetaSound data types which utilized a deprecated interface. It is not intended
							// to pass ownership of the proxy to the TUniquePtr<>.
							TUniquePtr<Audio::IProxyData> DeprecatedProxyDataPtr(InLiteral.Value.Get<TSharedPtr<Audio::IProxyData>>().Get());

							auto Result = FFactoryForDeprecatedPtr::CreateExplicitArgs(InSettings, (const TUniquePtr<Audio::IProxyData>&)DeprecatedProxyDataPtr);

							(void)DeprecatedProxyDataPtr.Release();
							return Result;
						}
						else
						{
							return CreateExplicitArgs<TSharedPtr<Audio::IProxyData>>(InSettings, InLiteral.Value);
						}
					}

				case ELiteralType::NoneArray:

					return CreateExplicitArgs<TArray<FLiteral::FNone>>(InSettings, InLiteral.Value);

				case ELiteralType::BooleanArray:

					return CreateExplicitArgs<TArray<bool>>(InSettings, InLiteral.Value);

				case ELiteralType::IntegerArray:

					return CreateExplicitArgs<TArray<int>>(InSettings, InLiteral.Value);

				case ELiteralType::FloatArray:

					return CreateExplicitArgs<TArray<float>>(InSettings, InLiteral.Value);

				case ELiteralType::StringArray:

					return CreateExplicitArgs<TArray<FString>>(InSettings, InLiteral.Value);

				case ELiteralType::UObjectProxyArray:
					{
						constexpr bool bIsParsable = TIsParsable<DataType, TArray<TSharedPtr<Audio::IProxyData>>>::Value;
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						constexpr bool bIsParsableWithDeprecatedPtr = TIsParsable<DataType, TArray<DataFactoryPrivate::ProxyDataPtrType_DEPRECATED>>::Value;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS

						// Temporarily convert TSharedPtr to TUniquePtr in scenario where only the deprecated proxy factory exists. 
						if constexpr (!bIsParsable && bIsParsableWithDeprecatedPtr)
						{
							using FFactoryForDeprecatedPtr = TDataFactory<DataType, CreatorType>;

							// The proxies are stored in a TArray<TUniquePtr<>>, but it is released before the TArray<TUniquePtr<>> destructor is called. 
							// This is done to match the interface for MetaSound data types which utilized a deprecated interface. It is not intended
							// to pass ownership of the proxies to the TArray<TUniquePtr<>>.
							const TArray<TSharedPtr<Audio::IProxyData>>& ProxyDataPtrArray = InLiteral.Value.Get<TArray<TSharedPtr<Audio::IProxyData>>>();
							TArray<TUniquePtr<Audio::IProxyData>> DeprecatedArray;

							for (const TSharedPtr<Audio::IProxyData>& ProxyDataPtr : ProxyDataPtrArray)
							{
								DeprecatedArray.Emplace(ProxyDataPtr.Get());
							}

							auto Result = FFactoryForDeprecatedPtr::CreateExplicitArgs(InSettings, (const TArray<TUniquePtr<Audio::IProxyData>>&)DeprecatedArray);

							for (TUniquePtr<Audio::IProxyData>& DeprecatedPtr : DeprecatedArray)
							{
								(void)DeprecatedPtr.Release();
							}

							return Result;
						}
					}

					return CreateExplicitArgs<TArray<TSharedPtr<Audio::IProxyData>>>(InSettings, InLiteral.Value);

				case ELiteralType::None:

					return CreateExplicitArgs<FLiteral::FNone>(InSettings, InLiteral.Value);

				case ELiteralType::Invalid:
				default:

					checkNoEntry();

					return CreateAny(InSettings, InLiteral.Value);
			}
		}
	};

	/** TDataTypeLiteralFactory creates a DataType.
	 *
	 * TDataTypeLiteralFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType.
	 */
	template<typename DataType>
	struct TDataTypeLiteralFactory : TDataLiteralFactory<DataType, DataFactoryPrivate::TDataTypeCreator<DataType>>
	{
	};

	/** TDataValueReferenceLiteralFactory creates TDataValueReferences for the given DataType.
	 *
	 * TDataValueReferenceLiteralFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataValueReference.
	 */
	template<typename DataType>
	struct TDataValueReferenceLiteralFactory : TDataLiteralFactory<DataType, TDataValueReference<DataType>>
	{
	};

	/** TDataReadReferenceLiteralFactory creates TDataReadReferences for the given DataType.
	 *
	 * TDataReadReferenceLiteralFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataReadReference.
	 */
	template<typename DataType>
	struct TDataReadReferenceLiteralFactory : TDataLiteralFactory<DataType, TDataReadReference<DataType>>
	{
	};

	/** TDataWriteReferenceLiteralFactory creates TDataWriteReferences for the given DataType.
	 *
	 * TDataWriteReferenceLiteralFactory provides several factory methods for forwarding 
	 * arguments from the factory method to the DataType constructor.  See TDataFactory
	 * for more information on the provided factory methods.
	 *
	 * @tparam DataType - The Metasound DataType of the TDataWriteReference.
	 */
	template<typename DataType>
	struct TDataWriteReferenceLiteralFactory : TDataLiteralFactory<DataType, TDataWriteReference<DataType>>
	{
	};
}
