// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Concepts/DerivedFrom.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "MetasoundDataTypeGetTraits.h"
#include "MetasoundPolymorphic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#define UE_API METASOUNDGRAPHCORE_API


namespace Audio
{
	// Forward declare
	class IProxyData;
}

namespace Metasound
{
	class FOperatorSettings;
	// Forward declare for LexToString
	enum class EDataReferenceAccessType : uint8;
}

/** Convert a EDataReferenceAccessType to FString. */
FString METASOUNDGRAPHCORE_API LexToString(Metasound::EDataReferenceAccessType InAccessType);

using FMetasoundDataTypeId = void const* const;

namespace Metasound
{
	// Unique ID type which corresponds to the underlying object referred to by a data reference.
	using FDataReferenceID = const void*;

	/** Returns array type associated with the base datatype provided(ex. 'Float:Array' if 'Float' is provided) */
	METASOUNDGRAPHCORE_API FName CreateArrayTypeNameFromElementTypeName(const FName InTypeName);

	/** Returns the base data type with the array extension(ex. 'Float' if 'Float:Array' is provided) */
	METASOUNDGRAPHCORE_API FName CreateElementTypeNameFromArrayTypeName(const FName InArrayTypeName);


	/** A Data Reference Interface.
	 *
	 * A parameter references provides information and access to a shared object in the graph.
	 */
	class IDataReference
	{
		public:
		static UE_API const FName RouterName;

		virtual ~IDataReference() = default;

		/** Returns the name of the data type. */
		virtual const FName& GetDataTypeName() const = 0;

		/** Returns the ID of the parameter type. */
		virtual const void* const GetDataTypeId() const = 0;

		/** Creates a copy of the parameter type. */
		virtual TUniquePtr<IDataReference> Clone() const = 0;

		/** provides a raw pointer to the storage where the data actually resides. */
		virtual void* GetRaw() const = 0;
	};

	/** Return the ID of the data reference. */
	inline FDataReferenceID GetDataReferenceID(const IDataReference& InDataReference)
	{
		return static_cast<FDataReferenceID>(InDataReference.GetRaw());
	}

	template <typename T>
	UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
	bool IsChildOf(const FName InBaseType)
	{
		static const FName ThisName = GetMetasoundDataTypeName<T>();
		if (InBaseType == ThisName)
		{
			return true;
		}

		if constexpr (TPolymorphicTraits<T>::bIsPolymorphic)
		{
			using Base = typename TPolymorphicTraits<T>::BaseType;
			static_assert(!std::is_same_v<Base, T>, "Polymorphic BaseType must not equal T");
			return IsChildOf<Base>(InBaseType);
		}
		else
		{
			return false;
		}
	}
	
	
	// Polymorphic overload
	template<typename DataType>
	requires TPolymorphicTraits<DataType>::bIsPolymorphic
	bool IsDataReferenceOfType(const IDataReference& InReference)
	{
		static const FName TypeName = GetMetasoundDataTypeName<DataType>();
		static const void* const TypeId = GetMetasoundDataTypeId<DataType>();

		const bool bEqualTypeName = InReference.GetDataTypeName() == TypeName;
		const bool bEqualTypeId = InReference.GetDataTypeId() == TypeId;
		if (bEqualTypeName && bEqualTypeId)
		{
			return true;
		}

		// Recurse to base type
		using FPolyType = TPolymorphicTraits<DataType>;
		return IsDataReferenceOfType<typename FPolyType::BaseType>(InReference);
	}

	// Non-polymorphic fallback
	template<typename DataType>
	requires (!TPolymorphicTraits<DataType>::bIsPolymorphic)
	bool IsDataReferenceOfType(const IDataReference& InReference)
	{
		static const FName TypeName = GetMetasoundDataTypeName<DataType>();
		static const void* const TypeId = GetMetasoundDataTypeId<DataType>();

		const bool bEqualTypeName = InReference.GetDataTypeName() == TypeName;
		const bool bEqualTypeId = InReference.GetDataTypeId() == TypeId;
		return bEqualTypeName && bEqualTypeId;
	}

	// This enum is used as a token to explicitly delineate when we should create a new object for the reference,
	// or use a different constructor.
	enum class EDataRefShouldConstruct
	{
		NewObject
	};
	
	namespace MetasoundDataReferencePrivate
	{
		template <typename Base, typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		TSharedRef<Base, ESPMode::NotThreadSafe> MakeRefOf(ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<Base, Derived>, "Target must derive from Base (or be Base)");
			static_assert(std::is_constructible_v<Derived, ArgTypes...>, "Args do not match Target's constructor");

			TSharedRef<Derived, ESPMode::NotThreadSafe> DerivedRef = MakeShared<Derived, ESPMode::NotThreadSafe>(Forward<ArgTypes>(Args)...);

			return StaticCastSharedRef<Base>(DerivedRef);
		}

		template <typename Base, typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		TSharedRef<Base, ESPMode::NotThreadSafe> MakeRefOf(ArgTypes&&... Args) requires CHasFactoryFunction<Derived, ArgTypes...>
		{
			static_assert(std::is_base_of_v<Base, Derived>, "Target must derive from Base (or be Base)");
			TSharedRef<Derived, ESPMode::NotThreadSafe> DerivedRef = Derived::CreateNew(Forward<ArgTypes>(Args)...);
 			return StaticCastSharedRef<Base>(DerivedRef);
		}
	}

	template <typename DataType>
	struct TDynamicDataReferencePolicy;

	template<typename DataType>
	requires (!TPolymorphicTraits<DataType>::bIsPolymorphic)
	struct TDynamicDataReferencePolicy<DataType>
	{
		template<typename QueryType> bool IsA(const DataType*) const { return GetMetasoundDataTypeName<DataType>() == GetMetasoundDataTypeName<QueryType>(); }
		const FName& GetDataTypeName(const DataType*) const { return GetMetasoundDataTypeName<DataType>(); }
		const void* GetDataTypeId(const DataType*) const { return GetMetasoundDataTypeId<DataType>(); }
	};

	template<typename DataType>
	requires (TPolymorphicTraits<DataType>::bIsPolymorphic)
	struct TDynamicDataReferencePolicy<DataType>
	{
		static_assert(CHasPolymorphicTypeFunctions<DataType>, "Dynamic Reference Policy Requires GetDataTypeName+GetDataTypeId functions on DataType");

		template<typename QueryType> bool IsA(const DataType* InOuter) const
		{
			return IsChildOfByName(GetDataTypeName(InOuter), GetMetasoundDataTypeName<QueryType>());
		}

		const FName& GetDataTypeName(const DataType* InOuter) const
		{
			return InOuter->GetDataTypeName();
		}

		const void* GetDataTypeId(const DataType* InOuter) const
		{
			return InOuter->GetDataTypeId();
		}
	};

	/** Template class for a paramter reference.
	 *
	 * This fulfills the IParamterRef interface, utilizing TDataReferenceTypeInfo to
	 * define the the TypeName and TypeId of the parameter.
	 */
	template <typename DataType>
	class TDataReference : public IDataReference
	{
		static_assert(std::is_same<DataType, std::decay_t<DataType>>::value, "Data types used as data references must not decay");
	protected:
		/**
		 * This constructor forwards arguments to an underlying constructor.
		 */
		template <typename... ArgTypes>
		TDataReference(EDataRefShouldConstruct InToken, ArgTypes&&... Args)
			: ObjectReference(MakeShared<DataType, ESPMode::NotThreadSafe>(Forward<ArgTypes>(Args)...))
		{
		}

		template <typename... ArgTypes>
		TDataReference(EDataRefShouldConstruct InToken, ArgTypes&&... Args) requires(CIsPolymorphicAndAbstract<DataType>)
			: ObjectReference(DataType::CreateNew(Forward<ArgTypes>(Args)...))
		{
			static_assert(CHasFactoryFunction<DataType, ArgTypes...>);
		}

		typedef TSharedRef<DataType, ESPMode::NotThreadSafe> FRefType;

		explicit TDataReference(EDataRefShouldConstruct /*InToken*/, FRefType InRef)
			: ObjectReference(MoveTemp(InRef))
		{}

		explicit TDataReference(EDataRefShouldConstruct,
								FRefType InRef,
								const FName InTypeName,
								const void* InTypeId)
			: ObjectReference(MoveTemp(InRef))
		{}
	
	public:

		typedef TDataReferenceTypeInfo<DataType> FInfoType;

		/** This should be used to construct a new DataType object and return this TDataReference as a wrapper around it.
		 */
		template <typename... ArgTypes>
		static TDataReference<DataType> CreateNew(ArgTypes&&... Args)
		{
			static_assert(std::is_constructible<DataType, ArgTypes...>::value, "Tried to call TDataReference::CreateNew with args that don't match any constructor for an underlying type!");
			return TDataReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		/**
		 * Polymorphic form of CreateNew.
		 */
		template <typename... ArgTypes>
		static TDataReference<DataType> CreateNew(ArgTypes&&... Args) requires (CIsPolymorphicAndAbstract<DataType>)
		{
			static_assert(CHasFactoryFunction<DataType, ArgTypes...>);
			return TDataReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		// Constructs Derived but stores as Base (DataType).
		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		static TDataReference CreateNewDerived(ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
			return TDataReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...)
			);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.8, "Polymorphic API is under development")
		static TDataReference CreateNewDerivedAs(const FName& InDerivedTypeName,
												 FMetasoundDataTypeId InDerivedTypeId,
												 ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
		#if DO_CHECK
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			check(Metasound::IsChildOfByName(InDerivedTypeName, GetMetasoundDataTypeName<DataType>()));
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		#endif

			return TDataReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...),
				InDerivedTypeName,
				InDerivedTypeId
			);
		}


		/** Enable copy constructor */
		TDataReference(const TDataReference<DataType>& Other) = default;

		/** Enable move constructor */
		TDataReference(TDataReference<DataType>&& Other) = default;

		/** Enable copy operator */
		TDataReference<DataType>& operator=(const TDataReference<DataType>& Other) = default;

		/** Enable move operator */
		TDataReference<DataType>& operator=(TDataReference<DataType>&& Other) = default;

		/** Return the name of the underlying type. */
		virtual const FName& GetDataTypeName() const override
		{
			return ObjectReferencePolicy.GetDataTypeName(&ObjectReference.Get());
		}

		/** Return the ID of the underlying type. */
		virtual const void* const GetDataTypeId() const override
		{
			return ObjectReferencePolicy.GetDataTypeId(&ObjectReference.Get());
		}

		/** Return a raw pointer to the data. */
		virtual void* GetRaw() const override
		{
			return &ObjectReference.Get();
		}

		template<typename QueryType>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		bool IsA() const
		{
			return this->ObjectReferencePolicy.template IsA<QueryType>(&ObjectReference.Get());
		}

		template <typename CastType>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		TOptional<TSharedRef<CastType, ESPMode::NotThreadSafe>> TryCast() const
		{
			if (IsA<CastType>())
			{
				return StaticCastSharedRef<CastType>(ObjectReference);
			}
			return {};
		}

	protected:
		
		// Protected object reference is utilized by subclasses which define what
		// access is provided to the ObjectReference. 
		FRefType ObjectReference;
		TDynamicDataReferencePolicy<DataType> ObjectReferencePolicy;
	};

	// Forward declare
	template <typename DataType>
	class TDataReadReference;

	// Forward declare
	template <typename DataType>
	class TDataWriteReference;

	/** TDataValueReference represents a constant value and provides read only access. 
	 * A TDataValueReference can never change value. */
	template<typename DataType>
	class TDataValueReference : public TDataReference<DataType>
	{
		// Construct operator with no arguments if the DataType has a default constructor.
		template <typename... ArgTypes>
		TDataValueReference(EDataRefShouldConstruct InToken, ArgTypes&&... Args)
			: TDataReference<DataType>(InToken, Forward<ArgTypes>(Args)...)
		{
		}

		// Constructor taking data reference. Used for casting
		TDataValueReference(const TDataReference<DataType>& InRef)
		: TDataReference<DataType>(InRef)
		{
		}

		template<typename T>
		friend TDataValueReference<T> ValueCast(const TDataReadReference<T>& InRef);

		template<typename T>
		friend TDataValueReference<T> ValueCast(const TDataWriteReference<T>& InRef);
		
		using FRefType = TDataReference<DataType>::FRefType;
		TDataValueReference(EDataRefShouldConstruct, FRefType InRef, FName InTypeName, const void* InTypeId)
			: TDataReference<DataType>(EDataRefShouldConstruct::NewObject, MoveTemp(InRef), InTypeName, InTypeId)
		{
		}

	public:

		/** This should be used to construct a new DataType object and return this TDataValueReference as a wrapper around it. */
		template <typename... ArgTypes>
		static TDataValueReference<DataType> CreateNew(ArgTypes&&... Args)
		{
			static_assert(std::is_constructible<DataType, ArgTypes...>::value, "TDataValueReference::CreateNew underlying type is not constructible with provided arguments.");
			return TDataValueReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		template <typename... ArgTypes>
		static TDataValueReference<DataType> CreateNew(ArgTypes&&... Args) requires (CIsPolymorphicAndAbstract<DataType>)
		{
			static_assert(CHasFactoryFunction<DataType, ArgTypes...>);
			return TDataValueReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.7, "Polymorphic API is under development")
		static TDataValueReference CreateNewDerived(ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
			return TDataValueReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...)
			);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.8, "Polymorphic API is under development")
		static TDataValueReference CreateNewDerivedAs(const FName& InDerivedTypeName,
													 FMetasoundDataTypeId InDerivedTypeId,
													 ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
		#if DO_CHECK
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			check(Metasound::IsChildOfByName(InDerivedTypeName, GetMetasoundDataTypeName<DataType>()));
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		#endif
			
			return TDataValueReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...),
				InDerivedTypeName,
				InDerivedTypeId
			);
		}
		
		/** Enable copy constructor */
		TDataValueReference(const TDataValueReference<DataType>& Other) = default;

		/** Enable move constructor */
		TDataValueReference(TDataValueReference<DataType>&& Other) = default;

		/** Enable assignment operator. */
		TDataValueReference<DataType>& operator=(const TDataValueReference<DataType>& Other) = default;

		/** Enable move operator. */
		TDataValueReference<DataType>& operator=(TDataValueReference<DataType>&& Other) = default;

		/** Implicit conversion to a readable parameter. */
		operator TDataReadReference<DataType>() const
		{
			return TDataReadReference<DataType>(*this);
		}

		/** Const access to the underlying parameter object. */
		inline const DataType& operator*() const
		{
			return *TDataReference<DataType>::ObjectReference;
		}

		/** Const access to the underlying parameter object. */
		inline const DataType* operator->() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		inline const DataType* Get() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		/** Create a clone of this parameter reference. */
		virtual TUniquePtr<IDataReference> Clone() const override
		{
			typedef TDataValueReference<DataType> FValueDataReference;

			return MakeUnique< FValueDataReference >(*this);
		}
	};

	/** Cast a TDataReadReference to a TDataValueReference. 
	 *
	 * In general TDataReadReferences should not be converted into TDataValueReferences unless the caller
	 * can be certain that no other TDataWriteReference exists for the underlying parameter. Having a 
	 * TDataWriteReferences to an existing TDataValueReference can cause confusing behavior as values
	 * references are not expected to change value.
	 */
	template<typename T>
	TDataValueReference<T> ValueCast(const TDataReadReference<T>& InRef)
	{
		return TDataValueReference<T>(InRef);
	}

	/** Cast a TDataWriteReference to a TDataValueReference. 
	 *
	 * In general TDataWriteReferences should never beconverted into TDataValueReferences unless the caller
	 * can be certain that no other TDataWriteReference exists for the underlying parameter. Having a 
	 * TDataWriteReferences to an existing TDataValueReference can cause confusing behavior as values
	 * references are not expected to change value.
	 */
	template<typename T>
	TDataValueReference<T> ValueCast(const TDataWriteReference<T>& InRef)
	{
		return TDataValueReference<T>(InRef);
	}

	/** TDataWriteReference provides write access to a shared parameter reference. */
	template <typename DataType>
	class TDataWriteReference : public TDataReference<DataType>
	{
		// Construct operator with no arguments if the DataType has a default constructor.
		template <typename... ArgTypes>
		TDataWriteReference(EDataRefShouldConstruct InToken, ArgTypes&&... Args)
		: TDataReference<DataType>(InToken, Forward<ArgTypes>(Args)...)
		{
		}

		/** Create a writable ref from a blank parameter ref. Should be done with care and understanding
		 * of side-effects of converting a ref to a writable ref. 
		 */
		TDataWriteReference<DataType>(const TDataReference<DataType>& InDataReference)
		: TDataReference<DataType>(InDataReference)
		{
		}

		// Friend because it calls protected constructor
		template <typename T>
		friend TDataWriteReference<T> WriteCast(const TDataReadReference<T>& InReadableRef);

		using FRefType = TDataReference<DataType>::FRefType;
		TDataWriteReference(EDataRefShouldConstruct, FRefType InRef, FName InTypeName, const void* InTypeId)
			: TDataReference<DataType>(EDataRefShouldConstruct::NewObject, MoveTemp(InRef), InTypeName, InTypeId)
		{
		}

	public:
		/** This should be used to construct a new DataType object and return this TDataWriteReference as a wrapper around it. */
		template <typename... ArgTypes>
		static TDataWriteReference<DataType> CreateNew(ArgTypes&&... Args)
		{
			static_assert(std::is_constructible<DataType, ArgTypes...>::value, "TDataWriteReference::CreateNew underlying type is not constructible with provided arguments.");
			return TDataWriteReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		/** This should be used to construct a new DataType object and return this TDataWriteReference as a wrapper around it. */
		template <typename... ArgTypes>
		static TDataWriteReference<DataType> CreateNew(ArgTypes&&... Args) requires (CIsPolymorphicAndAbstract<DataType>)
		{
			static_assert(CHasFactoryFunction<DataType, ArgTypes...>);
			return TDataWriteReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.8, "Polymorphic API is under development")
		static TDataWriteReference CreateNewDerived(ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
			return TDataWriteReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...)
			);
		}

		template <typename Derived, typename... ArgTypes>
		static TDataWriteReference CreateNewDerivedAs(const FName& InDerivedTypeName,
													 FMetasoundDataTypeId InDerivedTypeId,
													 ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
		#if DO_CHECK
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			check(Metasound::IsChildOfByName(InDerivedTypeName, GetMetasoundDataTypeName<DataType>()));
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		#endif
			
			return TDataWriteReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...),
				InDerivedTypeName,
				InDerivedTypeId
			);
		}
		
		/** Enable copy constructor */
		TDataWriteReference(const TDataWriteReference<DataType>& Other) = default;

		/** Enable move constructor */
		TDataWriteReference(TDataWriteReference<DataType>&& Other) = default;

		/** Enable assignment operator. */
		TDataWriteReference<DataType>& operator=(const TDataWriteReference<DataType>& Other) = default;

		/** Enable move operator. */
		TDataWriteReference<DataType>& operator=(TDataWriteReference<DataType>&& Other) = default;

		/** Implicit conversion to a readable parameter. */
		operator TDataReadReference<DataType>() const
		{
			return TDataReadReference<DataType>(*this);
		}

		/** Non-const access to the underlying parameter object. */
		inline DataType& operator*() const
		{
			return *TDataReference<DataType>::ObjectReference;
		}

		/** Non-const access to the underlying parameter object. */
		inline DataType* operator->() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		/** Non-const access to the underlying parameter object. */
		inline DataType* Get() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		/** Create a clone of this parameter reference. */
		virtual TUniquePtr<IDataReference> Clone() const override
		{
			typedef TDataWriteReference<DataType> FDataWriteReference;

			return MakeUnique< FDataWriteReference >(*this);
		}

		// Provide access to ObjectReference when converting from Write to Read.
		friend class TDataReadReference<DataType>;

	};

	/** Cast a TDataReadReference to a TDataWriteReference.
	 *
	 * In general TDataReadReferences should not be converted into TDataWriteReferences unless the caller
	 * can be certain that no other TDataWriteReference exists for the underlying parameter. Having multiple
	 * TDataWriteReferences to the same parameter can cause confusion behavior as values are overwritten in
	 * an underterministic fashion.
	 */
	template <typename DataType>
	TDataWriteReference<DataType> WriteCast(const TDataReadReference<DataType>& InReadableRef)
	{
		const TDataReference<DataType>& Ref = static_cast<const TDataReference<DataType>&>(InReadableRef); 
		return TDataWriteReference<DataType>(Ref);
	}



	/** TDataReadReference provides read access to a shared parameter reference. */
	template <typename DataType>
	class TDataReadReference : public TDataReference<DataType>
	{
		// Construct operator with no arguments if the DataType has a default constructor.
		template <typename... ArgTypes>
		TDataReadReference(EDataRefShouldConstruct InToken, ArgTypes&&... Args)
		: TDataReference<DataType>(InToken, Forward<ArgTypes>(Args)...)
		{
		}

		using FRefType = typename TDataReference<DataType>::FRefType;
		TDataReadReference(EDataRefShouldConstruct, FRefType InRef, FName InTypeName, const void* InTypeId)
			: TDataReference<DataType>(EDataRefShouldConstruct::NewObject, MoveTemp(InRef), InTypeName, InTypeId)
		{
		}

	public:

		// This should be used to construct a new DataType object and return this TDataReadReference as a wrapper around it.
		template <typename... ArgTypes>
		static TDataReadReference<DataType> CreateNew(ArgTypes&&... Args)
		{
			static_assert(std::is_constructible<DataType, ArgTypes...>::value, "DataType constructor does not support provided types.");
			return TDataReadReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		template <typename... ArgTypes>
		static TDataReadReference<DataType> CreateNew(ArgTypes&&... Args) requires (CIsPolymorphicAndAbstract<DataType>)
		{
			static_assert(CHasFactoryFunction<DataType, ArgTypes...>);
			return TDataReadReference<DataType>(EDataRefShouldConstruct::NewObject, Forward<ArgTypes>(Args)...);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.8, "Polymorphic API is under development")
		static TDataReadReference CreateNewDerived(ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
			return TDataReadReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...)
			);
		}

		template <typename Derived, typename... ArgTypes>
		UE_EXPERIMENTAL(5.8, "Polymorphic API is under development")
		static TDataReadReference CreateNewDerivedAs(const FName& InDerivedTypeName, FMetasoundDataTypeId InDerivedTypeId, ArgTypes&&... Args)
		{
			static_assert(std::is_base_of_v<DataType, Derived>, "Derived must inherit from DataType");
		#if DO_CHECK
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			check(Metasound::IsChildOfByName(InDerivedTypeName, GetMetasoundDataTypeName<DataType>()));
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		#endif
			
			return TDataReadReference(
				EDataRefShouldConstruct::NewObject,
				MetasoundDataReferencePrivate::MakeRefOf<DataType, Derived>(Forward<ArgTypes>(Args)...),
				InDerivedTypeName,
				InDerivedTypeId
			);
		}
		
		TDataReadReference(const TDataReadReference& Other) = default;

		/** Construct a readable parameter ref from a writable parameter ref. */
		explicit TDataReadReference(const TDataWriteReference<DataType>& WritableRef)
		: TDataReference<DataType>(WritableRef)
		{
		}

		/** Construct a readable reference from a value reference. */
		explicit TDataReadReference(const TDataValueReference<DataType>& ValueRef)
		: TDataReference<DataType>(ValueRef)
		{
		}

		/** Assign a readable parameter ref from a writable parameter ref. */
		TDataReadReference<DataType>& operator=(const TDataWriteReference<DataType>& Other)
		{
			TDataReference<DataType>::ObjectReference = Other.ObjectReference;
			return *this;
		}

		/** Enable copy operator */
		TDataReadReference<DataType>& operator=(const TDataReadReference<DataType>& Other) = default;

		/** Enable move operator */
		TDataReadReference<DataType>& operator=(TDataReadReference<DataType>&& Other) = default;

		/** Const access to the underlying parameter object. */
		inline const DataType& operator*() const
		{
			return *TDataReference<DataType>::ObjectReference;
		}

		/** Const access to the underlying parameter object. */
		inline const DataType* operator->() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		/** Non-const access to the underlying parameter object. */
		inline const DataType* Get() const
		{
			return TDataReference<DataType>::ObjectReference.operator->();
		}

		/** Create a clone of this parameter reference. */
		virtual TUniquePtr<IDataReference> Clone() const override
		{
			typedef TDataReadReference<DataType> FDataReadReference;

			return MakeUnique<FDataReadReference>(*this);
		}
	};

	/** EDataReferenceAccessType describes the underlying data reference access
	 * type for a data reference contained in a FAnyDataReference. 
	 *
	 * This value can be used to determine which methods are supported for accessing 
	 * a data reference using GetDataReadReference<>() or GetDataWriteReference<>()
	 */
	enum class EDataReferenceAccessType : uint8
	{
		None = 0x00, 	//< The data is inaccessible, or the data reference does not exist.
		Read = 0x01, 	//< The data is accessible through a TDataReadReference.
		Write = 0x02,	//< The data is accessible through a TDataWriteReference.
		Value = 0x04 	//< The data is accessible by value.
	};



	/** Container for any data reference. 
	 *
	 * This container maintains the underlying containers access type (Read or Write)
	 * and data type. This allows for convenient storage by implementing a virtual 
	 * copy constructor and assignment operator.
	 */
	class FAnyDataReference : public IDataReference
	{
		/** TGetAs allows for template specialization of data retrieval and conversion
		 * rules for the various flavors of data referencing. The default implementation
		 * assumes we are attempting to retrieve the value stored in the data reference. 
		 */
		template<typename ResultType>
		struct TGetAs
		{
		private:
			using DataType = typename std::decay_t<std::remove_pointer_t<ResultType>>;

			// Only deref a pointer if the returned type is specifically requested to be
			// a non-pointer. 
			static ResultType DerefPointerIfNonPointerResultType(const DataType* InData)
			{
				if constexpr (std::is_pointer_v<ResultType>)
				{
					return InData;
				}
				else
				{
					check(InData != nullptr);
					return *InData;
				}
			};

		public:
		
			// Return a the underlying data by value or by const pointer depending
			// upon the template parameter ResultType
			static ResultType Get(EDataReferenceAccessType InAccessType, const IDataReference& InRef)
			{
				check(IsDataReferenceOfType<DataType>(InRef));

				const DataType* DataPtr = nullptr;
				switch (InAccessType)
				{
					case EDataReferenceAccessType::Read:
					{
						DataPtr = static_cast<const TDataReadReference<DataType>*>(&InRef)->Get();
						break;
					}
					case EDataReferenceAccessType::Write:
					{
						DataPtr = static_cast<const TDataWriteReference<DataType>*>(&InRef)->Get();
						break;
					}
					case EDataReferenceAccessType::Value:
					{
						DataPtr = static_cast<const TDataValueReference<DataType>*>(&InRef)->Get();
						break;
					}
					default:
					{
						checkNoEntry();
						DataPtr = static_cast<const DataType*>(InRef.GetRaw());
					}
				}

				return DerefPointerIfNonPointerResultType(DataPtr);
			}
		};

		/* Template specialization for retrieving data reference as a TDataReadReference<DataType> */
		template<typename DataType>
		struct TGetAs<TDataReadReference<DataType>>
		{
			static TDataReadReference<DataType> Get(EDataReferenceAccessType InAccessType, const IDataReference& InRef)
			{
				PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
				check(Metasound::IsCastable(InRef.GetDataTypeName(), GetMetasoundDataTypeName<DataType>()));
				PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
				switch (InAccessType)
				{
					case EDataReferenceAccessType::Read:
					{
						return *static_cast<const TDataReadReference<DataType>*>(&InRef);
					}
					case EDataReferenceAccessType::Write:
					{
						return *static_cast<const TDataWriteReference<DataType>*>(&InRef);
					}
					case EDataReferenceAccessType::Value:
					{
						return *static_cast<const TDataValueReference<DataType>*>(&InRef);
					}
					default:
					{
						checkNoEntry();
						return *static_cast<const TDataReadReference<DataType>*>(&InRef);
					}
				}
			}
		};

		/* Template specialization for retrieving data reference as a TDataValueReference<DataType> */
		template<typename DataType>
		struct TGetAs<TDataValueReference<DataType>>
		{
			static TDataValueReference<DataType> Get(EDataReferenceAccessType InAccessType, const IDataReference& InRef)
			{
				check(IsDataReferenceOfType<DataType>(InRef));
				checkf(EDataReferenceAccessType::Value== InAccessType, TEXT("Invalid attempt to convert a data ref with \"%s\" access to \"%s\" access"), *LexToString(InAccessType), *LexToString(EDataReferenceAccessType::Value));
				return *static_cast<const TDataValueReference<DataType>*>(&InRef);
			}
		};

		/* Template specialization for retrieving data reference as a TDataWriteReference<DataType> */
		template<typename DataType>
		struct TGetAs<TDataWriteReference<DataType>>
		{
			static TDataWriteReference<DataType> Get(EDataReferenceAccessType InAccessType, const IDataReference& InRef)
			{
				PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
				checkf(Metasound::IsCastable(InRef.GetDataTypeName(), GetMetasoundDataTypeName<DataType>()), TEXT("Attempt to get data reference with underlying type \"%s\" when actual underlying type is \"%s\"."), *GetMetasoundDataTypeString<DataType>(), *InRef.GetDataTypeName().ToString());
				PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
				checkf(EDataReferenceAccessType::Write == InAccessType, TEXT("Invalid attempt to convert a data ref with \"%s\" access to \"%s\" access"), *LexToString(InAccessType), *LexToString(EDataReferenceAccessType::Write));
				return *static_cast<const TDataWriteReference<DataType>*>(&InRef);
			}
		};


		// Private constructor. 
		FAnyDataReference(EDataReferenceAccessType InAccessType, const IDataReference& InDataRef)
		: AccessType(InAccessType)
		, DataRefPtr(InDataRef.Clone())
		{
			check(DataRefPtr.IsValid());
			check(EDataReferenceAccessType::None != InAccessType);
		}

	public:
		/** Construct with a TDataReadReference. */
		template<typename DataType>
		FAnyDataReference(const TDataReadReference<DataType>& InDataRef)
		: FAnyDataReference(EDataReferenceAccessType::Read, InDataRef)
		{
		}

		/** Construct with a TDataWriteReference. */
		template<typename DataType>
		FAnyDataReference(const TDataWriteReference<DataType>& InDataRef)
		: FAnyDataReference(EDataReferenceAccessType::Write, InDataRef)
		{
		}

		/** Construct with a TDataValueReference. */
		template<typename DataType>
		FAnyDataReference(const TDataValueReference<DataType>& InDataRef)
		: FAnyDataReference(EDataReferenceAccessType::Value, InDataRef)
		{
		}

		/** Copy construct with a FAnyDataReference. */
		FAnyDataReference(const FAnyDataReference& InOther)
		: FAnyDataReference(InOther.AccessType, *InOther.DataRefPtr)
		{
		}

		/** Assignment. */
		FAnyDataReference& operator=(const FAnyDataReference& InOther)
		{
			AccessType = InOther.AccessType;
			DataRefPtr = InOther.DataRefPtr->Clone();

			check(DataRefPtr.IsValid());
			check(EDataReferenceAccessType::None != AccessType);

			return *this;
		}

		/** Returns the access type of the underlying data reference. */
		EDataReferenceAccessType GetAccessType() const
		{
			return AccessType;
		}

		/** Returns the data type name of the underlying data reference. */
		virtual const FName& GetDataTypeName() const override
		{
			check(DataRefPtr.IsValid());
			return DataRefPtr->GetDataTypeName();
		}
		
		/** Returns the data type ID of the underlying data reference. */
		virtual const void* const GetDataTypeId() const override
		{
			check(DataRefPtr.IsValid());
			return DataRefPtr->GetDataTypeId();
		}

		/** Returns a clone of the underlying data reference. */
		virtual TUniquePtr<IDataReference> Clone() const override
		{
			check(DataRefPtr.IsValid());
			return DataRefPtr->Clone();
		}

		/** Return a raw pointer to the data. */
		void* GetRaw() const
		{
			check(DataRefPtr.IsValid());
			return DataRefPtr->GetRaw();
		}

		/** Return the data in the specified format. The template argument
		 * can be any of the support TData*Reference<> types, the underlying DataType
		 * or a const DataType* pointer. 
		 */
		template<typename ReturnType>
		ReturnType GetAs() const
		{
			if constexpr (std::is_same_v<FAnyDataReference, ReturnType>)
			{
				return *this;
			}
			else
			{
				check(DataRefPtr.IsValid());
				return TGetAs<ReturnType>::Get(AccessType, *DataRefPtr);
			}
		}

		/** Returns the current value of a reference. 
		 *
		 * This method's behavior is undefined and an assert will be called if
		 * the DataType differs from the underlying data reference's DataType. 
		 */
		template<typename DataType>
		const DataType* GetValue() const
		{
			if (DataRefPtr.IsValid())
			{
				return GetAs<const DataType*>();
			}
			return nullptr;
		}

		/** Return a non-const pointer to the data.
		 * 
		 * If the vertex is bound with this will return a valid pointer. 
		 * Otherwise it will return a nullptr.  This method will assert
		 * if the reference is bound, but not writable or if the data type
		 * does match.  
		 */
		template<typename DataType>
		DataType* GetWritableValue() const
		{
			if (DataRefPtr.IsValid())
			{
				check(IsDataReferenceOfType<DataType>(*DataRefPtr));
				checkf(EDataReferenceAccessType::Write == AccessType, TEXT("Writable values can only be obtained with data references which have writable access"));

				return static_cast<TDataWriteReference<DataType>*>(DataRefPtr.Get())->Get();
			}
			return nullptr;
		}

		/** Get access to a TDataValueReference. 
		 *
		 * This method will return a valid TDataValueReference of the templated data
		 * type. The returned object is only valid if:
		 *     1. The template parameter DataType matches that of the underlying data reference.
		 *     2. The underlying data reference is has Value access.
		 *
		 * If this method's behavior is undefined and will assert if it is called 
		 * with a mismatched data type or unsupported access type.
		 */
		template<typename DataType>
		TDataValueReference<DataType> GetDataValueReference() const
		{
			check(DataRefPtr.IsValid());
			return GetAs<TDataValueReference<DataType>>();
		}

		/** Get access to a TDataReadReference. 
		 *
		 * This method will return a valid TDataReadReference of the templated data
		 * type. The returned object is only valid if:
		 *     1. The template parameter DataType matches that of the underlying data reference.
		 *     2. The underlying data reference is has Read, Write or Value access.
		 *
		 * If this method's behavior is undefined and will assert if it is called 
		 * with a mismatched data type or unsupported access type.
		 */
		template<typename DataType>
		TDataReadReference<DataType> GetDataReadReference() const
		{
			return GetAs<TDataReadReference<DataType>>();
		}

		/** Get access to a TDataWriteReference. 
		 *
		 * This method will return a valid TDataWriteReference of the templated data
		 * type. The returned object is only valid if:
		 *     1. The template paramter DataType matches that of the underlying data reference.
		 *     2. The underlying data reference is has Write access.
		 *
		 * If this method's behavior is undefined and will assert if it is called 
		 * with a mismatched data type or unsupported access type.
		 */
		template<typename DataType>
		TDataWriteReference<DataType> GetDataWriteReference() const
		{
			return GetAs<TDataWriteReference<DataType>>();
		}

	private:

		EDataReferenceAccessType AccessType = EDataReferenceAccessType::None;
		TUniquePtr<IDataReference> DataRefPtr;
	};
}

#undef UE_API
