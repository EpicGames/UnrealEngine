// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	template<typename VertexType>
	class TVertexInterfaceGroupData;

	class FDataReferenceCollection
	{
		using FDataReferenceMap = TMap<FVertexName, FAnyDataReference>;

		public:

			/** Add a readable data reference to the collection.
			 *
			 * @param InName - Name of data reference.
			 * @InDataReference - Readable data reference. 
			 */
			template<typename DataType>
			void AddDataReadReference(const FVertexName& InName, const TDataReadReference<DataType>& InDataReference)
			{
				FAnyDataReference Wrapper(InDataReference);

				AddDataReference(InName, MoveTemp(Wrapper));
			}

			/** Add a readable data reference to the collection.
			 *
			 * @param InName - Name of data reference.
			 * @InDataReference - Writable data reference. 
			 */
			template<typename DataType>
			void AddDataReadReference(const FVertexName& InName, const TDataWriteReference<DataType>& InDataReference)
			{
				TDataReadReference<DataType> Readable(InDataReference);

				AddDataReadReference(InName, Readable);
			}

			/** Add a writable data reference to the collection.
			 *
			 * @param InName - Name of data reference.
			 * @InDataReference - Writable data reference. 
			 */
			template<typename DataType>
			void AddDataWriteReference(const FVertexName& InName, const TDataWriteReference<DataType>& InDataReference)
			{
				FAnyDataReference Wrapper(InDataReference);

				AddDataReference(InName, MoveTemp(Wrapper));
			}

			/** Add a data reference to this collection. */
			UE_API void AddDataReference(const FVertexName& InName, FAnyDataReference&& InDataReference);

			/** Add a readable data reference from another collection.
			 *
			 * @param InLocalName - Name to give data added to this collection.
			 * @param InOtherCollection - Collection which contains the desired data reference. 
			 * @param InOtherName - Name of desired data in other collection.
			 * @param InOtherType - TypeName of desired data.
			 *
			 * @return True if the data was successfully added. False otherwise. 
			 */
			UE_API bool AddDataReadReferenceFrom(const FVertexName& InLocalName, const FDataReferenceCollection& InOtherCollection, const FVertexName& InOtherName, const FName& InOtherTypeName);

			/** Add a writable data reference from another collection.
			 *
			 * @param InLocalName - Name to give data added to this collection.
			 * @param InOtherCollection - Collection which contains the desired data reference. 
			 * @param InOtherName - Name of desired data in other collection.
			 * @param InOtherType - TypeName of desired data.
			 *
			 * @return True if the data was successfully added. False otherwise. 
			 */
			UE_API bool AddDataWriteReferenceFrom(const FVertexName& InLocalName, const FDataReferenceCollection& InOtherCollection, const FVertexName& InOtherName, const FName& InOtherTypeName);

			/** Query whether a readable data reference is within the collection. 
			 *
			 * @param InName - Name of the data.
			 * @param InTypeName - TypeName of teh data.
			 *
			 * @return True if the data exists in the collection. False otherwise. 
			 */
			UE_API bool ContainsDataReadReference(const FVertexName& InName, const FName& InTypeName) const;

			/** Query whether a writable data reference is within the collection. 
			 *
			 * @param InName - Name of the data.
			 * @param InTypeName - TypeName of teh data.
			 *
			 * @return True if the data exists in the collection. False otherwise. 
			 */
			UE_API bool ContainsDataWriteReference(const FVertexName& InName, const FName& InTypeName) const;

			/** Returns pointer to data reference if it exists for a given vertex name.
			 *
			 * If no data reference exists for the given vertex name, a nullptr is returned. 
			 */
			UE_API const FAnyDataReference* FindDataReference(const FVertexName& InName) const;

			/** Query whether a readable data reference is within the collection. 
			 *
			 * @param InName - Name of the data.
			 *
			 * @return True if the data exists in the collection. False otherwise. 
			 */
			template<typename DataType>
			bool ContainsDataReadReference(const FVertexName& InName) const
			{
				if (const FAnyDataReference* DataRef = FindDataReference(InName))
				{
					if (IsDataReferenceOfType<DataType>(*DataRef))
					{
						const EDataReferenceAccessType AccessType = DataRef->GetAccessType();
						return (AccessType == EDataReferenceAccessType::Read) || (AccessType == EDataReferenceAccessType::Write) || (AccessType == EDataReferenceAccessType::Value);
					}
				}

				return false;
			}

			/** Query whether a writable data reference is within the collection. 
			 *
			 * @param InName - Name of the data.
			 *
			 * @return True if the data exists in the collection. False otherwise. 
			 */
			template<typename DataType>
			bool ContainsDataWriteReference(const FVertexName& InName) const
			{
				if (const FAnyDataReference* DataRef = FindDataReference(InName))
				{
					if (IsDataReferenceOfType<DataType>(*DataRef))
					{
						const EDataReferenceAccessType AccessType = DataRef->GetAccessType();
						return AccessType == EDataReferenceAccessType::Write;
					}
				}

				return false;
			}

			/** Returns a readable data ref from the collection.
			 *
			 * @param InName - Name of readable data.
			 *
			 * @return A readable data reference.
			 */
			template<typename DataType>
			TDataReadReference<DataType> GetDataReadReference(const FVertexName& InName) const
			{
				const FAnyDataReference* DataRefPtr = FindDataReference(InName);
				check(nullptr != DataRefPtr);
				return DataRefPtr->GetDataReadReference<DataType>();

			}

			/** Returns a readable data ref from the collection or construct one
			 * if one is not there.
			 *
			 * @param InName - Name of readable data.
			 * @param ConstructorArgs - Arguments to pass to constructor of TDataReadReference<DataType>
			 *
			 * @return A readable data reference.
			 */
			template<typename DataType, typename... ConstructorArgTypes>
			TDataReadReference<DataType> GetDataReadReferenceOrConstruct(FVertexName InName, ConstructorArgTypes&&... ConstructorArgs) const
			{
				typedef TDataReadReference<DataType> FDataRefType;

				if (ContainsDataReadReference<DataType>(InName))
				{
					return GetDataReadReference<DataType>(InName);
				}
				else
				{
					return FDataRefType::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
				}
			}

			/** Returns a readable data ref from the collection or construct one
			 * if one is not there with the default provided from the given input vertex collection.
			 *
			 * @param InputVertices - Collection of input vertices to retrieve default from if reference is not available
			 * @param InName - Name of readable data.
			 * @param InSettings - OperatorSettings of Metasound Operator 
			 *
			 * @return A readable data reference.
			 */
			template<typename DataType>
			TDataReadReference<DataType> GetDataReadReferenceOrConstructWithVertexDefault(const FInputVertexInterface& InputVertices, FVertexName InName, const FOperatorSettings& InSettings) const
			{
				using FDataFactory = TDataReadReferenceLiteralFactory<DataType>;

				if (ContainsDataReadReference<DataType>(InName))
				{
					return GetDataReadReference<DataType>(InName);
				}
				else
				{
					if (ensure(InputVertices.Contains(InName)))
					{
						return FDataFactory::CreateExplicitArgs(InSettings, InputVertices[InName].GetDefaultLiteral());
					}
					else
					{
						return FDataFactory::CreateExplicitArgs(InSettings, FLiteral::CreateInvalid());
					}
				}
			}

			/** Returns a writable data ref from the collection.
			 *
			 * @param InName - Name of readable data.
			 *
			 * @return A readable data reference.
			 */
			template<typename DataType>
			TDataWriteReference<DataType> GetDataWriteReference(const FVertexName& InName) const
			{
				const FAnyDataReference* DataRefPtr = FindDataReference(InName);
				check(nullptr != DataRefPtr);

				return DataRefPtr->GetDataWriteReference<DataType>();
			}

			/** Returns a writable data ref from the collection or construct
			 * one if one is not there.
			 *
			 * @param InName - Name of readable data.
			 * @param ConstructorArgs - Arguments to pass to constructor of TDataWriteReference<DataType>
			 *
			 * @return A readable data reference.
			 */
			template<typename DataType, typename... ConstructorArgTypes>
			TDataWriteReference<DataType> GetDataWriteReferenceOrConstruct(const FVertexName& InName, ConstructorArgTypes&&... ConstructorArgs) const
			{
				typedef TDataWriteReference<DataType> FDataRefType;

				if (ContainsDataWriteReference<DataType>(InName))
				{
					return GetDataWriteReference<DataType>(InName);
				}
				else
				{
					return FDataRefType::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
				}
			}


		private:

			FDataReferenceMap DataRefMap;
	};
}

#undef UE_API
