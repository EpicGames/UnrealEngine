// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundVariable.h"

#define DECLARE_METASOUND_DATA_REFERENCE_CORE_TYPE(DataType, ModuleApi) \
	template<> \
	struct ::Metasound::TDataReferenceTypeInfo<DataType> \
	{ \
		static ModuleApi const TCHAR* TypeName; \
		static ModuleApi const FText& GetTypeDisplayText(); \
		static ModuleApi ::FMetasoundDataTypeId GetTypeId(); \
		\
		private: \
		\
		static DataType* TypePtr; \
	};

#define DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(DataType, DataTypeInfoTypeName, DataReadReferenceTypeName, DataWriteReferenceTypeName) \
	typedef ::Metasound::TDataReferenceTypeInfo<DataType> DataTypeInfoTypeName; \
	\
	typedef ::Metasound::TDataReadReference<DataType> DataReadReferenceTypeName; \
	typedef ::Metasound::TDataWriteReference<DataType> DataWriteReferenceTypeName;

#define DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(DataType, ModuleApi) \
	DECLARE_METASOUND_DATA_REFERENCE_CORE_TYPE(DataType, ModuleApi) \
	DECLARE_METASOUND_DATA_REFERENCE_CORE_TYPE(::Metasound::TVariable<DataType>, ModuleApi) \
	DECLARE_METASOUND_DATA_REFERENCE_CORE_TYPE(TArray<DataType>, ModuleApi) \
	DECLARE_METASOUND_DATA_REFERENCE_CORE_TYPE(::Metasound::TVariable<TArray<DataType>>, ModuleApi)

/** Macro to make declaring a metasound parameter simple.  */
// Declares a metasound parameter type by
// - Adding typedefs for commonly used template types.
// - Defining parameter type traits.
#define DECLARE_METASOUND_DATA_REFERENCE_TYPES(DataType, ModuleApi, DataTypeInfoTypeName, DataReadReferenceTypeName, DataWriteReferenceTypeName) \
	DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(DataType, ModuleApi) \
	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(DataType, DataTypeInfoTypeName, DataReadReferenceTypeName, DataWriteReferenceTypeName)

#define DEFINE_METASOUND_DATA_REFERENCE_CORE_TYPE(DataType, DataTypeName, DataTypeLoctextKey) \
	DataType* ::Metasound::TDataReferenceTypeInfo<DataType>::TypePtr = nullptr; \
	const TCHAR* ::Metasound::TDataReferenceTypeInfo<DataType>::TypeName = TEXT(DataTypeName); \
	const FText& ::Metasound::TDataReferenceTypeInfo<DataType>::GetTypeDisplayText() \
	{ \
		static const FText DisplayText = NSLOCTEXT("MetaSoundCore_DataReference", DataTypeLoctextKey, DataTypeName); \
		return DisplayText; \
	} \
	void const * const ::Metasound::TDataReferenceTypeInfo<DataType>::GetTypeId() \
	{ \
		return static_cast<void const * const>(&::Metasound::TDataReferenceTypeInfo<DataType>::TypePtr); \
	}

#define METASOUND_DATA_TYPE_NAME_VARIABLE_TYPE_SPECIFIER ":Variable"
#define METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER ":Array"

// This only needs to be called if you don't plan on calling REGISTER_METASOUND_DATATYPE.
#define DEFINE_METASOUND_DATA_TYPE(DataType, DataTypeName) \
	DEFINE_METASOUND_DATA_REFERENCE_CORE_TYPE(DataType, DataTypeName, DataTypeName) \
	DEFINE_METASOUND_DATA_REFERENCE_CORE_TYPE(::Metasound::TVariable<DataType>, DataTypeName METASOUND_DATA_TYPE_NAME_VARIABLE_TYPE_SPECIFIER, DataTypeName"_Variable") \
	DEFINE_METASOUND_DATA_REFERENCE_CORE_TYPE(TArray<DataType>, DataTypeName METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER, DataTypeName"_Array") \
	DEFINE_METASOUND_DATA_REFERENCE_CORE_TYPE(::Metasound::TVariable<TArray<DataType>>, DataTypeName METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER METASOUND_DATA_TYPE_NAME_VARIABLE_TYPE_SPECIFIER, DataTypeName"_Array_Variable")

