// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

namespace Metasound
{
	template <typename T>
	struct TDataReferenceTypeInfo;
	
	/** Specialize void data type for internal use. */
	template<>
	struct TDataReferenceTypeInfo<void>
	{
		static METASOUNDGRAPHCORE_API const TCHAR* TypeName;
		static METASOUNDGRAPHCORE_API const void* const TypeId;
		static METASOUNDGRAPHCORE_API const FText& GetTypeDisplayText();
		static const void* GetTypeId() { return TypeId; }

	private:

		static const void* const TypePtr;
	};
		
	/** Return the data type FName for a registered data type. */
	template<typename DataType>
	const FName& GetMetasoundDataTypeName()
	{
		static const FName TypeName = FName(TDataReferenceTypeInfo<std::decay_t<DataType>>::TypeName);

		return TypeName;
	}

	/** Return the data type string for a registered data type. */
	template<typename DataType>
	const FString& GetMetasoundDataTypeString()
	{
		static const FString TypeString = FString(TDataReferenceTypeInfo<std::decay_t<DataType>>::TypeName);

		return TypeString;
	}

	/** Return the display text for a registered data type. */
	template<typename DataType>
	const FText& GetMetasoundDataTypeDisplayText()
	{
		return TDataReferenceTypeInfo<std::decay_t<DataType>>::GetTypeDisplayText();
	}

	/** Return the data type ID for a registered data type.
	 *
	 * This ID is runtime constant but may change between executions and builds.
	 */
	template<typename DataType>
	const void* const GetMetasoundDataTypeId()
	{
		return TDataReferenceTypeInfo<std::decay_t<DataType>>::GetTypeId();
	}
}
