// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SpscQueue.h"
#include "Containers/UnrealString.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundGraph.h"
#include "MetasoundOperatorBuilder.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundRenderCost.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDGRAPHCORE_API

#ifndef METASOUND_DEBUG_DYNAMIC_TRANSACTOR
#define METASOUND_DEBUG_DYNAMIC_TRANSACTOR !UE_BUILD_SHIPPING
#endif

namespace Metasound
{
	class FOperatorSettings;
	struct FLiteral;
	class FAnyDataReference;

	namespace DynamicGraph
	{
		// Forward declare
		class IDynamicOperatorTransform;
		class FDynamicOperator;
		enum class EAudioFadeType : uint8;
		struct FDynamicGraphOperatorData;
		class FDynamicOperatorTransactor;

#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
		namespace Debug
		{
			class FDynamicOperatorDebugger;
			/** This can be used when debugging Dynamic Operator internals. The FDynamicOperatorTransactor and FDynamicOperator have three
			 * graph representations. Maintaining three different graph representations can be error prone. This method can be used to
			 * validate the graphs are equivalent, or help diagnose where they differ. 
			 * 
			 * The three graph representations are:
			 * 1. An IGraph contains in the FDynamicOperatorTransactor
			 * 2. A FDynamicGraphIncrementalSorter in the FDynamicOperatorTransactor
			 * 3. A FDynamicGraphOperatorData in the FDynamicOperator.
			 * 
			 * This method returns true if all three graph representations are equivalent, and false otherwsie. Graph differences are logged to help
			 * diagnose the issue. 
			 */
			bool CompareAndLogGraphRepresentationDiscrepancies(const FDynamicOperatorTransactor& InTransactor, const FDynamicOperator& InDynamicOperator);
		}
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR

		using FOperatorID = uintptr_t;
		using FLiteralAssignmentFunction = void(*)(const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral, const FAnyDataReference& OutDataRef);
		using FReferenceCreationFunction = TOptional<FAnyDataReference>(*)(const FOperatorSettings& InSettings, FName DataType, const FLiteral& InLiteral, EDataReferenceAccessType InAccessType);
		using FOnInputVertexUpdated = TFunction<void(const FVertexName&, const FInputVertexInterfaceData&)>;
		using FOnOutputVertexUpdated = TFunction<void(const FVertexName&, const FOutputVertexInterfaceData&)>;

		/** A collection of callbacks for handling updates to MetaSound dynamic operators.
		 * 
		 * Callbacks are invoked on the same thread which executes the dynamic operator. 
		 */
		struct FDynamicOperatorUpdateCallbacks
		{
			FOnInputVertexUpdated OnInputAdded;
			FOnInputVertexUpdated OnInputRemoved;
			FOnOutputVertexUpdated OnOutputAdded;
			FOnOutputVertexUpdated OnOutputUpdated;
			FOnOutputVertexUpdated OnOutputRemoved;
		};

		constexpr int32 ORDINAL_NONE = TNumericLimits<int32>::Max();

		/** FOrdinalSwap represents a change in ordinal for an individual operator. */
		struct FOrdinalSwap
		{
			FOperatorID OperatorID = 0;
			int32 OriginalOrdinal = ORDINAL_NONE;
			int32 NewOrdinal = ORDINAL_NONE;

			friend bool operator==(const FOrdinalSwap& InOrdinalSwap, FOperatorID InOperatorID)
			{
				return InOrdinalSwap.OperatorID == InOperatorID;
			}

			static bool OriginalOrdinalLessThan(const FOrdinalSwap& InLHS, const FOrdinalSwap& InRHS)
			{
				return InLHS.OriginalOrdinal < InRHS.OriginalOrdinal;
			}
		};


		/** FDynamicGraphIncrementalSorter implements a dynamic topological sorting algorithm which offers several 
		  * optimizations over doing full sorts.
		  * 
		  * - It can detect if a sort is needed or not. If a sort is unneeded we can skip sorts.
		  * - Sorts are generally not done on the entire set of operators. The sort is generally
		  *   done on a subset of nodes related to the nodes being connected. 
		  */
		class FDynamicGraphIncrementalSorter
		{
		public:
		
			/** Where to insert a new operator. */
			enum class EInsertLocation : uint8
			{
				First,
				Last	
			};

			FDynamicGraphIncrementalSorter();

			FDynamicGraphIncrementalSorter(const FGraph& InGraph);

			/** Insert a node into the graph.
			 * @return Ordinal of added operator.
			 */
			int32 InsertOperator(FOperatorID InOperator, EInsertLocation InLocation);

			/** Remove a node from the graph.
			 * 
			 * @param OpeartorID - ID of operator to remove
			 * @return Ordinal of removed operator.
			 */
			int32 RemoveOperator(FOperatorID InOperatorID);

			/** Populate a TMap<> with an ordinal for every operator. */
			void GenerateOrdinals(TMap<FOperatorID, int32>& OutOrdinals) const;

			/** Add an edge to the graph, connecting two vertices from two 
			 * nodes. 
			 *
			 * @param InFromOperatorID - Operator which contains the output vertex.
			 * @param InToOperatorID - Operator which contains the input vertex.
			 * @param OutOrdinalUpdates - Array to populate with ordinal updates to maintain topological sort of graph.
			 */
			void AddDataEdge(FOperatorID InFromOperatorID, FOperatorID InToOperatorID, TArray<FOrdinalSwap>& OutOrdinalUpdates);

			/** Remove the given data edge. */
			void RemoveDataEdge(FOperatorID InFromOperatorID, FOperatorID InToOperatorID);

		private:

#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
			friend class Debug::FDynamicOperatorDebugger;
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR

			struct FIncrementalSortOperatorInfo
			{
				int32 Ordinal;

				// We track connections in order to determine whether a dependency
				// exists between two operators. Because two operators can have
				// multiple shared edges, FOperatorIDs may appear multiple times 
				// in these arrays. 
				TArray<FOperatorID> Inputs;
				TArray<FOperatorID> Outputs;
			};

			void IncrementalTopologicalSortForNewEdge(FOperatorID InFromOperatorID, int32 InFromOrdinal, FOperatorID InToOperatorID, int32 InToOdinal, TArray<FOrdinalSwap>& OutUpdates);

			void Init(const FGraph& InGraph);

			int32 MaxOrdinal = 0;
			int32 MinOrdinal = 0;
			TMap<FOperatorID, FIncrementalSortOperatorInfo> OperatorMap;
		};


		/** The FDynamicOperatorTransactor is used for communicating with a dynamic
		 * MetaSound operator.
		 *
		 * Graph manipulations performed on the transactor are forwarded to dynamic
		 * operators using the transform queue. Each modification is converted into
		 * IDynamicOperatorTransforms which are consumed by dynamic operators during
		 * their execution.
		 */
		class FDynamicOperatorTransactor
		{
		public:
			FDynamicOperatorTransactor& operator=(const FDynamicOperatorTransactor&) = delete;
			FDynamicOperatorTransactor(const FDynamicOperatorTransactor&) = delete;

			UE_API FDynamicOperatorTransactor();
			UE_API FDynamicOperatorTransactor(const FGraph& InGraph);

			/** Create a queue for communication with a dynamic operator. */
			UE_DEPRECATED(5.5, "Replace with CreateTransformQueue overload including FGraphRenderCost")
			UE_API TSharedRef<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> CreateTransformQueue(const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment);
			UE_API TSharedRef<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> CreateTransformQueue(const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment, const TSharedPtr<FGraphRenderCost>& InRenderCost);

			/** Add a node to the graph. */
			UE_API bool AddNode(const FGuid& InNodeID, TUniquePtr<INode> InNode);

			/** Remove a node from the graph. */
			UE_API bool RemoveNode(const FGuid& InNodeID);

			/** Add an edge to the graph, connecting two vertices from two 
			 * nodes. 
			 *
			 * @param FromNode - Node which contains the output vertex.
			 * @param FromVertex - Key of the vertex in the FromNode.
			 * @param ToNode - Node which contains the input vertex.
			 * @param ToVertex - Key of the vertex in the ToNode.
			 */
			UE_API void AddDataEdge(const FGuid& InFromNodeID, const FVertexName& InFromVertex, const FGuid& InToNodeID, const FVertexName& InToVertex);

			UE_DEPRECATED(5.6, "Literal nodes will no longer be supported. Use RemoveDataEdge with an FLiteral instead.")
			UE_API void RemoveDataEdge(const FGuid& InFromNode, const FVertexName& InFromVertexName, const FGuid& InToNode, const FVertexName& InToVertexName, TUniquePtr<INode> InReplacementLiteralNode);

			/** Remove the given data edge. */
			UE_API void RemoveDataEdge(const FGuid& InFromNode, const FVertexName& InFromVertexName, const FGuid& InToNode, const FVertexName& InToVertexName, FLiteral InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc);

			UE_DEPRECATED(5.6, "Setting values using a literal node will no longer be supported. Set literal on the vertex interface instead.")
			UE_API void SetValue(const FGuid& InNodeID, const FVertexName& InVertexName, TUniquePtr<INode> InLiteralNode);

			/** Set the value on a unconnected node input vertex. */
			UE_API void SetValue(const FGuid& InNodeID, const FVertexName& InVertexName, const FLiteral& InLiteral, const FReferenceCreationFunction InReferenceCreateFunc);

			/** Add an input data destination to describe how data provided 
			 * outside this graph should be routed internally.
			 *
			 * @param InNode - Node which receives the data.
			 * @param InVertexName - Key for input vertex on InNode.
			 */
			UE_API void AddInputDataDestination(const FGuid& InNode, const FVertexName& InVertexName, const FLiteral& InDefaultLiteral, FReferenceCreationFunction InFunc);

			/** Remove an exposed input from the graph. */
			UE_API void RemoveInputDataDestination(const FVertexName& InVertexName);

			/** Add an output data source which describes routing of data which is 
			 * owned this graph and exposed externally.
			 *
			 * @param InNode - Node which produces the data.
			 * @param InVertexName - Key for output vertex on InNode.
			 */
			UE_API void AddOutputDataSource(const FGuid& InNode, const FVertexName& InVertexName);

			/** Remove an exposed output from the graph. */
			UE_API void RemoveOutputDataSource(const FVertexName& InVertexName);

			/** Return internal version of graph. */
			UE_API const FGraph& GetGraph() const;


		private:


			UE_API void RemoveNodeInternal(const INode& InNode, bool bInRemoveDataEdgesWithNode);
			UE_API void FadeAndRemoveNodeInternal(const INode& InNode, TArrayView<const FVertexName> InOutputsToFade, bool bInRemoveDataEdgesWithNode);

			UE_API void EnqueueInsertOperatorTransform(const INode& InNode, int32 InOrdinal);
			UE_API void EnqueueRemoveOperatorTransform(const INode& InNode, const TArray<FOperatorID>& InOperatorsConnectedToInput);
			UE_API void EnqueueBeginFadeOperatorTransform(const INode& InNode, EAudioFadeType InFadeType, TArrayView<const FVertexName> InInputsToFade, TArrayView<const FVertexName> InOutputsToFade);
			UE_API void EnqueueEndFadeOperatorTransform(const INode& InNode);

			// Only used by deprecated 5.6 functions
			UE_API void EnqueueRemoveEdgeOperatorTransform_Deprecated(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const INode& InReplacementLiteralNode, int32 InLiteralOrdinal);
			UE_API void EnqueueFadeAndRemoveEdgeOperatorTransform_Deprecated(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const INode& InReplacementLiteralNode, int32 InLiteralOrdinal);

			UE_API void EnqueueRemoveEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const FLiteral& InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc);
			UE_API void EnqueueFadeAndRemoveEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertexName, const INode& InToNode, const FVertexName& InToVertexName, const FLiteral& InReplacementLiteral, const FReferenceCreationFunction InReferenceCreateFunc);

			UE_API void EnqueueAddEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertex, const INode& InToNode, const FVertexName& InToVertex, const TArray<FOrdinalSwap>& InOrdinalUpdates);
			UE_API void EnqueueFadeAndAddEdgeOperatorTransform(const INode& InFromNode, const FVertexName& InFromVertex, const INode& InToNode, const FVertexName& InToVertex, const TArray<FOrdinalSwap>& InOrdinalUpdates);

			UE_API void AddDataEdgeInternal(const INode& InFromNode, const FVertexName& InFromVertex, const FGuid& InToNodeID, const INode& InToNode, const FVertexName& InToVertex);
			struct FDynamicOperatorInfo
			{
				FOperatorSettings OperatorSettings;
				FMetasoundEnvironment Environment;
				TSharedPtr<FGraphRenderCost> GraphRenderCost;
				TWeakPtr<TSpscQueue<TUniquePtr<IDynamicOperatorTransform>>> Queue;
			};

			using FCreateTransformFunctionRef = TFunctionRef<TUniquePtr<IDynamicOperatorTransform>(const FDynamicOperatorInfo& InOperatorInfo)>;

			UE_API TUniquePtr<IDynamicOperatorTransform> CreateInsertOperatorTransform(const INode& InNode, int32 InOrdinal, const FOperatorSettings& InOperatorSettings, const FMetasoundEnvironment& InEnvironment, FGraphRenderCost* InGraphRenderCost) const;

			UE_API void EnqueueTransformOnOperatorQueues(FCreateTransformFunctionRef InFunc);

			FOperatorBuilder OperatorBuilder;
			FGraph Graph;
			FDynamicGraphIncrementalSorter GraphSorter;


			TArray<FDynamicOperatorInfo> OperatorInfos;

			struct FLiteralNodeID
			{
				FGuid ToNode;
				FVertexName ToVertex;
			};
			friend bool operator<(const FLiteralNodeID& InLHS, const FLiteralNodeID& InRHS);

			TSortedMap<FLiteralNodeID, TUniquePtr<INode>> LiteralNodeMap;

#if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
			friend class Debug::FDynamicOperatorDebugger;
#endif // if METASOUND_DEBUG_DYNAMIC_TRANSACTOR
		};
	}
}

#undef UE_API
