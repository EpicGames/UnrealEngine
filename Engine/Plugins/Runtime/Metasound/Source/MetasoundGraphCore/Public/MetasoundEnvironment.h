// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SortedMap.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "Misc/Build.h"

#include <type_traits>

#define UE_API METASOUNDGRAPHCORE_API

#ifndef WITH_METASOUND_DEBUG_ENVIRONMENT
#define WITH_METASOUND_DEBUG_ENVIRONMENT !UE_BUILD_SHIPPING
#endif

#define DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(DLL_API, VarType) \
	template<> \
	struct DLL_API ::Metasound::TEnvironmentVariableTypeInfo<VarType> \
	{ \
	private: \
		\
		static VarType* TypePtr; \
	public: \
		typedef VarType Type; \
		static FMetasoundEnvironmentVariableTypeId GetTypeId(); \
	};

#define DEFINE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(VarType) \
	VarType* ::Metasound::TEnvironmentVariableTypeInfo<VarType>::TypePtr = nullptr; \
	void const* const ::Metasound::TEnvironmentVariableTypeInfo<VarType>::GetTypeId() { return static_cast<FMetasoundEnvironmentVariableTypeId>(&::Metasound::TEnvironmentVariableTypeInfo<VarType>::TypePtr); }

	
using FMetasoundEnvironmentVariableTypeId = void const* const;

namespace Metasound
{
	namespace MetasoundEnvironmentPrivate
	{
		// Helper class to ensure that DECLARE_METASOUND_ENVIRONMENT_VARIABLE is used. 
		template<typename DataType>
		struct TSpecializationHelper 
		{
			enum { Value = false };
		};
	}


	template<typename VarType>
	//struct METASOUNDGRAPHCORE_API TEnvironmentVariableTypeInfo
	struct TEnvironmentVariableTypeInfo
	{
		typedef VarType Type;

		static const FMetasoundEnvironmentVariableTypeId TypeId;

		// Trigger an assert if this template has not been specialized. 
		static_assert(MetasoundEnvironmentPrivate::TSpecializationHelper<VarType>::Value, "TEnvironmentVariableTypeInfo must be specialized.  Use macro DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE");
	};


	/** Return the type ID for a environment variable type. */
	template<typename Type>
	FMetasoundEnvironmentVariableTypeId GetMetasoundEnvironmentVariableTypeId()
	{
		using FTypeInfo = TEnvironmentVariableTypeInfo< std::decay_t<Type> >;

		return FTypeInfo::GetTypeId();
	}

	/** Interface for a metasound environment variable which supports
	 * a name, a runtime type ID, and a clone method.
	 */
	class IMetasoundEnvironmentVariable
	{
	private:
		// Only allow TMetasoundEnvironmentVariable to implement this class
		// by making constructor private so that casting can be effectively checked. 
		template<typename Type>
		friend class TMetasoundEnvironmentVariable;

		IMetasoundEnvironmentVariable() = default;

	public:
		virtual ~IMetasoundEnvironmentVariable() = default;

		/** The name of this environment variable. */
		virtual const FName& GetName() const = 0;

		/** The type id of this environment variable. */
		virtual FMetasoundEnvironmentVariableTypeId GetTypeId() const = 0;

		/** Create a copy of this environment variable. */
		virtual TUniquePtr<IMetasoundEnvironmentVariable> Clone() const = 0;
	};

	template<typename Type>
	class TMetasoundEnvironmentVariable : public IMetasoundEnvironmentVariable
	{
		// Make sure that the `Type` is not a reference or CV qualified.
		static_assert(std::is_same<Type, typename std::decay<Type>::type>::value, "Type must not change when decayed");

		// Make sure the `Type` has appropriate construction and copy characteristics. 
		static_assert(std::is_constructible<Type>::value, "Environment variable types must be default constructible");
		static_assert(std::is_copy_constructible<Type>::value, "Environment variable types must be copy constructible");
		static_assert(std::is_copy_assignable<Type>::value, "Environment variable types must be copy assignable");

	public:
		/** Construct an environment variable 
		 *
		 * @param InName - Name of the environment variable.
		 * @param ...Args - Parameter pack to be forwarded to underlying types constructor.
		 */
		template<typename... ArgTypes>
		TMetasoundEnvironmentVariable(const FName& InName, ArgTypes&&... Args)
		:	Name(InName)
		,	Value(Forward<ArgTypes>(Args)...)
		{
		}

		/** Get the current value. */
		const Type& GetValue() const
		{
			return Value;
		}

		/** Set the current value. */
		void SetValue(const Type& InValue)
		{
			Value = InValue;
		}

		/** Get the name of this environment variable. */
		const FName& GetName() const override
		{
			return Name;
		}

		/** Get the type ID of this environment variable. */
		FMetasoundEnvironmentVariableTypeId GetTypeId() const override
		{
			return GetMetasoundEnvironmentVariableTypeId<Type>();
		}

		/** Create a clone of this environment variable. */
		TUniquePtr<IMetasoundEnvironmentVariable> Clone() const override
		{
			return MakeUnique<TMetasoundEnvironmentVariable<Type>>(Name, Value);
		}

	private:
		FName Name;
		Type Value;
	};


	/** Returns true if the environment variable contains the same type as the `VarType` */
	template<typename VarType>
	bool IsEnvironmentVariableOfType(const IMetasoundEnvironmentVariable& InVar)
	{
		return GetMetasoundEnvironmentVariableTypeId<VarType>() == InVar.GetTypeId();
	}

	/** Casts an environment variable to the derived type. */
	template<typename Type>
	TMetasoundEnvironmentVariable<Type>& CastMetasoundEnvironmentVariableChecked(IMetasoundEnvironmentVariable& InVar)
	{
		using FDerivedType = TMetasoundEnvironmentVariable<Type>;

		check(IsEnvironmentVariableOfType<Type>(InVar));
		return static_cast<FDerivedType&>(InVar);
	}

	/** Casts an environment variable to the derived type. */
	template<typename Type>
	const TMetasoundEnvironmentVariable<Type>& CastMetasoundEnvironmentVariableChecked(const IMetasoundEnvironmentVariable& InVar)
	{
		using FDerivedType = TMetasoundEnvironmentVariable<Type>;

		check(IsEnvironmentVariableOfType<Type>(InVar));
		return static_cast<const FDerivedType&>(InVar);
	}

	/** FMetasoundEnvironment contains a set of TMetasoundEnvironmentVariables requiring
	 * that each environment variable has a unique name.
	 */
	class FMetasoundEnvironment
	{
	public:
		FMetasoundEnvironment() = default;
		~FMetasoundEnvironment() = default;

		UE_API FMetasoundEnvironment(const FMetasoundEnvironment& InOther);

		UE_API FMetasoundEnvironment& operator=(const FMetasoundEnvironment& InOther);

		/** Returns true if the environment variable with the given name contains
		 * the data of the same type as `VarType`
		 *
		 * @param InVariableName - Name of environment variable.
		 * @tparam VarType - Type of the underlying data stored in the environment variable.
		 */
		template<typename VarType>
		bool IsType(const FName& InVariableName) const
		{
			return IsEnvironmentVariableOfType<VarType>(*Variables[InVariableName]);
		}

		/** Returns true if the environment contains a variable with the name `InVariableName`
		 * and the type `VarType`.
		 *
		 * @param InVariableName - Name of environment variable. 
		 * @tparam VarType - Type of the underlying data stored in the environment variable.
		 */
		template<typename VarType>
		bool Contains(const FName& InVariableName) const
		{
			if (Variables.Contains(InVariableName))
			{
				return IsType<VarType>(InVariableName);
			}
			return false;
		}

		/** Returns the environment variable data.
		 *
		 * @param InVariableName - Name of environment variable.
		 * @tparam VarType - Type of the underlying data stored in the environment variable.
		 *
		 * @return copy of the underlying data.
		 */
		template<typename VarType>
		VarType GetValue(const FName& InVariableName) const
		{
			if (ensure(Contains<VarType>(InVariableName)))
			{
				const TMetasoundEnvironmentVariable<VarType>& Var = CastMetasoundEnvironmentVariableChecked<VarType>(*Variables[InVariableName]);

				return Var.GetValue();
			}
			return VarType{};
		}

		/** Sets the environment variable data
		 *
		 * @param InVariableName - Name of environment variable.
		 * @tparam VarType - Type of the underlying data stored in the environment variable.
		 *
		 * @return const ref to the underlying data.
		 */
		template<typename VarType>
		void SetValue(const FName& InVariableName, const VarType& InValue)
		{
			Variables.Add(InVariableName, MakeUnique<TMetasoundEnvironmentVariable<VarType>>(InVariableName, InValue));
		}

		/** Sets the environment variable data from a given environment variable
		 * 
		 * @param InValue - Environment variable to set.
		 */
		void SetValue(TUniquePtr<IMetasoundEnvironmentVariable> InValue)
		{
			Variables.Add(InValue->GetName(), MoveTemp(InValue));
		}

		using FRangedForConstIteratorType = TSortedMap<FName, TUniquePtr<IMetasoundEnvironmentVariable>, FDefaultAllocator, FNameFastLess>::RangedForConstIteratorType;
		FRangedForConstIteratorType begin() const
		{
			return Variables.begin();
		}

		FRangedForConstIteratorType end() const
		{
			return Variables.end();
		}

	private:
		TSortedMap<FName, TUniquePtr<IMetasoundEnvironmentVariable>, FDefaultAllocator, FNameFastLess> Variables;
	};

	namespace CoreInterface
	{
		namespace Environment
		{
			// The InstanceID acts as an external ID for communicating and in and out of MetaSounds. Each MetaSound
			// has a unique InstanceID
			METASOUNDGRAPHCORE_API const extern FLazyName InstanceID;

			// An array representing the graph hierarchy.
			METASOUNDGRAPHCORE_API const extern FLazyName GraphHierarchy;
		}
	}
}

/** Declare basic set of variable types. */
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, void);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, bool);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, int8);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, uint8);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, int16);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, uint16);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, int32);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, uint32);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, int64);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, uint64);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, float);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, double);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, FString);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, FName);
DECLARE_METASOUND_ENVIRONMENT_VARIABLE_TYPE(METASOUNDGRAPHCORE_API, TArray<FGuid>);

#undef UE_API
