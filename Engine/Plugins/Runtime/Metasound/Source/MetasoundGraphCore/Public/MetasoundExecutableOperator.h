// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundOperatorInterface.h"

#include <type_traits>


namespace Metasound
{
	namespace MetasoundExecutableOperatorPrivate
	{
		// Helper template to determine whether a member function is declared
		// for a given template class.
		template <typename U>
		class TIsResetMethodDeclared 
		{
			private:
				template<typename T, T> 
				struct Helper;

				template<typename T>
				static uint8 Check(Helper<void(T::*)(const IOperator::FResetParams&), &T::Reset>*);

				template<typename T> static uint16 Check(...);

			public:
				static constexpr bool Value = sizeof(Check<U>(0)) == sizeof(uint8);
		};

		// Helper to determine whether an Reset(...) function exists. 
		//
		// Some nodes have an Execute() call without an Reset(...) call because
		// Reset(...) was introduced after several MetaSound releases. In order
		// allow operators to compile, we make the Intiailize(...) function optional
		// for subclasses of TExecutableOperator. This adapter determines whether
		// it Reset(...) exists or not.
		//
		// Note: This helper must be instantiated inside the functions and cannot
		// be part of the `TExecutableOperator<OperatorType>` class definition as
		// classes are not completely defined at the point in the compilation process
		// when static constexpr class members are evaluated. 
		template<typename OperatorType>
		struct TResetFunctionAdapter
		{
			// Returns an IOperator::FResetFunction if the OperatorType has a
			// Reset(...) class member function.
			static IOperator::FResetFunction GetResetFunction() 
			{
				if (TIsResetMethodDeclared<OperatorType>::Value)
				{
					return &TResetFunctionAdapter::ResetFunction;
				}
				else
				{
					return nullptr;
				}
			}

		private:
			static void ResetFunction(IOperator* InOperator, const IOperator::FResetParams& InParams)
			{
				if constexpr (TIsResetMethodDeclared<OperatorType>::Value)
				{
					OperatorType* DerivedOperator = static_cast<OperatorType*>(InOperator);

					check(nullptr != DerivedOperator);

					DerivedOperator->Reset(InParams);
				}
				else
				{
					checkNoEntry();
				}
			}
		};

		// Helper template to determine whether a member function is declared
		// for a given template class.
		template <typename U>
		class TIsPostExecuteMethodDeclared 
		{
			private:
				template<typename T, T> 
				struct Helper;

				template<typename T>
				static uint8 Check(Helper<void(T::*)(), &T::PostExecute>*);

				template<typename T> static uint16 Check(...);

			public:
				static constexpr bool Value = sizeof(Check<U>(0)) == sizeof(uint8);
		};

		// Helper to determine whether an PostExecute(...) function exists. 
		//
		// Note: This helper must be instantiated inside the functions and cannot
		// be part of the `TExecutableOperator<OperatorType>` class definition as
		// classes are not completely defined at the point in the compilation process
		// when static constexpr class members are evaluated. 
		template<typename OperatorType>
		struct TPostExecuteFunctionAdapter
		{
			// Returns an IOperator::FPostExecuteFunction if the OperatorType has a
			// PostExecute(...) class member function.
			static IOperator::FPostExecuteFunction GetPostExecuteFunction() 
			{
				if (TIsPostExecuteMethodDeclared<OperatorType>::Value)
				{
					return &TPostExecuteFunctionAdapter::PostExecuteFunction;
				}
				else
				{
					return nullptr;
				}
			}

		private:
			static void PostExecuteFunction(IOperator* InOperator)
			{
				if constexpr (TIsPostExecuteMethodDeclared<OperatorType>::Value)
				{
					OperatorType* DerivedOperator = static_cast<OperatorType*>(InOperator);

					check(nullptr != DerivedOperator);

					DerivedOperator->PostExecute();
				}
				else
				{
					checkNoEntry();
				}
			}
		};
	}

	// As a general rule, ExecutableDataTypes should be avoided whenever possible
	// as they incur an undesired cost and are generally not typically necessary.
	// This is primarily for the special case of trigger types, where state management
	// cannot be avoided (or rather an avoidable design has yet to be formulated).
	template<class DataType>
	struct TExecutableDataType
	{
		static constexpr bool bIsExecutable = false;

		static void Execute(const DataType& InData, const DataType& OutData)
		{
			// No-Op for base case as most DataTypes (ex POD) are not executable.
		}

		static void ExecuteInline(DataType& InData, bool bInUpdated)
		{
			// No-Op for base case as most DataTypes (ex POD) are not executable.
		}
	};

	template<class DataType>
	struct TPostExecutableDataType
	{
		static constexpr bool bIsPostExecutable = false;

		/*
		 * static void PostExecute(DataType& InOutData) { ... }
		 */
	};


	/** Convenience class for supporting the IOperator interface's GetExecuteFunction virtual member function.
	 *
	 * Derived classes should inherit from this template class as well as implement a void Execute() member
	 * function.
	 *
	 * 	class MyOperator : public TExecutableOperator<MyOperator>
	 * 	{
	 * 	  public:
	 * 	  ...
	 * 	  void Execute()
	 * 	  {
	 *		  ...
	 * 	  }
	 * 	};
	 */
	template<class DerivedOperatorType>
	class TExecutableOperator : public IOperator
	{
	public:

		virtual ~TExecutableOperator() = default;

		virtual FResetFunction GetResetFunction() override
		{
			return MetasoundExecutableOperatorPrivate::TResetFunctionAdapter<DerivedOperatorType>::GetResetFunction();
		}

		virtual FExecuteFunction GetExecuteFunction() override
		{
			return &TExecutableOperator<DerivedOperatorType>::ExecuteFunction;
		}

		virtual FPostExecuteFunction GetPostExecuteFunction() override
		{
			return MetasoundExecutableOperatorPrivate::TPostExecuteFunctionAdapter<DerivedOperatorType>::GetPostExecuteFunction();
		}

	private:

		static void ExecuteFunction(IOperator* InOperator)
		{
			DerivedOperatorType* DerivedOperator = static_cast<DerivedOperatorType*>(InOperator);

			check(nullptr != DerivedOperator);

			DerivedOperator->Execute();
		}
	};

	/** FNoOpOperator is for IOperators which do not perform any execution.
	 * Their only behavior is to perform operations on constructor or through
	 * the Bind(...) methods.
	 */
	class FNoOpOperator : public IOperator
	{
		public:
			virtual ~FNoOpOperator() = default;

			virtual FExecuteFunction GetExecuteFunction() override
			{
				return nullptr;
			}

			virtual FResetFunction GetResetFunction() override
			{
				return nullptr;
			}

			virtual FPostExecuteFunction GetPostExecuteFunction() override
			{
				return nullptr;
			}
	};

	/** FExecuter
	 *
	 * Wraps an IOperator and provides an Execute() member function.
	 */
	class FExecuter : public IOperator
	{
		public:
			using FOperatorPtr = TUniquePtr<IOperator>;

			FExecuter()
			{
			}

			FExecuter(FOperatorPtr InOperator)
			{
				SetOperator(MoveTemp(InOperator));
			}

			void SetOperator(FOperatorPtr InOperator)
			{
				Operator = MoveTemp(InOperator);

				ExecuteFunction = nullptr;
				PostExecuteFunction = nullptr;
				ResetFunction = nullptr;

				if (Operator.IsValid())
				{
					ExecuteFunction = Operator->GetExecuteFunction();
					PostExecuteFunction = Operator->GetPostExecuteFunction();
					ResetFunction = Operator->GetResetFunction();
				}
			}

			TUniquePtr<IOperator> ReleaseOperator()
			{
				ExecuteFunction = nullptr;
				PostExecuteFunction = nullptr;
				ResetFunction = nullptr;
				return MoveTemp(Operator);
			}

			void Execute()
			{
				if (ExecuteFunction)
				{
					ExecuteFunction(Operator.Get());
				}
			}

			void PostExecute()
			{
				if (PostExecuteFunction)
				{
					PostExecuteFunction(Operator.Get());
				}
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				if (ResetFunction)
				{
					ResetFunction(Operator.Get(), InParams);
				}
			}

			bool IsNoOp()
			{
				return (nullptr == ExecuteFunction) && (nullptr == ResetFunction) && (nullptr == PostExecuteFunction);
			}

			bool IsValid() const
			{
				return Operator.IsValid();
			}

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				if (Operator.IsValid())
				{
					Operator->BindInputs(InVertexData);
				}
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
			{
				if (Operator.IsValid())
				{
					Operator->BindOutputs(InVertexData);
				}
			}

			virtual IOperator::FExecuteFunction GetExecuteFunction() override
			{
				return ExecuteFunction;
			}

			virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override
			{
				return PostExecuteFunction;
			}

			virtual IOperator::FResetFunction GetResetFunction() override
			{
				return ResetFunction;
			}

		private:
			FOperatorPtr Operator;

			IOperator::FExecuteFunction ExecuteFunction = nullptr;
			IOperator::FPostExecuteFunction PostExecuteFunction = nullptr;
			IOperator::FResetFunction ResetFunction = nullptr;
	};
}
