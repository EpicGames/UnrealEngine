// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundBasicNode.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include <type_traits>

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	namespace MetasoundFacadePrivate
	{
		// Helper template to determine whether a static class function is declared
		// for a given template class.
		template <typename U>
		class TIsFactoryMethodDeclared
		{
			private:
				template<typename T, T> 
				struct Helper;

				// Check for "static TUniquePtr<IOperator> U::CreateOperator(const FBuildOperatorParams& Inparams, FBuildResults& OutErrors)"
				template<typename T>
				static uint8 Check(Helper<TUniquePtr<IOperator>(*)(const FBuildOperatorParams& InParams, FBuildResults& OutResults), &T::CreateOperator>*);

				template<typename T> static uint16 Check(...);

			public:

				// If the function exists, then "Value" is true. Otherwise "Value" is false.
				static constexpr bool Value = sizeof(Check<U>(0)) == sizeof(uint8_t);
		};

		// Helper template to determine whether a static class function is declared
		// for a given template class.
		template <typename U>
		class TIsNodeInfoDeclared
		{
			private:
				template<typename T, T> 
				struct Helper;

				// Check for "static const FNodeClassMetadata& U::GetNodeInfo()"
				template<typename T>
				static uint8 Check(Helper<const FNodeClassMetadata&(*)(), &T::GetNodeInfo>*);

				// Check for "static FNodeClassMetadata U::GetNodeInfo()"
				template<typename T>
				static uint8 Check(Helper<FNodeClassMetadata(*)(), &T::GetNodeInfo>*);

				template<typename T> static uint16 Check(...);

			public:

				// If the function exists, then "Value" is true. Otherwise "Value" is false.
				static constexpr bool Value = sizeof(Check<U>(0)) == sizeof(uint8_t);
		};
	}

	/** TFacadeOperatorClass encapsulates an operator type and checks that the
	 * required static functions exist to build the facade operator class.  It 
	 * is required to call the FNodeFacade template constructor.
	 */
	template<typename OperatorType>
	struct TFacadeOperatorClass
	{
		// Require that OperatorType is subclass of IOperator
		static_assert(std::is_base_of<IOperator, OperatorType>::value, "To use the FNodeFacade constructor, the OperatorType must be derived from IOperator");

		// Require static TUniquePtr<IOperator> OperatorType::CreateOperator(const FCreateOperatorParams&, TArray<TUniquePtr<IOperatorBuildError>>&) exists.
		static_assert(MetasoundFacadePrivate::TIsFactoryMethodDeclared<OperatorType>::Value, "To use the FNodeFacade constructor, the OperatorType must have the static function \"static TUniquePtr<IOperator> OperatorType::CreateOperator(const FBuildOperatorParams&, TArray<TUniquePtr<IOperatorBuildError>>&)\"");


		// Require static const FNodeClassMetadata& OperatorType::GetNodeInfo() exists.
		static_assert(MetasoundFacadePrivate::TIsNodeInfoDeclared<OperatorType>::Value, "To use the FNodeFacade constructor, the OperatorType must have the static function \"static const FNodeClassMetadata& OperatorType::GetNodeInfo()\"");

		typedef OperatorType Type;
	};

	/** FNodeFacade implements a significant amount of boilerplate code required
	 * to build a Metasound INode. FNodeFacade is particularly useful for an INode
	 * which has a static FVertexInterface, and always creates the same IOperator type.
	 * 
	 * The type of the concrete IOperator class to instantiate is defined in the
	 * FNodeFacade constructors TFacadeOperatorClass<>.
	 */
	class FNodeFacade : public FBasicNode
	{
		// Factory class for create an IOperator by using a TFunction matching
		// the signature of the CreateOperator function.
		class FFactory : public IOperatorFactory
		{
			using FCreateOperatorFunction = TUniquePtr<IOperator>(*)(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
			friend class FNodeFacade;

			public:
				UE_API FFactory(FCreateOperatorFunction InCreateFunc);

				UE_API virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;

			private:

				FCreateOperatorFunction CreateFunc;
		};

	public: 

		FNodeFacade() = delete;

		/** FNodeFacade constructor using the OperatorType template parameter
		 * to get the node info, operator factory method and vertex interface. 
		 *
		 * @param InInstanceName - Instance name for the node.
		 * @param InInstanceID - Instance ID for the node.
		 * @param OperatorClass - Template class wrapper for the underlying 
		 * 						  IOperator which is created by this node.
		 */
		template<typename OperatorType>
		FNodeFacade(const FVertexName& InInstanceName, const FGuid& InInstanceID, TFacadeOperatorClass<OperatorType> OperatorClass)
		: FNodeFacade(InInstanceName, InInstanceID, OperatorType::GetNodeInfo(), &OperatorType::CreateOperator)
		{
		}

		template<typename OperatorType>
		FNodeFacade(const FNodeInitData& InInitData, TFacadeOperatorClass<OperatorType> OperatorClass)
		: FNodeFacade(InInitData.InstanceName, InInitData.InstanceID, OperatorType::GetNodeInfo(), &OperatorType::CreateOperator)
		{
		}

		template<typename OperatorType>
		FNodeFacade(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata, TFacadeOperatorClass<OperatorType> OperatorClass)
		: FNodeFacade(InNodeData, InClassMetadata, &OperatorType::CreateOperator)
		{
		}

		virtual ~FNodeFacade() = default;

		/** Return a reference to the default operator factory. */
		UE_API virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override;

	private:
		FORCENOINLINE FNodeFacade(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FNodeClassMetadata& InNodeInfo, FFactory::FCreateOperatorFunction InCreate)
		: FBasicNode(FNodeData(InInstanceName, InInstanceID, InNodeInfo.DefaultInterface.CreateVertexInterface()), MakeShared<FNodeClassMetadata>(InNodeInfo))
		, Factory(MakeShared<FFactory, ESPMode::ThreadSafe>(InCreate))
		{
		}

		FORCENOINLINE FNodeFacade(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata, FFactory::FCreateOperatorFunction InCreate)
		: FBasicNode(InNodeData, InClassMetadata)
		, Factory(MakeShared<FFactory, ESPMode::ThreadSafe>(InCreate))
		{
		}

		TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> Factory;
	};


	/** TNodeFacade further reduces boilerplate code by allowing shorthand node implementations.
	 *  in the form: TNodeFacade<FMyOperator>
	 */
	template<typename OperatorType>
	class TNodeFacade : public FNodeFacade
	{
	public:
		TNodeFacade(const FName& InNodeName, const FGuid& InNodeID)
		: TNodeFacade(FNodeInitData{InNodeName, InNodeID})
		{
		}

		TNodeFacade(const FNodeInitData& InInitData)
			: FNodeFacade(InInitData, TFacadeOperatorClass<OperatorType>())
		{
		}

		TNodeFacade(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata)
			: FNodeFacade(InNodeData, InClassMetadata, TFacadeOperatorClass<OperatorType>())
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata()
		{
			return OperatorType::GetNodeInfo();
		}
	};
}

#undef UE_API
