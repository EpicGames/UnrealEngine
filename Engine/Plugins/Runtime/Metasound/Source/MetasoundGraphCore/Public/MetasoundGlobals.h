// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreGlobals.h"
#include "Misc/Build.h"

namespace Metasound
{
	// Returns true if a MetaSound is expected to execute, false if not.
	METASOUNDGRAPHCORE_API bool CanEverExecuteGraph(bool bIsCooking = false);
} // namespace Metasound
