// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"

#define UE_API METASOUNDGRAPHCORE_API


namespace Metasound
{
	/** FGraph contains the edges between nodes as well as input and output 
	 * vertices.  FGraph does not maintain ownership over any node. Nodes used
	 * within the graph must be valid for the lifetime of the graph. 
	 */
	class FGraph : public IGraph
	{
		public:
			UE_DEPRECATED(5.7, "Use constructor that provides an FName or AssetPath instead")
			UE_API FGraph(const FString& InInstanceName, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InNodeConfig = { });

			// Constructor to be used for dynamically generated MetaSounds not associated with a given asset (ex. test graphs, graphs
			// not represented by an asset-contained frontend document, etc.)
			UE_API FGraph(const FName& InInstanceName, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InNodeConfig = { });

			// Constructor to be used for MetaSounds associated with a given asset. Both transient and serialized assets should use this constructor,
			// as transient assets are contained in the transient package)
			UE_API FGraph(const FTopLevelAssetPath& InAssetPath, const FGuid& InInstanceID, TSharedPtr<const IOperatorData> InNodeConfig = { });

			virtual ~FGraph() = default;

			/** Returns path to graph's authoring content (invalid if generated dynamically). */
			UE_API virtual const FTopLevelAssetPath& GetAssetPath() const override;

			/** Return the name of this specific instance of the node class. */
			UE_API virtual const FVertexName& GetInstanceName() const override;

			/** Return the ID of this specific instance of the node class. */
			UE_API virtual const FGuid& GetInstanceID() const override;

			/** Return metadata about this graph. */
			UE_API virtual const FNodeClassMetadata& GetMetadata() const override;

			/** Retrieve all the edges associated with a graph. */
			UE_API virtual const TArray<FDataEdge>& GetDataEdges() const override;

			/** Return the current vertex interface. */
			UE_API virtual const FVertexInterface& GetVertexInterface() const override;

			/** Set the default value associated with an input vertex. */
			UE_API virtual void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral) override;

			/** Return the configuration for this graph. */
			UE_API virtual TSharedPtr<const IOperatorData> GetOperatorData() const override;

			/** Get vertices which contain input parameters. */
			UE_API virtual const FInputDataDestinationCollection& GetInputDataDestinations() const override;

			/** Get vertices which contain output parameters. */
			UE_API virtual const FOutputDataSourceCollection& GetOutputDataSources() const override;

			/** Add an edge to the graph. */
			UE_API void AddDataEdge(const FDataEdge& InEdge);

			/** Add an edge to the graph, connecting two vertices from two 
			 * nodes. 
			 *
			 * @param FromNode - Node which contains the output vertex.
			 * @param FromVertexKey - Key of the vertex in the FromNode.
			 * @param ToNode - Node which contains the input vertex.
			 * @param ToVertexKey - Key of the vertex in the ToNode.
			 *
			 * @return True if the edge was successfully added. False otherwise.
			 */
			UE_API bool AddDataEdge(const INode& FromNode, const FVertexName& FromVertexKey, const INode& ToNode, const FVertexName& ToVertexKey);

			/** Remove the given data edge. 
			 *
			 * @return True on success, false on failure.
			 */
			UE_API bool RemoveDataEdge(const INode& FromNode, const FVertexName& FromVertexKey, const INode& ToNode, const FVertexName& ToVertexKey);

			/** Removes all edges for which that predicate returns true.
			 *
			 * @param Predicate - A callable object which accepts an FDataEdge and returns true if
			 *                    the edge should be removed.
			 */
			template<typename PredicateType>
			UE_DEPRECATED(5.3, "Removing data edges by predicate is no longer supported")
			void RemoveDataEdgeByPredicate(const PredicateType& Predicate)
			{
				Edges.RemoveAllSwap(Predicate);
			}

			/** Removes all edges with connected to the node. */
			UE_API void RemoveDataEdgesWithNode(const INode& InNode);

			UE_DEPRECATED(5.6, "Use AddNode(...) which accepts a unique ptr")
			UE_API void AddNode(const FGuid& InNodeID, TSharedPtr<const INode> InNode);

			/** Store a node on this graph. 
			 *
			 * @param InNodeID - A unique ID associated with the node.
			 * @param InNode - A shared pointer to a node. 
			 */
			UE_API void AddNode(const FGuid& InNodeID, TUniquePtr<INode> InNode);

			/** Set the input default literal for a node that exists in the graph. 
			 *
			 * @param InNodeID - ID of not to update. 
			 * @param InVertexName - Name of input vertex on the node.
			 * @param InLiteral - Literal used to construct default input. 
			 */
			UE_API void SetNodeDefaultInput(const FGuid& InNodeID, const FVertexName& InVertexName, const FLiteral& InLiteral);

			/** Add an input node to this graph.
			 *
			 * @param InNodeID - A unique ID associated with the node.
			 * @param InVertexName - The key for the graph input vertex.
			 * @param InNode - A shared pointer to an input node. 
			 */
			UE_DEPRECATED(5.6, "Use AddInputNode(...) which accepts a unique ptr")
			UE_API void AddInputNode(FGuid InNodeID, const FVertexName& InVertexName, TSharedPtr<const INode> InNode);

			/** Add an input node to this graph.
			 *
			 * @param InNodeID - A unique ID associated with the node.
			 * @param InVertexName - The key for the graph input vertex.
			 * @param InNode - A unique pointer to an input node. 
			 */
			UE_API void AddInputNode(FGuid InNodeID, const FVertexName& InVertexName, TUniquePtr<INode> InNode);

			/** Add an output node to this graph.
			 *
			 * @param InNodeID - A unique ID associated with the node.
			 * @param InVertexName - The key for the graph output vertex.
			 * @param InNode - A shared pointer to an output node. 
			 */
			UE_DEPRECATED(5.6, "Use AddOutputNode(...) which accepts a unique ptr")
			UE_API void AddOutputNode(FGuid InNodeID, const FVertexName& InVertexName, TSharedPtr<const INode> InNode);

			/** Add an output node to this graph.
			 *
			 * @param InNodeID - A unique ID associated with the node.
			 * @param InVertexName - The key for the graph output vertex.
			 * @param InNode - A shared pointer to an output node. 
			 */
			UE_API void AddOutputNode(FGuid InNodeID, const FVertexName& InVertexName, TUniquePtr<INode> InNode);

			/** Retrieve node by node ID.
			 *
			 * @param InNodeID - The NodeID of the requested Node.
			 *
			 * @return Pointer to the Node if it is stored on this graph. nullptr otherwise. 
			 */
			UE_API const INode* FindNode(const FGuid& InNodeID) const;

			/** Populates an array with all nodes which exist in the graph but do not have
			 * any connections.
			 * 
			 * @param OutUnconnectedNodes - Array to populate.
			 * @return Number of unconnected nodes found.
			 */
			UE_API int32 FindUnconnectedNodes(TArray<TPair<FGuid, const INode*>>& OutUnconnectedNodes) const;

			/** Removes node from graph.
			 *
			 * @param InNodeID - ID of node to remove.
			 * @param bInRemoveDataEdgesWithNode - If true, will also call RemoveDataEdgesWithNode(...)
			 *
			 * @return true if node exists and is removed, false otherwise. 
			 */
			UE_API bool RemoveNode(const FGuid& InNodeID, bool bInRemoveDataEdgesWithNode = true);

			/** Add an input data destination to describe how data provided 
			 * outside this graph should be routed internally.
			 *
			 * @param InNode - Node which receives the data.
			 * @param InVertexName - Key for input vertex on InNode.
			 *
			 * @return True if the destination was successfully added. False 
			 * otherwise.
			 */
			UE_API bool AddInputDataDestination(const INode& InNode, const FVertexName& InVertexName);


			/** Add an input data destination to describe how data provided 
			 * outside this graph should be routed internally.
			 */
			UE_API void AddInputDataDestination(const FInputDataDestination& InDestination);

			/** Remove an input data destination by vertex name
			 * 
			 * @param InVertexName - Name of vertex to remove.
			 *
			 * @return True if destination existed and was successfully removed. False otherwise.
			 */
			UE_API bool RemoveInputDataDestination(const FVertexName& InVertexName);

			/** Add an output data source which describes routing of data which is 
			 * owned this graph and exposed externally.
			 *
			 * @param InNode - Node which produces the data.
			 * @param InVertexName - Key for output vertex on InNode.
			 *
			 * @return True if the source was successfully added. False 
			 * otherwise.
			 */
			UE_API bool AddOutputDataSource(const INode& InNode, const FVertexName& InVertexName);

			/** Add an output data source which describes routing of data which is 
			 * owned this graph and exposed externally.
			 */
			UE_API void AddOutputDataSource(const FOutputDataSource& InSource);

			/** Remove an output data source by vertex name
			 * 
			 * @param InVertexName - Name of vertex to remove.
			 *
			 * @return True if source existed and was successfully removed. False otherwise.
			 */
			UE_API bool RemoveOutputDataSource(const FVertexName& InVertexName);

			/** Return a reference to the default operator factory. */
			UE_API virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override;

		private:
			// Non-const access to nodes should not be exposed publicly so that
			// graph edges can be assured to be consistent. 
			UE_API INode* FindMutableNode(const FGuid& InNodeID);

			class FFactory : public IOperatorFactory
			{
			public:
				virtual ~FFactory() = default;

				virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;
			};

			FTopLevelAssetPath AssetPath;
			FVertexName InstanceName;
			FGuid InstanceID;

			FNodeClassMetadata Metadata;

			// Mutable vertex interface used for graph input/output mutations.
			// Initialized from Metadata.DefaultInterface
			FVertexInterface GraphInterface;

			TSharedPtr<const IOperatorData> OperatorData;

			TArray<FDataEdge> Edges;

			// This struct is used to handle back-compatibility with the deprecated
			// API. To support the old API, we still hold onto shared pointers of
			// nodes in the graph. Once the deprecated API is removed, we can remove
			// this struct and simply store unique ptrs. 
			struct FDeprecationNodeStorageAdapter
			{
				FDeprecationNodeStorageAdapter() = default;

				FDeprecationNodeStorageAdapter(TUniquePtr<INode> InNode)
				: Storage(InNode.Get())
				{
					MutableNode = InNode.Release(); // Node lifetime managed by storage in TSharedPtr<>
				}

				FDeprecationNodeStorageAdapter(TSharedPtr<const INode> InNode)
				: Storage(MoveTemp(InNode))
				{
				}

				const INode* GetConstNode() const;
				INode* GetMutableNode() const;

			private:
				INode* MutableNode = nullptr;
				TSharedPtr<const INode> Storage;
			};

			TSortedMap<FGuid, FDeprecationNodeStorageAdapter> Nodes;

			FInputDataDestinationCollection InputDestinations;
			FOutputDataSourceCollection OutputSources;
	};
}

#undef UE_API
