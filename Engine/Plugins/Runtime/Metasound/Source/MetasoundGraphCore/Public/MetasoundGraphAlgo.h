// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "MetasoundNodeInterface.h"
#include "Misc/CoreMiscDefines.h"
#include "Templates/PimplPtr.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	// Forward declare 
	class FDirectedGraphAlgoAdapter;
	class INode;
	class IGraph;

	struct UE_DEPRECATED(5.3, "Use the equivalent methods declared in `namespace DirectedGrpahAlgo {...}`") FDirectedGraphAlgo;

	namespace DirectedGraphAlgo
	{
		/** A strongly connected component containing Metasound INodes and 
		 * FDataEdges.
		 */
		struct FStronglyConnectedComponent
		{
			/** Nodes in the strongly connected component. */
			TArray<const INode*> Nodes;

			/** FDataEdges in the strongly connected component. */
			TArray<FDataEdge> Edges;
		};
	}

	using FStronglyConnectedComponent UE_DEPRECATED(5.3, "Use the equivalent type declared in `namespace DirectedGrpahAlgo {...}`") = DirectedGraphAlgo::FStronglyConnectedComponent;

	struct FDirectedGraphAlgo
	{
		/** The FDirectedGraphAlgoAdapter caches internal representations of the
		 * passed in IGraph. Using this can reduce redundant calcuations when 
		 * multiple algorithms are run on the same IGraph (without altering the IGraph).
		 *
		 * This function creates the adapter. 
		 */
		static UE_API TPimplPtr<FDirectedGraphAlgoAdapter> CreateDirectedGraphAlgoAdapter(const IGraph& InGraph);

		/** Sort the nodes of a directed acyclic graph using a depth-first 
		 * topological sorting algorithm.
		 *
		 * @param InGraph      - The graph containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		static UE_API bool DepthFirstTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using a depth-first 
		 * topological sorting algorithm.
		 *
		 * @param InAdapter    - The adapter containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		static UE_API bool DepthFirstTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using Kahn's topological
		 * sorting algorithm.
		 *
		 * @param InGraph      - The graph containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		static UE_API bool KahnTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using Kahn's topological
		 * sorting algorithm.
		 *
		 * @param InAdapter    - The adapter containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		static UE_API bool KahnTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder);

		/** Find strongly connected components given a Metasound IGraph.
		 *
		 * @param InGraph              - Graph to be analyzed. 
		 * @param OutComponents        - Strongly connected components found in
		 * 								 the graph are be added to this array.
		 * @param bExcludeSingleVertex - If true, single vertices are not be 
		 *                               returned as strongly connected components. 
		 *                               If false, single vertices may be returned
		 *                               as strongly connected components. 
		 *
		 * @return True if one or more strongly connected components are added
		 *         to OutComponents. False otherwise. 
		 */
		static UE_API bool TarjanStronglyConnectedComponents(const IGraph& InGraph, TArray<DirectedGraphAlgo::FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);

		/** Find strongly connected components given a Metasound IGraph.
		 *
		 * @param InAdapter            - The adapter containing edges and nodes to analyze.
		 * @param OutComponents        - Strongly connected components found in
		 * 								 the graph are be added to this array.
		 * @param bExcludeSingleVertex - If true, single vertices are not be 
		 *                               returned as strongly connected components. 
		 *                               If false, single vertices may be returned
		 *                               as strongly connected components. 
		 *
		 * @return True if one or more strongly connected components are added
		 *         to OutComponents. False otherwise. 
		 */
		static UE_API bool TarjanStronglyConnectedComponents(const FDirectedGraphAlgoAdapter& InAdapter, TArray<DirectedGraphAlgo::FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);

		/** Finds all nodes which can be reached by traversing the graph beginning
		 * from FInputDataDestination nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodesFromInput(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph beginning
		 * from FInputDataDestination nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodesFromInput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the transpose graph 
		 * beginning from FOutputDataSource nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodesFromOutput(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the transpose graph 
		 * beginning from FOutputDataSource nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodesFromOutput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph 
		 * beginning from FInputDataDestinations or by traversing the transpose
		 * graph beginning from FOutputDataSource nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodes(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph 
		 * beginning from FInputDataDestinations or by traversing the transpose
		 * graph beginning from FOutputDataSource nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		static UE_API void FindReachableNodes(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);
	};

	namespace DirectedGraphAlgo
	{
		/** The FDirectedGraphAlgoAdapter caches internal representations of the
		 * passed in IGraph. Using this can reduce redundant calcuations when 
		 * multiple algorithms are run on the same IGraph (without altering the IGraph).
		 *
		 * This function creates the adapter. 
		 */
		METASOUNDGRAPHCORE_API TPimplPtr<FDirectedGraphAlgoAdapter> CreateDirectedGraphAlgoAdapter(const IGraph& InGraph);

		/** Sort the nodes of a directed acyclic graph using a depth-first 
		 * topological sorting algorithm.
		 *
		 * @param InGraph      - The graph containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool DepthFirstTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using a depth-first 
		 * topological sorting algorithm.
		 *
		 * @param InAdapter    - The adapter containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool DepthFirstTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using Kahn's topological
		 * sorting algorithm.
		 *
		 * @param InGraph      - The graph containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool KahnTopologicalSort(const IGraph& InGraph, TArray<const INode*>& OutNodeOrder);

		/** Sort the nodes of a directed acyclic graph using Kahn's topological
		 * sorting algorithm.
		 *
		 * @param InAdapter    - The adapter containing edges and nodes to sort. 
		 * @param OutNodeOrder - An array of nodes from InGraph put in a valid 
		 *                       topological sorted order.
		 *
		 * @return True if sorting was successful, false otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool KahnTopologicalSort(const FDirectedGraphAlgoAdapter& InAdapter, TArray<const INode*>& OutNodeOrder);

		/** Find strongly connected components given a Metasound IGraph.
		 *
		 * @param InGraph              - Graph to be analyzed. 
		 * @param OutComponents        - Strongly connected components found in
		 * 								 the graph are be added to this array.
		 * @param bExcludeSingleVertex - If true, single vertices are not be 
		 *                               returned as strongly connected components. 
		 *                               If false, single vertices may be returned
		 *                               as strongly connected components. 
		 *
		 * @return True if one or more strongly connected components are added
		 *         to OutComponents. False otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool TarjanStronglyConnectedComponents(const IGraph& InGraph, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);

		/** Find strongly connected components given a Metasound IGraph.
		 *
		 * @param InAdapter            - The adapter containing edges and nodes to analyze.
		 * @param OutComponents        - Strongly connected components found in
		 * 								 the graph are be added to this array.
		 * @param bExcludeSingleVertex - If true, single vertices are not be 
		 *                               returned as strongly connected components. 
		 *                               If false, single vertices may be returned
		 *                               as strongly connected components. 
		 *
		 * @return True if one or more strongly connected components are added
		 *         to OutComponents. False otherwise. 
		 */
		METASOUNDGRAPHCORE_API bool TarjanStronglyConnectedComponents(const FDirectedGraphAlgoAdapter& InAdapter, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);

		/** Finds all nodes which can be reached by traversing the graph beginning
		 * from FInputDataDestination nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodesFromInput(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph beginning
		 * from FInputDataDestination nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodesFromInput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the transpose graph 
		 * beginning from FOutputDataSource nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodesFromOutput(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the transpose graph 
		 * beginning from FOutputDataSource nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodesFromOutput(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph 
		 * beginning from FInputDataDestinations or by traversing the transpose
		 * graph beginning from FOutputDataSource nodes. 
		 *
		 * @param InGraph  - The graph to be traversed.
		 * @param OutNodes - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodes(const IGraph& InGraph, TSet<const INode*>& OutNodes);

		/** Finds all nodes which can be reached by traversing the graph 
		 * beginning from FInputDataDestinations or by traversing the transpose
		 * graph beginning from FOutputDataSource nodes. 
		 *
		 * @param InAdapter - The adapter to be traversed.
		 * @param OutNodes  - The nodes which were reached during traversal.
		 */
		METASOUNDGRAPHCORE_API void FindReachableNodes(const FDirectedGraphAlgoAdapter& InAdapter, TSet<const INode*>& OutNodes);
	};
}

#undef UE_API
