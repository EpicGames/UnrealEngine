// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MetasoundNodeInterface.h"
#include "MetasoundBuilderInterface.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	// Forward declaration
	class FDirectedGraphAlgoAdapter;

	/** FGraphLinter
	 *
	 * Functions for detecting issues and errors in an IGraph.
	 */
	class FGraphLinter
	{
		public:
			using FBuildErrorPtr = TUniquePtr<IOperatorBuildError>;


			/** 
			 * Validate each FDataEdge in the IGraph by checking that the FInputDataSource 
			 * and FOutputDataDestination data types are equal. Errors will be generated 
			 * if unequal data types are detected.
			 *
			 * @param InGraph - Graph to analyze.
			 * @param OutErrors - Array to hold detected errors.
			 *
			 * @return True if no errors detected. False otherwise.
			 */
			static UE_API bool ValidateEdgeDataTypesMatch(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors);

			/** 
			 * Validate each FDataEdge in the IGraph by checking that the corresponding 
			 * INodes contain matching FDataVertex information as described by the 
			 * FDataEdge. Errors will be generated if inconsistencies are detected.
			 *
			 * @param InGraph - Graph to analyze.
			 * @param OutErrors - Array to hold detected errors.
			 *
			 * @return True if no errors detected. False otherwise.
			 */
			static UE_API bool ValidateVerticesExist(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors);

			/** 
			 * Validate the IGraph by checking for duplicate inputs connected to 
			 * an individual vertex on a given node. Errors will be generated if 
			 * duplicates are detected. 
			 *
			 * @param InGraph - Graph to analyze.
			 * @param OutErrors - Array to hold detected errors.
			 *
			 * @return True if no errors detected. False otherwise.
			 */
			static UE_API bool ValidateNoDuplicateInputs(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors);

			/** 
			 * Validate the IGraph by checking for cycles. Errors will be generated
			 * if a cycle is detected in the graph.
			 *
			 * @param InGraph - Graph to analyze.
			 * @param OutErrors - Array to hold detected errors.
			 *
			 * @return True if no errors detected. False otherwise.
			 */
			static UE_API bool ValidateNoCyclesInGraph(const IGraph& InGraph, TArray<FBuildErrorPtr>& OutErrors);

			/** 
			 * Validate the IGraph by checking for cycles. Errors will be generated
			 * if a cycle is detected in the graph.
			 *
			 * @param InAdapter - Graph adapter to analyze.
			 * @param OutErrors - Array to hold detected errors.
			 *
			 * @return True if no errors detected. False otherwise.
			 */
			static UE_API bool ValidateNoCyclesInGraph(const FDirectedGraphAlgoAdapter& InAdapter, TArray<FBuildErrorPtr>& OutErrors);
	};
}

#undef UE_API
