// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	// Forward declare
	namespace DirectedGraphAlgo
	{
		struct FStaticGraphOperatorData;
	}

	class FGraphOperator : public TExecutableOperator<FGraphOperator>
	{
		public:
			using FOperatorPtr = TUniquePtr<IOperator>;
			using FExecuteFunction = IOperator::FExecuteFunction;
			using FResetFunction = IOperator::FResetFunction;
			using FResetParams = IOperator::FResetParams;

			FGraphOperator() = default;
			UE_API FGraphOperator(TUniquePtr<DirectedGraphAlgo::FStaticGraphOperatorData> InOperatorData);

			virtual ~FGraphOperator() = default;

			// Add an operator to the end of the executation stack.
			UE_API void AppendOperator(FOperatorPtr InOperator);

			// Set the vertex interface data. This data will be copied to output 
			// during calls to Bind(InOutVertexData).
			UE_API void SetVertexInterfaceData(FVertexInterfaceData&& InVertexData);

			// Bind the graph's interface data references to FVertexInterfaceData.
			UE_API virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			UE_API virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			UE_API virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override;

			UE_API void Execute();
			UE_API void PostExecute();
			UE_API void Reset(const FResetParams& InParams);

		private:
			// Delete copy operator because underlying types cannot be copied. 
			FGraphOperator& operator=(const FGraphOperator&) = delete;
			FGraphOperator(const FGraphOperator&) = delete;

			static UE_API void StaticPostExecute(IOperator* Operator);

			struct FExecuteEntry
			{
				FExecuteEntry(IOperator& InOperator, FExecuteFunction InFunc);
				void Execute();

				IOperator* Operator;
				FExecuteFunction Function;	
			};

			struct FPostExecuteEntry
			{
				FPostExecuteEntry(IOperator& InOperator, FPostExecuteFunction InFunc);
				void PostExecute();

				IOperator* Operator;
				FPostExecuteFunction Function;	
			};

			struct FResetEntry
			{
				FResetEntry(IOperator& InOperator, FResetFunction InFunc);
				void Reset(const FResetParams& InParams);

				IOperator* Operator;
				FResetFunction Function;	
			};

			TArray<FExecuteEntry> ExecuteStack;
			TArray<FPostExecuteEntry> PostExecuteStack;
			TArray<FResetEntry> ResetStack;
			TArray<TUniquePtr<IOperator>> ActiveOperators;
			FVertexInterfaceData VertexData;
	};
}

#undef UE_API
