// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IAudioProxyInitializer.h"
#include "Misc/TVariant.h"
#include <type_traits>

#define UE_API METASOUNDGRAPHCORE_API

#define METASOUND_DEBUG_LITERALS 0

// Forward Declare
struct FGuid;
class FString;


namespace Metasound
{
	/** ELiteralType describes the format of the values held in the literal. */
	enum class ELiteralType : uint8
	{
		None, //< If the literal is None, TType(const FOperatorSettings&) or TType() will be invoked.
		Boolean, 
		Integer,
		Float,
		String,
		UObjectProxy,
		NoneArray, //< For NoneArray types,  TType(const FOperatorSettings&) or TType() will be invoked for each element in the array. 
		BooleanArray,
		IntegerArray,
		FloatArray,
		StringArray,
		UObjectProxyArray,
		Invalid,  //< The literal is in an invalid state and cannot be used to construct an object.

		COUNT
	};


	/**
	 * FLiteral represents a constant value in a Metasound graph and is primarily
	 * used to invoke the correct constructor of a Metasound data type. To be constructed
	 * using a FLiteral, the Metasound data type must support a constructor which 
	 * accepts one of the types supported by a FLiteral. The Metasound data type may 
	 * optionally accept a `const FOperatorSettings&` in addition.
	 *
	 * Example:
	 * // Somewhere before DECLARE_METASOUND_DATA_REFERENCE_TYPES(FAudioBuffer...), ParseFrom<FAudioBuffer>(int32) is defined
	 * // which means this data type can be created from an integer.
	 * Metasound::FAudioBuffer ::Metasound::ParseFrom<Metasound::FAudioBuffer>(int32 InNumFrames, const ::Metasound::FOperatorSettings&)
	 * {
	 *     return Metasound::FAudioBuffer(InNumFrames);
	 * }
	 *
	 * //...
	 * // In the frontend, we know that int and pass it to somewhere in MetasoundGraphCore...
	 * FLiteral InitParam(512);
	 *
	 * //...
	 * // In the backend, we can safely construct an FAudioBuffer.
	 * if(InitParam.IsCompatibleWithType<Metasound::FAudioBuffer>())
	 * {
	 *     Metasound::FAudioBuffer AudioBuffer = InitParam.ParseTo<Metasound::FAudioBuffer>();
	 *     //...
	 * }
	 */
	struct FLiteral
	{
	private:
		// Forward declare
		template<typename U>
		struct TIsSupportedLiteralType;
	public:
		struct FInvalid {};

		/* FNone is used in scenarios where an object is constructed with an FLiteral
		 * where the expected object constructor is the default constructor, or a
		 * constructor which accepts an FOperatorSettings.
		 */
		struct FNone {};

		using FVariantType = TVariant<
			FNone, bool, int32, float, FString, TSharedPtr<Audio::IProxyData>, // Single value types
			TArray<FNone>, TArray<bool>, TArray<int32>, TArray<float>, TArray<FString>, TArray<TSharedPtr<Audio::IProxyData>>, // Array of values types
			FInvalid
		>;


		/** Construct a literal param with a single argument. */
		template<
			typename ArgType,
			typename std::enable_if<TIsSupportedLiteralType<ArgType>::Value, int>::type = 0
			>
		FLiteral(ArgType&& Arg)
		{
			Value.Set<typename std::decay<ArgType>::type>(Forward<ArgType>(Arg));
#if METASOUND_DEBUG_LITERALS
			InitDebugString();
#endif
		}

		/** Construct a literal param with no arguments. */
		FLiteral()
		{
			Value.Set<FNone>(FNone());
#if METASOUND_DEBUG_LITERALS
			InitDebugString();
#endif
		}

		FVariantType Value;

		// builds an invalid FLiteral.
		static UE_API FLiteral CreateInvalid();
		static UE_API FLiteral GetDefaultForType(ELiteralType InType);

		UE_API bool IsValid() const;

		UE_API ELiteralType GetType() const;

		template<typename ArgType>
		void Set(ArgType&& Arg)
		{
			Value.Set<typename std::decay<ArgType>::type>(Forward<ArgType>(Arg));
#if METASOUND_DEBUG_LITERALS
			InitDebugString();
#endif
		}

	private:
#if METASOUND_DEBUG_LITERALS

		FString DebugString;

		UE_API void InitDebugString() const;

#endif // METAOUND_DEBUG_LITERALS
		
		// Helper function to determine if type "U" can be stored in this FLiteral.
		template<typename U, typename ... Ts>
		struct TIsSupportedLiteralTypeHelper
		{
			static constexpr bool Value = UE::Core::Private::TParameterPackTypeIndex<U, Ts...>::Value != (SIZE_T)-1;
		};

		// Helper function to determine if type "U" can be stored in this FLiteral.
		template<typename U, typename T, typename ... Ts>
		struct TIsSupportedLiteralTypeHelper<U, TVariant<T, Ts...>>
		{
			static constexpr bool Value = TIsSupportedLiteralTypeHelper<U, T, Ts...>::Value;
		};

		// Determine if type "U" can be stored in this FLiteral.
		template<typename U>
		struct TIsSupportedLiteralType
		{
			static constexpr bool Value = TIsSupportedLiteralTypeHelper<typename std::decay<U>::type, FVariantType>::Value;
		};
	};

	namespace MetasoundLiteralIntrinsics
	{
		// Default template for converting a template parameter to a literal argument type
		template<typename... ArgTypes>
		ELiteralType GetLiteralArgTypeFromDecayed()
		{
			return ELiteralType::Invalid;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<>()
		{
			checkNoEntry();
			return ELiteralType::None;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<bool>()
		{
			return ELiteralType::Boolean;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<int32>()
		{
			return ELiteralType::Integer;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<float>()
		{
			return ELiteralType::Float;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<FString>()
		{
			return ELiteralType::String;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TSharedPtr<Audio::IProxyData>>()
		{
			return ELiteralType::UObjectProxy;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<FLiteral::FNone>>()
		{
			return ELiteralType::NoneArray;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<bool>>()
		{
			return ELiteralType::BooleanArray;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<int32>>()
		{
			return ELiteralType::IntegerArray;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<float>>()
		{
			return ELiteralType::FloatArray;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<FString>>()
		{
			return ELiteralType::StringArray;
		}

		template<>
		inline ELiteralType GetLiteralArgTypeFromDecayed<TArray<TSharedPtr<Audio::IProxyData>>>()
		{
			return ELiteralType::UObjectProxyArray;
		}

		template<typename ... ArgTypes>
		struct TLiteralDefaultValueFromDecayed {};

		template<typename ArgType>
		struct TLiteralDefaultValueFromDecayed<ArgType>
		{
			static const ArgType& GetValue()
			{
				static const ArgType Value = static_cast<ArgType>(0);
				return Value;
			}
		};

		template<>
		struct TLiteralDefaultValueFromDecayed<FLiteral::FNone>
		{
			static FLiteral::FNone GetValue()
			{
				return FLiteral::FNone{};
			}
		};

		template<>
		struct TLiteralDefaultValueFromDecayed<FString>
		{
			static const FString& GetValue()
			{
				static const FString Value = TEXT("");
				return Value;
			}
		};

		template<typename ElementType>
		struct TLiteralDefaultValueFromDecayed<TArray<ElementType>>
		{
			static const TArray<ElementType>& GetValue()
			{
				static const TArray<ElementType> Value;
				return Value;
			}
		};

		template<>
		struct TLiteralDefaultValueFromDecayed<TSharedPtr<Audio::IProxyData>>
		{
			static TSharedPtr<Audio::IProxyData> GetValue()
			{
				return TSharedPtr<Audio::IProxyData>(nullptr);
			}
		};

		template<>
		struct TLiteralDefaultValueFromDecayed<TArray<TSharedPtr<Audio::IProxyData>>>
		{
			static TArray<TSharedPtr<Audio::IProxyData>> GetValue()
			{
				return TArray<TSharedPtr<Audio::IProxyData>>();
			}
		};
	}


	/** Provides literal type information for a given type. 
	 *
	 * @tparam ArgType - A C++ type.
	 */
	template<typename... ArgTypes>
	struct TLiteralTypeInfo{};

	template<typename ArgType>
	struct TLiteralTypeInfo<ArgType>
	{
		using FDecayedType = typename std::decay<ArgType>::type;
		/** Returns the associated ELiteralType for the C++ type provided in the TLiteralTypeInfo<Type> */
		static const ELiteralType GetLiteralArgTypeEnum()
		{
			// Use decayed version of template arg to remove references and cv qualifiers. 
			return MetasoundLiteralIntrinsics::GetLiteralArgTypeFromDecayed<FDecayedType>();
		}

		static auto GetDefaultValue()
		{
			return MetasoundLiteralIntrinsics::TLiteralDefaultValueFromDecayed<FDecayedType>::GetValue();
		}
	};

	/** Provides literal type information for a given type. 
	 *
	 * @tparam ArgType - A C++ type.
	 */
	template<>
	struct TLiteralTypeInfo<>
	{
		/** Returns the associated ELiteralType for the C++ type provided in the TLiteralTypeInfo<Type> */
		static const ELiteralType GetLiteralArgTypeEnum()
		{
			// Use decayed version of template arg to remove references and cv qualifiers. 
			return MetasoundLiteralIntrinsics::GetLiteralArgTypeFromDecayed<>();
		}

		static FLiteral::FNone GetDefaultValue()
		{
			return FLiteral::FNone{};
		}
	};
}

METASOUNDGRAPHCORE_API FString LexToString(const Metasound::FLiteral& InLiteral);

#undef UE_API
