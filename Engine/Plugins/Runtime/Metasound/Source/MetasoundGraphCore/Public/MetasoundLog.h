// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

METASOUNDGRAPHCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogMetaSound, Log, All);

