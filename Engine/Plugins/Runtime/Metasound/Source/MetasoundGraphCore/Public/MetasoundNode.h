// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDGRAPHCORE_API


namespace Metasound
{
	class IOperatorData;

	class FNode : public INode
	{
		public:
			UE_API FNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FNodeClassMetadata& InInfo, TSharedPtr<const IOperatorData> InOperatorData={});

			virtual ~FNode() = default;

			/** Return the name of this specific instance of the node class. */
			UE_API virtual const FVertexName& GetInstanceName() const override;

			/** Return the ID of this specific instance of the node class. */
			UE_API virtual const FGuid& GetInstanceID() const override;

			/** Return metadata associated with this node. */
			UE_API virtual const FNodeClassMetadata& GetMetadata() const override;

			UE_API virtual const FVertexInterface& GetVertexInterface() const override;

			UE_API virtual void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral) override;

			UE_API virtual TSharedPtr<const IOperatorData> GetOperatorData() const override;
		private:

			FVertexName InstanceName;
			FGuid InstanceID;
			FNodeClassMetadata Info;

			// Mutable vertex interface for SetDefaultInput. Initialized from
			// Info.DefaultInterface (implicit conversion from FClassInterface).
			FVertexInterface VertexInterface;

			TSharedPtr<const IOperatorData> OperatorData;
	};
}

#undef UE_API
