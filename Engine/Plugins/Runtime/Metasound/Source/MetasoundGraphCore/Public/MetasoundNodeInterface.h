// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertex.h"
#include "MetasoundLiteral.h"
#include "Misc/Guid.h"

#define UE_API METASOUNDGRAPHCORE_API

// This define helps in migrating code which has not yet implemented the purevirtual
// INode::SetDefaultInput(...) which was introduced in UE 5.6. In later releases
// the method will become pure virtual and this macro will no longer exist. 
#ifndef UE_METASOUND_PURE_VIRTUAL_SET_DEFAULT_INPUT
#define UE_METASOUND_PURE_VIRTUAL_SET_DEFAULT_INPUT 0
#endif

// This define helps in migrating code which has not yet implemented the purevirtual
// INode::GetOperatorData(...) which was introduced in UE 5.6. In later releases
// the method will become pure virtual and this macro will no longer exist. 
#ifndef UE_METASOUND_PURE_VIRTUAL_GET_OPERATOR_DATA
#define UE_METASOUND_PURE_VIRTUAL_GET_OPERATOR_DATA 0
#endif

namespace Metasound
{
	// Forward declare
	class IOperatorData;

	extern const FString METASOUNDGRAPHCORE_API PluginAuthor;
	extern const FText METASOUNDGRAPHCORE_API PluginNodeMissingPrompt;

	/** Data used to construct a Node. */
	struct FNodeData
	{
		UE_API FNodeData();
		UE_API FNodeData(const FNodeData&);
		UE_API FNodeData(const FName& InName, const FGuid& InID, FVertexInterface InInterface);
		UE_API FNodeData(const FName& InName, const FGuid& InID, FVertexInterface InInterface, TSharedPtr<const IOperatorData> InOperatorData);
		UE_API ~FNodeData();

		FName Name;
		FGuid ID;
		FVertexInterface Interface;
		TSharedPtr<const IOperatorData> OperatorData;
	};

	/**
	 * Node style data
	 */
	struct FNodeDisplayStyle
	{
		/** Icon name identifier associated with node. */
		FName ImageName;

		/** Whether or not to show name in visual layout. */
		bool bShowName = true;

		/** Whether or not to show input names in visual layout. */
		bool bShowInputNames = true;

		/** Whether or not to show output names in visual layout. */
		bool bShowOutputNames = true;

		/** Whether or not to show input literals in visual layout. */
		bool bShowLiterals = true;

		/** Name of the Slate style associated with node (color, icon). */
		FName StyleSet;

		/** Name of the node's icon defined in Style. */
		FName Icon;

		/** Name of the color defined in Style. */
		FName Color;
	};

	/** Name of a node class.
	 *
	 * FNodeClassName is used for lookup and declaring interoperability.
	 *
	 * Namespaces are provided as a convenience to avoid name collisions.
	 *
	 * Nodes with equal Namespace and Name, but different Variants are considered
	 * to be interoperable. They can be used to define nodes that perform the same
	 * function, but have differing vertex types.
	 */
	class FNodeClassName
	{
	public:
		UE_API FNodeClassName();

		UE_API FNodeClassName(const FName& InNamespace, const FName& InName, const FName& InVariant);

		/** Namespace of node class. */
		UE_API const FName& GetNamespace() const;

		/** Name of node class. */
		UE_API const FName& GetName() const;

		/** Variant of node class. */
		UE_API const FName& GetVariant() const;

		/** The full name of the Node formatted Namespace.Name and optionally appended with .Variant if valid */
		UE_API const FString ToString() const;

		/** Whether or not this instance of a node class name is a valid name.*/
		UE_API bool IsValid() const;

		static UE_API FName FormatFullName(const FName& InNamespace, const FName& InName, const FName& InVariant);
		static UE_API FName FormatScopedName(const FName& InNamespace, const FName& InName);

		static UE_API void FormatFullName(FNameBuilder& InBuilder, const FName& InNamespace, const FName& InName, const FName& InVariant);
		static UE_API void FormatScopedName(FNameBuilder& InBuilder, const FName& InNamespace, const FName& InName);

		/** Invalid form of node class name (i.e. empty namespace, name, and variant)*/
		static UE_API const FNodeClassName InvalidNodeClassName;

		friend inline bool operator==(const FNodeClassName& InLHS, const FNodeClassName& InRHS)
		{
			return (InLHS.Namespace == InRHS.Namespace) && (InLHS.Name == InRHS.Name) && (InLHS.Variant == InRHS.Variant);
		}

	private:

		FName Namespace;
		FName Name;
		FName Variant;
	};

	enum class ENodeClassAccessFlags : uint16
	{
		None = 0,

		/** Node class is deprecated and should not be used in new MetaSounds. */
		Deprecated = 1 << 0,

		/** Node class can be referenced by MetaSound graphs */
		Referenceable = 1 << 1,

		Default = Referenceable
	};

	/** Provides metadata for a given node. */
	struct FNodeClassMetadata
	{
		UE_API FNodeClassMetadata();
		UE_API FNodeClassMetadata(const FNodeClassMetadata& InOther);
		UE_API FNodeClassMetadata(
			FNodeClassName InClassName,
			int32 InMajorVersion,
			int32 InMinorVersion,
			FText InDisplayName = { },
			FText InDescription = { },
			FString InAuthor = { },
			FText InPromptIfMissing = { },
			FClassInterface InDefaultInterface = { },
			TArray<FText> InCategoryHierarchy = { },
			TArray<FText> InKeywords = { },
			FNodeDisplayStyle InDisplayStyle = { });

		UE_API ~FNodeClassMetadata();

		UE_API FNodeClassMetadata& operator=(const FNodeClassMetadata& InOther);
		UE_API FNodeClassMetadata& operator=(FNodeClassMetadata&& InOther);

		/** Returns an empty FNodeClassMetadata object. */
		static UE_API const FNodeClassMetadata& GetEmpty();

		/** Name of class. Used for registration and lookup. */
		FNodeClassName ClassName;

		/** Major version of node. Used for registration and lookup. */
		int32 MajorVersion;

		/** Minor version of node. */
		int32 MinorVersion;

		/** Display name of node class. */
		FText DisplayName;

		/** Human readable description of node. */
		FText Description;

		/** Author information. */
		FString Author;

		/** Human readable prompt for acquiring plugin in case node is not loaded. */
		FText PromptIfMissing;

		/** Default class interface for the node. */
		FClassInterface DefaultInterface;

		/** Hierarchy of categories for displaying node. */
		TArray<FText> CategoryHierarchy;

		/** List of keywords for contextual node searching. */
		TArray<FText> Keywords;

		/** Display style for node when visualized. */
		FNodeDisplayStyle DisplayStyle;

		ENodeClassAccessFlags AccessFlags;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.7, "Use AccessFlags instead")
		bool bDeprecated;
#endif // WITH_EDITORONLY_DATA
	};

	/** INodeBase
	 * 
	 * Interface for all nodes that can describe their name, type, inputs and outputs.
	 */
	class INodeBase
	{
		public:
			virtual ~INodeBase() = default;

			/** Return the name of this specific instance of the node class. */
			virtual const FName& GetInstanceName() const = 0;

			/** Return the ID of this node instance. */
			virtual const FGuid& GetInstanceID() const = 0;

			/** Return the type name of this node. */
			virtual const FNodeClassMetadata& GetMetadata() const = 0;

			/** Return the current vertex interface. */
			virtual const FVertexInterface& GetVertexInterface() const = 0;

			/** Set the default input literal for a vertex. This literal will be 
			 * used if nothing is connected to the node's input vertex. */
#if UE_METASOUND_PURE_VIRTUAL_SET_DEFAULT_INPUT
			virtual void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral) = 0;
#else 
			virtual METASOUNDGRAPHCORE_API void SetDefaultInput(const FVertexName& InVertexName, const FLiteral& InLiteral);
#endif

			/** Returns the configuration for the node. */
#if UE_METASOUND_PURE_VIRTUAL_GET_OPERATOR_DATA
			UE_EXPERIMENTAL(5.6, "Node operator data is still under development")
			virtual TSharedPtr<const IOperatorData> GetOperatorData() const = 0;
#else 
			UE_EXPERIMENTAL(5.6, "Node operator data is still under development")
			virtual METASOUNDGRAPHCORE_API TSharedPtr<const IOperatorData> GetOperatorData() const;
#endif
	};

	// Forward declare
	class IOperatorFactory;

	/** Shared ref type of operator factory. */
	typedef TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> FOperatorFactorySharedRef;

	/** Convenience function for making operator factory references */
	template<typename FactoryType, typename... ArgTypes>
	TSharedRef<FactoryType, ESPMode::ThreadSafe> MakeOperatorFactoryRef(ArgTypes&&... Args)
	{
		return MakeShared<FactoryType, ESPMode::ThreadSafe>(Forward<ArgTypes>(Args)...);
	}

	/** INode 
	 * 
	 * Interface for all nodes that can create operators. 
	 */
	class INode : public INodeBase
	{
		public:
			virtual ~INode() = default;

			/** Return a reference to the default operator factory. */
			virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const = 0;
	};

	/** FOutputDataSource describes the source of data which is produced within
	 * a graph and exposed external to the graph. 
	 */
	struct FOutputDataSource
	{
		/** Node containing the output data vertex. */
		const INode* Node = nullptr;

		/** Output data vertex. */
		FOutputDataVertex Vertex;

		FOutputDataSource()
		{
		}

		FOutputDataSource(const INode& InNode, const FOutputDataVertex& InVertex)
		:	Node(&InNode)
		,	Vertex(InVertex)
		{
		}


		/** Check if two FOutputDataSources are equal. */
		friend bool METASOUNDGRAPHCORE_API operator==(const FOutputDataSource& InLeft, const FOutputDataSource& InRight);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FOutputDataSource& InLeft, const FOutputDataSource& InRight);
		friend bool METASOUNDGRAPHCORE_API operator<(const FOutputDataSource& InLeft, const FOutputDataSource& InRight);
	};

	/** FInputDataSource describes the destination of data which produced 
	 * external to the graph and read internal to the graph.
	 */
	struct FInputDataDestination
	{
		/** Node containing the input data vertex. */
		const INode* Node = nullptr;

		/** Input data vertex of edge. */
		FInputDataVertex Vertex;

		FInputDataDestination()
		{
		}

		FInputDataDestination(const INode& InNode, const FInputDataVertex& InVertex)
		:	Node(&InNode)
		,	Vertex(InVertex)
		{
		}

		/** Check if two FInputDataDestinations are equal. */
		friend bool METASOUNDGRAPHCORE_API operator==(const FInputDataDestination& InLeft, const FInputDataDestination& InRight);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FInputDataDestination& InLeft, const FInputDataDestination& InRight);
		friend bool METASOUNDGRAPHCORE_API operator<(const FInputDataDestination& InLeft, const FInputDataDestination& InRight);
	};

	/** Key type for an FOutputDataSource or FInputDataDestination. */
	typedef TTuple<const INode*, FVertexName> FNodeDataVertexKey;

	/** FOutputDataSourceCollection contains multiple FOutputDataSources mapped 
	 * by FNodeDataVertexKeys.
	 */
	typedef TMap<FNodeDataVertexKey, FOutputDataSource> FOutputDataSourceCollection;

	/** FInputDataDestinationCollection contains multiple 
	 * FInputDataDestinations mapped by FNodeDataVertexKeys.
	 */
	typedef TMap<FNodeDataVertexKey, FInputDataDestination> FInputDataDestinationCollection;

	/** Make a FNodeDataVertexKey from an FOutputDataSource. */
	inline FNodeDataVertexKey MakeSourceDataVertexKey(const FOutputDataSource& InSource)
	{
		return FNodeDataVertexKey(InSource.Node, InSource.Vertex.VertexName);
	}

	inline FNodeDataVertexKey MakeDestinationDataVertexKey(const FInputDataDestination& InDestination)
	{
		return FNodeDataVertexKey(InDestination.Node, InDestination.Vertex.VertexName);
	}

	/** FDataEdge
	 *
	 * An edge describes a connection between two Nodes. 
	 */
	struct FDataEdge
	{
		FOutputDataSource From;

		FInputDataDestination To;

		FDataEdge()
		{
		}

		FDataEdge(const FOutputDataSource& InFrom, const FInputDataDestination& InTo)
		:	From(InFrom)
		,	To(InTo)
		{
		}

		/** Check if two FDataEdges are equal. */
		friend bool METASOUNDGRAPHCORE_API operator==(const FDataEdge& InLeft, const FDataEdge& InRight);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FDataEdge& InLeft, const FDataEdge& InRight);
		friend bool METASOUNDGRAPHCORE_API operator<(const FDataEdge& InLeft, const FDataEdge& InRight);
	};


	/** Interface for graph of nodes. */
	class IGraph : public INode
	{
		public:
			virtual ~IGraph() = default;

			/** Returns path to graph's authoring content (invalid if generated dynamically). */
			virtual const FTopLevelAssetPath& GetAssetPath() const = 0;

			/** Retrieve all the edges associated with a graph. */
			virtual const TArray<FDataEdge>& GetDataEdges() const = 0;

			/** Get vertices which contain input parameters. */
			virtual const FInputDataDestinationCollection& GetInputDataDestinations() const = 0;

			/** Get vertices which contain output parameters. */
			virtual const FOutputDataSourceCollection& GetOutputDataSources() const = 0;
	};

}

#undef UE_API
