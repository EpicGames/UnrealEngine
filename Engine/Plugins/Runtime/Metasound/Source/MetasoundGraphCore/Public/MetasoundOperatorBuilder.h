// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SpscQueue.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundEnvironment.h"
#include "MetasoundGraphOperator.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorBuilderSettings.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertexData.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	// Forward declare.
	class FDirectedGraphAlgoAdapter;
	class FOperatorSettings;
	class FMetasoundEnvironment;

	namespace OperatorBuilder
	{
		struct FBuildContext;
	}

	namespace DynamicGraph
	{
		class IDynamicOperatorTransform;
		struct FDynamicOperatorUpdateCallbacks;
		struct FDynamicGraphOperatorData;
		class IDynamicGraphInPlaceBuildable;
	}

	namespace DirectedGraphAlgo
	{
		struct FGraphOperatorData;
	}

	// Parameters for building a dynamic graph operator consist of the parameters
	// needed to build a standard graph operator as well as a transform queue and
	// set of callbacks in order to communicate and react to changes in the dynamic operator. 
	struct FBuildDynamicGraphOperatorParams : FBuildGraphOperatorParams
	{
		TSharedPtr<TSpscQueue<TUniquePtr<DynamicGraph::IDynamicOperatorTransform>>> TransformQueue; 
		const DynamicGraph::FDynamicOperatorUpdateCallbacks& OperatorUpdateCallbacks;
	};

	/** FOperatorBuilder builds an IOperator from an IGraph. */
	class FOperatorBuilder : public IOperatorBuilder
	{
		public:

			/** FOperatorBuilder constructor.
			 *
			 * @param InBuilderSettings  - Settings to configure builder options.
			 */
			UE_API FOperatorBuilder(const FOperatorBuilderSettings& InBuilderSettings);

			UE_API virtual ~FOperatorBuilder();

			/** Create an IOperator from an IGraph.
			 *
			 * @param InParams   - Params of the current build
			 * @param OutResults - Results data pertaining to the given build operator result.
			 *
			 * @return A TUniquePtr to an IOperator. If the processes was unsuccessful, 
			 *         the returned pointer will contain a nullptr and be invalid.
			 */
			UE_API virtual TUniquePtr<IOperator> BuildGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const override;

			/** Create an dynamic IOperator from an IGraph.
			 *
			 * @param InParams                  - Params of the current build
			 * @param OutResults                - Results data pertaining to the given build operator result.
			 *
			 * @return A TUniquePtr to an IOperator. If the processes was unsuccessful, 
			 *         the returned pointer will contain a nullptr and be invalid.
			 */
			UE_API TUniquePtr<IOperator> BuildDynamicGraphOperator(const FBuildDynamicGraphOperatorParams& InParams,  FBuildResults& OutResults);

		private:

			// Handles build status of current build operation.
			struct FBuildStatus
			{
				// Enumeration of build status states. 
				//
				// Note: plain enum used here instead of enum class so that implicit 
				// conversion to int32 can be utilized. It is assumed that the 
				// build status int32 values increase as the build status deteriorates.
				// Build statuses are merged by taking the maximum int32 value of
				// the EStatus. 
				enum EStatus
				{
					// No error has been encountered.
					NoError = 0,
					
					// A non fatal error has been encountered.
					NonFatalError = 1,

					// A fatal error has been encountered.
					FatalError = 2
				};


				FBuildStatus() = default;

				FBuildStatus(FBuildStatus::EStatus InStatus)
				:	Value(InStatus)
				{
				}

				// Merge build statuses by taking the maximum of EStatus.
				FBuildStatus& operator |= (FBuildStatus RHS)
				{
					Value = Value > RHS.Value ? Value : RHS.Value;
					return *this;
				}

				operator EStatus() const
				{
					return Value;
				}

			private:
				EStatus Value = NoError;
			};


			// Perform topological sort using depth first algorithm.
			FBuildStatus DepthFirstTopologicalSort(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& OutNodes) const;

			// Perform topological sort using kahns algorithm.
			FBuildStatus KahnsTopologicalSort(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& OutNodes) const;

			// Prune unreachable nodes from InOutNodes
			FBuildStatus PruneNodes(OperatorBuilder::FBuildContext& InOutContext, TArray<const INode*>& InOutNodes) const;

			// Creates FVertexInterfaceData from the IGraph.
			void InitializeVertexInterfaceData(const IGraph& InGraph, DirectedGraphAlgo::FGraphOperatorData& InOutGraphOperatorData) const;

			// Initialize Operator Info
			void InitializeOperatorInfo(const IGraph& InGraph ,TArray<const INode*>& InSortedNodes, DirectedGraphAlgo::FGraphOperatorData& InOutGraphOperatorData) const;

			// Get and route input data references for inputs provided externally to the graph.
			FBuildStatus GatherExternalInputDataReferences(OperatorBuilder::FBuildContext& InOutContext, const FInputVertexInterfaceData& InExternalInputData) const;

			// Graphs all internal graph references and places them in output map.
			void GatherInternalGraphDataReferences(OperatorBuilder::FBuildContext& InOutContext, const TArray<const INode*>& InNodes, TMap<FGuid, FDataReferenceCollection>& OutNodeVertexData) const;

			// Validates whether all operator outputs are bound to data references. 
			FBuildStatus ValidateOperatorOutputsAreBound(const INode& InNode, const FOutputVertexInterfaceData& InVertexData) const;

			// Get all input/output data references for a given graph.
			FBuildStatus GatherGraphDataReferences(OperatorBuilder::FBuildContext& InOutContext, FVertexInterfaceData& OutVertexData) const;

			// Call the operator factories for the nodes
			FBuildStatus CreateOperators(OperatorBuilder::FBuildContext& InOutContext, const TArray<const INode*>& InSortedNodes, const FInputVertexInterfaceData& InExternalInputData) const;

			// Creates a graph operator where the set of operators does not change, but the input data can be rebound.
			TUniquePtr<IOperator> BuildRebindableGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const;

			// Creates a graph operator where teh set of operators is constant and the input data cannot be rebound.
			TUniquePtr<IOperator> BuildStaticGraphOperator(const FBuildGraphOperatorParams& InParams, FBuildResults& OutResults) const;

			// Builds graph operator data structure. 
			bool BuildGraphOperatorData(const FBuildGraphOperatorParams& InParams, DirectedGraphAlgo::FGraphOperatorData& OutGraphOperatorData, TArray<const INode*>& OutNodeOrder, FBuildResults& OutResults) const;

			// Access function for retrieving FDynamicGraphOperatorData from an IDynamicGraphInPlaceBuildable.
			DynamicGraph::FDynamicGraphOperatorData& GetDynamicGraphOperatorData(DynamicGraph::IDynamicGraphInPlaceBuildable& InBuildable) const;
			
			FBuildStatus::EStatus GetMaxErrorLevel() const;

			FOperatorBuilderSettings BuilderSettings;
	};
}

#undef UE_API
