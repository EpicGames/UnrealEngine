// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#define UE_API METASOUNDGRAPHCORE_API


namespace Metasound
{
	/** EOperatorBuildNodePruning expresses the desired pruning behavior during
	 * the node pruning step.  
	 *
	 * Some nodes are unreachable in the graph either by traversing from the
	 * input nodes to from the output nodes. Because they are not dependent, 
	 * they have no impact on the produced output and can be pruned without 
	 * causing any change to the declared behavior of the graph.
	 */
	enum class EOperatorBuilderNodePruning : uint8
	{
		/** Do not prune any nodes. */
		None, 

		/** Prune nodes which cannot be reached from the output nodes. */
		PruneNodesWithoutOutputDependency, 

		/** Prune nodes which cannot be reached from the input nodes. */
		PruneNodesWithoutInputDependency, 

		/** Prune nodes which cannot be reached from the input nodes or output nodes. */
		PruneNodesWithoutExternalDependency, 
	};

	/** FOperatorBuilderSettings
	 *
	 * Settings for building IGraphs into IOperators.
	 */
	struct FOperatorBuilderSettings
	{
		/** Desired node pruning behavior. */
		EOperatorBuilderNodePruning PruningMode = EOperatorBuilderNodePruning::None;

		/** If true, the IGraph will be analyzed to detect cycles. Errors will be
		 * generated if a cycle is detected in the graph.
		 */
		bool bValidateNoCyclesInGraph = true;

		/** If true, the inputs to each node in the IGraph will be analyzed to
		 * detect duplicate inputs connected to an individual vertex on a given 
		 * node.  Errors will be generated if duplicates are detected. */
		bool bValidateNoDuplicateInputs = true;

		/** If true, each FDataEdge in the IGraph will be validated by checking
		 * that the corresponding INodes contain matching FDataVertex information
		 * as described by the FDataEdge. Errors will be generated if 
		 * inconsistencies are detected.
		 */
		bool bValidateVerticesExist = true;

		/** If true, each FDataEdge in the IGraph will be validated by checking
		 * that the FInputDataSource and FOutputDataDestination data types are
		 * equal. Errors will be generated if unequal data types are detected.
		 */
		bool bValidateEdgeDataTypesMatch = true;

		/** If true, each IOperator in the graph will be validated by checking
		 * that each output of the FVertexInterface is bound to data. Errors will
		 * be generated outputs are not bound.
		 */
		bool bValidateOperatorOutputsAreBound = true;

		/** If true, the builder will return an invalid IOperator if any errors
		 * are detected. If false, the builder will return an invalid IOperator
		 * only if fatal errors are detected.
		 */
		bool bFailOnAnyError = false;

		/** If true, enables tracking all internal data references (can be used
		  * for analyzing internal graph state by managing sound generator). */
		bool bPopulateInternalDataReferences = false;

		/** If true, the operator created by the builder will be able to update
		 * the underlying data of it's input and output data references through 
		 * the IOperator::BindInputs(...) and IOperator BindOutputs(...) methods.
		 * This requires that the created operator maintain additional data and
		 * results in a modest increase in memory usage.
		 */
		bool bEnableOperatorRebind = false;

		/** If true, the graph's operators will be wrapped in ProfilingOperator 
		* instances and insights events will be generated for their Execute
		* functions.
		*/
		bool bProfileOperators = false;

		/** Return the default settings for the current build environment. */
		static UE_API const FOperatorBuilderSettings& GetDefaultSettings();

		/** Return the default settings for a debug build environment. */
		static UE_API const FOperatorBuilderSettings& GetDefaultDebugSettings();

		/** Return the default settings for a development build environment. */
		static UE_API const FOperatorBuilderSettings& GetDefaultDevelopementSettings();

		/** Return the default settings for a test build environment. */
		static UE_API const FOperatorBuilderSettings& GetDefaultTestSettings();

		/** Return the default settings for a shipping build environment. */
		static UE_API const FOperatorBuilderSettings& GetDefaultShippingSettings();
	};
} // namespace Metasound

#undef UE_API
