// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include <type_traits>

namespace Metasound
{
	namespace OperatorDataPrivate
	{
		// Helper template for determining whether a static OperatorDataTypeName exists on the node configuration class declaration. 
		template<typename OperatorDataType> 
		struct THasOperatorDataTypeNameMember
		{
		private:
			template<typename T> 
			static std::negation<std::is_member_pointer<decltype(&T::OperatorDataTypeName)>> HasStaticMemberHelper(int32);

			template<typename T>
			static std::false_type HasStaticMemberHelper(...);

			// If there is a static member named T::OperatorDataTypeName, than THasStaticMember<T> will be of type std::true_type
			template<typename T>
			using THasStaticMember = decltype(HasStaticMemberHelper<T>(0));

			template<typename T>
			static std::is_convertible<decltype(T::OperatorDataTypeName), FName> IsConvertibleToFNameHelper(int32);

			template<typename T>
			static std::false_type IsConvertibleToFNameHelper(...);

			// If the member T::OperatorDataTypeName exists and is convertible to an FName, this will be of std::true_type
			template<typename T>
			using TIsConvertibleToFName = decltype(IsConvertibleToFNameHelper<T>(0));

		public:
			static constexpr bool Value = TIsConvertibleToFName<OperatorDataType>::value && THasStaticMember<OperatorDataType>::value;
		};
	}

	/** Base class for node configurations. */
	class IOperatorData
	{
	public:
		IOperatorData() = default;
		virtual ~IOperatorData() = default;

		/** Return a typename use for safe downcasting of the IOperatorData. */
		virtual FName GetOperatorDataTypeName() const = 0;
	};

	/** Node configuration CRTP. Node configurations should derived from this class
	 * to enable downcasting. 
	 *
	 * class FMyOperatorData : public TOperatorData<FMyOperatorData>
	 * {
	 * public:
	 *    static const FLazyName OperatorDataTypeName;
	 * };
	 */
	template<typename DerivedType>
	class TOperatorData : public IOperatorData
	{
	public:

		TOperatorData() = default;
		virtual ~TOperatorData() = default;

		virtual FName GetOperatorDataTypeName() const override final 
		{
			static_assert(OperatorDataPrivate::THasOperatorDataTypeNameMember<DerivedType>::Value, "Types deriving from TOperatorData must define a static class member which can describes the name of the data type");
			return DerivedType::OperatorDataTypeName;
		}
	};

	/** Returns the type name of the node configuration object. */
	template<typename OperatorDataType>
	FName GetStaticOperatorDataTypeName()
	{
		return OperatorDataType::OperatorDataTypeName;
	}

	/** Returns true if the provided node configuration is of the type T. */
	template<
		typename OperatorDataType, 
		typename U=std::enable_if_t<std::is_base_of_v<TOperatorData<std::decay_t<OperatorDataType>>, OperatorDataType>>
	>
	bool IsOperatorDataOfType(const IOperatorData& InNodeConfig)
	{
		return InNodeConfig.GetOperatorDataTypeName() == GetStaticOperatorDataTypeName<OperatorDataType>();
	}

	/** Returns a non-null pointer to a downcast node configuration object if the 
	 * provided node configuration is non-null and of the same derived type. */
	template< typename DesiredOperatorDataType, typename ProvidedOperatorDataType >
	std::enable_if_t<
		std::conjunction_v< 
			std::is_base_of<TOperatorData<std::decay_t<DesiredOperatorDataType>>, DesiredOperatorDataType>, 
			std::is_base_of<IOperatorData, ProvidedOperatorDataType>
		>, 
	DesiredOperatorDataType*> 
	CastOperatorData(ProvidedOperatorDataType* InOperatorData)
	{
		if (InOperatorData)
		{
			if (IsOperatorDataOfType<DesiredOperatorDataType>(*InOperatorData))
			{
				return static_cast<DesiredOperatorDataType*>(InOperatorData);
			}
		}
		return nullptr;
	}
}

