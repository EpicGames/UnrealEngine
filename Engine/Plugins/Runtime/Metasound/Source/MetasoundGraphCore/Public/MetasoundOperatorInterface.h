// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReferenceCollection.h"
#include "MetasoundRenderCost.h"
#include "MetasoundVertexData.h"

namespace Metasound
{
	/** IOperator
	 *
	 *  IOperator defines the interface for render time operations.  IOperators are created using an INodeOperatorFactory.
	 */
	class IOperator
	{
	public:
		/** FResetOperatorParams holds the parameters provided to an IOperator's 
		 * reset function.
		 */
		struct FResetParams
		{
			/** General operator settings for the graph. */
			const FOperatorSettings& OperatorSettings; 

			/** Environment settings available. */
			const FMetasoundEnvironment& Environment;

			/** Object for tracking runtime render cost of a MetaSound */
			const FGraphRenderCost* GraphRenderCost = nullptr;
		};

		virtual ~IOperator() = default;

		/** GetInputs() has been deprecated in favor of BindInputs(...). Please update
		 * your code by removing the implementation to GetInputs() and implementing
		 * BindInputs(...).  In future releases, the GetInputs() virtual method
		 * will be removed.
		 *
		 * Note: IOperators which do not correctly implement BindInputs(...) and
		 * BindOutputs(...) will not function correctly with MetaSound operator
		 * caching and live auditioning in the MetaSound Builder BP API. 
		 */
		UE_DEPRECATED(5.3, "GetInputs() has been replaced by BindInputs(FInputVertexInterfaceData&).")
		virtual FDataReferenceCollection METASOUNDGRAPHCORE_API GetInputs() const;

		/** GetOutputs() has been deprecated in favor of BindOutputs(...). Please update
		 * your code by removing the implementation to GetOutputs() and implementing
		 * BindOutputs(...).  In future releases, the GetOutputs() virtual method
		 * will be removed.
		 *
		 * Note: IOperators which do not correctly implement BindInputs(...) and
		 * BindOutputs(...) will not function correctly with MetaSound operator
		 * caching and live auditioning in the MetaSound Builder BP API. 
		 */
		UE_DEPRECATED(5.3, "GetOutputs() has been replaced by BindOutputs(FOutputVertexInterfaceData&)")
		virtual FDataReferenceCollection METASOUNDGRAPHCORE_API GetOutputs() const;

		/** Bind(...) has been deprecated in favor of BindInputs(...) and BindOutputs(...). 
		 * Please update your code by removing the implementation to Bind(...) 
		 * and implementing BindOutputs(...).  In future releases, the Bind(...) 
		 * virtual method will be removed.
		 *
		 * Note: IOperators which do not correctly implement BindInputs(...) and
		 * BindOutputs(...) will not function correctly with MetaSound operator
		 * caching and live auditioning in the MetaSound Builder BP API. 
		 */
		UE_DEPRECATED(5.3, "Bind(FVertexInterfaceData&) has been replaced by BindInputs(FInputVertexInterfaceDAta&) and BindOutputs(FOutputVertexInterfaceData&)")
		virtual void METASOUNDGRAPHCORE_API Bind(FVertexInterfaceData& InVertexData) const;

		/** BindInputs binds data references in the IOperator with the FInputVertexInterfaceData.
		 *
		 * All input data references should be bound to the InVertexData to support
		 * other MetaSound systems such as MetaSound BP API, Operator Caching, 
		 * and live auditioning. 
		 *
		 * Note: The virtual function IOPerator::BindInputs(...) will be made a
		 * pure virtual when IOperator::GetInputs() is removed at or after release 5.5
		 *
		 * Note: Binding an data reference may update the which underlying object 
		 * the reference points to. Any operator which caches pointers or values 
		 * from data references must update their cached pointers in the call to 
		 * BindInputs(...).  Operators which do not cache the underlying pointer 
		 * of a data reference do not need to update anything after BindInputs(...)
		 *
		 * Example:
		 * 	FMyOperator::FMyOperator(TDataReadReference<float> InGain, TDataReadReference<FAudioBuffer> InAudioBuffer)
		 * 	: Gain(InGain)
		 * 	, AudioBuffer(InAudioBuffer)
		 * 	{
		 * 		MyBufferPtr = AudioBuffer->GetData();		
		 * 	}
		 *
		 * 	void FMyOperator::BindInputs(FInputVertexInterfaceData& InVertexData)
		 * 	{
		 * 		InVertexData.BindReadVertex("Gain", Gain);
		 * 		InVertexData.BindReadVertex("Audio", AudioBuffer);
		 *
		 * 		// Update MyBufferPtr in case AudioBuffer references a new FAudioBuffer.
		 * 		MyBufferPtr = AudioBuffer->GetData();
		 * 	}
		 */
		virtual void METASOUNDGRAPHCORE_API BindInputs(FInputVertexInterfaceData& InVertexData);

		/** BindOutputs binds data references in the IOperator with the FOutputVertexInterfaceData.
		 *
		 * All output data references should be bound to the InVertexData to support
		 * other MetaSound systems such as MetaSound BP API, Operator Caching, 
		 * and live auditioning. 
		 *
		 * Note: The virtual function IOPerator::BindOutputs(...) will be made a
		 * pure virtual when IOperator::GetOutputs() is removed at or after release 5.5
		 *
		 * Note: Binding an data reference may update the which underlying object 
		 * the reference points to. Any operator which caches pointers or values 
		 * from data references must update their cached pointers in the call to 
		 * BindOutputs(...).  Operators which do not cache the underlying pointer 
		 * of a data reference do not need to update anything after BindOutputs(...)
		 *
		 * Example:
		 * 	FMyOperator::FMyOperator(TDataWriteReference<float> InGain, TDataWriteReference<FAudioBuffer> InAudioBuffer)
		 * 	: Gain(InGain)
		 * 	, AudioBuffer(InAudioBuffer)
		 * 	{
		 * 		MyBufferPtr = AudioBuffer->GetData();		
		 * 	}
		 *
		 * 	void FMyOperator::BindOutputs(FOutputVertexInterfaceData& InVertexData)
		 * 	{
		 * 		InVertexData.BindReadVertex("Gain", Gain);
		 * 		InVertexData.BindReadVertex("Audio", AudioBuffer);
		 *
		 * 		// Update MyBufferPtr in case AudioBuffer references a new FAudioBuffer.
		 * 		MyBufferPtr = AudioBuffer->GetData();
		 * 	}
		 */
		virtual void METASOUNDGRAPHCORE_API BindOutputs(FOutputVertexInterfaceData& InVertexData);

		/** Pointer to initialize function for an operator.
		 *
		 * @param IOperator* - The operator associated with the function pointer.
		 */
		using FResetFunction = void(*)(IOperator*, const FResetParams& InParams);

		/** Return the reset function to call during graph execution.
		 *
		 * The IOperator* argument to the FExecutionFunction will be the same IOperator instance
		 * which returned the execution function.
		 *
		 * nullptr return values are valid and signal an IOperator which does not need to be
		 * reset.
		 */
		virtual FResetFunction GetResetFunction() = 0;

		/** Pointer to execute function for an operator.
		 *
		 * @param IOperator* - The operator associated with the function pointer.
		 */
		using FExecuteFunction = void(*)(IOperator*);

		/** Return the execution function to call during graph execution.
		 *
		 * The IOperator* argument to the FExecutionFunction will be the same IOperator instance
		 * which returned the execution function.
		 *
		 * nullptr return values are valid and signal an IOperator which does not need to be
		 * executed.
		 */
		virtual FExecuteFunction GetExecuteFunction() = 0;

		/** Pointer to post execute function for an operator.
		 *
		 * @param IOperator* - The operator associated with the function pointer.
		 */
		using FPostExecuteFunction = void(*)(IOperator*);

		/** Return the FPostExecute function to call during graph post execution.
		 *
		 * The IOperator* argument to the FPostExecutionFunction will be the same IOperator instance
		 * which returned the execution function.
		 *
		 * nullptr return values are valid and signal an IOperator which does not need to be
		 * post executed.
		 */
		virtual FPostExecuteFunction GetPostExecuteFunction() = 0;
	};

	// todo: better place to put this? needs to be in core
	struct FOperatorAndInputs
	{
		TUniquePtr<IOperator> Operator;
		FInputVertexInterfaceData Inputs;
		TSharedPtr<FGraphRenderCost> RenderCost;
	};
}
