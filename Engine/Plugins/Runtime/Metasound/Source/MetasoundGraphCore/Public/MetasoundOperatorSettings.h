// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	using FSampleRate = int32;
	static_assert(TIsIntegral<FSampleRate>::Value, "FSampleRate must be integral type.");

	/** FOperatorSettings
	 *
	 * Audio settings for Metasound IOperators including audio sample rate, 
	 * block rate and frames per block.
	 *
	 * The audio SampleRate defines the number of audio samples per a second of
	 * a mono waveform. 
	 *
	 * The BlockRate defines the number of times an IOperator's execution 
	 * function will be triggered per a second of audio content.
	 *
	 * The NumFramesPerBlock defines the number of audio frames written-to 
	 * and/or read-from during a single call to an IOperator's execution 
	 * function.
	 *
	 * GetNumFramesPerBlock() is required to return a value that abides by the
	 * alignment requirements of FAudioBuffer, FAlignedFloatBuffer and the math
	 * routines declared in BufferVectorOperations.h. This enables the vast 
	 * majority of Metasound IOperators to take advantage of SIMD hardware
	 * accelerations.  
	 *
	 * In order to achieve SIMD alignment, desired block rates must be slightly 
	 * adjusted to match the boundaries of the given sample rate and SIMD 
	 * alignment. GetActualBlockRate() returns the true block rate after 
	 * adhering to sample rate and SIMD boundaries. 
	 */
	class FOperatorSettings
	{
		public:
			/** FOperatorSettings constructor.
			 *
			 * @param InSampleRate      - Audio sample rate in Hz. 
			 * @param InTargetBlockRate - The desired block rate in Hz.
			 */
			UE_API FOperatorSettings(FSampleRate InSampleRate, float InTargetBlockRate);

			/** Set the audio sample rate in Hz. */
			UE_API void SetSampleRate(FSampleRate InSampleRate);

			/** Get the audio sample rate in Hz. */
			inline float GetSampleRate() const
			{
				return SampleRate;
			}

			/** Set the target block rate in Hz. 
			 *
			 * In order to achieve SIMD alignment, target block rates must be 
			 * adjusted to match the boundaries of the given sample rate and 
			 * SIMD alignment. GetActualBlockRate() returns the true block rate 
			 * after adhering to sample rate and SIMD boundaries. 
			 */
			UE_API void SetTargetBlockRate(float InTargetBlockRate);

			/** Get the target block rate in Hz. */
			UE_API float GetTargetBlockRate() const;

			/** Get the actual block rate in Hz after adhering to sample rate 
			 * and SIMD alignment.
			 */
			UE_API float GetActualBlockRate() const;

			/** Get the number of audio frames in a block. 
			 *
			 * This function is required to return a value that abides by the
			 * alignment requirements of FAudioBuffer, FAlignedFloatBuffer and 
			 * the math routines declared in BufferVectorOperations.h. This 
			 * enables the vast majority of Metasound IOperators to take 
			 * advantage of SIMD hardware accelerations.
			 */
			inline int32 GetNumFramesPerBlock() const
			{
				return NumFramesPerBlock;
			}

			/** Convert to human readable string. */
			UE_API FString ToString() const;

			METASOUNDGRAPHCORE_API friend bool operator<(const FOperatorSettings& InLHS, const FOperatorSettings& InRHS);
			METASOUNDGRAPHCORE_API friend bool operator==(const FOperatorSettings& InLHS, const FOperatorSettings& InRHS);

			friend inline uint32 GetTypeHash(const FOperatorSettings& InSettings)
			{
				// Ignore TargetBlockRate in comparison because it is not used in any external calculations
				// Ignore ActualBlockRate because it is derived from NumFramesPerBlock
				return HashCombineFast(GetTypeHash(InSettings.SampleRate), GetTypeHash(InSettings.NumFramesPerBlock));
			}

		private:
			// Update actual block rate and frames per block.
			UE_API void Update();

			UE_API int32 RoundToAligned(int32 Alignment, int32 InNum) const;

			FSampleRate SampleRate = 1;
			
			float TargetBlockRate = 1.f;

			float ActualBlockRate = 1.f;

			int32 NumFramesPerBlock = 0;
	};
}

#undef UE_API
