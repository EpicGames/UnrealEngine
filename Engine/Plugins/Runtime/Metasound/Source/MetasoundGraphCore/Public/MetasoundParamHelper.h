// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundVertex.h"
#include "UObject/NameTypes.h"

/*

	 Macros to make parameter definitions a little easier
	 Use in a namespace

	namespace Metasound
	{
		namespace MyNodeVertexNames
		{
			METASOUND_PARAM(InputParam1, "Param 1", "Tooltip for param1.");
			METASOUND_PARAM(InputParam2, "Param 2", "Tooltip for param2.");
			METASOUND_PARAM(OutputParam1, "Out Param1", "Tooltip for output param 2.");
			METASOUND_PARAM(OutputParam2, "Out Param2", "Tooltip for output param 2.");
		}
	}

	Then to retrieve:

	using namespace MyNodeVertexNames;

	METASOUND_GET_PARAM_NAME(InputParam1)
	METASOUND_GET_PARAM_TT(InputParam1)

	To ensure display names and tooltips are localized: 
	METASOUND_GET_PARAM_NAME_AND_METADATA(InputParam1)

	For variable parameters (e.g. templated on number), do the following:

		namespace TriggerAccumulatorVertexNames
		{
			METASOUND_PARAM(InputParam, "Param {0}", "Do {0} tooltip.");
		}

	Then to retrieve the name/tooltip use:

	METASOUND_GET_PARAM_NAME_WITH_INDEX(InputParam, NUMBER);

	Where NUMBER is a number (or whatever) to slot into the format specifier you defined in the param def.

*/

/**
 * Declare a node parameter that can be accessed by other code (e.g. tests)
 * @param API_EXPORT If this parameter should be available to other modules (common), provide your module API export here
 * @param PARAM_NAME The parameter name
 */
#define DECLARE_METASOUND_PARAM(API_EXPORT, PARAM_NAME) \
	API_EXPORT extern const FLazyName PARAM_NAME##Name;

/**
 * Use this macro to define your parameter if you used DECLARE_METASOUND_PARAM, otherwise just use METASOUND_PARAM in a source file
 */
#if WITH_EDITOR
#define DEFINE_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT) \
	const FLazyName NAME##Name = TEXT(NAME_TEXT); \
	static const FText NAME##Tooltip = LOCTEXT(#NAME "Tooltip", TOOLTIP_TEXT); \
	static const FText NAME##DisplayName = LOCTEXT(#NAME "DisplayName", NAME_TEXT);
#else 
#define DEFINE_METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT) \
	const FLazyName NAME##Name = TEXT(NAME_TEXT);
#endif // WITH_EDITOR

#if WITH_EDITOR
#define METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT) \
	static const FLazyName NAME##Name = TEXT(NAME_TEXT); \
	static const FText NAME##Tooltip = LOCTEXT(#NAME "Tooltip", TOOLTIP_TEXT); \
	static const FText NAME##DisplayName = LOCTEXT(#NAME "DisplayName", NAME_TEXT); 
#else 
#define METASOUND_PARAM(NAME, NAME_TEXT, TOOLTIP_TEXT) \
	static const FLazyName NAME##Name = TEXT(NAME_TEXT);
#endif // WITH_EDITOR

#define METASOUND_GET_PARAM_NAME(NAME) NAME##Name

#if WITH_EDITOR
#define METASOUND_GET_PARAM_TT(NAME) NAME##Tooltip
#define METASOUND_GET_PARAM_NAME_AND_TT(NAME) NAME##Name, NAME##Tooltip
#define METASOUND_GET_PARAM_METADATA(NAME) FDataVertexMetadata { NAME##Tooltip, NAME##DisplayName }
#define METASOUND_GET_PARAM_METADATA_ADVANCED(NAME) FDataVertexMetadata { NAME##Tooltip, NAME##DisplayName, true }
#define METASOUND_GET_PARAM_DISPLAYNAME(NAME) NAME##DisplayName
#else
#define METASOUND_GET_PARAM_TT(NAME) FText{}
#define METASOUND_GET_PARAM_NAME_AND_TT(NAME) NAME##Name, FText{}
#define METASOUND_GET_PARAM_METADATA(NAME) FDataVertexMetadata::EmptyBasic
#define METASOUND_GET_PARAM_METADATA_ADVANCED(NAME) FDataVertexMetadata::EmptyAdvanced
#define METASOUND_GET_PARAM_DISPLAYNAME(NAME) FText{}
#endif // WITH_EDITOR
#define METASOUND_GET_PARAM_NAME_AND_METADATA(NAME) METASOUND_GET_PARAM_NAME(NAME), METASOUND_GET_PARAM_METADATA(NAME)

#define METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(NAME) METASOUND_GET_PARAM_NAME(NAME), METASOUND_GET_PARAM_METADATA_ADVANCED(NAME)

#define METASOUND_GET_PARAM_NAME_WITH_INDEX(NAME, INDEX) *FString::Format(*static_cast<FName>(NAME##Name).ToString(), {INDEX})
#if WITH_EDITOR
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_TT(NAME, INDEX)  METASOUND_GET_PARAM_NAME_WITH_INDEX(NAME, INDEX), FText::Format(NAME##Tooltip, INDEX)
#define METASOUND_GET_PARAM_TT_WITH_INDEX(NAME, INDEX)  FText::Format(NAME##Tooltip, INDEX)
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(NAME, INDEX)  METASOUND_GET_PARAM_NAME_WITH_INDEX(NAME, INDEX), FDataVertexMetadata {FText::Format(NAME##Tooltip, INDEX)}
#else 
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_TT(NAME, INDEX)  *FString::Format(*static_cast<FName>(NAME##Name).ToString(), {INDEX}), FText{}
#define METASOUND_GET_PARAM_TT_WITH_INDEX(NAME, INDEX)  FText{}
#define METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(NAME, INDEX)  *FString::Format(*static_cast<FName>(NAME##Name).ToString(), {INDEX}), FDataVertexMetadata::EmptyBasic
#endif // WITH_EDITOR

