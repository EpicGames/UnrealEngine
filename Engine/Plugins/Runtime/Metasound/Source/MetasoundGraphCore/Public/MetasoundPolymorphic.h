// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataTypeGetTraits.h"
#include "Concepts/DerivedFrom.h"

namespace Metasound
{
	class FOperatorSettings;

	template<typename T>
	struct TPolymorphicTraits
	{
		static constexpr bool bIsPolymorphic = false;
		static constexpr bool bIsAbstract = false;
		using BaseType = void;
	};

	struct FPolyTypeInfo
	{
		FName TypeName = NAME_None;
		FName BaseTypeName = NAME_None;
		bool bIsPolymorphic = false;
	};

	template<typename DataType>
	concept CIsPolymorphic = TPolymorphicTraits<DataType>::bIsPolymorphic;

	template<typename DataType>
	concept CIsNotPolymorphic = !TPolymorphicTraits<DataType>::bIsPolymorphic;

	template<typename DataType>
	concept CIsPolymorphicAndAbstract = TPolymorphicTraits<DataType>::bIsPolymorphic && TPolymorphicTraits<DataType>::bIsAbstract;

	template<typename DataType, typename... ArgTypes>
	concept CHasFactoryFunction = requires(ArgTypes&&... Args)
	{
		{ DataType::CreateNew(Forward<ArgTypes>(Args)...) } -> UE::CDerivedFrom<TSharedRef<DataType, ESPMode::NotThreadSafe>>;
	};

	// Checks if the Constructor is public and accessible (std::is_constructable does not do that).
	template <class DataType, class... ArgTypes>
	concept CHasPublicConstructor =
	requires(ArgTypes&&... Args)
	{
		requires std::is_constructible_v<DataType, ArgTypes...>;
		::new DataType(Forward<ArgTypes>(Args)...);
	};

	template<typename DataType>
	concept CHasFactoryFunctionWithSettings = CHasFactoryFunction<DataType, const FOperatorSettings&>;
	template<typename DataType, typename... ArgTypes>
	concept CHasFactoryFunctionWithSettingsAndArgs = CHasFactoryFunction<DataType, const FOperatorSettings&, ArgTypes...>;
	template<typename DataType, typename... ArgTypes>
	concept CHasFactoryFunctionWithArgs = CHasFactoryFunction<DataType, ArgTypes...>;
	template<typename DataType>
	concept CHasFactoryFunctionDefaultConstruction = CHasFactoryFunction<DataType>;

	template <typename T>
	concept CHasPolymorphicTypeGetter =
	requires(const T& t)
	{
		{ t.GetDataTypeName() } -> UE::CSameAs<const FName&>;
	};

	template <typename T>
	concept CHasPolymorphicTypeIdGetter =
	requires(const T& t)
	{
		{ t.GetDataTypeId() } -> UE::CSameAs<const void*>;
	};

	template <typename T>
	concept CHasPolymorphicTypeFunctions = CHasPolymorphicTypeIdGetter<T> && CHasPolymorphicTypeGetter<T>;

	class FPolyRegistry
	{
	public:
		UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
		METASOUNDGRAPHCORE_API static FPolyRegistry& Get();

		UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
		METASOUNDGRAPHCORE_API void Register(const FPolyTypeInfo& Info);

		UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
		METASOUNDGRAPHCORE_API const FPolyTypeInfo* Find(const FName Type) const;

		UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
		void Unregister(const FPolyTypeInfo& Info);

		METASOUNDGRAPHCORE_API void GetAllRegisteredTypes(TArray<FPolyTypeInfo>& Out) const;

	private:
		TMap<FName, FPolyTypeInfo> Map;
	};

	template <typename T>
	struct TPolyRegistrar
	{
		TPolyRegistrar()
		{
			static const FName ThisName = GetMetasoundDataTypeName<T>();
			FPolyTypeInfo Info;
			Info.TypeName = ThisName;
			Info.bIsPolymorphic = TPolymorphicTraits<T>::bIsPolymorphic;

			if constexpr (TPolymorphicTraits<T>::bIsPolymorphic)
			{
				using Base = TPolymorphicTraits<T>::BaseType;
				static const FName BaseName = GetMetasoundDataTypeName<Base>();
				static_assert(!std::is_same_v<Base, T>, "Polymorphic BaseType must not equal T");
				Info.BaseTypeName = BaseName;
			}
			else
			{
				Info.BaseTypeName = NAME_None;
			}
			PRAGMA_DISABLE_INTERNAL_WARNINGS
			FPolyRegistry::Get().Register(Info);
			PRAGMA_ENABLE_INTERNAL_WARNINGS
		}
	};

	UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
	METASOUNDGRAPHCORE_API bool IsChildOfByName(const FName InDataType, const FName InPotentialBase);

	UE_EXPERIMENTAL(5.7, "Metasound Polymorphism is still under development")
	METASOUNDGRAPHCORE_API bool IsCastable(const FName InType, const FName InPotentialBase);


}

#define DECLARE_METASOUND_POLY_TYPE(TYPE, BASE, IS_ABSTRACT)\
	template<>\
	struct Metasound::TPolymorphicTraits<TYPE>\
	{\
		static constexpr bool bIsPolymorphic = true;\
		static constexpr bool bIsAbstract = IS_ABSTRACT || TIsAbstract<TYPE>::Value;\
		using BaseType = BASE;\
		static_assert((CHasFactoryFunctionWithSettings<TYPE> && bIsAbstract && bIsPolymorphic) || !bIsAbstract, "Abstract Types Must Define a Factory Function 'CreateNew'"); \
		static_assert(CHasPolymorphicTypeFunctions<TYPE>);\
	};

#define REGISTER_METASOUND_POLY_TYPE(T) static Metasound::TPolyRegistrar<T> UE_JOIN(GPolyReg,__COUNTER__);