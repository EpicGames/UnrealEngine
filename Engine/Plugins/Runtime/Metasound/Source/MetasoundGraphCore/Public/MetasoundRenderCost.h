// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "MetasoundEnvironment.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDGRAPHCORE_API

#ifndef UE_METASOUNDRENDERCOST_TRACK_NODE_HIERARCHY 
#define UE_METASOUNDRENDERCOST_TRACK_NODE_HIERARCHY (!UE_BUILD_SHIPPING)
#endif

namespace Metasound
{
	class FNodeRenderCost;

	/** FGraphRenderCost represents the accumulated render cost of a graph. Individual
	 * nodes in a graph can report their render cost through an FNodeRenderCost.
	 *
	 * The render cost of each node is added together to determine the graph's render
	 * cost. 
	 */
	class FGraphRenderCost : public TSharedFromThis<FGraphRenderCost>
	{
		// Private token to enforce creation of shared reference. 
		enum EPrivateToken { PrivateToken };
		friend class FNodeRenderCost;

	public:

		UE_API FGraphRenderCost(EPrivateToken InToken);

		static UE_API TSharedRef<FGraphRenderCost> MakeGraphRenderCost();

		/* Add a node to the graph's render cost.
		 *
		 * @param InNodeInstanceID - ID of the node to add.
		 * @param InEnv - The environment used to create the node. 
		 *
		 * @return FNodeRenderCost object for reporting the render cost of individual nodes. */
		UE_API FNodeRenderCost AddNode(const FGuid& InNodeInstanceID, const FMetasoundEnvironment& InEnv);

		/** Reset the individual node render costs to zero. */
		UE_API void ResetNodeRenderCosts();

		/** Adds all the individual node render costs and returns the result. */
		UE_API float ComputeGraphRenderCost() const;
		
	private:
		UE_API void SetNodeRenderCost(int32 InNodeIndex, float InRenderCost);

		TArray<float> NodeCosts;
		UE_API void AddNodeHierarchy(const FGuid& InNodeInstanceID, const FMetasoundEnvironment& InEnv);
#if UE_METASOUNDRENDERCOST_TRACK_NODE_HIERARCHY
		TArray<TArray<FGuid>> NodeHierarchies;
#endif
	};

	/** FNodeRenderCost allows individual nodes to report their render cost. 
	 * 
	 * FNodeRenderCost should be created with FGraphRenderCost::AddNode(...)
	 */
	class FNodeRenderCost
	{
		friend class FGraphRenderCost;
		UE_API FNodeRenderCost(int32 InNodeIndex, TSharedRef<FGraphRenderCost> InGraphRenderCost);
	public:
		/** The default constructor is provided for convenience, but creating a
		 * FNodeRenderCost with the default constructor will allow any method for
		 * reading or accumulating a nodes render cost. Instead, FNodeRenderCost
		 * should be created with FGraphRenderCost::AddNode(...) so that it is 
		 * associated and aggregated within a graph.
		 */
		FNodeRenderCost() = default;

		/** Set the render cost of this node. */
		UE_API void SetRenderCost(float InCost);

	private:
		int32 NodeIndex = INDEX_NONE;
		TSharedPtr<FGraphRenderCost> GraphRenderCost;
	};
}

#undef UE_API
