// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

/** MetaSound Thread Local Debugging allows node info to be propagated down to 
 * calling functions for debug purposes. It passes the information by storing thread_local
 * debug data which can be accessed on other functions. thread_local access can
 * have negative performance implications, and so this debug info is disabled by
 * default when DO_CHECK and DO_ENSURE are false. 
 *
 * To set which node is active in the current thread, use the following macros.
 *
 * 	{
 * 		// Declare a debug scope to minimize access to thread locals
 * 		METASOUND_DEBUG_DECLARE_SCOPE
 *
 * 		for (const INode* Node : Nodes)
 * 		{
 * 			// Set the active node in this scope to be Node
 * 			METASOUND_DEBUG_SET_ACTIVE_NODE_SCOPE(Node)
 *
 *
 * 			// Any calls in this scope can get debug info about the "active node"
 * 			// by using the METASOUND_DEBUG macros
 * 			DoOtherThings().
 * 		}
 * 	}
 *
 *
 * 	In another function in the same callstack, debug info can be accessed.
 *
 * 	{ 
 * 		UE_LOGF(LogMetaSound, Error, "Error with node %ls", METASOUND_DEBUG_ACTIVE_NODE_NAME);
 * 	}
 */

#ifndef UE_METASOUND_DEBUG_ENABLED
#define UE_METASOUND_DEBUG_ENABLED (DO_CHECK || DO_ENSURE)
#endif

#if UE_METASOUND_DEBUG_ENABLED

namespace Metasound
{
	class INode;

	// Debug util for setting stack scope debug variables
	namespace ThreadLocalDebug
	{
		class FDebugInfo;

		struct FAssetMetaData
		{
			FName ClassName;
			FName AssetName;
			FName AssetPath;
		}; // struct FAssetMetaData

		METASOUNDGRAPHCORE_API FDebugInfo* GetDebugInfoOnThisThread(); 
		METASOUNDGRAPHCORE_API void SetAssetScopeForThisThread(FName Class, FName Name, FName Path);
		
		const TCHAR* GetActiveNodeClassNameAndVersionOnThisThread();


		struct FScopeDebugActiveNode final
		{
			explicit FScopeDebugActiveNode(FDebugInfo* InDebugInfo, const INode* InNode);

			~FScopeDebugActiveNode();

			FScopeDebugActiveNode(const FScopeDebugActiveNode&) = delete;
			FScopeDebugActiveNode(FScopeDebugActiveNode&&) = delete;
			FScopeDebugActiveNode& operator=(const FScopeDebugActiveNode&) = delete;
			FScopeDebugActiveNode& operator=(FScopeDebugActiveNode&&) = delete;

			const FAssetMetaData& GetAssetMetaData() const { return AssetMetaData; };

		private:
			void UpdateAssetMetaData();
			FAssetMetaData AssetMetaData;
			const INode* PriorNode = nullptr;
			FDebugInfo* DebugInfo = nullptr;
		};
	}
}

#endif // if UE_METASOUND_DEBUG_ENABLED

#if UE_METASOUND_DEBUG_ENABLED

#define METASOUND_DEBUG_SCOPE_NAME __MetaSoundDebugScope
#define METASOUND_DEBUG_DECLARE_SCOPE ::Metasound::ThreadLocalDebug::FDebugInfo* METASOUND_DEBUG_SCOPE_NAME = ::Metasound::ThreadLocalDebug::GetDebugInfoOnThisThread();

#define METASOUND_DEBUG_SET_ACTIVE_NODE_SCOPE(InNodePtr)\
	::Metasound::ThreadLocalDebug::FScopeDebugActiveNode __MetaSoundDebugScope_ActiveNode(METASOUND_DEBUG_SCOPE_NAME, InNodePtr);\
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(\
		__MetaSoundDebugScope_ActiveNode.GetAssetMetaData().ClassName,\
		__MetaSoundDebugScope_ActiveNode.GetAssetMetaData().AssetName,\
		__MetaSoundDebugScope_ActiveNode.GetAssetMetaData().AssetPath);\

#define METASOUND_DEBUG_ACTIVE_ASSET_SCOPE (__MetaSoundDebugScope_ActiveNode.GetAssetMetaData())

#define METASOUND_DEBUG_ACTIVE_NODE_NAME (::Metasound::ThreadLocalDebug::GetActiveNodeClassNameAndVersionOnThisThread())

#define METASOUND_DEBUG_SET_ACTIVE_ASSET_SCOPE(AssetName, AssetClass, AssetPath)\
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(AssetName, AssetClass, AssetPath);\
	::Metasound::ThreadLocalDebug::SetAssetScopeForThisThread(AssetName, AssetClass, AssetPath);\

#else // if UE_METASOUND_DEBUG_ENABLED

#define METASOUND_DEBUG_SCOPE_NAME 
#define METASOUND_DEBUG_DECLARE_SCOPE 
#define METASOUND_DEBUG_SET_ACTIVE_NODE_SCOPE(InNodePtr)
#define METASOUND_DEBUG_ACTIVE_ASSET_SCOPE
#define METASOUND_DEBUG_ACTIVE_NODE_NAME TEXT("[Metasound debug disable. Use UE_METASOUND_DEBUG_ENABLED to enable]")
#define METASOUND_DEBUG_SET_ACTIVE_ASSET_SCOPE(AssetName, AssetClass, AssetPath)

#endif // if UE_METASOUND_DEBUG_ENABLED

 