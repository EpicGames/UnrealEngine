// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#if !defined(METASOUND_CPUPROFILERTRACE_ENABLED)
#if CPUPROFILERTRACE_ENABLED && ENABLE_NAMED_EVENTS
#define METASOUND_CPUPROFILERTRACE_ENABLED 1
#else
#define METASOUND_CPUPROFILERTRACE_ENABLED 0
#endif
#endif

#if METASOUND_CPUPROFILERTRACE_ENABLED
// Metasound CPU profiler trace enabled

// Copied from SCOPED_NAMED_EVENT but modified
// to accommodate event names containing ::
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Name, Cond)\
	FScopedNamedEventConditionalStatic UE_JOIN(MetaSound_NamedEvent_,__LINE__)(FColor::Green, NAMED_EVENT_STR(#Name), GCycleStatsShouldEmitNamedEvents > 0 && (Cond));\
	TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Name, Cond);

#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Name)\
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Name, true)

#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_COND(Name, Cond) \
	FScopedNamedEventConditional ANONYMOUS_VARIABLE(NamedEvent_)(FColor::Green, Name, GCycleStatsShouldEmitNamedEvents > 0 && (Cond));\
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Name, Cond);

#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name) \
	METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_COND(Name, true)

// Uses cached Insights SpecId to avoid string lookup
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_FAST(TraceSpecId, Name) \
	FScopedNamedEventConditional ANONYMOUS_VARIABLE(NamedEvent_)(FColor::Green, Name, GCycleStatsShouldEmitNamedEvents > 0);\
	TRACE_CPUPROFILER_EVENT_SCOPE_USE(TraceSpecId, Name, UE_JOIN(MetaSound_NamedEventScope_, __LINE__), true)

#else
// Metasound CPU profiler trace *not* enabled

#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_COND(Name, Cond)
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE(Name)
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_COND(Name, Cond)
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name)
#define METASOUND_TRACE_CPUPROFILER_EVENT_SCOPE_FAST(TraceSpecId, Name)

#endif

LLM_DECLARE_TAG_API(Audio_MetaSound, METASOUNDGRAPHCORE_API);
// Convenience macro for Audio_MetaSound LLM scope to avoid misspells.
#define METASOUND_LLM_SCOPE LLM_SCOPE_BYTAG(Audio_MetaSound);
