// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundLiteral.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertexData.h"
#include "MetasoundPolymorphic.h"

namespace Metasound
{
	namespace MetasoundVariablePrivate
	{
		// This tag resolves issues around the behavior of the TLiteralTraits<>::bIsParsable.
		// Use of the tag dispatch in constructing TVariable<> disables implicit conversions
		// of FLiteral::FVariantType underlying types to FLiteral. 
		struct FConstructWithLiteral {};
	}

	/** A MetaSound Variable contains a data reference's prior and current value.
	 *
	 * @tparam DataType - Underlying data type of data reference.
	 */
	template<typename DataType>
	struct TVariable
	{
		using FWriteReference = TDataWriteReference<DataType>;
		using FReadReference = TDataReadReference<DataType>;

		TVariable() = delete;
		TVariable(TVariable&&) = default;
		TVariable(const TVariable&) = default;


		/** Create a delayed variable.
		 *
		 * @param InInitialValue - A literal used to construct the initial value.
		 */
		TVariable(const FLiteral& InInitialValue, MetasoundVariablePrivate::FConstructWithLiteral)
		: Literal(InInitialValue)
		{
		}

		bool RequiresDelayedDataCopy() const
		{
			if constexpr (TPolymorphicTraits<DataType>::bIsPolymorphic)
			{
				return false; // Polymorphic types do not support delayed data copy.
			}
			else
			{
				return DelayedDataReference.IsSet() && DataReference.IsSet();
			}
		}

		void CopyReferencedData()
		{
			if constexpr (!TPolymorphicTraits<DataType>::bIsPolymorphic)
			{
				check(RequiresDelayedDataCopy());
				*(*DelayedDataReference) = *(*DataReference);
			}
		}

		void SetDataReference(FReadReference InDataReference)
		{
			DataReference.Emplace(InDataReference);
		}
		
		void InitDataReference(const FOperatorSettings& InOperatorSettings)
		{
			if (!DataReference.IsSet())
			{
				InitDelayedDataReference(InOperatorSettings);
				DataReference = GetDelayedDataReference();
			}
		}

		/** Get the current data reference. */
		FReadReference GetDataReference() const
		{
			checkf(DataReference.IsSet(), TEXT("InitDataReference must be called before accessing"));
			return *DataReference;
		}

		void BindDataReference(FOutputVertexInterfaceData& InVertexData, const FVertexName& InVertexName)
		{
			checkf(DataReference.IsSet(), TEXT("InitDataReference must be called before accessing"));
			InVertexData.BindReadVertex(InVertexName, *DataReference);
		}

		void BindDelayedDataReference(FOutputVertexInterfaceData& InVertexData, const FVertexName& InVertexName)
		{
			checkf(DelayedDataReference.IsSet(), TEXT("InitDelayedDataReference must be called before accessing"));
			InVertexData.BindReadVertex(InVertexName, *DelayedDataReference);
		}

		void InitDelayedDataReference(const FOperatorSettings& InOperatorSettings)
		{
			if (!DelayedDataReference.IsSet())
			{
				DelayedDataReference.Emplace(TDataWriteReferenceLiteralFactory<DataType>::CreateExplicitArgs(InOperatorSettings, Literal));
			}
		}

		/** Get the delayed data reference */
		FReadReference GetDelayedDataReference() const
		{
			checkf(DelayedDataReference.IsSet(), TEXT("InitDelayedDataReference must be called before accessing"));
			return *DelayedDataReference;
		}

		void Reset(const FOperatorSettings& InOperatorSettings)
		{
			if constexpr (!TPolymorphicTraits<DataType>::bIsPolymorphic)
			{
				if (DelayedDataReference.IsSet())
				{
					FWriteReference DelayedWritable = *DelayedDataReference;
					*DelayedWritable = TDataTypeLiteralFactory<DataType>::CreateExplicitArgs(InOperatorSettings, Literal);
				}
			}
		}

	private:

		FLiteral Literal;
		TOptional<FWriteReference> DelayedDataReference;
		TOptional<FReadReference> DataReference;
	};

	/** Template to determine if data type is a variable. */
	template<typename DataType>
	struct TIsVariable
	{
		static constexpr bool Value = false;
	};

	/** Template specialization to determine if data type is a variable. */
	template<typename UnderlyingDataType>
	struct TIsVariable<TVariable<UnderlyingDataType>>
	{
		static constexpr bool Value = true;
	};
}

