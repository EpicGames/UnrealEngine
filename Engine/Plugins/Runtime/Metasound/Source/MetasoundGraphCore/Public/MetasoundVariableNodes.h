// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundLog.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundVariable.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundGraphCore"


namespace Metasound
{
	namespace VariableNames
	{
		DECLARE_METASOUND_PARAM(METASOUNDGRAPHCORE_API, InputData)
		DECLARE_METASOUND_PARAM(METASOUNDGRAPHCORE_API, OutputData)
		DECLARE_METASOUND_PARAM(METASOUNDGRAPHCORE_API, InputVariable)
		DECLARE_METASOUND_PARAM(METASOUNDGRAPHCORE_API, OutputVariable)

		extern METASOUNDGRAPHCORE_API const FDataVertexMetadata InputDataMetadata;
		extern METASOUNDGRAPHCORE_API const FDataVertexMetadata OutputDataMetadata;
		extern METASOUNDGRAPHCORE_API const FDataVertexMetadata InputVariableMetadata;
		extern METASOUNDGRAPHCORE_API const FDataVertexMetadata OutputVariableMetadata;

		/** Class name for variable node. */
		METASOUNDGRAPHCORE_API FNodeClassName GetVariableNodeClassName(const FName& InDataTypeName);

		/** Class name for variable mutator node. */
		METASOUNDGRAPHCORE_API FNodeClassName GetVariableMutatorNodeClassName(const FName& InDataTypeName);

		/** Class name for variable accessor node */
		METASOUNDGRAPHCORE_API FNodeClassName GetVariableAccessorNodeClassName(const FName& InDataTypeName);

		/** Class name for variable deferred accessor node. */
		METASOUNDGRAPHCORE_API FNodeClassName GetVariableDeferredAccessorNodeClassName(const FName& InDataTypeName);

		/** Class name for variable node. */
		template<typename DataType>
		const FNodeClassName& GetVariableNodeClassName()
		{
			static const FNodeClassName ClassName = GetVariableNodeClassName(GetMetasoundDataTypeName<DataType>());
			return ClassName;
		}

		/** Class name for variable mutator node. */
		template<typename DataType>
		const FNodeClassName& GetVariableMutatorNodeClassName()
		{
			static const FNodeClassName ClassName = GetVariableMutatorNodeClassName(GetMetasoundDataTypeName<DataType>());
			return ClassName;
		}

		/** Class name for variable accessor node */
		template<typename DataType>
		const FNodeClassName& GetVariableAccessorNodeClassName()
		{
			static const FNodeClassName ClassName = GetVariableAccessorNodeClassName(GetMetasoundDataTypeName<DataType>());
			return ClassName;
		}

		/** Class name for variable deferred accessor node. */
		template<typename DataType>
		const FNodeClassName& GetVariableDeferredAccessorNodeClassName()
		{
			static const FNodeClassName ClassName = GetVariableDeferredAccessorNodeClassName(GetMetasoundDataTypeName<DataType>());
			return ClassName;
		}

	}

	namespace MetasoundVariableNodesPrivate
	{
		METASOUNDGRAPHCORE_API bool IsSupportedVertexData(const IDataReference& InCurrentVariableRef, const FInputVertexInterfaceData& InNew);

		template<typename DataType>
		class TVariableOperator : public FNoOpOperator
		{
			using FVariable = TVariable<DataType>;

		public:
			TVariableOperator(const FOperatorSettings& InOperatorSettings, const FLiteral& InLiteral)
			: Variable(TDataWriteReference<FVariable>::CreateNew(InLiteral, MetasoundVariablePrivate::FConstructWithLiteral()))
			{
			}

			virtual ~TVariableOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& OutVertexData) override
			{
				using namespace VariableNames; 

				OutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputVariable), Variable);
			}

		private:

			TDataWriteReference<FVariable> Variable;
		};

		/** Operator for providing setting a variable */
		template<typename DataType>
		class TVariableMutatorOperator : public IOperator
		{
		public:
			using FVariable = TVariable<DataType>;

			static FVertexInterface DeclareVertexInterface()
			{
				using namespace VariableNames;

				return FVertexInterface(
					FInputVertexInterface(
						TInputDataVertex<DataType>(METASOUND_GET_PARAM_NAME(InputData), InputDataMetadata),
						TInputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(InputVariable), InputVariableMetadata)
					),
					FOutputVertexInterface(
						TOutputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), OutputVariableMetadata)
					)
				);
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto CreateNodeClassMetadata = []()
				{
					FNodeClassMetadata Info;

					Info.ClassName = VariableNames::GetVariableMutatorNodeClassName<DataType>();
					Info.MajorVersion = 1;
					Info.MinorVersion = 0;

#if WITH_EDITOR
					Info.DisplayStyle.bShowName = false;
					Info.DisplayName = LOCTEXT("Metasound_VariableMutatorNodeDisplayName", "Set");
					Info.Description = LOCTEXT("Metasound_VariableMutatorNodeDescription", "Set variable on MetaSound graph.");
					Info.CategoryHierarchy = { LOCTEXT("Metasound_VariableCategory", "Variable") };
#endif // WITH_EDITOR
					Info.Author = PluginAuthor;
					Info.PromptIfMissing = PluginNodeMissingPrompt;
					Info.DefaultInterface = DeclareVertexInterface();

					return Info;
				};

				static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
				return Metadata;
			}

			TVariableMutatorOperator(TDataWriteReference<FVariable> InVariable, TDataReadReference<DataType> InData)
			: Variable(InVariable)
			, Data(InData)
			{
				Variable->SetDataReference(Data);
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) 
			{
				using namespace VariableNames;

				if (const FAnyDataReference* VariableRef = InParams.InputData.FindDataReference(METASOUND_GET_PARAM_NAME(InputVariable)))
				{
					TDataWriteReference<FVariable> Variable = VariableRef->GetDataWriteReference<FVariable>();
					TDataReadReference<DataType> Data = InParams.InputData.GetOrCreateDefaultDataReadReference<DataType>(METASOUND_GET_PARAM_NAME(InputData), InParams.OperatorSettings);

					return MakeUnique<TVariableMutatorOperator<DataType>>(Variable, Data);
				}
				else 
				{
					// Nothing to do if there's no input data.
					UE_LOGF(LogMetaSound, Verbose, "Missing internal variable connection. Failed to create valid \"SetVariable\" operator");
					return MakeUnique<FNoOpOperator>();
				}
			}

			virtual ~TVariableMutatorOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 

				checkf(IsSupportedVertexData(Variable, InVertexData), TEXT("Input variables cannot be bound to new underlying objects"));

				InVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(InputVariable), Variable);
				InVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputData), Data);

				// Update input variable with new input data in case it has been updated. 
				Variable->SetDataReference(Data);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 
				InVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputVariable), Variable);
			}

			void PostExecute()
			{
				Variable->CopyReferencedData();
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				Variable->Reset(InParams.OperatorSettings);
			}

			virtual IOperator::FExecuteFunction GetExecuteFunction() override
			{
				// Do not do anything during execution
				return nullptr;
			}

			virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override
			{
				if (Variable->RequiresDelayedDataCopy())
				{
					// Only copy over data if it's required. Otherwise, avoid the copy
					// to improve performance.
					return &StaticPostExecute;
				}
				else
				{
					return nullptr;
				}
			}

			virtual IOperator::FResetFunction GetResetFunction() override
			{
				if (Variable->RequiresDelayedDataCopy())
				{
					// Reset is only applied to the delayed data reference. If we
					// are not modifying the delayed data reference, do not perform
					// any reset.
					return &StaticReset;
				}
				else
				{
					return nullptr;
				}
			}

		private:
			static void StaticPostExecute(IOperator* InOperator)
			{
				static_cast<TVariableMutatorOperator<DataType>*>(InOperator)->PostExecute();
			}

			static void StaticReset(IOperator* InOperator, const IOperator::FResetParams& InParams)
			{
				static_cast<TVariableMutatorOperator<DataType>*>(InOperator)->Reset(InParams);
			}

			TDataWriteReference<FVariable> Variable;
			TDataReadReference<DataType> Data;
		};

		/** Operator for providing delayed access to a variable */
		template<typename DataType>
		class TVariableDeferredAccessorOperator : public FNoOpOperator 
		{
		public:

			using FVariable = TVariable<DataType>;

			static FVertexInterface DeclareVertexInterface()
			{
				using namespace VariableNames; 

				return FVertexInterface(
					FInputVertexInterface(
						TInputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(InputVariable), InputVariableMetadata)
					),
					FOutputVertexInterface(
						TOutputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), OutputVariableMetadata),
						TOutputDataVertex<DataType>(METASOUND_GET_PARAM_NAME(OutputData), OutputDataMetadata)
					)
				);
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto CreateNodeClassMetadata = []()
				{
					FNodeClassMetadata Info;

					Info.ClassName = VariableNames::GetVariableDeferredAccessorNodeClassName<DataType>();
					Info.MajorVersion = 1;
					Info.MinorVersion = 0;
#if WITH_EDITOR
					Info.DisplayStyle.bShowName = false;
					Info.DisplayName = LOCTEXT("Metasound_VariableDeferredAccessorNodeDisplayName", "Get Delayed");
					Info.Description = LOCTEXT("Metasound_VariableDeferredAccessorNodeDescription", "Get a delayed variable on MetaSound graph.");
					Info.CategoryHierarchy = { LOCTEXT("Metasound_VariableCategory", "Variable") };
#endif // WITH_EDITOR
					Info.Author = PluginAuthor;
					Info.PromptIfMissing = PluginNodeMissingPrompt;
					Info.DefaultInterface = DeclareVertexInterface();

					return Info;
				};

				static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
				return Metadata;
			}

			TVariableDeferredAccessorOperator(TDataWriteReference<FVariable> InVariable)
			: Variable(InVariable)
			, DelayedData(Variable->GetDelayedDataReference())
			{
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) 
			{
				using namespace VariableNames;

				if (const FAnyDataReference* VariableRef = InParams.InputData.FindDataReference(METASOUND_GET_PARAM_NAME(InputVariable)))
				{
					TDataWriteReference<FVariable> DelayedVariable = VariableRef->GetDataWriteReference<FVariable>();
					DelayedVariable->InitDelayedDataReference(InParams.OperatorSettings);

					return MakeUnique<TVariableDeferredAccessorOperator>(DelayedVariable);
				}
				else 
				{
					// Nothing to do if there's no input data.
					UE_LOGF(LogMetaSound, Verbose, "Missing internal variable connection. Failed to create valid \"GetDelayedVariable\" operator");
					return MakeUnique<FNoOpOperator>();
				}
			}

			virtual ~TVariableDeferredAccessorOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 

				checkf(IsSupportedVertexData(Variable, InVertexData), TEXT("Input variables cannot be bound to new underlying objects"));

				InVertexData.BindWriteVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), Variable);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 

				InVertexData.BindWriteVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), Variable);
				Variable->BindDelayedDataReference(InVertexData, METASOUND_GET_PARAM_NAME(OutputData));
			}

		private:

			TDataWriteReference<FVariable> Variable;
			TDataReadReference<DataType> DelayedData;
		};

		/** Operator for providing inline access to a variable */
		template<typename DataType>
		class TVariableAccessorOperator : public FNoOpOperator 
		{
		public:

			using FVariable = TVariable<DataType>;

			static FVertexInterface DeclareVertexInterface()
			{
				using namespace VariableNames; 

				return FVertexInterface(
					FInputVertexInterface(
						TInputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(InputVariable), InputDataMetadata)
					),
					FOutputVertexInterface(
						TOutputDataVertex<DataType>(METASOUND_GET_PARAM_NAME(OutputData), OutputDataMetadata),
						TOutputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), OutputVariableMetadata)
					)
				);
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto CreateNodeClassMetadata = []()
				{
					FNodeClassMetadata Info;

					Info.ClassName = VariableNames::GetVariableAccessorNodeClassName<DataType>();
					Info.MajorVersion = 1;
					Info.MinorVersion = 0;
#if WITH_EDITOR
					Info.DisplayStyle.bShowName = false;
					Info.DisplayName = LOCTEXT("Metasound_VariableAccessorNodeDisplayName", "Get");
					Info.Description = LOCTEXT("Metasound_VariableAccessorNodeDescription", "Get variable on MetaSound graph.");
#endif // WITH_EDITOR

					Info.Author = PluginAuthor;
					Info.PromptIfMissing = PluginNodeMissingPrompt;
					Info.DefaultInterface = DeclareVertexInterface();

					return Info;
				};

				static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
				return Metadata;
			}

			TVariableAccessorOperator(TDataWriteReference<FVariable> InVariable)
			: Variable(InVariable)
			{
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) 
			{
				using namespace VariableNames;

				if (const FAnyDataReference* VariableRef = InParams.InputData.FindDataReference(METASOUND_GET_PARAM_NAME(InputVariable)))
				{
					TDataWriteReference<FVariable> Variable = VariableRef->GetDataWriteReference<FVariable>();
					Variable->InitDataReference(InParams.OperatorSettings);

					return MakeUnique<TVariableAccessorOperator>(Variable);
				}
				else 
				{
					// Nothing to do if there's no input data.
					UE_LOGF(LogMetaSound, Verbose, "Missing internal variable connection. Failed to create valid \"GetVariable\" operator");
					return MakeUnique<FNoOpOperator>();
				}
			}

			virtual ~TVariableAccessorOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 

				checkf(IsSupportedVertexData(Variable, InVertexData), TEXT("Input variables cannot be bound to new underlying objects"));

				InVertexData.BindWriteVertex<FVariable>(METASOUND_GET_PARAM_NAME(InputVariable), Variable);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InVertexData) override
			{
				using namespace VariableNames; 

				InVertexData.BindWriteVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), Variable);
				Variable->BindDataReference(InVertexData, METASOUND_GET_PARAM_NAME(OutputData));
			}

		private:

			TDataWriteReference<FVariable> Variable;
		};
	}


	/** Variable nodes initialize variable values. The output of a VariableNode
	 * is a TVariable.  */
	template<typename DataType>
	class TVariableNode : public FBasicNode
	{
		using FVariable = TVariable<DataType>;

		class FFactory : public IOperatorFactory
		{
		public:
			FFactory(FLiteral&& InLiteral)
			: Literal(MoveTemp(InLiteral))
			{
			}

			virtual ~FFactory() = default;

			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				return MakeUnique<MetasoundVariableNodesPrivate::TVariableOperator<DataType>>(InParams.OperatorSettings, Literal);
			}
		private:
			FLiteral Literal;
		};


		static FVertexInterface DeclareVertexInterface()
		{
			using namespace VariableNames; 

			return FVertexInterface(
				FInputVertexInterface(
				),
				FOutputVertexInterface(
					TOutputDataVertex<FVariable>(METASOUND_GET_PARAM_NAME(OutputVariable), OutputVariableMetadata)
				)
			);
		}


	public:
		TVariableNode(const FVertexName& InNodeName, const FGuid& InInstanceID, FLiteral&& InLiteral)
		: TVariableNode(MoveTemp(InLiteral), FNodeData{InNodeName, InInstanceID, DeclareVertexInterface()}, MakeShared<const FNodeClassMetadata>(CreateNodeClassMetadata()))
		{
		}

		TVariableNode(FLiteral InLiteral, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		, Factory(MakeShared<FFactory>(MoveTemp(InLiteral)))
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata()
		{
			FNodeClassMetadata Info;

			Info.ClassName = VariableNames::GetVariableNodeClassName<DataType>();
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;

#if WITH_EDITOR
			Info.DisplayStyle.bShowName = false;
			Info.Description = LOCTEXT("Metasound_InitVariableNodeDescription", "Initialize a variable of a MetaSound graph.");
			Info.CategoryHierarchy = { LOCTEXT("Metasound_VariableCategory", "Variable") };
#endif // WITH_EDITOR
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();

			return Info;
		}


		virtual TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override
		{
			return Factory;
		}

	private:

		FOperatorFactorySharedRef Factory;
	};

	/** TVariableMutatorNode allows variable values to be set.  */
	template<typename DataType>
	using TVariableMutatorNode = TNodeFacade<MetasoundVariableNodesPrivate::TVariableMutatorOperator<DataType>>;


	/** TVariableDeferredAccessorNode provides access to the prior executions variable value.
	 * TVariableDeferredAccessorNodes must always be before TVariableMutatorNodes in the dependency
	 * order.
	 */
	template<typename DataType>
	using TVariableDeferredAccessorNode = TNodeFacade<MetasoundVariableNodesPrivate::TVariableDeferredAccessorOperator<DataType>>;

	/** FGetVariable node provides delay free, cpu free access to a set variable. */
	template<typename DataType>
	using TVariableAccessorNode = TNodeFacade<MetasoundVariableNodesPrivate::TVariableAccessorOperator<DataType>>;
}

#undef LOCTEXT_NAMESPACE // MetasoundGraphCore
