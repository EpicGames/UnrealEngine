// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/NumericLimits.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundEnvironment.h"
#include "MetasoundLiteral.h"

#include <type_traits>

#define UE_API METASOUNDGRAPHCORE_API

namespace Metasound
{
	/** Name of a given vertex.  Only unique for a given node interface. */
	using FVertexName = FName;

	
	/** Describes a variant on a ClassInterface */
	struct FVariantDescription
	{
		FName VariantName; 
		TArray<FName> DataTypes; //< Array of supported data types.
	};


	namespace VertexPrivate
	{
		// Forward declare for friendship
		class FInputInterfaceConfigurationBuilder;
		class FOutputInterfaceConfigurationBuilder;

		struct FPrivateAccessTag;

		namespace SubInterface
		{
			// These are utilized to make sure we don't overflow uint16 capacity
			// in case someone declares a ridiculously large interface.
			static constexpr uint16 InvalidInstancePosition = TNumericLimits<uint16>::Max();
			static constexpr uint16 MaxInstancePosition = TNumericLimits<uint16>::Max() - 1;
			static constexpr uint16 MinInstancePosition = 0;
		}

		/** Contains a list of sub-interface spans where the span indices
		 * refer to vertex indices of an array containing vertices. */
		struct FSubInterfaceLayout
		{
			struct FInstance
			{
				uint16 Begin = 0;
				uint16 End = 0; //< Exclusive
			};

			FName SubInterfaceName;
			TArray<FInstance> Instances;
		};

		/** Contains the location of an individual sub interface in a FClassInterface */
		struct FSubInterfaceClassLayout
		{
			FName SubInterfaceName;
			uint16 Begin = 0;
			uint16 End = 0; //< Exclusive. 
		};

		/** Denotes whether a vertex belongs to the input or output interface */
		enum class EVertexDirection : uint8
		{
			Input,
			Output
		};

		/** A key for placing input and output vertices in a shared container. */
		struct FVertexKey
		{
			FVertexName Name;
			EVertexDirection Direction;

			friend bool operator==(const FVertexKey& InLHS, const FVertexKey& InRHS)
			{
				return (InLHS.Name == InRHS.Name) && (InLHS.Direction == InRHS.Direction);
			}

			friend bool operator<(const FVertexKey& InLHS, const FVertexKey& InRHS)
			{
				if (InLHS.Name == InRHS.Name)
				{
					return InLHS.Direction < InRHS.Direction;
				}
				return InLHS.Name.FastLess(InRHS.Name);
			}
		};

		/** Functor for finding vertices with equal names */
		template<typename VertexType>
		struct TEqualVertexName
		{
			const FVertexName& NameRef;

			TEqualVertexName(const FVertexName& InName)
			: NameRef(InName)
			{
			}

			inline bool operator()(const VertexType& InOther) const
			{
				return InOther.VertexName == NameRef;
			}
		};

		/** constexpr count of the number of a instances of type T in a parameter pack */
		template<typename T>
		struct TNumOfTypeInPack
		{
			template<typename ... Ts>
			static constexpr uint32 Get()
			{
				constexpr uint32 Num = ((std::is_base_of_v<T, Ts> ? 1 : 0) + ...);
				return Num;
			}
		};
	} // namespace VertexPrivate

	/** Convenience for using a TSortedMap with FVertexName Key type.
	 *
	 * This template makes it convenient to create a TSortedMap with an FVertexName 
	 * while also avoiding compilation errors incurred from using the FName default
	 * "less than" operator in the TSortedMap implementation. 
	 *
	 * - FVertexName is an alias to FName. 
	 * - TSortedMap<FName, ValueType> fails to compile since the "less than" operator
	 *   specific implementation needs to be chosen (FastLess vs LexicalLess)
	 * - Due to the template argument order of TSortedMap this also forces you to
	 *   choose the allocator. 
	 * - This is all a bit of an annoyance to do every time we use a TSortedMap 
	 *   with an FVertexName as the key.
	 */
	template<typename ValueType, typename AllocatorType=FDefaultAllocator>
	using TSortedVertexNameMap = TSortedMap<FVertexName, ValueType, AllocatorType, FNameFastLess>;

	// Vertex metadata
	struct FDataVertexMetadata
	{
		FText Description;
		FText DisplayName;
		bool bIsAdvancedDisplay = false;

		METASOUNDGRAPHCORE_API static FDataVertexMetadata EmptyBasic;
		METASOUNDGRAPHCORE_API static FDataVertexMetadata EmptyAdvanced;
	};

	/** Describe how the vertex will access connected data. */
	enum class EVertexAccessType
	{
		Reference, //< Vertex accesses the data references
		Value      //< Vertex accesses the data value.
	};


	/** FDataVertex
	 *
	 * An FDataVertex is a named vertex of a MetaSound node which can contain data.
	 */
	class FDataVertex
	{

	public:

		FDataVertex() = default;

		/** FDataVertex Constructor
		 *
		 * @InVertexName - Name of vertex.
		 * @InDataTypeName - Name of data type.
		 * @InMetadata - Metadata pertaining to given vertex.
		 * @InAccessType - The access type of the vertex.
		 */
		FDataVertex(const FVertexName& InVertexName, const FName& InDataTypeName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType)
		: VertexName(InVertexName)
		, DataTypeName(InDataTypeName)
#if WITH_EDITORONLY_DATA
		, Metadata(InMetadata)
#endif // WITH_EDITORONLY_DATA
		, AccessType(InAccessType)
		{
		}


		virtual ~FDataVertex() = default;

		/** Name of vertex. */
		FVertexName VertexName;

		/** Type name of data. */
		FName DataTypeName;

#if WITH_EDITORONLY_DATA
		/** Metadata associated with vertex. */
		FDataVertexMetadata Metadata;
#endif // WITH_EDITORONLY_DATA

		/** Access type of the vertex. */
		EVertexAccessType AccessType;
	};

	/** FInputDataVertex */
	class FInputDataVertex : public FDataVertex
	{
	public:

		FInputDataVertex() = default;

		/** Construct an FInputDataVertex. */
		FInputDataVertex(const FVertexName& InVertexName, const FName& InDataTypeName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType=EVertexAccessType::Reference)
			: FDataVertex(InVertexName, InDataTypeName, InMetadata, InAccessType)
			, Literal(FLiteral::FNone{})
		{
		}

		/** Construct an FInputDataVertex with a default literal. */
		template<typename LiteralValueType>
		FInputDataVertex(const FVertexName& InVertexName, const FName& InDataTypeName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType, const LiteralValueType& InLiteralValue)
			: FDataVertex(InVertexName, InDataTypeName, InMetadata, InAccessType)
			, Literal(InLiteralValue)
		{
		}

		FInputDataVertex(const FVertexName& InVertexName, const FName& InDataTypeName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType, const FLiteral& InLiteral)
			: FDataVertex(InVertexName, InDataTypeName, InMetadata, InAccessType)
			, Literal(InLiteral)
		{
		}

		/** Returns the default literal associated with this input. */
		FLiteral GetDefaultLiteral() const 
		{
			return Literal;
		}
		
		/** Set the default literal for this vertex */
		void SetDefaultLiteral(const FLiteral& InLiteral)
		{
			Literal = InLiteral;
		}

		friend bool METASOUNDGRAPHCORE_API operator==(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator<(const FInputDataVertex& InLHS, const FInputDataVertex& InRHS);

	private:

		FLiteral Literal;
	};

	/** Create a FInputDataVertex with a templated MetaSound data type. */
	template<typename DataType>
	class TInputDataVertex : public FInputDataVertex
	{
	public:
		TInputDataVertex() = default;

		template<typename... ArgTypes>
		FORCENOINLINE TInputDataVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FInputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}

		template<typename... ArgTypes>
		FORCENOINLINE TInputDataVertex(const FLazyName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FInputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}

		template<typename... ArgTypes>
		FORCENOINLINE TInputDataVertex(const char* InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FInputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}
	};

	/** Create a FInputDataVertex with a templated MetaSound data type which only
	 * reads data at operator time. */
	template<typename DataType>
	class TInputConstructorVertex : public FInputDataVertex
	{
	public:
		TInputConstructorVertex() = default;

		template<typename... ArgTypes>
		TInputConstructorVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FInputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Value, Forward<ArgTypes>(Args)...)
		{
		}
	};
	
	/** FOutputDataVertex
	 *
	 * Vertex for outputs.
	 */
	class FOutputDataVertex : public FDataVertex
	{
	public:
		using FDataVertex::FDataVertex;

		friend bool METASOUNDGRAPHCORE_API operator==(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator<(const FOutputDataVertex& InLHS, const FOutputDataVertex& InRHS);
	};

	/** Create a FOutputDataVertex with a templated MetaSound data type. */
	template<typename DataType>
	class TOutputDataVertex : public FOutputDataVertex
	{
	public:

		TOutputDataVertex() = default;

		template<typename... ArgTypes>
		FORCENOINLINE TOutputDataVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FOutputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}

		template<typename... ArgTypes>
		FORCENOINLINE TOutputDataVertex(const FLazyName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FOutputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}

		template<typename... ArgTypes>
		FORCENOINLINE TOutputDataVertex(const char* InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FOutputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Reference, Forward<ArgTypes>(Args)...)
		{
		}
	};

	/** Create a FOutputDataVertex with a templated MetaSound data type which is only
	 * sets data at operator construction time. 
	 */
	template<typename DataType>
	class TOutputConstructorVertex : public FOutputDataVertex
	{
	public:

		TOutputConstructorVertex() = default;

		template<typename... ArgTypes>
		TOutputConstructorVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FOutputDataVertex(InVertexName, GetMetasoundDataTypeName<DataType>(), InMetadata, EVertexAccessType::Value, Forward<ArgTypes>(Args)...)
		{
		}
	};


	/** FEnvironmentVertex
	 *
	 * A vertex for environment variables. 
	 */
	class FEnvironmentVertex
	{
	public:
		/** FEnvironmentVertex Construtor
		 *
		 * @InVertexName - Name of vertex.
		 * @InDescription - Human readable vertex description.
		 */
		FEnvironmentVertex(const FVertexName& InVertexName, const FText& InDescription)
		:	VertexName(InVertexName)
#if WITH_EDITORONLY_DATA
		,	Description(InDescription)
#endif // WITH_EDITORONLY_DATA
		{
		}

		virtual ~FEnvironmentVertex() = default;

		/** Name of vertex. */
		FVertexName VertexName;

#if WITH_EDITORONLY_DATA
		/** Description of the vertex. */
		FText Description;
#endif // WITH_EDITORONLY_DATA

		friend bool METASOUNDGRAPHCORE_API operator==(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator<(const FEnvironmentVertex& InLHS, const FEnvironmentVertex& InRHS);
	};

	/** TVertexInterfaceImpls encapsulates multiple related data vertices. It 
	 * requires that each vertex in the group have a unique FVertexName.
	 */
	template<typename VertexType>
	class TVertexInterfaceImpl
	{
		using FEqualVertexName = VertexPrivate::TEqualVertexName<VertexType>;

		void AddOrUpdateVertex(VertexType&& InVertex)
		{
			if (VertexType* Vertex = Find(InVertex.VertexName))
			{
				*Vertex = MoveTemp(InVertex);
			}
			else
			{
				Vertices.Add(MoveTemp(InVertex));
			}
		}
	public:

		using RangedForConstIteratorType = typename TArray<VertexType>::RangedForConstIteratorType;

		TVertexInterfaceImpl() = default;

		/** Construct with prebuilt array of vertices. */
		explicit TVertexInterfaceImpl(TArray<VertexType> InVertices)
		: Vertices(MoveTemp(InVertices))
		{
		}
		
		/** TVertexInterfaceImpl constructor with variadic list of vertex
		 * models.
		 */
		template<typename... VertexTypes>
		UE_DEPRECATED(5.6, "Use the constructors of derived classes instead (FInputVertexInterface, FOutputVertexInterface, etc).")
		explicit TVertexInterfaceImpl(VertexTypes&&... InVertices)
		{
			static_assert(
				(std::is_constructible_v<VertexType, VertexTypes&&> && ...),
				"Vertex types must be move constructible from the base type");
			
			// Reserve array to hold exact number of vertices to avoid
			// over allocation.
			Vertices.Reserve(sizeof...(VertexTypes));

			// Unfold parameter pack and add t
			(AddOrUpdateVertex(Forward<VertexTypes>(InVertices)), ...);
		}

		/** Add a vertex. */
		void Add(const VertexType& InVertex)
		{
			AddOrUpdateVertex(VertexType(InVertex));
		}

		void Add(VertexType&& InVertex)
		{
			AddOrUpdateVertex(MoveTemp(InVertex));
		}

		void Append(TArrayView<const VertexType> InVertices)
		{
			for (const VertexType& Vertex : InVertices)
			{
				Add(Vertex);
			}
		}

		/** Remove a vertex by key. */
		bool Remove(const FVertexName& InKey)
		{
			int32 NumRemoved = Vertices.RemoveAll(FEqualVertexName(InKey));
			return (NumRemoved > 0);
		}

		/** Returns true if this contains a vertex with a matching key. */
		bool Contains(const FVertexName& InKey) const
		{
			return Vertices.ContainsByPredicate(FEqualVertexName(InKey));
		}

		/** Find a vertex with a given VertexName */
		VertexType* Find(const FVertexName& InKey)
		{
			return Vertices.FindByPredicate(FEqualVertexName(InKey));
		}

		/** Find a vertex with a given VertexName */
		const VertexType* Find(const FVertexName& InKey) const
		{
			return Vertices.FindByPredicate(FEqualVertexName(InKey));
		}

		/** Return the sort order index of a vertex with the given name.
		 *
		 * @param InKey - FVertexName of vertex of interest.
		 *
		 * @return The index of the vertex. INDEX_NONE if the vertex does not exist. 
		 */
		int32 GetSortOrderIndex(const FVertexName& InKey) const
		{
			return Vertices.IndexOfByPredicate(FEqualVertexName(InKey));
		}

		/** Return the vertex for a given vertex key. */
		const VertexType& operator[](const FVertexName& InName) const
		{
			const VertexType* Vertex = Find(InName);
			checkf(nullptr != Vertex, TEXT("Vertex with name '%s' does not exist"), *InName.ToString());
			return *Vertex;
		}

		/** Iterator for ranged for loops. */
		RangedForConstIteratorType begin() const
		{
			return Vertices.begin();
		}

		/** Iterator for ranged for loops. */
		RangedForConstIteratorType end() const
		{
			return Vertices.end();
		}

		/** Returns the number of vertices in the group. */
		int32 Num() const
		{
			return Vertices.Num();
		}

		/** Return a vertex at an index. */
		const VertexType& At(int32 InIndex) const
		{
			return Vertices[InIndex];
		}
		
		/** Return a vertex at an index. */
		VertexType& At(int32 InIndex)
		{
			return Vertices[InIndex];
		}

		/** Compare whether two vertex interfaces are equal. */
		friend bool operator==(const TVertexInterfaceImpl<VertexType>& InLHS, const TVertexInterfaceImpl<VertexType>& InRHS)
		{
			return InLHS.Vertices == InRHS.Vertices;
		}

		/** Compare whether two vertex interfaces are unequal. */
		friend bool operator!=(const TVertexInterfaceImpl<VertexType>& InLHS, const TVertexInterfaceImpl<VertexType>& InRHS)
		{
			return !(InLHS == InRHS);
		}

	protected:

		TArray<VertexType> Vertices;
	};

	template<typename VertexType>
	using TVertexInterfaceGroup UE_DEPRECATED(5.6, "Use TVertexInterfaceImpl instead") = TVertexInterfaceImpl<VertexType>;

	namespace VertexPrivate
	{
		/** Interface Declaration Builders create vertex interfaces from a template
		 * parameter pack. This allows developers to express their node interfaces 
		 * as declarations as opposed to requiring them to sequentially construct
		 * their interfaces.  
		 *
		 * For example, we can do:
		 * 
		 *  FInputVertexInterface Interface
		 *  {
		 *  	TInputDataVertex<float>(...),
	 	 * 		TInputDataVertex<int32>(...),
		 *  	TInputDataVertex<float>(...)
		 *  };
		 *
		 * As opposed to:
		 *  FInputVertexInterface Interface;
		 * 
		 * 	Interface.Add(TInputDataVertex<float>(...));
	     * 	Interface.Add(TInputDataVertex<int32>(...));
		 * 	Interface.Add(TInputDataVertex<float>(...);
		 *
		 * The builders size memory allocations exactly to remove any addition
		 * slack. In general, there can be many interfaces in memory and minimizing
		 * their memory footprint is important. 
		 */

		/** Interface builder for FInputVertexInterface declarations. */
		class FInputVertexInterfaceDeclarationBuilder
		{
		public:
			UE_API FInputVertexInterfaceDeclarationBuilder(TArray<FInputDataVertex>& OutVertices);

			template<typename... ArgTypes>
			void Build(ArgTypes&&... InArgs)
			{
				// Size vertices exactly to avoid wasted memory in array allocations
				constexpr uint32 NumVertices = TNumOfTypeInPack<FInputDataVertex>::Get<ArgTypes...>();
				if constexpr (NumVertices > 0)
				{
					Vertices.Reserve(NumVertices);
				}

				// Add all the elements of the vertex interface. This uses a fold expression
				// to call Add(...) on each input. The various overloads of the Add(...) method
				// then assemble the appropriate structures.
				(Add(Forward<ArgTypes>(InArgs)), ...);
			}

		private:

			UE_API void Add(FInputDataVertex&& InVertex);

			TArray<FInputDataVertex>& Vertices;
		};

		/** Interface builder for FOutputVertexInterface declarations. */
		class FOutputVertexInterfaceDeclarationBuilder 
		{
		public:
			UE_API FOutputVertexInterfaceDeclarationBuilder(TArray<FOutputDataVertex>& OutVertices);

			template<typename... ArgTypes>
			void Build(ArgTypes&&... InArgs)
			{
				// Size vertices exactly to avoid wasted memory in array allocations
				constexpr uint32 NumVertices = TNumOfTypeInPack<FOutputDataVertex>::Get<ArgTypes...>();
				if constexpr (NumVertices > 0)
				{
					Vertices.Reserve(NumVertices);
				}

				// Add all the elements of the vertex interface. This uses a fold expression
				// to call Add(...) on each input. The various overloads of the Add(...) method
				// then assemble the appropriate structures.
				(Add(Forward<ArgTypes>(InArgs)), ...);
			}

		private:

			UE_API void Add(FOutputDataVertex&& InVertex);

			TArray<FOutputDataVertex>& Vertices;
		};

		/** Interface builder for FEnvironmentInterface declarations. */
		class FEnvironmentDeclarationBuilder 
		{
		public:
			UE_API FEnvironmentDeclarationBuilder(TArray<FEnvironmentVertex>& OutVertices);

			template<typename... ArgTypes>
			void Build(ArgTypes&&... InArgs)
			{
				// Size vertices exactly to avoid wasted memory in array allocations
				constexpr uint32 NumVertices = TNumOfTypeInPack<FEnvironmentVertex>::Get<ArgTypes...>();
				if constexpr (NumVertices > 0)
				{
					Vertices.Reserve(NumVertices);
				}

				// Add all the elements of the vertex interface. 
				(Vertices.Emplace(MoveTemp(InArgs)), ...);
			}
		private:

			TArray<FEnvironmentVertex>& Vertices;
		};
	}

	/** Interface representing the inputs of a node. */
	class FInputVertexInterface : public TVertexInterfaceImpl<FInputDataVertex>
	{
	public:
		UE_API FInputVertexInterface();

		/** Construct an FInputVertexInterface from a parameter pack. This allows 
		 * node interfaces to be declared in a single function.
		 *
		 * For example:
		 * 
		 *  FInputVertexInterface Interface
		 *  {
		 *  	TInputDataVertex<float>(...),
	 	 * 		TInputDataVertex<int32>(...),
		 *  	TInputDataVertex<float>(...)
		 *  };
		 */
		template<typename... ArgTypes>
		explicit FInputVertexInterface(ArgTypes && ... InArgs)
		{
			VertexPrivate::FInputVertexInterfaceDeclarationBuilder InterfaceBuilder(Vertices);
			InterfaceBuilder.Build(Forward<ArgTypes>(InArgs)...);
		}

		UE_API FInputVertexInterface(TArray<FInputDataVertex> InVertices, TArray<VertexPrivate::FSubInterfaceLayout> InSubInterfaces={});

		/** Iterate through all repetitions of a sub interface. */
		UE_API void ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TConstArrayView<FInputDataVertex>)> Callable) const;
		/** Iterate through all repetitions of a sub interface. */
		UE_API void ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TArrayView<FInputDataVertex>)> Callable);
		UE_API TConstArrayView<VertexPrivate::FSubInterfaceLayout> GetSubInterfaces(const VertexPrivate::FPrivateAccessTag& InTag) const;

	private:
		friend class VertexPrivate::FInputInterfaceConfigurationBuilder;
		UE_API const VertexPrivate::FSubInterfaceLayout* FindSubInterfaceLayout(const FName& InName) const;

		TArray<VertexPrivate::FSubInterfaceLayout> SubInterfaces;
	};

	/** Interface representing the outputs of a node. */
	class FOutputVertexInterface : public TVertexInterfaceImpl<FOutputDataVertex>
	{
	public:
		UE_API FOutputVertexInterface();

		/** Construct an FOutputVertexInterface from a parameter pack. This allows 
		 * node interfaces to be declared in a single function.
		 *
		 * For example:
		 * 
		 *  FOutputVertexInterface Interface
		 *  {
		 *  	TOutputDataVertex<float>(...),
		 *  	TOutputDataVertex<int32>(...),
		 *  	TOutputDataVertex<float>(...)
		 *  };
		 */
		template<typename... ArgTypes>
		explicit FOutputVertexInterface(ArgTypes && ... InArgs)
		: FOutputVertexInterface()
		{
			VertexPrivate::FOutputVertexInterfaceDeclarationBuilder InterfaceBuilder(Vertices);
			InterfaceBuilder.Build(Forward<ArgTypes>(InArgs)...);
		}

		UE_API FOutputVertexInterface(TArray<FOutputDataVertex> InVertices, TArray<VertexPrivate::FSubInterfaceLayout> InSubInterfaces={});

		/** Iterate through all repetitions of a sub interface. */
		UE_API void ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TConstArrayView<FOutputDataVertex>)> Callable) const;
		/** Iterate through all repetitions of a sub interface. */
		UE_API void ForEachSubInterfaceInstance(const FName& InSubInterfaceName, TFunctionRef<void (TArrayView<FOutputDataVertex>)> Callable);

		UE_API TConstArrayView<VertexPrivate::FSubInterfaceLayout> GetSubInterfaces(const VertexPrivate::FPrivateAccessTag& InTag) const;

	private:

		friend class VertexPrivate::FOutputInterfaceConfigurationBuilder;

		UE_API const VertexPrivate::FSubInterfaceLayout* FindSubInterfaceLayout(const FName& InName) const;

		TArray<VertexPrivate::FSubInterfaceLayout> SubInterfaces;
	};

	/** Interface representing the environment variables used by a node. */
	class FEnvironmentVertexInterface : public TVertexInterfaceImpl<FEnvironmentVertex>
	{
	public:
		FEnvironmentVertexInterface() = default;

		template<typename... ArgTypes>
		explicit FEnvironmentVertexInterface(ArgTypes && ... InArgs)
		{
			VertexPrivate::FEnvironmentDeclarationBuilder InterfaceBuilder(Vertices);
			InterfaceBuilder.Build(Forward<ArgTypes>(InArgs)...);
		}

		UE_API explicit FEnvironmentVertexInterface(TArray<FEnvironmentVertex> InVertices);
	};

	/** FVertexInterface provides access to a collection of input and output vertex
	 * interfaces. 
	 */
	class FVertexInterface
	{
		public:

			/** Default constructor. */
			UE_API FVertexInterface();

			/** Construct with an input and output interface. */
			UE_API FVertexInterface(const FInputVertexInterface& InInputs, const FOutputVertexInterface& InOutputs);

			/** Construct with input, output and environment interface. */
			UE_API FVertexInterface(FInputVertexInterface InInputs, FOutputVertexInterface InOutputs, FEnvironmentVertexInterface InEnvironmentVariables);

			/** Destructor. */
			UE_API ~FVertexInterface();

			/** Return the input interface. */
			UE_API const FInputVertexInterface& GetInputInterface() const;

			/** Return the input interface. */
			UE_API FInputVertexInterface& GetInputInterface();

			/** Return an input vertex. */
			UE_API const FInputDataVertex& GetInputVertex(const FVertexName& InKey) const;

			/** Returns true if an input vertex with the given key exists. */
			UE_API bool ContainsInputVertex(const FVertexName& InKey) const;

			/** Return the output interface. */
			UE_API const FOutputVertexInterface& GetOutputInterface() const;

			/** Return the output interface. */
			UE_API FOutputVertexInterface& GetOutputInterface();

			/** Return an output vertex. */
			UE_API const FOutputDataVertex& GetOutputVertex(const FVertexName& InName) const;

			/** Returns true if an output vertex with the given name exists. */
			UE_API bool ContainsOutputVertex(const FVertexName& InName) const;

			/** Return the output interface. */
			UE_API const FEnvironmentVertexInterface& GetEnvironmentInterface() const;

			/** Return the output interface. */
			UE_API FEnvironmentVertexInterface& GetEnvironmentInterface();

			/** Return an output vertex. */
			UE_API const FEnvironmentVertex& GetEnvironmentVertex(const FVertexName& InKey) const;

			/** Returns true if an output vertex with the given key exists. */
			UE_API bool ContainsEnvironmentVertex(const FVertexName& InKey) const;

			/** Test for equality between two interfaces. */
			friend bool METASOUNDGRAPHCORE_API operator==(const FVertexInterface& InLHS, const FVertexInterface& InRHS);

			/** Test for inequality between two interfaces. */
			friend bool METASOUNDGRAPHCORE_API operator!=(const FVertexInterface& InLHS, const FVertexInterface& InRHS);

		private:

			FInputVertexInterface InputInterface;
			FOutputVertexInterface OutputInterface;
			FEnvironmentVertexInterface EnvironmentInterface;
	};

	/** A description of a sub interface which is used when declaring a FClassInterface */
	struct FSubInterfaceDescription
	{
		FName SubInterfaceName;
		uint32 Min; 		//< Minimum number of instances of the sub interface. 
		uint32 Max; 		//< Maximum number of instances of the sub interface. 
		uint32 NumDefault; 	//< Default number of instances of the sub interface if desired num is unspecified. 
	};



	/** The data type name used for a variant vertex which has not been given
	 * a concrete MetaSound data type. */
	extern UE_API const FLazyName GVariantDataTypeName;

	/** A variant vertex represents a vertex that has a variant data type. Vertices
	 * are associated with a variant by using the VariantName. Multiple vertices
	 * may share the same variant. */
	class FVariantVertex
	{
	protected:
		
		// Derived classes using this constructor must remember to initialize the DataTypeNames array
		UE_API FVariantVertex(const FVertexName& InVertexName, const FVertexName& InVariantName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType);

	public:

		UE_API FVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const TArray<FName>& InDataTypeNames, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType);
		 
		/** Name of vertex. */
		FVertexName VertexName;

		/** Name of variant associated with the vertex. */
		FName VariantName;

		/** Possible data types supported by the variant. */
		TArray<FName> DataTypeNames;

#if WITH_EDITORONLY_DATA
		/** Metadata associated with vertex. */
		FDataVertexMetadata Metadata;
#endif // WITH_EDITORONLY_DATA

		/** Access type of the vertex. */
		EVertexAccessType AccessType;
	};

	/** An input variant vertex represents an input vertex that has a variant data 
	 * type. Vertices are associated with a variant by using the VariantName. 
	 * Multiple vertices may share the same variant. */
	class FInputVariantVertex : public FVariantVertex
	{
	protected:

		// Derived classes using this constructor must remember to initialize the DataTypeNames array and Literals array.
		UE_API FInputVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType);

	public:
		UE_API FInputVariantVertex(const FVertexName& InVertexName, const FName& InVariantName, const TArray<FName>& InDataTypeNames, const FDataVertexMetadata& InMetadata, EVertexAccessType InAccessType, const TArray<FLiteral>& InLiterals);

		TArray<FLiteral> Literals;

		friend bool METASOUNDGRAPHCORE_API operator==(const FInputVariantVertex& InLHS, const FInputVariantVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FInputVariantVertex& InLHS, const FInputVariantVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator<(const FInputVariantVertex& InLHS, const FInputVariantVertex& InRHS);
	};

	/** An output variant vertex represents an output vertex that has a variant data 
	 * type. Vertices are associated with a variant by using the VariantName. 
	 * Multiple vertices may share the same variant. */
	class FOutputVariantVertex : public FVariantVertex
	{
	public:
		using FVariantVertex::FVariantVertex;

		friend bool METASOUNDGRAPHCORE_API operator==(const FOutputVariantVertex& InLHS, const FOutputVariantVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator!=(const FOutputVariantVertex& InLHS, const FOutputVariantVertex& InRHS);
		friend bool METASOUNDGRAPHCORE_API operator<(const FOutputVariantVertex& InLHS, const FOutputVariantVertex& InRHS);
	};

	namespace VertexPrivate
	{
		template<typename ... DataTypes>
		void BuildArrayOfTypeNames(TArray<FName>& InOutArray)
		{
			checkf(InOutArray.Num() == 0, TEXT("Initial array must be empty"));

			InOutArray.Reserve(sizeof...(DataTypes));
			( InOutArray.Add(GetMetasoundDataTypeName<DataTypes>()), ... );
		}

		template<typename ... ArgTypes>
		void BuildArrayOfLiterals(int32 InNumDataTypes, TArray<FLiteral>& InOutArray, ArgTypes&&...Args)
		{
			constexpr int32 NumArgs = sizeof...(ArgTypes);

			checkf(InOutArray.Num() == 0, TEXT("Initial array must be empty"));
			checkf(InNumDataTypes >= NumArgs, TEXT("The number of default arguments (%d) must be less than or equal to the number of data types (%d)"), NumArgs, InNumDataTypes);

			// Create literals for each supplied arg. 
			InOutArray.Reserve(InNumDataTypes);
			( InOutArray.Add(FLiteral{Args}), ... );

			// If there were fewer args than data types, use a default FLiteral for remaining data types. 
			int32 NumRemaining = InNumDataTypes - NumArgs;
			if (NumRemaining > 0)
			{
				InOutArray.AddDefaulted(NumRemaining);
			}
		}
	}

	/** An input variant vertex represents an input vertex that has a variant data 
	 * type. Vertices are associated with a variant by using the VariantName. 
	 * Multiple vertices may share the same variant. The supported data types of
	 * the variant are declared by the template arguments to the vertex. 
	 *
	 * TInputVariantVertex<float, int32>(...) will produce a variant which supports
	 * float or int32.
	 */
	template<typename ... DataTypes>
	class TInputVariantVertex : public FInputVariantVertex
	{
	public:
		TInputVariantVertex() = default;

		template<typename... ArgTypes>
		FORCENOINLINE TInputVariantVertex(const FVertexName& InVertexName, const FVertexName& InVariantName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: FInputVariantVertex(InVertexName, InVariantName, InMetadata, EVertexAccessType::Reference)
		{
			using namespace VertexPrivate;

			constexpr int32 NumDataTypes = sizeof...(DataTypes);
			BuildArrayOfTypeNames<DataTypes...>(DataTypeNames);
			BuildArrayOfLiterals(NumDataTypes, Literals, Forward<ArgTypes>(Args)...);
		}

		/** Vertex w/ variant name matching vertex name. */
		template<typename... ArgTypes>
		FORCENOINLINE TInputVariantVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata, ArgTypes&&... Args)
		: TInputVariantVertex(InVertexName, InVertexName, InMetadata, Forward<ArgTypes>(Args)...)
		{
		}
	};

	/** An output variant vertex represents an output vertex that has a variant data 
	 * type. Vertices are associated with a variant by using the VariantName. 
	 * Multiple vertices may share the same variant. The supported data types of
	 * the variant are declared by the template arguments to the vertex. 
	 *
	 * TOutputVariantVertex<float, int32>(...) will produce a variant which supports
	 * float or int32.
	 */
	template<typename ... DataTypes>
	class TOutputVariantVertex : public FOutputVariantVertex
	{
	public:
		TOutputVariantVertex() = default;

		FORCENOINLINE TOutputVariantVertex(const FVertexName& InVertexName, const FVertexName& InVariantName, const FDataVertexMetadata& InMetadata)
		: FOutputVariantVertex(InVertexName, InVariantName, InMetadata, EVertexAccessType::Reference)
		{
			using namespace VertexPrivate;

			constexpr int32 NumDataTypes = sizeof...(DataTypes);
			BuildArrayOfTypeNames<DataTypes...>(DataTypeNames);
		}

		/** Vertex with variant name matching vertex name. */
		FORCENOINLINE TOutputVariantVertex(const FVertexName& InVertexName, const FDataVertexMetadata& InMetadata)
		: TOutputVariantVertex(InVertexName, InVertexName, InMetadata)
		{
		}
	};

	/** FClassInterface describes the interface of a node class. It is a
	 * factory for FVertexInterfaces. */
	class FClassInterface;

	class FSubInterface;

	namespace VertexPrivate
	{
		// Data structure used to efficiently store class declarations w/o introducing
		// unneeded allocations. FClassInterfaceData generally only lives for the
		// duration of a function call. 
		struct FClassInterfaceData
		{
			TArray<FSubInterfaceDescription, TInlineAllocator<4>> SubInterfaces;
			TArray<FSubInterfaceClassLayout, TInlineAllocator<2>> InputSubInterfaceLayouts;
			TArray<FSubInterfaceClassLayout, TInlineAllocator<2>> OutputSubInterfaceLayouts;

			TSortedMap<FVertexKey, uint8, TInlineAllocator<4>> VertexKeyToVariantIndex;
			TArray<FVariantDescription, TInlineAllocator<4>> Variants;
			TSortedVertexNameMap<TArray<FLiteral, TInlineAllocator<4>>, TInlineAllocator<4>> VariantLiterals;

			TArray<FInputDataVertex, TInlineAllocator<8>> Inputs;
			TArray<FOutputDataVertex, TInlineAllocator<8>> Outputs;
			TArray<FEnvironmentVertex, TInlineAllocator<4>> EnvironmentVariables;
		};

		// Stores information to describe variants on a class interface
		struct FVariantSchema
		{
			TSortedMap<VertexPrivate::FVertexKey, uint8> VertexKeyToVariantIndex;
			TArray<FVariantDescription> Variants;
			TSortedVertexNameMap<TArray<FLiteral, TInlineAllocator<2>>> VariantLiterals;

			UE_API static FVariantSchema FromClassInterfaceData(const VertexPrivate::FClassInterfaceData& InClassInterfaceData);
		};

		// Stores information to describe sub interfaces on a class interface. 
		struct FSubInterfaceSchema
		{
			TArray<FSubInterfaceDescription> SubInterfaces;
			TArray<VertexPrivate::FSubInterfaceClassLayout> InputSubInterfaceLayouts;
			TArray<VertexPrivate::FSubInterfaceClassLayout> OutputSubInterfaceLayouts;

			UE_API static FSubInterfaceSchema FromClassInterfaceData(const VertexPrivate::FClassInterfaceData& InClassInterfaceData);
		};

		/** The FClassInterfaceDataBuilder assembles declared inputs, outputs,
		 * variants and subinterfaces into a FClassInterfaceData object. This is
		 * largely to simply the task of node authors when they declare a nodes
		 * interface. Instead of building an interface sequentially, they can
		 * declare the interface and allow the builder to build the correct object.
		 *
		 * e.g:
		 *
		 * FClassInterface ClassInterface = CreateClassInterface(
		 * 	TInputDataVertex<float>(...)
		 * 	FSubInterface{
		 * 		"MySub",
		 * 		TInputVariantVertex<float, int32>(...)
		 * 	}
		 * 	TOutDataVertex<FAudioBuffer>(...)
		 * );
		 */
		class FClassInterfaceDataBuilder
		{
		public:
			template<typename ... Args>
			FClassInterfaceDataBuilder& Build(Args&& ... InArgs)
			{
				(this->Add(&InArgs), ...);
				return *this;
			}

			UE_API const FClassInterfaceData& GetClassInterfaceData() const;
			
		private:
			UE_API void Add(const FInputDataVertex* InInputVertex);
			UE_API void Add(const FOutputDataVertex* InOutputVertex);
			UE_API void Add(const FEnvironmentVertex* InEnvironmentVertex);

			UE_API void Add(const FSubInterfaceDescription* InSubInterfaceDescription);
			UE_API void Add(const FSubInterface* InSubInterface);
			UE_API void Add(const FInputVariantVertex* InInputVariantVertex);
			UE_API void Add(const FOutputVariantVertex* InOutputVariantVertex);

			bool ContainsAnySubInterfaces() const;
			void AddSubInterfaceVertices(const FName& InSubInterfaceName, const FClassInterfaceData& InSubInterfaceData);
			void MergeSubInterfaceVariantData(const FClassInterfaceData& InSubInterfaceData);

			uint8 FindOrAddVariant(const FVariantVertex& InVariantVertex);
			int32 IndexOfVariant(const FName& InName) const;

			FClassInterfaceData ClassInterfaceData;
		};

		/** FBuildableInterface supports composition when declaring FSubInterfaces */
		class FBuildableInterface
		{
		public:
			FBuildableInterface() = default;
			FBuildableInterface(const FBuildableInterface&) = default;
			FBuildableInterface(FBuildableInterface&&) = default;

			template<class FirstArgType, class... AdditionalArgTypes>
			FBuildableInterface(FirstArgType&& InFirstArg, AdditionalArgTypes&&... InAdditionalArgs)
			{
				Builder.Build(Forward<FirstArgType>(InFirstArg), Forward<AdditionalArgTypes>(InAdditionalArgs)...);
			}

			UE_INTERNAL
			UE_API const FClassInterfaceDataBuilder& GetClassInterfaceDataBuilder() const;

		private:
			FClassInterfaceDataBuilder Builder;
		};
	} // namespace VertexPrivate;

	
	class FSubInterface : public VertexPrivate::FBuildableInterface
	{
	public:
		template<typename ... ArgTypes>
		FSubInterface(FLazyName InName, ArgTypes&&... InArgs)
		: FBuildableInterface(Forward<ArgTypes>(InArgs)...)
		, SubInterfaceName(InName)
		{
		}

		UE_API const FName GetName() const;
	private:

		FName SubInterfaceName;
	};

	/** An FClassInterface defines the class interface of a MetaSound C++ node class. This
	 * interface may contain basic vertices, but also subinterfaces and variants. 
	 *
	 * The FClassInterface is used to create the FVertexInterface for individual 
	 * node instances. It does so by combining the FClassInterface definition with
	 * subinterface and variant configurations. */
	class FClassInterface
	{
	public:
		FClassInterface() = default;
		~FClassInterface() = default;

		/** Copy constructor (deep-copies schema data). */
		UE_API FClassInterface(const FClassInterface& Other);

		/** Copy assignment (deep-copies schema data). */
		UE_API FClassInterface& operator=(const FClassInterface& Other);

		/** Move constructor. */
		FClassInterface(FClassInterface&&) = default;

		/** Move assignment. */
		FClassInterface& operator=(FClassInterface&&) = default;

		/** Construct from a FVertexInterface. The resulting FClassInterface has
		 * no sub-interfaces or variants — it simply wraps the flat vertex lists. */
		UE_API FClassInterface(const FVertexInterface& InVertexInterface);

		/** Conversion to FVertexInterface. Returns the default-configured
		 * interface (no sub-interface or variant configurations). */
		UE_API operator FVertexInterface() const;

		UE_INTERNAL
		UE_API FClassInterface(const VertexPrivate::FClassInterfaceData& InClassInterfaceData);

		/** Create a FVertexInterface with default configuration (no sub-interface or variant overrides). */
		UE_API FVertexInterface CreateVertexInterface() const;

		/** Create a FVertexInterface with the given configuration. */
		UE_API FVertexInterface CreateVertexInterface(const TMap<FName, uint32>& InSubInterfaceCounts, const TMap<FName, FName>& InVariantSelections) const;

		/** Return any sub interfaces on this class interface. */
		UE_API TConstArrayView<FSubInterfaceDescription> GetSubInterfaceDescriptions() const;

		/** Return any variants on this class interface. */
		UE_API TConstArrayView<FVariantDescription> GetVariantDescriptions() const;

		/** Return a variant description for a given input vertex if the vertex
		 * is associated with a variant. */
		UE_API const FVariantDescription* FindVariantDescriptionForInput(const FVertexName& InVertexName) const;

		/** Return a variant description for a given output vertex if the vertex
		 * is associated with a variant. */
		UE_API const FVariantDescription* FindVariantDescriptionForOutput(const FVertexName& InVertexName) const;

		/** Returns true if this class interface contains sub-interface definitions. */
		bool ContainsSubInterfaces() const { return SubInterfaceSchema && !SubInterfaceSchema->SubInterfaces.IsEmpty(); }

		/** Returns true if this class interface contains variant definitions. */
		bool ContainsVariants() const { return VariantSchema && !VariantSchema->Variants.IsEmpty(); }

	private:

		// Many nodes do not need this extra configurable information. Save on memory
		// storage by offloading this extra functionality behind a unique ptr.
		// Deep-copied in copy constructor/assignment to preserve value semantics.
		TUniquePtr<VertexPrivate::FVariantSchema> VariantSchema;
		TUniquePtr<VertexPrivate::FSubInterfaceSchema> SubInterfaceSchema;

		TArray<FInputDataVertex, TInlineAllocator<1>> Inputs;
		TArray<FOutputDataVertex, TInlineAllocator<1>> Outputs;
		TArray<FEnvironmentVertex> EnvironmentVariables;
	};

	/** Create a FClassInterface by declaring the vertices, subinterfaces and variants.
	 * e.g:
	 *
	 * FClassInterface ClassInterface = CreateClassInterface(
	 * 	TInputDataVertex<float>(...)
	 * 	FSubInterface{
	 * 		"MySub",
	 * 		TInputVariantVertex<float, int32>(...)
	 * 	}
	 * 	TOutDataVertex<FAudioBuffer>(...)
	 * );
	 */
	template<typename ... ArgTypes>
	FClassInterface CreateClassInterface(ArgTypes&& ... InArgs)
	{
		return FClassInterface{VertexPrivate::FClassInterfaceDataBuilder().Build(Forward<ArgTypes>(InArgs)...).GetClassInterfaceData()};
	}

	/**
	 * This struct is used to pass in any arguments required for constructing a single node instance.
	 * because of this, all FNode implementations have to implement a constructor that takes an FNodeInitData instance.
	 */
	struct FNodeInitData
	{
		FVertexName InstanceName;
		FGuid InstanceID;
	};
}

/** Convert EVertexAccessType to string */
METASOUNDGRAPHCORE_API FString LexToString(Metasound::EVertexAccessType InAccessType);

#undef UE_API
