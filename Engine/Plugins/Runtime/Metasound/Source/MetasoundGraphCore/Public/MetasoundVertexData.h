// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AllOf.h"
#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundLog.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundVertex.h"
#include "Traits/ElementType.h"
#include "MetasoundPolymorphic.h"

#include <type_traits>

#define UE_API METASOUNDGRAPHCORE_API

/** Enable runtime tests for compatible access types between vertex access types
 * and data references bound to the vertex. */
#define ENABLE_METASOUND_ACCESS_TYPE_COMPATIBILITY_TEST DO_CHECK

namespace Metasound
{
	namespace VertexDataPrivate
	{
		// Tests to see that the access type of the data reference is compatible 
		// with the access type of the vertex.
#if ENABLE_METASOUND_ACCESS_TYPE_COMPATIBILITY_TEST
		void METASOUNDGRAPHCORE_API CheckAccessTypeCompatibility(const FDataVertex& InDataVertex, const FAnyDataReference& InDataReference);
#else
		inline void CheckAccessTypeCompatibility(const FDataVertex& InDataVertex, const FAnyDataReference& InDataReference) {}
#endif // #if DO_CHECK

		METASOUNDGRAPHCORE_API EVertexAccessType DataReferenceAccessTypeToVertexAccessType(EDataReferenceAccessType InReferenceAccessType);

		// Helper for getting a EVertexAccessType from a TData*Reference<DataType> object
		template<typename ...>
		struct TGetVertexAccess
		{
		};

		template<typename DataType>
		struct TGetVertexAccess<TDataReadReference<DataType>>
		{
			static constexpr EVertexAccessType VertexAccess = EVertexAccessType::Reference;
		};

		template<typename DataType>
		struct TGetVertexAccess<TDataWriteReference<DataType>>
		{
			static constexpr EVertexAccessType VertexAccess = EVertexAccessType::Reference;
		};

		template<typename DataType>
		struct TGetVertexAccess<TDataValueReference<DataType>>
		{
			static constexpr EVertexAccessType VertexAccess = EVertexAccessType::Value;
		};

		/** TGetReferencDataTypeT is a template trait which determines the underlying
		 * type stored in a data reference
		 */
		template<typename ...>
		struct TGetReferenceDataTypeT
		{
			static_assert("Invalid template parameter. This must be instantiated with a TDataReadReference<>, TDataWriteReference or TDataValueReference");
		};

		template<typename DataType>
		struct TGetReferenceDataTypeT<TDataReadReference<DataType>>
		{
			using Type = DataType;
		};

		template<typename DataType>
		struct TGetReferenceDataTypeT<TDataWriteReference<DataType>>
		{
			using Type = DataType;
		};

		template<typename DataType>
		struct TGetReferenceDataTypeT<TDataValueReference<DataType>>
		{
			using Type = DataType;
		};
		
		/** TGetReferencDataType is the underlying type stored in a data reference */
		template<typename T>
		using TGetReferenceDataType = typename TGetReferenceDataTypeT<T>::Type;

		/** TFactoryFacace is a factory for data types which uses the class template
		 * type to determine which flavor of DataReference to create. 
		 */
		template<typename Type>
		struct TFactoryFacade
		{
			static Type CreateWithLiteral(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			{
				return TDataTypeLiteralFactory<Type>::CreateExplicitArgs(InSettings, InLiteral);
			}

			template<typename ... ArgTypes>
			static Type CreateWithArgs(ArgTypes&&... Args)
			{
				return Type(Forward<ArgTypes>(Args)...);
			}
		};

		template<typename DataType>
		struct TFactoryFacade<TDataReadReference<DataType>>
		{
			static TDataReadReference<DataType> CreateWithLiteral(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			{
				return TDataReadReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral);
			}

			template<typename ... ArgTypes>
			static TDataReadReference<DataType> CreateWithArgs(ArgTypes&&... Args)
			{
				return TDataReadReference<DataType>::CreateNew(Forward<ArgTypes>(Args)...);
			}
		};

		template<typename DataType>
		struct TFactoryFacade<TDataWriteReference<DataType>>
		{
			static TDataWriteReference<DataType> CreateWithLiteral(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			{
				return TDataWriteReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral);
			}

			template<typename ... ArgTypes>
			static TDataWriteReference<DataType> CreateWithArgs(ArgTypes&&... Args)
			{
				return TDataWriteReference<DataType>::CreateNew(Forward<ArgTypes>(Args)...);
			}
		};

		template<typename DataType>
		struct TFactoryFacade<TDataValueReference<DataType>>
		{
			static TDataValueReference<DataType> CreateWithLiteral(const FOperatorSettings& InSettings, const FLiteral& InLiteral)
			{
				return TDataValueReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, InLiteral);
			}

			template<typename ... ArgTypes>
			static TDataValueReference<DataType> CreateWithArgs(ArgTypes&&... Args)
			{
				return TDataValueReference<DataType>::CreateNew(Forward<ArgTypes>(Args)...);
			}
		};

		template<typename T>
		struct TTriggerDeprecatedBindAPIWarning
		{
		};

		template<typename T>
		struct TTriggerDeprecatedBindAPIWarning<const T>
		{
			UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
			inline void Trigger() 
			{
			}
		};
		
		// An input binding which connects an FInputVertex to a IDataReference
		class FInputBinding
		{
		public:

			UE_API FInputBinding(FInputDataVertex&& InVertex);
			UE_API FInputBinding(const FInputDataVertex& InVertex);
			UE_API FInputBinding(const FVertexName& InVertexName, FAnyDataReference&& InReference);

			template<typename DataReferenceType>
			void BindRead(DataReferenceType& InOutDataReference)
			{
				if (Data.IsSet())
				{
					InOutDataReference = Data->GetAs<DataReferenceType>();
				}
				else
				{
					Set(TDataReadReference<TGetReferenceDataType<DataReferenceType>>(InOutDataReference));
				}
			}

			template<typename DataType>
			void BindWrite(TDataWriteReference<DataType>& InOutDataReference)
			{
				if (Data.IsSet())
				{
					InOutDataReference = Data->GetDataWriteReference<DataType>();
				}
				else
				{
					Set(InOutDataReference);
				}
			}

			template<typename DataReferenceType>
			void Bind(DataReferenceType& InOutDataReference)
			{
				if constexpr (std::is_const_v<DataReferenceType>)
				{
					// Bind operations need to be able to modify the provided
					// data reference in place. A prior API had performed binding
					// on const references without modifications. That API has 
					// since been deprecated. 
					//
					// If you trigger this deprecation warning, it is likely 
					// that the code is using the deprecated API, or that it is
					// incidentally calling this API because it passes a temporary 
					// object to this function which then is bound to a const reference.
					TTriggerDeprecatedBindAPIWarning<DataReferenceType>::Trigger();
					Bind(const_cast<std::remove_const_t<DataReferenceType>&>(InOutDataReference));
				}
				else
				{
					if (Data.IsSet())
					{
						InOutDataReference = Data->GetAs<DataReferenceType>();
					}
					else
					{
						Set(InOutDataReference);
					}
				}
			}

			UE_API void Bind(FInputBinding& InBinding);

			// Set the data reference, overwriting any existing bound data references
			template<typename DataReferenceType>
			void Set(const DataReferenceType& InDataReference)
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
				check(Metasound::IsCastable(InDataReference.GetDataTypeName(), Vertex.DataTypeName));
				PRAGMA_ENABLE_INTERNAL_WARNINGS
				
				Data.Emplace(InDataReference);
				CheckAccessTypeCompatibility(Vertex, *Data);
			}

			// Set the data reference, overwriting any existing bound data references
			UE_API void Set(FAnyDataReference&& InAnyDataReference);

			UE_API const FInputDataVertex& GetVertex() const;

			UE_API void SetDefaultLiteral(const FLiteral& InLiteral);

			UE_API bool IsBound() const;

			UE_API EDataReferenceAccessType GetAccessType() const;

			UE_API const FAnyDataReference* GetDataReference() const;

			UE_API FDataReferenceID GetDataReferenceID() const;

			// Get the bound data reference. This assumes the data reference exists
			// and is convertible to the AsType.
			template<typename AsType>
			AsType GetDataReferenceAs() const
			{
				check(Data.IsSet());
				return Data->GetAs<AsType>();
			}

			// Get the bound data reference if it exists. Otherwise create and 
			// return a data reference by constructing one using the Vertex's 
			// default literal.
			template<typename AsType>
			AsType GetOrCreateDefaultAs(const FOperatorSettings& InSettings) const
			{
				if (Data.IsSet())
				{
					return GetDataReferenceAs<AsType>();
				}
				else
				{
					return TFactoryFacade<AsType>::CreateWithLiteral(InSettings, Vertex.GetDefaultLiteral());
				}
			}

			// Get the bound data reference if it exists. Otherwise create and 
			// return a data reference by constructing one the supplied constructor 
			// arguments.
			template<typename AsType, typename ... ArgTypes>
			AsType GetOrConstructAs(ArgTypes&&... Args) const
			{
				if (Data.IsSet())
				{
					return GetDataReferenceAs<AsType>();
				}
				else
				{
					return TFactoryFacade<AsType>::CreateWithArgs(Forward<ArgTypes>(Args)...);
				}
			}


			// Set the value with a constant value reference.
			template<typename DataType>
			void SetValue(const DataType& InValue)
			{
				check(Vertex.DataTypeName == GetMetasoundDataTypeName<DataType>());
				Data.Emplace(TDataValueReference<DataType>::CreateNew(InValue));
				CheckAccessTypeCompatibility(Vertex, *Data);
			}

		private:

			FInputDataVertex Vertex;
			TOptional<FAnyDataReference> Data;
		};


		// Binds an IDataReference to a FOutputDataVertex
		class FOutputBinding
		{
		public:
			UE_API FOutputBinding(FOutputDataVertex&& InVertex);
			UE_API FOutputBinding(const FOutputDataVertex& InVertex);
			UE_API FOutputBinding(const FVertexName& InVertexName, FAnyDataReference&& InReference);

			template<typename DataType>
			void BindValue(const TDataValueReference<DataType>& InOutDataReference)
			{
				Set(InOutDataReference);
			}

			template<typename DataReferenceType>
			void BindRead(DataReferenceType& InOutDataReference)
			{
				Set(TDataReadReference<TGetReferenceDataType<DataReferenceType>>(InOutDataReference));
			}

			template<typename DataType>
			void BindReadFromAny(FAnyDataReference& InOutDataReference)
			{
				Set(InOutDataReference.GetAs<TDataReadReference<DataType>>());
			}

			template<typename DataType>
			void BindWrite(TDataWriteReference<DataType>& InOutDataReference)
			{
				Set(InOutDataReference);
			}

			template<typename DataReferenceType>
			void Bind(DataReferenceType& InOutDataReference)
			{
				if constexpr (std::is_const_v<DataReferenceType>)
				{
					// Bind operations need to be able to modify the provided
					// data reference in place. A prior API had performed binding
					// on const references without modifications. That API has 
					// since been deprecated. 
					//
					// If you trigger this deprecation warning, it is likely 
					// that the code is using the deprecated API, or that it is
					// incidentally calling this API because it passes a temporary 
					// object to this function which then is bound to a const reference.
					TTriggerDeprecatedBindAPIWarning<DataReferenceType>::Trigger();
					Bind(const_cast<std::remove_const_t<DataReferenceType>&>(InOutDataReference));
				}
				else
				{
					Set(InOutDataReference);
				}
			}

			UE_API void Bind(FOutputBinding& InBinding);

			// Set the data reference, overwriting any existing bound data references
			template<typename DataReferenceType>
			void Set(const DataReferenceType& InDataReference)
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
				checkf(Metasound::IsCastable(InDataReference.GetDataTypeName(), Vertex.DataTypeName),
					TEXT("Ref=%s, Vertex=%s"), *InDataReference.GetDataTypeName().ToString(), *Vertex.DataTypeName.ToString());
				PRAGMA_ENABLE_INTERNAL_WARNINGS				
				Data.Emplace(InDataReference);
				CheckAccessTypeCompatibility(Vertex, *Data);
			}

			// Set the data reference, overwriting any existing bound data references
			UE_API void Set(FAnyDataReference&& InAnyDataReference);
			
			UE_API const FOutputDataVertex& GetVertex() const;

			UE_API bool IsBound() const;

			UE_API EDataReferenceAccessType GetAccessType() const;

			// Get data reference 
			UE_API const FAnyDataReference* GetDataReference() const;

			UE_API FDataReferenceID GetDataReferenceID() const;

			// Get the bound data reference. This assumes the data reference exists
			// and is convertible to the AsType.
			template<typename AsType>
			AsType GetDataReferenceAs() const
			{
				check(Data.IsSet());
				return Data->GetAs<AsType>();
			}

			// Get the bound data reference if it exists. Otherwise create and 
			// return a data reference by constructing one the supplied constructor 
			// arguments.
			template<typename AsType, typename ... ArgTypes>
			AsType GetOrConstructAs(ArgTypes&&... Args) const
			{
				if (Data.IsSet())
				{
					return GetDataReferenceAs<AsType>();
				}
				else
				{
					return TFactoryFacade<AsType>::CreateWithArgs(Forward<ArgTypes>(Args)...);
				}
			}

			// Set the value with a constant value reference.
			template<typename DataType>
			void SetValue(const DataType& InValue)
			{
				Data.Emplace(TDataValueReference<DataType>::CreateNew(InValue));
				CheckAccessTypeCompatibility(Vertex, *Data);
			}

		private:

			FOutputDataVertex Vertex;
			TOptional<FAnyDataReference> Data;
		};


		// Functor for creating a new FInputBinding from a IDataReference derived class and a vertex name.
		template<typename DataReferenceType>
		struct TCreateInputBinding
		{
			const FVertexName& VertexName;
			DataReferenceType& Ref;

			TCreateInputBinding(const FVertexName& InVertexName, DataReferenceType& InRef)
			: VertexName(InVertexName)
			, Ref(InRef)
			{}

			FInputBinding operator()() const
			{
				FInputDataVertex Vertex(VertexName, Ref.GetDataTypeName(), FDataVertexMetadata{}, TGetVertexAccess<std::decay_t<DataReferenceType>>::VertexAccess);
				return FInputBinding(Vertex);
			}
		};

		// Functor for creating a new FOutputBinding from a IDataReference derived class and a vertex name.
		template<typename DataReferenceType>
		struct TCreateOutputBinding
		{
			const FVertexName& VertexName;
			DataReferenceType& Ref;

			TCreateOutputBinding(const FVertexName& InVertexName, DataReferenceType& InRef)
			: VertexName(InVertexName)
			, Ref(InRef)
			{}

			FOutputBinding operator()() const
			{
				FOutputDataVertex Vertex(VertexName, Ref.GetDataTypeName(), FDataVertexMetadata{}, TGetVertexAccess<std::decay_t<DataReferenceType>>::VertexAccess);
				return FOutputBinding(Vertex);
			}
		};
	}

	/** MetaSound Binding 
	 *
	 * MetaSound IOperators read and write to shared data. For example: a TDataWriteReference<float> 
	 * written to by one operator may be read by several other operators holding
	 * onto TDataReadReference<float>. Binding empowers sharing of the underlying 
	 * objects (a `float` in the example) between IOperators and minimizing copying
	 * of data. In general, it is assumed that any individual node does not need 
	 * to worry about the specific implications of a binding, but can rather assume 
	 * that data references are managed by external systems. 
	 *
	 * There are 2 known scenarios where an IOperator must be aware of what binding
	 * can do.
	 *
	 * 1. If the IOperator internally caches raw pointers to the underlying data 
	 * of a data reference.
	 *
	 * 2. If the IOperator manages multiple internal IOperators with their own 
	 * shared connections (e.g. FGraphOperator)
	 *
	 * In the case that a IOperator does need to know manage bound connections, it's
	 * important that the IOperator follows the binding rules.
	 *
	 * General Binding Rules:
	 * - Binding cannot change the access type of an existing data reference.
	 * - IOperators will ignore new TDataValueReferences because Value references
	 *   cannot be updated after the operator has been constructed. 
	 *
	 * Input Binding Rules:
	 * - Binding an input may update the underlying data object to point to
	 * a new object.
	 * - When updating input pointers of underlying data, a data reference from
	 * and outer scope should replace that of an inner scope. For example, a graph 
	 * representing multiple IOperators can override the input pointers of individual
	 * IOperators, but an individual IOperator cannot override the input pointers of 
	 * the containing graph.
	 *
	 *
	 * Output Binding Rules:
	 * - Binding an output may NOT update the location of the underlying data object.
	 * - When determining output pointers of underlying data, a data reference from
	 *  the inner scope can override the data reference of an outerscope. For example, 
	 *  a graph representing multiple IOperators cannot override the output pointers 
	 *  of individual IOperators, but an individual IOperator can override the output 
	 *  pointers of the containing graph.
	 *
	 *
	 *  These rules apply to any method on the FInputVertexInterfaData or 
	 *  FOutptuVertexInterfaceData which begins with `Bind`
	 */

	// Forward declare
	struct FVertexDataState;

	/** An input vertex interface with optionally bound data references. */
	class FInputVertexInterfaceData
	{
		using FInputBinding = VertexDataPrivate::FInputBinding;

	public:

		using FRangedForIteratorType = typename TArray<FInputBinding>::RangedForIteratorType;
		using FRangedForConstIteratorType = typename TArray<FInputBinding>::RangedForConstIteratorType;

		/** Construct with an FInputVertexInterface. This will default to a unfrozen vertex interface. */
		UE_API FInputVertexInterfaceData();

		/** Construct with an FInputVertexInterface. This will default to a frozen vertex interface. */
		UE_API FInputVertexInterfaceData(const FInputVertexInterface& InVertexInterface);

		/** Returns true if the vertex interface is frozen. */
		UE_API bool IsVertexInterfaceFrozen() const;

		/** Set whether the vertex interface is frozen or not. 
		 *
		 * If frozen, attempts to access vertices which do not already exist will result in an error. 
		 *
		 * If not frozen, attempts to bind to a missing vertex will automatically
		 * add the missing vertex.
		 */
		UE_API void SetIsVertexInterfaceFrozen(bool bInIsVertexInterfaceFrozen);

		/** Returns true if a vertex exists with the provided name. This does not
		 * reflect whether the vertex is bound or not. */
		UE_API bool Contains(const FVertexName& InVertexName) const;

		/** Set the value of a vertex. */
		template<typename DataType>
		void SetValue(const FVertexName& InVertexName, const DataType& InValue)
		{
			auto CreateBinding = [&]()
			{
				FInputDataVertex Vertex(InVertexName, GetMetasoundDataTypeName<DataType>(), FDataVertexMetadata{}, EVertexAccessType::Value);
				return FInputBinding(Vertex);
			};

			auto BindData = [&](FInputBinding& Binding) { Binding.SetValue<DataType>(InValue); };
			
			Apply(InVertexName, CreateBinding, BindData);
		}

		/** Bind a read vertex from a read reference.*/
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindReadVertex(const FVertexName& InVertexName, const TDataReadReference<DataType>& InOutDataReference)
		{
			BindReadVertex(InVertexName, const_cast<TDataReadReference<DataType>&>(InOutDataReference));
		}

		/** Bind a read vertex from a read reference. */
		template<typename DataType>
		void BindReadVertex(const FVertexName& InVertexName, TDataReadReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FInputBinding& Binding) { Binding.BindRead(InOutDataReference); };
			
			Apply(InVertexName, TCreateInputBinding(InVertexName, InOutDataReference), BindData);
		}
		
		/** Bind a read vertex from a write reference. */
		template<typename DataType>
		void BindReadVertex(const FVertexName& InVertexName, TDataWriteReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			
			auto BindData = [&](FInputBinding& Binding) { Binding.BindRead(InOutDataReference); };
			Apply(InVertexName, TCreateInputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a write vertex from a write reference.*/
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindWriteVertex(const FVertexName& InVertexName, const TDataWriteReference<DataType>& InOutDataReference)
		{
			BindWriteVertex(InVertexName, const_cast<TDataWriteReference<DataType>&>(InOutDataReference));
		}

		/** Bind a write vertex from a write reference. */
		template<typename DataType>
		void BindWriteVertex(const FVertexName& InVertexName, TDataWriteReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FInputBinding& Binding) { Binding.BindWrite(InOutDataReference); };
			Apply(InVertexName, TCreateInputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a vertex with a any data reference. */
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		UE_API void BindVertex(const FVertexName& InVertexName, const FAnyDataReference& InOutDataReference);
		
		/** Bind a vertex with a a data reference. */
		template<typename DataReferenceType>
		void BindVertex(const FVertexName& InVertexName, DataReferenceType& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FInputBinding& Binding) { Binding.Bind(InOutDataReference); };
			Apply(InVertexName, TCreateInputBinding(InVertexName, InOutDataReference), BindData);
		}
		
		/** Bind a vertex with a any data reference. */
		UE_API void BindVertex(const FVertexName& InVertexName, FAnyDataReference& InOutDataReference);

		/** Return the number of instances of a particular sub interface. */
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		UE_API uint32 GetNumSubInterfaceInstances(const FName& InSubInterfaceName) const;

		/** Bind the vertices of a sub interface vertex. 
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			auto BindData = [&InOutDataReferences](uint32 Index, FInputBinding& Binding)
			{
				Binding.Bind(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as write references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceWriteVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			auto BindData = [&InOutDataReferences](uint32 Index, FInputBinding& Binding)
			{
				Binding.BindWrite(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as read references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceReadVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			auto BindData = [&InOutDataReferences](uint32 Index, FInputBinding& Binding)
			{
				Binding.BindRead(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as value references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void SetSubInterfaceValues(const FName& InSubInterfaceName, const FName& InVertexName, const RangeType& InValues)
		{
			auto BindData = [&InValues](uint32 Index, FInputBinding& Binding)
			{
				Binding.SetValue<TElementType_T<RangeType>>(InValues[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InValues.Num(), BindData);
		}

		/** Bind vertex data using other vertex data.*/
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		UE_API void Bind(const FInputVertexInterfaceData& InOutInputVertexData);

		/** Bind vertex data using other vertex data. */
		UE_API void Bind(FInputVertexInterfaceData& InOutInputVertexData);

		/** Sets a vertex to use a data reference, ignoring any existing data bound to the vertex. */
		UE_API void SetVertex(const FVertexName& InVertexName, const FAnyDataReference& InDataReference);
		/** Sets a vertex to use a data reference, ignoring any existing data bound to the vertex. */
		UE_API void SetVertex(const FVertexName& InVertexName, FAnyDataReference&& InDataReference);

		/** Set a data references to vertices with matching vertex names. Ignores any existing data bound to the vertex. */
		UE_DEPRECATED(5.6, "Do not use FDataReferenceCollection.")
		UE_API void Set(const FDataReferenceCollection& InCollection);

		/** Convert vertex data to a data reference collection. */
		UE_DEPRECATED(5.6, "Do not use FDataReferenceCollection.")
		UE_API FDataReferenceCollection ToDataReferenceCollection() const;

		/** Return the vertex associated with the vertex name. */
		UE_API const FInputDataVertex& GetVertex(const FVertexName& InVertexName) const;

		/** Set the default literal used to create default values. */
		UE_API void SetDefaultLiteral(const FVertexName& InVertexName, const FLiteral& InLiteral);

		/** Add a vertex. VertexInterfaceData must be unfrozen. */
		UE_API void AddVertex(const FInputDataVertex& InVertex);

		/** Remove a vertex. VertexInterfaceData must be unfrozen. */
		UE_API void RemoveVertex(const FVertexName& InVertexName);

		/** Returns true if a vertex with the given vertex name exists and is bound
		 * to a data reference. */
		UE_API bool IsVertexBound(const FVertexName& InVertexName) const;

		/** Returns the access type of a bound vertex. If the vertex does not exist
		 * or if it is unbound, this will return EDataReferenceAccessType::None
		 */
		UE_API EDataReferenceAccessType GetVertexDataAccessType(const FVertexName& InVertexName) const;

		/** Returns true if all vertices in the FInputVertexInterface are bound to 
		 * data references. */
		UE_API bool AreAllVerticesBound() const;

		FRangedForIteratorType begin()
		{
			return Bindings.begin();
		}

		FRangedForIteratorType end()
		{
			return Bindings.end();
		}

		FRangedForConstIteratorType begin() const
		{
			return Bindings.begin();
		}

		FRangedForConstIteratorType end() const
		{
			return Bindings.end();
		}

		/** Find data reference bound to vertex. Returns a nullptr if no data reference is bound. */
		UE_API const FAnyDataReference* FindDataReference(const FVertexName& InVertexName) const;

		/** Returns the current value of a vertex. */
		template<typename DataType>
		const DataType* GetValue(const FVertexName& InVertexName) const
		{
			const FInputBinding* Binding = FindChecked(InVertexName);
			if (Binding->IsBound())
			{
				return Binding->GetDataReferenceAs<const DataType*>();
			}
			return nullptr;
		}

		/**  Gets the value of the bound data reference if it exists. Otherwise
		 * create and return a value by constructing one using the Vertex's 
		 * default literal. */
		template<typename DataType>
		DataType GetOrCreateDefaultValue(const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			const FInputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetOrCreateDefaultAs<DataType>(InSettings);
		}

		/**  Gets the value of the bound data reference if it exists. Otherwise
		 * create and return a value by constructing one using the Vertex's 
		 * default literal. */
		template<typename DataType>
		TDataValueReference<DataType> GetOrCreateDefaultDataValueReference(const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			const FInputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetOrCreateDefaultAs<TDataValueReference<DataType>>(InSettings);
		}

		/** Get data read reference assuming data is bound and read or write accessible. */
		template<typename DataType> 
		TDataReadReference<DataType> GetDataReadReference(const FVertexName& InVertexName) const
		{
			const FInputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetDataReferenceAs<TDataReadReference<DataType>>();
		}

		/** Get the bound data read reference if it exists. Otherwise create and 
		 * return a data read reference by constructing one using the Vertex's 
		 * default literal.
		 */
		template<typename DataType> 
		TDataReadReference<DataType> GetOrCreateDefaultDataReadReference(const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			if (const FInputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrCreateDefaultAs<TDataReadReference<DataType>>(InSettings);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataReadReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, FLiteral::CreateInvalid());
		}

		/** Get the bound data read reference if it exists. Otherwise create and 
		 * return a data read reference by constructing one the supplied constructor 
		 * arguments.
		 */
		template<typename DataType, typename ... ConstructorArgTypes>
		UE_DEPRECATED(5.6, "Please GetOrCreateDefaultDataReadReference instead.")
		TDataReadReference<DataType> GetOrConstructDataReadReference(const FVertexName& InVertexName, ConstructorArgTypes&&... ConstructorArgs) const
		{
			if (const FInputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrConstructAs<TDataReadReference<DataType>>(Forward<ConstructorArgTypes>(ConstructorArgs)...);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataReadReference<DataType>::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
		}


		/** Get data write reference assuming data is bound and write accessible. */
		template<typename DataType> 
		TDataWriteReference<DataType> GetDataWriteReference(const FVertexName& InVertexName) const
		{
			const FInputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetDataReferenceAs<TDataWriteReference<DataType>>();
		}

		/** Get the bound data write reference if it exists. Otherwise create and 
		 * return a data write reference by constructing one using the Vertex's 
		 * default literal.
		 */
		template<typename DataType> 
		TDataWriteReference<DataType> GetOrCreateDefaultDataWriteReference(const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			if (const FInputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrCreateDefaultAs<TDataWriteReference<DataType>>(InSettings);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataWriteReferenceLiteralFactory<DataType>::CreateExplicitArgs(InSettings, FLiteral::CreateInvalid());
		}

		/** Get the bound data write reference if it exists. Otherwise create and 
		 * return a data write reference by constructing one the supplied constructor 
		 * arguments.
		 */
		template<typename DataType, typename ... ConstructorArgTypes>
		UE_DEPRECATED(5.6, "Please GetOrCreateDefaultDataWriteReference instead.")
		TDataWriteReference<DataType> GetOrConstructDataWriteReference(const FVertexName& InVertexName, ConstructorArgTypes&&... ConstructorArgs) const
		{
			if (const FInputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrConstructAs<TDataWriteReference<DataType>>(Forward<ConstructorArgTypes>(ConstructorArgs)...);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataWriteReference<DataType>::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
		}

		/** Get or create an array of data references for all instances of a sub interface vertex.
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 * @param InSettings         - Operator settings to use if creating a data reference.
		 */
		template<typename DataReferenceType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		TArray<DataReferenceType> GetOrCreateDefaultSubInterfaceAs(const FName& InSubInterfaceName, const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			TArray<DataReferenceType> References;
			auto GetOrCreateDefault = [&References, &InSettings](uint32 InIndex, const FInputBinding& InBinding)
			{
				check(References.Num() == InIndex); // This check will be false if the references array 
													// is not being built in the correct order. 
				References.Add(InBinding.GetOrCreateDefaultAs<DataReferenceType>(InSettings));
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, INDEX_NONE, GetOrCreateDefault);
			return References;
		}

		/** Get or create an array of data read references for all instances of a sub interface vertex.
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 * @param InSettings         - Operator settings to use if creating a data reference.
		 */
		template<typename DataType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		TArray<TDataReadReference<DataType>> GetOrCreateDefaultSubInterfaceDataReadReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			return GetOrCreateDefaultSubInterfaceAs<TDataReadReference<DataType>>(InSubInterfaceName, InVertexName, InSettings);
		}

		/** Get or create an array of data write references for all instances of a sub interface vertex.
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 * @param InSettings         - Operator settings to use if creating a data reference.
		 */
		template<typename DataType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		TArray<TDataWriteReference<DataType>> GetOrCreateDefaultSubInterfaceDataWriteReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			return GetOrCreateDefaultSubInterfaceAs<TDataWriteReference<DataType>>(InSubInterfaceName, InVertexName, InSettings);
		}

		/** Get or create an array of values for all instances of a sub interface vertex.
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 * @param InSettings         - Operator settings to use if creating a data reference.
		 */
		template<typename DataType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		TArray<DataType> GetOrCreateDefaultSubInterfaceValues(const FName& InSubInterfaceName, const FVertexName& InVertexName, const FOperatorSettings& InSettings) const
		{
			return GetOrCreateDefaultSubInterfaceAs<DataType>(InSubInterfaceName, InVertexName, InSettings);
		}

		/** Find data references bound to the sub interface vertices. If there is
		 * no bound data for a sub interface instance, the array element will be
		 * a nullptr. 
		 *
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 */
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		UE_API TArray<const FAnyDataReference*> FindSubInterfaceReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName) const;
	
	private:
		friend METASOUNDGRAPHCORE_API void GetVertexInterfaceDataState(const FInputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState);
		friend METASOUNDGRAPHCORE_API void CompareVertexInterfaceDataToPriorState(const FInputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates);

		UE_API void Apply(const FVertexName& InVertexName, TFunctionRef<FInputBinding ()> InCreateFunc, TFunctionRef<void (FInputBinding&)> InApplyFunc);
		UE_API void ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, FInputBinding&)> InApplyFunc);
		UE_API void ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, const FInputBinding&)> InApplyFunc) const;

		UE_API FInputBinding* Find(const FVertexName& InVertexName);
		UE_API const FInputBinding* Find(const FVertexName& InVertexName) const;

		UE_API FInputBinding* FindChecked(const FVertexName& InVertexName);
		UE_API const FInputBinding* FindChecked(const FVertexName& InVertexName) const;

		UE_API const VertexPrivate::FSubInterfaceLayout* FindSubInterface(const FName& InSubInterfaceName) const;
		
		bool bIsVertexInterfaceFrozen = false;
		TArray<FInputBinding> Bindings;
		TArray<VertexPrivate::FSubInterfaceLayout> SubInterfaces;
	};


	/** An output vertex interface with optionally bound data references. */
	class FOutputVertexInterfaceData
	{
		using FOutputBinding = VertexDataPrivate::FOutputBinding;

	public:

		using FRangedForIteratorType = typename TArray<FOutputBinding>::RangedForIteratorType;
		using FRangedForConstIteratorType = typename TArray<FOutputBinding>::RangedForConstIteratorType;

		/** Construct with an FOutputVertexInterface. This will default to a unfrozen vertex interface. */
		UE_API FOutputVertexInterfaceData();

		/** Construct with an FOutputVertexInterface. This will default to a frozen vertex interface. */
		UE_API FOutputVertexInterfaceData(const FOutputVertexInterface& InVertexInterface);

		/** Returns true if the vertex interface is frozen. */
		UE_API bool IsVertexInterfaceFrozen() const;

		/** Set whether the vertex interface is frozen or not. 
		 *
		 * If frozen, attempts to access vertices which do not already exist will result in an error. 
		 *
		 * If not frozen, attempts to bind to a missing vertex will automatically
		 * add the missing vertex.
		 */
		UE_API void SetIsVertexInterfaceFrozen(bool bInIsVertexInterfaceFrozen);

		/** Returns true if a vertex exists with the provided name. This does not
		 * reflect whether the vertex is bound or not. */
		UE_API bool Contains(const FVertexName& InVertexName) const;

		/** Set the value of a vertex. */
		template<typename DataType>
		void SetValue(const FVertexName& InVertexName, const DataType& InValue)
		{
			auto CreateBinding = [&]()
			{
				FOutputDataVertex Vertex(InVertexName, GetMetasoundDataTypeName<DataType>(), FDataVertexMetadata{}, EVertexAccessType::Value);
				return FOutputBinding(Vertex);
			};

			auto BindData = [&](FOutputBinding& Binding) { Binding.SetValue<DataType>(InValue); };
			Apply(InVertexName, CreateBinding, BindData);
		}

		/** Bind a value vertex from a value reference. */
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindValueVertex(const FVertexName& InVertexName, const TDataValueReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FOutputBinding& Binding) { Binding.BindValue(InOutDataReference); };
			Apply(InVertexName, TCreateOutputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a read vertex from a value reference. */
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindReadVertex(const FVertexName& InVertexName, const TDataValueReference<DataType>& InOutDataReference)
		{
			BindReadVertex(InVertexName, static_cast<TDataReadReference<DataType>>(InOutDataReference));
		}

		/** Bind a read vertex from a read reference.*/
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindReadVertex(const FVertexName& InVertexName, const TDataReadReference<DataType>& InOutDataReference)
		{
			BindReadVertex(InVertexName, const_cast<TDataReadReference<DataType>&>(InOutDataReference));
		}

		/** Bind a read vertex from a read reference. */
		template<typename DataType>
		void BindReadVertex(const FVertexName& InVertexName, TDataReadReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FOutputBinding& Binding) { Binding.BindRead(InOutDataReference); };
			Apply(InVertexName, TCreateOutputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a read vertex from a write reference.*/
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindReadVertex(const FVertexName& InVertexName, const TDataWriteReference<DataType>& InOutDataReference)
		{
			BindReadVertex(InVertexName, const_cast<TDataWriteReference<DataType>&>(InOutDataReference));
		}

		/** Bind a read vertex from a write reference. */
		template<typename DataType>
		void BindReadVertex(const FVertexName& InVertexName, TDataWriteReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FOutputBinding& Binding) { Binding.BindRead(InOutDataReference); };

			Apply(InVertexName, TCreateOutputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a read vertex from a any reference. */
		template<typename DataType>
		void BindReadVertex(const FVertexName& InVertexName, FAnyDataReference& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			check(InOutDataReference.GetDataTypeName() == GetMetasoundDataTypeName<DataType>());

			auto BindData = [&](FOutputBinding& Binding) { Binding.BindReadFromAny<DataType>(InOutDataReference); };
			auto CreateBinding = [&]()
			{
				FOutputDataVertex Vertex(InVertexName, GetMetasoundDataTypeName<DataType>(), FDataVertexMetadata{}, EVertexAccessType::Reference);
				return FOutputBinding(Vertex);
			};

			Apply(InVertexName, CreateBinding, BindData);
		}

		/** Bind a write vertex from a write reference.*/
		template<typename DataType>
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		void BindWriteVertex(const FVertexName& InVertexName, const TDataWriteReference<DataType>& InOutDataReference)
		{
			BindWriteVertex(InVertexName, const_cast<TDataWriteReference<DataType>&>(InOutDataReference));
		}

		/** Bind a write vertex from a write reference. */
		template<typename DataType>
		void BindWriteVertex(const FVertexName& InVertexName, TDataWriteReference<DataType>& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FOutputBinding& Binding) { Binding.BindWrite(InOutDataReference); };
			Apply(InVertexName, TCreateOutputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a vertex with a any data reference. */
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		UE_API void BindVertex(const FVertexName& InVertexName, const FAnyDataReference& InOutDataReference);
		
		/** Bind a vertex with a data reference. */
		template<typename DataReferenceType>
		void BindVertex(const FVertexName& InVertexName, DataReferenceType& InOutDataReference)
		{
			using namespace VertexDataPrivate;
			auto BindData = [&](FOutputBinding& Binding) { Binding.Bind(InOutDataReference); };
			Apply(InVertexName, TCreateOutputBinding(InVertexName, InOutDataReference), BindData);
		}

		/** Bind a vertex with any data reference. */		
		UE_API void BindVertex(const FVertexName& InVertexName, FAnyDataReference& InOutDataReference);

		/** Return the number of instances of a particular sub interface. */
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		UE_API uint32 GetNumSubInterfaceInstances(const FName& InSubInterfaceName) const;

		/** Bind the vertices of a sub interface vertex. 
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			using namespace VertexDataPrivate;

			auto BindData = [&InOutDataReferences](uint32 Index, FOutputBinding& Binding)
			{
				Binding.Bind(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as write references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceWriteVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			using namespace VertexDataPrivate;

			auto BindData = [&InOutDataReferences](uint32 Index, FOutputBinding& Binding)
			{
				Binding.BindWrite(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as read references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void BindSubInterfaceReadVertices(const FName& InSubInterfaceName, const FName& InVertexName, RangeType& InOutDataReferences)
		{
			using namespace VertexDataPrivate;

			auto BindData = [&InOutDataReferences](uint32 Index, FOutputBinding& Binding)
			{
				Binding.BindRead(InOutDataReferences[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InOutDataReferences.Num(), BindData);
		}

		/** Bind the vertices of a sub interface vertex as value references
		 * @param InSubInterfaceName  - Name of sub interface.
		 * @param InVertexName        - Name of prototype vertex on sub interface.
		 * @param InOutDataReferences - An indexable sequence container of data references. 
		 */
		template<typename RangeType>
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		void SetSubInterfaceValues(const FName& InSubInterfaceName, const FName& InVertexName, const RangeType& InValues)
		{
			using namespace VertexDataPrivate;

			auto BindData = [&InValues](uint32 Index, FOutputBinding& Binding)
			{
				Binding.SetValue<TElementType_T<RangeType>>(InValues[Index]);
			};
			ForEachSubInterfaceBinding(InSubInterfaceName, InVertexName, InValues.Num(), BindData);
		}

		/** Bind vertex data using other vertex data.*/
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data references passed to this function. Const data references can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		UE_API void Bind(const FOutputVertexInterfaceData& InOutOutputVertexData);
		/** Bind vertex data using other vertex data. */
		UE_API void Bind(FOutputVertexInterfaceData& InOutOutputVertexData);

		/** Set a data reference to a vertex, ignoring any existing data bound to the vertex*/
		UE_API void SetVertex(const FVertexName& InVertexName, const FAnyDataReference& InDataReference);
		/** Set a data reference to a vertex, ignoring any existing data bound to the vertex*/
		UE_API void SetVertex(const FVertexName& InVertexName, FAnyDataReference&& InDataReference);

		/** Set a data references to vertices with matching vertex names. */
		UE_DEPRECATED(5.6, "Do not use FDataReferenceCollection.")
		UE_API void Set(const FDataReferenceCollection& InCollection);

		/** Converts the vertex data to a data reference collection. */
		UE_DEPRECATED(5.6, "Do not use FDataReferenceCollection.")
		UE_API FDataReferenceCollection ToDataReferenceCollection() const;

		/** Returns true if a vertex with the given vertex name exists and is bound
		 * to a data reference. */
		UE_API bool IsVertexBound(const FVertexName& InVertexName) const;

		/** Return the vertex associated with the vertex name. */
		UE_API const FOutputDataVertex& GetVertex(const FVertexName& InVertexName) const;

		/** Add a vertex. VertexInterfaceData must be unfrozen. */
		UE_API void AddVertex(const FOutputDataVertex& InVertex);

		/** Remove a vertex. VertexInterfaceData must be unfrozen. */
		UE_API void RemoveVertex(const FVertexName& InVertexName);

		/** Returns the access type of a bound vertex. If the vertex does not exist
		 * or if it is unbound, this will return EDataReferenceAccessType::None
		 */
		UE_API EDataReferenceAccessType GetVertexDataAccessType(const FVertexName& InVertexName) const;

		/** Returns true if all vertices in the FInputVertexInterface are bound to 
		 * data references. */
		UE_API bool AreAllVerticesBound() const;

		FRangedForIteratorType begin()
		{
			return Bindings.begin();
		}

		FRangedForIteratorType end()
		{
			return Bindings.end();
		}

		FRangedForConstIteratorType begin() const
		{
			return Bindings.begin();
		}

		FRangedForConstIteratorType end() const
		{
			return Bindings.end();
		}

		/** Find data reference bound to vertex. Returns a nullptr if no data reference is bound. */
		UE_API const FAnyDataReference* FindDataReference(const FVertexName& InVertexName) const;

		/** Returns the current value of a vertex. */
		template<typename DataType>
		const DataType* GetValue(const FVertexName& InVertexName) const
		{
			const FOutputBinding* Binding = FindChecked(InVertexName);
			if (Binding->IsBound())
			{
				return Binding->GetDataReferenceAs<const DataType*>();
			}
			else
			{
				return nullptr;
			}
		}


		/** Get data read reference assuming data is bound and read or write accessible. */
		template<typename DataType> 
		TDataReadReference<DataType> GetDataReadReference(const FVertexName& InVertexName) const
		{
			const FOutputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetDataReferenceAs<TDataReadReference<DataType>>();
		}

		/** Get the bound data read reference if it exists. Otherwise create and 
		 * return a data read reference by constructing one the supplied constructor 
		 * arguments.
		 */
		template<typename DataType, typename ... ConstructorArgTypes>
		TDataReadReference<DataType> GetOrConstructDataReadReference(const FVertexName& InVertexName, ConstructorArgTypes&&... ConstructorArgs) const
		{
			if (const FOutputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrConstructAs<TDataReadReference<DataType>>(Forward<ConstructorArgTypes>(ConstructorArgs)...);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataReadReference<DataType>::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
		}


		/** Get data write reference assuming data is bound and write accessible. */
		template<typename DataType> 
		TDataWriteReference<DataType> GetDataWriteReference(const FVertexName& InVertexName) const
		{
			const FOutputBinding* Binding = FindChecked(InVertexName);
			return Binding->GetDataReferenceAs<TDataWriteReference<DataType>>();
		}

		/** Get the bound data write reference if it exists. Otherwise create and 
		 * return a data write reference by constructing one the supplied constructor 
		 * arguments.
		 */
		template<typename DataType, typename ... ConstructorArgTypes>
		TDataWriteReference<DataType> GetOrConstructDataWriteReference(const FVertexName& InVertexName, ConstructorArgTypes&&... ConstructorArgs) const
		{
			if (const FOutputBinding* Binding = Find(InVertexName))
			{
				return Binding->GetOrConstructAs<TDataWriteReference<DataType>>(Forward<ConstructorArgTypes>(ConstructorArgs)...);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Failed find vertex with name '%ls'. Cannot check for existing bound data.", *InVertexName.ToString());
			}

			return TDataWriteReference<DataType>::CreateNew(Forward<ConstructorArgTypes>(ConstructorArgs)...);
		}


		/** Find data references bound to the sub interface vertices. If there is
		 * no bound data for a sub interface instance, the array element will be
		 * a nullptr. 
		 *
		 * @param InSubInterfaceName - Name of the sub interface
		 * @param InVertexName       - Name of the prototype vertex in the sub interface.
		 */
		UE_EXPERIMENTAL(5.6, "Sub interfaces are still under development")
		UE_API TArray<const FAnyDataReference*> FindSubInterfaceReferences(const FName& InSubInterfaceName, const FVertexName& InVertexName) const;
	
	private:
		friend METASOUNDGRAPHCORE_API void GetVertexInterfaceDataState(const FOutputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState);
		friend METASOUNDGRAPHCORE_API void CompareVertexInterfaceDataToPriorState(const FOutputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates);

		UE_API void Apply(const FVertexName& InVertexName, TFunctionRef<FOutputBinding ()> InCreateFunc, TFunctionRef<void (FOutputBinding&)> InBindFunc);
		UE_API void ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, FOutputBinding&)> InApplyFunc);
		UE_API void ForEachSubInterfaceBinding(const FName& InSubInterfaceName, const FVertexName& InVertexName, uint32 InExpectedNumSubInterfaceInstances, TFunctionRef<void (uint32, const FOutputBinding&)> InApplyFunc) const;

		UE_API FOutputBinding* Find(const FVertexName& InVertexName);
		UE_API const FOutputBinding* Find(const FVertexName& InVertexName) const;
		UE_API const VertexPrivate::FSubInterfaceLayout* FindSubInterface(const FName& InSubInterfaceName) const;

		UE_API FOutputBinding* FindChecked(const FVertexName& InVertexName);
		UE_API const FOutputBinding* FindChecked(const FVertexName& InVertexName) const;

		bool bIsVertexInterfaceFrozen = false;
		TArray<FOutputBinding> Bindings;
		TArray<VertexPrivate::FSubInterfaceLayout> SubInterfaces;
	};

	/** A vertex interface with optionally bound data. */
	class FVertexInterfaceData
	{
	public:

		FVertexInterfaceData() = default;

		/** Construct using an FVertexInterface. */
		UE_API FVertexInterfaceData(const FVertexInterface& InVertexInterface);

		/** Set vertex data using other vertex data. */
		UE_DEPRECATED(5.6, "Please use a non-const lvalue reference for data  passed to this function. Const data  can sometimes be created by passing in temporary objects and/or triggering implicit conversion.")
		UE_API void Bind(const FVertexInterfaceData& InVertexData);
		
		/** Set vertex data using other vertex data. */
		UE_API void Bind(FVertexInterfaceData& InVertexData);

		/** Get input vertex interface data. */
		FInputVertexInterfaceData& GetInputs()
		{
			return InputVertexInterfaceData;
		}

		/** Get input vertex interface data. */
		const FInputVertexInterfaceData& GetInputs() const
		{
			return InputVertexInterfaceData;
		}

		/** Get output vertex interface data. */
		FOutputVertexInterfaceData& GetOutputs()
		{
			return OutputVertexInterfaceData;
		}

		/** Get output vertex interface data. */
		const FOutputVertexInterfaceData& GetOutputs() const
		{
			return OutputVertexInterfaceData;
		}

	private:

		FInputVertexInterfaceData InputVertexInterfaceData;
		FOutputVertexInterfaceData OutputVertexInterfaceData;
	};

	/** FVertexDataState encapsulates which data reference a vertex is associated
	 * with. The ID refers to the underlying object associated with the IDataReference.
	 */
	struct FVertexDataState
	{
		FVertexName VertexName;
		FDataReferenceID ID;

		METASOUNDGRAPHCORE_API friend bool operator<(const FVertexDataState& InLHS, const FVertexDataState& InRHS);
		METASOUNDGRAPHCORE_API friend bool operator==(const FVertexDataState& InLHS, const FVertexDataState& InRHS);
	};

	/** Caches a representation of the current data references bound to the vertex interface */
	METASOUNDGRAPHCORE_API void GetVertexInterfaceDataState(const FInputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState);
	/** Caches a representation of the current data references bound to the vertex interface */
	METASOUNDGRAPHCORE_API void GetVertexInterfaceDataState(const FOutputVertexInterfaceData& InVertexInterface, TArray<FVertexDataState>& OutState);

	/** Compares the current data bound to the vertex interface with a prior cached state. */
	METASOUNDGRAPHCORE_API void CompareVertexInterfaceDataToPriorState(const FInputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates);
	/** Compares the current data bound to the vertex interface with a prior cached state. */
	METASOUNDGRAPHCORE_API void CompareVertexInterfaceDataToPriorState(const FOutputVertexInterfaceData& InCurrentInterface, const TArray<FVertexDataState>& InPriorState, TSortedVertexNameMap<FAnyDataReference>& OutUpdates);
}

#undef UE_API
