// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MetasoundStandardNodes : ModuleRules
	{
		public MetasoundStandardNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AudioExtensions",
					"Core",
					"MetasoundFrontend",
					"Serialization",
					"SignalProcessing"
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"CoreUObject",
					"MetasoundGraphCore"
				}
			);

			NumIncludedBytesPerUnityCPPOverride = 120 * 1024;

			PrivateDefinitions.AddRange(
				new string[]
				{
					"METASOUND_PLUGIN=Metasound",
					"METASOUND_MODULE=MetasoundStandardNodes"
				});
		}
	}
}
