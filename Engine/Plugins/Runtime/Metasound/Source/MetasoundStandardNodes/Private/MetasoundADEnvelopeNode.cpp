// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_ADEnvelopeNode"

namespace Metasound
{
	namespace ADEnvelopeVertexNames
	{
		METASOUND_PARAM(InputTrigger, "Trigger", "Trigger to start the attack phase of the envelope generator.");
		METASOUND_PARAM(InputAttackTime, "Attack Time", "Attack time of the envelope.");
		METASOUND_PARAM(InputDecayTime, "Decay Time", "Decay time of the envelope.");
		METASOUND_PARAM(InputAttackCurve, "Attack Curve", "The exponential curve factor of the attack. 1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth.");
		METASOUND_PARAM(InputDecayCurve, "Decay Curve", "The exponential curve factor of the decay. 1.0 = linear decay, < 1.0 exponential decay, > 1.0 logarithmic decay.");
		METASOUND_PARAM(InputLooping, "Looping", "Set to true to enable looping of the envelope. This will allow the envelope to be an LFO or wave generator.");
		METASOUND_PARAM(InputHardReset, "Hard Reset", "Set to true to always reset the envelope level to 0 when triggering.");

		METASOUND_PARAM(OutputOnTrigger, "On Trigger", "Triggers when the envelope is triggered.");
		METASOUND_PARAM(OutputOnDone, "On Done", "Triggers when the envelope finishes or loops back if looping is enabled.");
		METASOUND_PARAM(OutputEnvelopeValue, "Out Envelope", "The output value of the envelope.");
	}

	namespace ADEnvelopeNodePrivate
	{
		struct FEnvState
		{
			// Where the envelope is. If INDEX_NONE, then the envelope is not triggered
			int32 CurrentSampleIndex = INDEX_NONE;

			// Number of samples for attack
			int32 AttackSampleCount = 1;

			// Number of samples for Decay
			int32 DecaySampleCount = 1;

			// Curve factors for attack/Decay
			float AttackCurveFactor = 0.0f;
			float DecayCurveFactor = 0.0f;

			Audio::FExponentialEase EnvEase;

			// Where the envelope value was when it was triggered
			float StartingEnvelopeValue = 0.0f;
			float CurrentEnvelopeValue = 0.0f;

			bool bLooping = false;
			bool bHardReset = false;

			void Reset()
			{
				CurrentSampleIndex = INDEX_NONE;
				AttackSampleCount = 1;
				DecaySampleCount = 1;

				AttackCurveFactor = 0.0f;
				DecayCurveFactor = 0.0f;

				StartingEnvelopeValue = 0.0f;
				CurrentEnvelopeValue = 0.0f;

				bLooping = false;
				bHardReset = false;
				EnvEase.Init(0.f, 0.01f);
			}
		};

		template<typename ValueType>
		struct TADEnvelope
		{
		};

		template<>
		struct TADEnvelope<float>
		{
			static void GetNextEnvelopeOutput(FEnvState& InState, int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, float& OutEnvelopeValue)
			{
				// Don't need to do anything if we're not generating the envelope at the top of the block since this is a block-rate envelope
				if (StartFrame > 0 || InState.CurrentSampleIndex == INDEX_NONE)
				{
					OutEnvelopeValue = 0.0f;
					return;
				}

				// We are in attack
				if (InState.CurrentSampleIndex < InState.AttackSampleCount)
				{
					if (InState.AttackSampleCount > 1)
					{
						float AttackFraction = (float)InState.CurrentSampleIndex++ / InState.AttackSampleCount;
						OutEnvelopeValue = InState.StartingEnvelopeValue + (1.0f - InState.StartingEnvelopeValue) * FMath::Pow(AttackFraction, InState.AttackCurveFactor);
					}
					else
					{
						// Attack is effectively 0, skip Attack fade-in
						InState.CurrentSampleIndex++;
						OutEnvelopeValue = 1.f;
					}
				}
				else
				{
					int32 TotalEnvSampleCount = (InState.AttackSampleCount + InState.DecaySampleCount);
					
					// We are in Decay
					if (InState.CurrentSampleIndex < TotalEnvSampleCount)
					{
						int32 SampleCountInDecayState = InState.CurrentSampleIndex++ - InState.AttackSampleCount;
						float DecayFraction = (float)SampleCountInDecayState / InState.DecaySampleCount;
						OutEnvelopeValue = 1.0f - FMath::Pow(DecayFraction, InState.DecayCurveFactor);
					}
					// We are looping so reset the sample index
					else if (InState.bLooping)
					{
						InState.CurrentSampleIndex = 0;
						OutFinishedFrames.Add(EndFrame);
					}
					else
					{
						// Envelope is done
						InState.CurrentSampleIndex = INDEX_NONE;
						OutEnvelopeValue = 0.0f;
						OutFinishedFrames.Add(EndFrame);
					}
				}
			}

			static void GetInitialOutputEnvelope(float& OutputEnvelope)
			{
				OutputEnvelope = 0.f;
			}

			static constexpr bool IsAudio() { return false; }
		};

		template<>
		struct TADEnvelope<FAudioBuffer>
		{
			static void GetNextEnvelopeOutput(FEnvState& InState, int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, FAudioBuffer& OutEnvelopeValue)
			{
				// If we are not active zero the buffer and early exit
				if (InState.CurrentSampleIndex == INDEX_NONE)
				{
					OutEnvelopeValue.Zero();
					return;
				}

				float* OutEnvPtr = OutEnvelopeValue.GetData();
				for (int32 i = StartFrame; i < EndFrame; ++i)
				{
					// We are in attack
					if (InState.CurrentSampleIndex < InState.AttackSampleCount)
					{
						float AttackFraction = (float)InState.CurrentSampleIndex++ / InState.AttackSampleCount;
						float EnvValue = FMath::Pow(AttackFraction, InState.AttackCurveFactor);
						float TargetEnvelopeValue = InState.StartingEnvelopeValue + (1.0f - InState.StartingEnvelopeValue) * EnvValue;
						InState.EnvEase.SetValue(TargetEnvelopeValue);
						InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
						OutEnvPtr[i] = InState.CurrentEnvelopeValue;
					}
					else 
					{
						int32 TotalEnvSampleCount = (InState.AttackSampleCount + InState.DecaySampleCount);

						// We are in Decay
						if (InState.CurrentSampleIndex < TotalEnvSampleCount)
						{
							int32 SampleCountInDecayState = InState.CurrentSampleIndex++ - InState.AttackSampleCount;
							float DecayFracton = (float)SampleCountInDecayState / InState.DecaySampleCount;
							float TargetEnvelopeValue = 1.0f - FMath::Pow(DecayFracton, InState.DecayCurveFactor);
							InState.EnvEase.SetValue(TargetEnvelopeValue);
							InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
							OutEnvPtr[i] = InState.CurrentEnvelopeValue;
						}
						// We are looping so reset the sample index
						else if (InState.bLooping)
						{
							InState.StartingEnvelopeValue = 0.0f;
							InState.CurrentEnvelopeValue = InState.AttackSampleCount? 0.f : 1.f;
							InState.CurrentSampleIndex = 0;
							OutFinishedFrames.Add(i);
						}
						else
						{
							InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
							OutEnvPtr[i] = InState.CurrentEnvelopeValue;

							// Envelope is done
							if (InState.EnvEase.IsDone())
							{
								// Zero out the rest of the envelope
								int32 NumSamplesLeft = EndFrame - i - 1;
								if (NumSamplesLeft > 0)
								{
									FMemory::Memzero(&OutEnvPtr[i + 1], sizeof(float) * NumSamplesLeft);
								}
								InState.CurrentSampleIndex = INDEX_NONE;
								OutFinishedFrames.Add(i);
								break;
							}
						}
					}
				}
			}

			static void GetInitialOutputEnvelope(FAudioBuffer& OutputEnvelope)
			{
				OutputEnvelope.Zero();
			}

			static constexpr bool IsAudio() { return true; }
		};
	}

	template<typename ValueType>
	class TADEnvelopeNodeOperator : public TExecutableOperator<TADEnvelopeNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ADEnvelopeVertexNames;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackTime), 0.01f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDecayTime), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackCurve), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDecayCurve), 1.0f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLooping), false),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputHardReset), false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnDone)),
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputEnvelopeValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = "AD Envelope";
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ADEnvelopeDisplayNamePattern", "AD Envelope ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("ADEnevelopeDesc", "Generates an attack-decay envelope value output when triggered.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				const FNodeClassMetadata Metadata
				{
					FNodeClassName { "AD Envelope", OperatorName, DataTypeName },
					1, // Major Version
					0, // Minor Version
					NodeDisplayName,
					NodeDescription,
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Envelopes },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ADEnvelopeVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
			FTimeReadRef AttackTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputAttackTime), InParams.OperatorSettings);
			FTimeReadRef DecayTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputDecayTime), InParams.OperatorSettings);
			FFloatReadRef AttackCurveFactor = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputAttackCurve), InParams.OperatorSettings);
			FFloatReadRef DecayCurveFactor = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDecayCurve), InParams.OperatorSettings);
			FBoolReadRef bLooping = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputLooping), InParams.OperatorSettings);
			FBoolReadRef bHardReset = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputHardReset), InParams.OperatorSettings);

			return MakeUnique<TADEnvelopeNodeOperator<ValueType>>(InParams, TriggerIn, AttackTime, DecayTime, AttackCurveFactor, DecayCurveFactor, bLooping, bHardReset);
		}

		TADEnvelopeNodeOperator(const FBuildOperatorParams& InParams,
			const FTriggerReadRef& InTriggerIn,
			const FTimeReadRef& InAttackTime,
			const FTimeReadRef& InDecayTime,
			const FFloatReadRef& InAttackCurveFactor,
			const FFloatReadRef& InDecayeCurveFactor,
			const FBoolReadRef& bInLooping,
			const FBoolReadRef& bInHardReset)
			: TriggerAttackIn(InTriggerIn)
			, AttackTime(InAttackTime)
			, DecayTime(InDecayTime)
			, AttackCurveFactor(InAttackCurveFactor)
			, DecayCurveFactor(InDecayeCurveFactor)
			, bLooping(bInLooping)
			, bHardReset(bInHardReset)
			, OnAttackTrigger(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InParams.OperatorSettings))
			, OnDone(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InParams.OperatorSettings))
			, OutputEnvelope(TDataWriteReferenceFactory<ValueType>::CreateExplicitArgs(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual ~TADEnvelopeNodeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ADEnvelopeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerAttackIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackTime), AttackTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDecayTime), DecayTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackCurve), AttackCurveFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDecayCurve), DecayCurveFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLooping), bLooping);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputHardReset), bHardReset);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ADEnvelopeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnTrigger), OnAttackTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnDone), OnDone);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputEnvelopeValue), OutputEnvelope);
		}

		void UpdateParams()
		{
			float AttackTimeSeconds = AttackTime->GetSeconds();
			float DecayTimeSeconds = DecayTime->GetSeconds();
			EnvState.AttackSampleCount = FMath::Max(0, SampleRate * AttackTimeSeconds);

			// if our attack phase is zero, force decay phase to be at least a single sample
			const int32 DecaySampleCountMin = (EnvState.AttackSampleCount? 0 : 1);
			EnvState.DecaySampleCount = FMath::Max(DecaySampleCountMin, SampleRate * DecayTimeSeconds);
			EnvState.AttackCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, *AttackCurveFactor);
			EnvState.DecayCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, *DecayCurveFactor);
			EnvState.bLooping = *bLooping;
			EnvState.bHardReset = *bHardReset;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();


			if constexpr (ADEnvelopeNodePrivate::TADEnvelope<ValueType>::IsAudio())
			{
				SampleRate = InParams.OperatorSettings.GetSampleRate();
			}
			else
			{
				SampleRate = InParams.OperatorSettings.GetActualBlockRate();
			}

			OnAttackTrigger->Reset();
			OnDone->Reset();
			ADEnvelopeNodePrivate::TADEnvelope<ValueType>::GetInitialOutputEnvelope(*OutputEnvelope);
			EnvState.Reset();
		}

		void Execute()
		{
			using namespace ADEnvelopeNodePrivate;

			OnAttackTrigger->AdvanceBlock();
			OnDone->AdvanceBlock();

			// check for any updates to input params
			UpdateParams();

			TriggerAttackIn->ExecuteBlock(
				// OnPreTrigger
				[&](int32 StartFrame, int32 EndFrame)
				{
					TArray<int32> FinishedFrames;
					TADEnvelope<ValueType>::GetNextEnvelopeOutput(EnvState, StartFrame, EndFrame, FinishedFrames, *OutputEnvelope);

					for (int32 FrameFinished : FinishedFrames)
					{
						OnDone->TriggerFrame(FrameFinished);
					}
				},
				// OnTrigger
					[&](int32 StartFrame, int32 EndFrame)
				{
					// Get latest params
					UpdateParams();

					// Set the sample index to the top of the envelope
					EnvState.CurrentSampleIndex = 0;
					EnvState.StartingEnvelopeValue = EnvState.bHardReset ? 0 : EnvState.CurrentEnvelopeValue;
					EnvState.EnvEase.SetValue(EnvState.StartingEnvelopeValue, true /* bInit */);

					// Generate the output (this will no-op if we're block rate)
					TArray<int32> FinishedFrames;
					TADEnvelope<ValueType>::GetNextEnvelopeOutput(EnvState, StartFrame, EndFrame, FinishedFrames, *OutputEnvelope);
					for (int32 FrameFinished : FinishedFrames)
					{
						OnDone->TriggerFrame(FrameFinished);
					}

					// Forward the trigger
					OnAttackTrigger->TriggerFrame(StartFrame);
				}
			);
		}

	private:

		FTriggerReadRef TriggerAttackIn;
		FTimeReadRef AttackTime;
		FTimeReadRef DecayTime;
		FFloatReadRef AttackCurveFactor;
		FFloatReadRef DecayCurveFactor;
		FBoolReadRef bLooping;
		FBoolReadRef bHardReset;

		FTriggerWriteRef OnAttackTrigger;
		FTriggerWriteRef OnDone;
		TDataWriteReference<ValueType> OutputEnvelope;

		// This will either be the block rate or sample rate depending on if this is block-rate or audio-rate envelope
		float SampleRate = 0.0f;
		int32 NumFramesPerBlock = 0;

		ADEnvelopeNodePrivate::FEnvState EnvState;
	};

	/** TADEnvelopeNode
	 *
	 *  Creates an Attack/Decay envelope node.
	 */
	template<typename ValueType>
	using TADEnvelopeNode = TNodeFacade<TADEnvelopeNodeOperator<ValueType>>;

	using FADEnvelopeNodeFloat = TADEnvelopeNode<float>;
	METASOUND_REGISTER_NODE(FADEnvelopeNodeFloat)

	using FADEnvelopeAudioBuffer = TADEnvelopeNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FADEnvelopeAudioBuffer)
}

#undef LOCTEXT_NAMESPACE
