// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_ADSR"

namespace Metasound
{
	namespace ADSREnvelopeVertexNames
	{
		METASOUND_PARAM(InputAttackTrigger, "Trigger Attack", "Trigger to start the attack phase of the envelope generator.");
		METASOUND_PARAM(InputReleaseTrigger, "Trigger Release", "Trigger to start the release phase of the envelope generator.");

		METASOUND_PARAM(InputAttackTime, "Attack Time", "Attack time (seconds) of the envelope. Time to reach 1.0 in the envelope output.");
		METASOUND_PARAM(InputDecayTime, "Decay Time", "Decay time (seconds) of the envelope. Time to reach the sustain level in the envelope output.");
		METASOUND_PARAM(InputSustainLevel, "Sustain Level", "The sustain level (between 0.0 and 1.0).");
		METASOUND_PARAM(InputReleaseTime, "Release Time", "Release time (seconds) of the envelope. Time to reach 0.0 in the envelope output.");

		METASOUND_PARAM(InputAttackCurve, "Attack Curve", "The exponential curve factor of the attack. 1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth.");
		METASOUND_PARAM(InputDecayCurve, "Decay Curve", "The exponential curve factor of the decay. 1.0 = linear decay, < 1.0 exponential decay, > 1.0 logorithmic decay.");
		METASOUND_PARAM(InputReleaseCurve, "Release Curve", "The exponential curve factor of the release. 1.0 = linear release, < 1.0 exponential release, > 1.0 logorithmic release.");

		METASOUND_PARAM(InputHardReset, "Hard Reset", "Set to true to always reset the envelope level to 0 when triggering.");

		METASOUND_PARAM(OutputOnAttackTrigger, "On Attack Triggered", "Triggers when the envelope attack is triggered.");
		METASOUND_PARAM(OutputOnDecayTrigger, "On Decay Triggered", "Triggers when the envelope decay begins and attack is finished.");
		METASOUND_PARAM(OutputOnSustainTrigger, "On Sustain Triggered", "Triggers when the envelope sustain begins and attack is finished.");
		METASOUND_PARAM(OutputOnReleaseTrigger, "On Release Triggered", "Triggers when the envelope release is triggered.");
		METASOUND_PARAM(OutputOnDone, "On Done", "Triggers when the envelope finishes.");
		METASOUND_PARAM(OutputEnvelopeValue, "Out Envelope", "The output value of the envelope.");
	}

	namespace ADSREnvelopeNodePrivate
	{
		struct FEnvState
		{
			// Where the envelope is. If INDEX_NONE, then the envelope is not triggered
			int32 CurrentSampleIndex = INDEX_NONE;

			// Number of samples for attack
			int32 AttackSampleCount = 0;

			// Number of samples for Decay
			int32 DecaySampleCount = 0;

			// Number of samples for Release 
			int32 ReleaseSampleCount = 0;
			
			// Sustain level
			float SustainLevel = 0.0f;

			// Curve factors for attack/decay/release
			float AttackCurveFactor = 0.0f;
			float DecayCurveFactor = 0.0f;
			float ReleaseCurveFactor = 0.0f;

			Audio::FExponentialEase EnvEase;

			// Where the envelope value was when it was triggered
			float StartingEnvelopeValue = 0.0f;
			float CurrentEnvelopeValue = 0.0f;
			float EnvelopeValueAtReleaseStart = 0.0f;
			bool bHardReset = false;

			// If this is set, we are in release mode
			bool bIsInRelease = false;

			bool IsTriggered() const 
			{
				return CurrentSampleIndex != INDEX_NONE; 
			}

			void Reset()
			{
				CurrentSampleIndex = INDEX_NONE;

				AttackSampleCount = 0;
				DecaySampleCount = 0;
				ReleaseSampleCount = 0;
				
				SustainLevel = 0.0f;

				AttackCurveFactor = 0.0f;
				DecayCurveFactor = 0.0f;
				ReleaseCurveFactor = 0.0f;

				StartingEnvelopeValue = 0.0f;
				CurrentEnvelopeValue = 0.0f;
				EnvelopeValueAtReleaseStart = 0.0f;

				bHardReset = false;
				bIsInRelease = false;
			
				EnvEase.Init(0.f, 0.1f);
			}
		};


		struct FFloatADSREnvelope
		{
			static void GetInitialOutputEnvelope(float& OutputEnvelope)
			{
				OutputEnvelope = 0.f;
			}

			static float GetSampleRate(const FOperatorSettings& InOperatorSettings)
			{		
				return InOperatorSettings.GetActualBlockRate();
			}

			static void GetNextEnvelopeOutput(FEnvState& InState, int32 StartFrame, int32 EndFrame, TArray<int32>& OutOnDecayFrames, TArray<int32>& OutOnSustainFrames, TArray<int32>& OutOnDoneFrames, float& OutEnvelopeValue)
			{
				// Don't need to do anything if we're not generating the envelope at the top of the block since this is a block-rate envelope
				if (StartFrame > 0)
				{
					return;
				}

				if (InState.CurrentSampleIndex == INDEX_NONE)
				{
					OutEnvelopeValue = 0.0f;
					return;
				}

				// If we are in the release state, jump forward in our sample count
				if (InState.bIsInRelease)
				{
					int32 SampleStartOfRelease = InState.AttackSampleCount + InState.DecaySampleCount;
					if (InState.CurrentSampleIndex < SampleStartOfRelease)
					{
						InState.EnvelopeValueAtReleaseStart = InState.CurrentEnvelopeValue;
						InState.CurrentSampleIndex = InState.AttackSampleCount + InState.DecaySampleCount;
					}
				}

				// We are in attack
				if (InState.CurrentSampleIndex < InState.AttackSampleCount)
				{
					float AttackFraction = (float)++(InState.CurrentSampleIndex) / InState.AttackSampleCount;
					float EnvValue = FMath::Pow(AttackFraction, InState.AttackCurveFactor);
					float TargeEnvelopeValue = InState.StartingEnvelopeValue + (1.0f - InState.StartingEnvelopeValue) * EnvValue;
					InState.EnvEase.SetValue(TargeEnvelopeValue, true /* bInit */);
					InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
					OutEnvelopeValue = InState.CurrentEnvelopeValue;

					if (InState.CurrentSampleIndex == InState.AttackSampleCount)
					{
						OutOnDecayFrames.Add(0);

						// Decay stage is skipped
						if (InState.DecaySampleCount == 0)
						{
							OutOnSustainFrames.Add(0);
						}
					}
				}
				// We are in decay
				else
				{
					// Sample count to the end of the decay phase
					int32 DecayEnvSampleCount = (InState.AttackSampleCount + InState.DecaySampleCount);
					if (InState.CurrentSampleIndex < DecayEnvSampleCount)
					{
						int32 SampleCountInDecayState = InState.CurrentSampleIndex++ - InState.AttackSampleCount;
						float DecayFracton = (float)SampleCountInDecayState / InState.DecaySampleCount;
						float TargetEnvelopeValue = 1.0f - (1.0f - InState.SustainLevel) * FMath::Pow(DecayFracton, InState.DecayCurveFactor);
						InState.EnvEase.SetValue(TargetEnvelopeValue);
						InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
						OutEnvelopeValue = InState.CurrentEnvelopeValue;

						if (InState.CurrentSampleIndex == DecayEnvSampleCount)
						{
							OutOnSustainFrames.Add(0);
						}
					}
					// We are in sustain
					else if (!InState.bIsInRelease)
					{
						InState.EnvEase.SetValue(InState.SustainLevel);
						InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
						OutEnvelopeValue = InState.CurrentEnvelopeValue;
						InState.EnvelopeValueAtReleaseStart = OutEnvelopeValue;
					}
					// We are in release mode or finished
					else
					{
						int32 ReleaseEnvSampleCount = (InState.AttackSampleCount + InState.DecaySampleCount + InState.ReleaseSampleCount);
						// We are in release
						if (InState.CurrentSampleIndex < ReleaseEnvSampleCount)
						{
							int32 SampleCountInReleaseState = InState.CurrentSampleIndex++ - InState.DecaySampleCount - InState.AttackSampleCount;
							float ReleaseFraction = (float)SampleCountInReleaseState / InState.ReleaseSampleCount;
							float TargetEnvelopeValue = InState.EnvelopeValueAtReleaseStart * (1.0f - FMath::Pow(ReleaseFraction, InState.ReleaseCurveFactor));
							InState.EnvEase.SetValue(TargetEnvelopeValue);
							InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
							OutEnvelopeValue = InState.CurrentEnvelopeValue;
						}
						// We are done
						else
						{
							InState.CurrentSampleIndex = INDEX_NONE;
							OutEnvelopeValue = 0.0f;
							OutOnDoneFrames.Add(EndFrame);
						}
					}
				}
			}
		};

		struct FAudioADSREnvelope
		{
			static void GetInitialOutputEnvelope(FAudioBuffer& OutputEnvelope)
			{
				OutputEnvelope.Zero();
			}

			static float GetSampleRate(const FOperatorSettings& InOperatorSettings)
			{
					return InOperatorSettings.GetSampleRate();
			}

			static void GetNextEnvelopeOutput(FEnvState& InState, int32 StartFrame, int32 EndFrame, TArray<int32>& OutOnDecayFrames, TArray<int32>& OutOnSustainFrames, TArray<int32>& OutOnDoneFrames, FAudioBuffer& OutEnvelopeValue)
			{
				// If we are not active zero the buffer and early exit
				if (InState.CurrentSampleIndex == INDEX_NONE)
				{
					OutEnvelopeValue.Zero();
					return;
				}

				// If we are in the release state, jump forward in our sample count
				if (InState.bIsInRelease)
				{
					int32 SampleStartOfRelease = InState.AttackSampleCount + InState.DecaySampleCount;
					if (InState.CurrentSampleIndex <= SampleStartOfRelease)
					{
						InState.EnvelopeValueAtReleaseStart = InState.CurrentEnvelopeValue;
						InState.CurrentSampleIndex = SampleStartOfRelease;
					}
				}

				// Init the end attack and decay frames to the start frame. If we're not in these states, it'll just start loops at the start frame
				// But we want to keep track of which frames we have finished previous envelope states so the next state can start rendering at the
				// correct sample vs assume it's at the start of the block.
				int32 EndAttackFrame = StartFrame;
				int32 EndDecayFrame = StartFrame;

				float* OutEnvPtr = OutEnvelopeValue.GetData();
				int32 AttackSamplesLeft = InState.AttackSampleCount - InState.CurrentSampleIndex;

				// We are in attack
				if (AttackSamplesLeft > 0)
				{
					EndAttackFrame = FMath::Max(1, FMath::Min(StartFrame + AttackSamplesLeft, EndFrame));
					for (int32 i = StartFrame; i < EndAttackFrame; ++i)
					{
						float AttackFraction = (float)++(InState.CurrentSampleIndex) / InState.AttackSampleCount;
						float EnvValue = FMath::Pow(AttackFraction, InState.AttackCurveFactor);
						OutEnvPtr[i] = InState.StartingEnvelopeValue + (1.0f - InState.StartingEnvelopeValue) * EnvValue;
					}

					InState.CurrentEnvelopeValue = OutEnvPtr[EndAttackFrame - 1];
					InState.EnvEase.SetValue(InState.CurrentEnvelopeValue, true /* bInit */);

					// We have finished our attack phase in this block
					if (InState.CurrentSampleIndex == InState.AttackSampleCount)
					{
						OutOnDecayFrames.Add(EndAttackFrame);

						// Decay Stage is Skipped
						if (InState.DecaySampleCount == 0)
						{
							OutOnSustainFrames.Add(EndAttackFrame);
						}
					}
				}

				// We are now done with our attack and may need to immediately start rendering our decay block
				if (EndAttackFrame < EndFrame)
				{
					int32 DecaySampelsFromStart = InState.AttackSampleCount + InState.DecaySampleCount;
					int32 DecaySamplesLeft = DecaySampelsFromStart - InState.CurrentSampleIndex;
					// We are in decay
					if (DecaySamplesLeft > 0)
					{
						EndDecayFrame = FMath::Min(StartFrame + EndAttackFrame + DecaySamplesLeft, EndFrame);

						for (int32 i = EndAttackFrame; i < EndDecayFrame; ++i)
						{
							int32 SampleCountInDecayState = InState.CurrentSampleIndex++ - InState.AttackSampleCount;
							float DecayFracton = (float)SampleCountInDecayState / InState.DecaySampleCount;
							float TargetEnvelopeValue = 1.0f - (1.0f - InState.SustainLevel) * FMath::Pow(DecayFracton, InState.DecayCurveFactor);
							InState.EnvEase.SetValue(TargetEnvelopeValue);
							InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
							OutEnvPtr[i] = InState.CurrentEnvelopeValue;
						}

						// We have finished the decay phase in this block
						if (InState.CurrentSampleIndex == DecaySampelsFromStart)
						{
							OutOnSustainFrames.Add(EndDecayFrame);
						}
					}

					// if the end decay frame is not the end of this current render frame, we are in a post-decay mode
					// could be in sustain or release, or we're done.
					if (EndDecayFrame < EndFrame)
					{
						// If we're not in release mode, we're in sustain mode
						if (!InState.bIsInRelease)
						{

							for (int32 i = EndDecayFrame; i < EndFrame; ++i)
							{
								InState.EnvEase.SetValue(InState.SustainLevel);
								InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
								InState.EnvelopeValueAtReleaseStart = InState.CurrentEnvelopeValue;
								OutEnvPtr[i] = InState.CurrentEnvelopeValue;
							}
						}
						// We're in release mode
						else
						{
							int32 ReleaseSamplesLeft = (InState.AttackSampleCount + InState.DecaySampleCount + InState.ReleaseSampleCount) - InState.CurrentSampleIndex;

							int32 EndReleaseFrame = FMath::Min(StartFrame + ReleaseSamplesLeft, EndFrame);
							for (int32 i = EndDecayFrame; i < EndReleaseFrame; ++i)
							{
								int32 SampleCountInReleaseState = InState.CurrentSampleIndex++ - InState.DecaySampleCount - InState.AttackSampleCount;
								float ReleaseFraction = (float)SampleCountInReleaseState / InState.ReleaseSampleCount;
								float TargetEnvelopeValue = InState.EnvelopeValueAtReleaseStart * (1.0 - FMath::Pow(ReleaseFraction, InState.ReleaseCurveFactor));
								InState.EnvEase.SetValue(TargetEnvelopeValue);
								InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
								OutEnvPtr[i] = InState.CurrentEnvelopeValue;
							}

							// We're now done, lets taper it off and finish things
							if (EndReleaseFrame < EndFrame)
							{
								InState.EnvEase.SetValue(0.f);
								for (int32 i = EndReleaseFrame; i < EndFrame; ++i)
								{
									InState.CurrentEnvelopeValue = InState.EnvEase.GetNextValue();
									OutEnvPtr[i] = InState.CurrentEnvelopeValue;

									// Envelope is done
									if (InState.EnvEase.IsDone())
									{
										// Zero out the rest of the envelope
										int32 NumSamplesLeft = EndFrame - i - 1;
										if (NumSamplesLeft > 0)
										{
											FMemory::Memzero(&OutEnvPtr[i + 1], sizeof(float) * NumSamplesLeft);
										}

										InState.CurrentSampleIndex = INDEX_NONE;
										OutOnDoneFrames.Add(i);
										break;
									}
								}
							}
						}
					}
				}
			}
		};

	}

	template<typename EnvelopeClass, typename ValueType>
	class TADSREnvelopeNodeOperator : public TExecutableOperator<TADSREnvelopeNodeOperator<EnvelopeClass, ValueType>>
	{
		enum class ETriggerType : uint8
		{
			Attack,
			Release,
			None
		};

		struct FTriggerInfo
		{
			ETriggerType Type;
			int32 FrameIndex;
		};

		struct FTriggerIterator
		{
			FTriggerIterator(const FTrigger& InAttackTrigger, const FTrigger& InReleaseTrigger, int32 InNumFramesPerBlock)
			: AttackTrigger(InAttackTrigger)
			, ReleaseTrigger(InReleaseTrigger)
			, NumFramesPerBlock(InNumFramesPerBlock)
			{
				if (AttackTrigger.NumTriggeredInBlock() > 0)
				{
					NextAttackFrame = AttackTrigger[0];
					AttackTriggerIndex++;
				}
				else
				{
					NextAttackFrame = NumFramesPerBlock;
				}

				if (ReleaseTrigger.NumTriggeredInBlock() > 0)
				{
					NextReleaseFrame = ReleaseTrigger[0];
					ReleaseTriggerIndex++;
				}
				else
				{
					NextReleaseFrame = NumFramesPerBlock;
				}
			}

			FTriggerInfo NextTrigger(bool EnvelopeIsTriggered)
			{
				FTriggerInfo Info;

				// If attack and release triggers arrive on the same frame we will process the 
				// release first if the envelope is currently triggered, or process the attack first if 
				// the envelope is not alreay triggered. In fact, in this second case we could do NOTHING and 
				// "swallow" them both. Since we know that the attack will be followed immediately by a release.
				// But we don't that as there may be nodes down the line watching the ADSR's output triggers
				// and they may want to know about the attack and release even if they do cancel out. 
				if (NextAttackFrame == NextReleaseFrame && EnvelopeIsTriggered && NextReleaseFrame < NumFramesPerBlock)
				{
					Info.Type = ETriggerType::Release;
					Info.FrameIndex = NextReleaseFrame;

					if (ReleaseTriggerIndex < ReleaseTrigger.NumTriggeredInBlock())
					{
						NextReleaseFrame = ReleaseTrigger[ReleaseTriggerIndex];
						ReleaseTriggerIndex++;
					}
					else
					{
						NextReleaseFrame = NumFramesPerBlock;
					}
				}
				else if ((NextAttackFrame <= NextReleaseFrame) && (NextAttackFrame < NumFramesPerBlock))
				{
					Info.Type = ETriggerType::Attack;
					Info.FrameIndex = NextAttackFrame;

					if (AttackTriggerIndex < AttackTrigger.NumTriggeredInBlock())
					{
						NextAttackFrame = AttackTrigger[AttackTriggerIndex];
						AttackTriggerIndex++;
					}
					else
					{
						NextAttackFrame = NumFramesPerBlock;
					}
				}
				else if (NextReleaseFrame < NumFramesPerBlock)
				{
					Info.Type = ETriggerType::Release;
					Info.FrameIndex = NextReleaseFrame;

					if (ReleaseTriggerIndex < ReleaseTrigger.NumTriggeredInBlock())
					{
						NextReleaseFrame = ReleaseTrigger[ReleaseTriggerIndex];
						ReleaseTriggerIndex++;
					}
					else
					{
						NextReleaseFrame = NumFramesPerBlock;
					}
				}
				else
				{
					Info.Type = ETriggerType::None;
					Info.FrameIndex = NumFramesPerBlock;
				}

				return Info;
			}

		private:
			const FTrigger& AttackTrigger;
			const FTrigger& ReleaseTrigger;

			int32 NumFramesPerBlock = 0;
			int32 AttackTriggerIndex = 0;
			int32 NextAttackFrame = 0;

			int32 ReleaseTriggerIndex = 0;
			int32 NextReleaseFrame = 0;
		};

	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ADSREnvelopeVertexNames;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackTrigger)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseTrigger)),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackTime), 0.01f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDecayTime), 0.2f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSustainLevel), 0.5f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseTime), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackCurve), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDecayCurve), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseCurve), 1.0f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputHardReset), false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnAttackTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnDecayTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSustainTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnReleaseTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnDone)),
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputEnvelopeValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = "ADSR Envelope";
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ADSREnvelopeDisplayNamePattern", "ADSR Envelope ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("ADSREnevelopeDesc", "Generates an attack-decay-sustain-release envelope value output when triggered.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				const FNodeClassMetadata Metadata
				{
					FNodeClassName { "ADSR Envelope", OperatorName, DataTypeName },
					1, // Major Version
					0, // Minor Version
					NodeDisplayName,
					NodeDescription,
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Envelopes },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		struct FOperatorArgs
		{
			const FBuildOperatorParams& BuildOperatorParams;
			FTriggerReadRef TriggerAttackIn;
			FTriggerReadRef TriggerReleaseIn;
			FTimeReadRef AttackTime;
			FTimeReadRef DecayTime;
			FFloatReadRef SustainLevel;
			FTimeReadRef ReleaseTime;
			FFloatReadRef AttackCurveFactor;
			FFloatReadRef DecayCurveFactor;
			FFloatReadRef ReleaseCurveFactor;
			FBoolReadRef bInHardReset;
		};
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ADSREnvelopeVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FOperatorArgs Args 
			{
				InParams,
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputAttackTrigger), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReleaseTrigger), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputAttackTime), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputDecayTime), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputSustainLevel), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputReleaseTime), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputAttackCurve), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDecayCurve), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputReleaseCurve), InParams.OperatorSettings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputHardReset), InParams.OperatorSettings)
			};

			return MakeUnique<TADSREnvelopeNodeOperator<EnvelopeClass, ValueType>>(Args);
		}

		TADSREnvelopeNodeOperator(const FOperatorArgs& InArgs)
			: TriggerAttackIn(InArgs.TriggerAttackIn)
			, TriggerReleaseIn(InArgs.TriggerReleaseIn)
			, AttackTime(InArgs.AttackTime)
			, DecayTime(InArgs.DecayTime)
			, SustainLevel(InArgs.SustainLevel)
			, ReleaseTime(InArgs.ReleaseTime)
			, AttackCurveFactor(InArgs.AttackCurveFactor)
			, DecayCurveFactor(InArgs.DecayCurveFactor)
			, ReleaseCurveFactor(InArgs.ReleaseCurveFactor)
			, bHardReset(InArgs.bInHardReset)
			, OnDecayTrigger(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InArgs.BuildOperatorParams.OperatorSettings))
			, OnSustainTrigger(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InArgs.BuildOperatorParams.OperatorSettings))
			, OnDone(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InArgs.BuildOperatorParams.OperatorSettings))
			, OutputEnvelope(TDataWriteReferenceFactory<ValueType>::CreateExplicitArgs(InArgs.BuildOperatorParams.OperatorSettings))
		{
			Reset(InArgs.BuildOperatorParams);
		}

		virtual ~TADSREnvelopeNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ADSREnvelopeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackTrigger), TriggerAttackIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseTrigger), TriggerReleaseIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackTime), AttackTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDecayTime), DecayTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSustainLevel), SustainLevel);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseTime), ReleaseTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackCurve), AttackCurveFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDecayCurve), DecayCurveFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseCurve), ReleaseCurveFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputHardReset), bHardReset);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ADSREnvelopeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnAttackTrigger), TriggerAttackIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnDecayTrigger), OnDecayTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnSustainTrigger), OnSustainTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnReleaseTrigger), TriggerReleaseIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnDone), OnDone);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputEnvelopeValue), OutputEnvelope);
		}

		void UpdateParams()
		{
			// Clamp each segment sample count to Max/4 so that the sum of all three 
			// can never overflow int32. 
			static constexpr float MaxSegmentSampleCount = static_cast<float>(TNumericLimits<int32>::Max() / 4);

			float AttackTimeSeconds = AttackTime->GetSeconds();
			float DecayTimeSeconds = DecayTime->GetSeconds();
			float ReleaseTimeSeconds = ReleaseTime->GetSeconds();
			EnvState.AttackSampleCount = FMath::Max(1, static_cast<int32>(FMath::Min(SampleRate * FMath::Max(0.0f, AttackTimeSeconds), MaxSegmentSampleCount)));
			EnvState.DecaySampleCount = FMath::Max(0, static_cast<int32>(FMath::Min(SampleRate * FMath::Max(0.0f, DecayTimeSeconds), MaxSegmentSampleCount)));
			EnvState.SustainLevel = FMath::Clamp(*SustainLevel, 0.0f, 1.0f);
			EnvState.ReleaseSampleCount = FMath::Max(0, static_cast<int32>(FMath::Min(SampleRate * FMath::Max(0.0f, ReleaseTimeSeconds), MaxSegmentSampleCount)));
			EnvState.AttackCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, *AttackCurveFactor);
			EnvState.DecayCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, *DecayCurveFactor);
			EnvState.ReleaseCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, *ReleaseCurveFactor);
			EnvState.bHardReset = *bHardReset;
		}

		void ProcessEnvelopeOutput(int32 InStartFrame, int32 InEndFrame)
		{
			TArray<int32> OnDecayFrames;
			TArray<int32> OnSustainFrames;
			TArray<int32> OnDoneFrames;
			EnvelopeClass::GetNextEnvelopeOutput(EnvState, InStartFrame, InEndFrame, OnDecayFrames, OnSustainFrames, OnDoneFrames, *OutputEnvelope);

			for (int32 OnDecayFrame : OnDecayFrames)
			{
				OnDecayTrigger->TriggerFrame(OnDecayFrame);
			}

			for (int32 OnSustainFrame : OnSustainFrames)
			{
				OnSustainTrigger->TriggerFrame(OnSustainFrame);
			}

			for (int32 OnDoneFrame : OnDoneFrames)
			{
				OnDone->TriggerFrame(OnDoneFrame);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();

			SampleRate = EnvelopeClass::GetSampleRate(InParams.OperatorSettings);
			EnvelopeClass::GetInitialOutputEnvelope(*OutputEnvelope);
			EnvState.Reset();

			OnDecayTrigger->Reset();
			OnSustainTrigger->Reset();
			OnDone->Reset();
		}

		void Execute()
		{
			using namespace ADSREnvelopeNodePrivate;

			OnDecayTrigger->AdvanceBlock();
			OnSustainTrigger->AdvanceBlock();
			OnDone->AdvanceBlock();
 				
			FTriggerIterator TriggerIter(*TriggerAttackIn, *TriggerReleaseIn, NumFramesPerBlock);

			FTriggerInfo NextTrigger = TriggerIter.NextTrigger(EnvState.IsTriggered());
			if (NextTrigger.FrameIndex > 0)
			{
				// Process envelope before receiving any triggers on this block
				ProcessEnvelopeOutput(0, NextTrigger.FrameIndex);
			}

			while (NextTrigger.Type != ETriggerType::None)
			{
				FTriggerInfo CurrentTrigger = NextTrigger;
				NextTrigger = TriggerIter.NextTrigger(EnvState.IsTriggered());

				switch (CurrentTrigger.Type)
				{
					case ETriggerType::Attack:
					// check for any updates to input params
					UpdateParams();
					// Set the sample index to the top of the envelope
					EnvState.CurrentSampleIndex = 0;
					EnvState.StartingEnvelopeValue = EnvState.bHardReset ? 0 : EnvState.CurrentEnvelopeValue;
					EnvState.EnvEase.SetValue(EnvState.StartingEnvelopeValue, true /* bInit */);
					EnvState.bIsInRelease = false;

						break;

					case ETriggerType::Release:

						EnvState.bIsInRelease = true;

						break;

					default:
						{
							checkNoEntry();
						}
				}

				ProcessEnvelopeOutput(CurrentTrigger.FrameIndex, NextTrigger.FrameIndex);
			}
		}

	private:
		FTriggerReadRef TriggerAttackIn;
		FTriggerReadRef TriggerReleaseIn;
		FTimeReadRef AttackTime;
		FTimeReadRef DecayTime;
		FFloatReadRef SustainLevel;
		FTimeReadRef ReleaseTime;
		FFloatReadRef AttackCurveFactor;
		FFloatReadRef DecayCurveFactor;
		FFloatReadRef ReleaseCurveFactor;
		FBoolReadRef bHardReset;

		FTriggerWriteRef OnDecayTrigger;
		FTriggerWriteRef OnSustainTrigger;
		FTriggerWriteRef OnDone;
		TDataWriteReference<ValueType> OutputEnvelope;

		// This will either be the block rate or sample rate depending on if this is block-rate or audio-rate envelope
		float SampleRate = 0.0f;
		int32 NumFramesPerBlock = 0;

		ADSREnvelopeNodePrivate::FEnvState EnvState;
	};

	/** TADSREnvelopeNode
	 *
	 *  Creates an Attack/Decay envelope node.
	 */
	template<typename EnvelopeClass, typename ValueType>
	using TADSREnvelopeNode = TNodeFacade<TADSREnvelopeNodeOperator<EnvelopeClass, ValueType>>;

	using FADSREnvelopeNodeFloat = TADSREnvelopeNode<ADSREnvelopeNodePrivate::FFloatADSREnvelope, float>;
	METASOUND_REGISTER_NODE(FADSREnvelopeNodeFloat)

	using FADSREnvelopeAudioBuffer = TADSREnvelopeNode<ADSREnvelopeNodePrivate::FAudioADSREnvelope, FAudioBuffer>;
	METASOUND_REGISTER_NODE(FADSREnvelopeAudioBuffer)
}

#undef LOCTEXT_NAMESPACE
