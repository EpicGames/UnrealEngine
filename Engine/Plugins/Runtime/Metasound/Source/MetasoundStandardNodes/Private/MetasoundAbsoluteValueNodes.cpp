// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeRegistrationMacro.h"
#include "CoreMinimal.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_AbsNode"

namespace Metasound
{
	namespace AbsVertexNames
	{
		METASOUND_PARAM(InputValue, "Input", "Input value.");
		METASOUND_PARAM(OutputValue, "Value", "The Absolute Value of the input.");
	}

	namespace MetasoundAbsNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "AbsoluteValue", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Math },
				{ },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}

		template<typename ValueType>
		struct TAbs
		{
			bool bSupported = false;
		};

		template<>
		struct TAbs<int32>
		{
			static void GetAbs(const int32 In, int32& OutAbs)
			{
				OutAbs = FMath::Abs(In);
			}

			static TDataReadReference<int32> CreateInRef(const FBuildOperatorParams& InParams)
			{
				using namespace AbsVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}
		};

		template<>
		struct TAbs<float>
		{
			static void GetAbs(const float In, float& OutAbs)
			{
				OutAbs = FMath::Abs(In);
			}

			static TDataReadReference<float> CreateInRef(const FBuildOperatorParams& InParams)
			{
				using namespace AbsVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static bool IsAudioBuffer() { return false; }
		};
		
		template<>
		struct TAbs<FTime>
		{
			static void GetAbs(const FTime& In, FTime& OutAbs)
			{
				OutAbs = FTime(FMath::Abs(In.GetSeconds()));
			}

			static TDataReadReference<FTime> CreateInRef(const FBuildOperatorParams& InParams)
			{
				using namespace AbsVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static bool IsAudioBuffer() { return false; }
		};

		template<>
		struct TAbs<FAudioBuffer>
		{
			static void GetAbs(const FAudioBuffer& In, FAudioBuffer& OutAbs)
			{
				TArrayView<float> OutAbsView(OutAbs.GetData(), OutAbs.Num());
				TArrayView<const float> InView(In.GetData(), OutAbs.Num());
				
				Audio::ArrayAbs(InView, OutAbsView);
			}

			static TDataReadReference<FAudioBuffer> CreateInRef(const FBuildOperatorParams& InParams)
			{
				using namespace AbsVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}
		};
	}

	template<typename ValueType>
	class TAbsNodeOperator : public TExecutableOperator<TAbsNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace AbsVertexNames;
			using namespace MetasoundAbsNodePrivate;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValue))
				),
				FOutputVertexInterface(
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = TEXT("Abs");
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("AbsDisplayNamePattern", "Abs ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("AbsDesc", "Returns the Absolute Value of the input.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundAbsNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace AbsVertexNames;
			using namespace MetasoundAbsNodePrivate;

			TDataReadReference<ValueType> Input = TAbs<ValueType>::CreateInRef(InParams);

			return MakeUnique<TAbsNodeOperator<ValueType>>(InParams.OperatorSettings, Input);
		}


		TAbsNodeOperator(const FOperatorSettings& InSettings,
			const TDataReadReference<ValueType>& InInput)
			: Input(InInput)
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
		{
			GetAbs();
		}

		virtual ~TAbsNodeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AbsVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), Input);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AbsVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void GetAbs()
		{
			using namespace MetasoundAbsNodePrivate;

			TAbs<ValueType>::GetAbs(*Input, *OutputValue);
		}

		void Execute()
		{
			GetAbs();
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			GetAbs();
		}

	private:

		TDataReadReference<ValueType> Input;
		TDataWriteReference<ValueType> OutputValue;
	};

	/** TAbsNode
	 *
	 *  Returns the Aof the input
	 */
	template<typename ValueType>
	using TAbsNode = TNodeFacade<TAbsNodeOperator<ValueType>>;

	using FAbsNodeInt32 = TAbsNode<int32>;
 	METASOUND_REGISTER_NODE(FAbsNodeInt32)

	using FAbsNodeFloat = TAbsNode<float>;
	METASOUND_REGISTER_NODE(FAbsNodeFloat)
	
	using FAbsNodeTime = TAbsNode<FTime>;
    METASOUND_REGISTER_NODE(FAbsNodeTime)

	using FAbsNodeAudioBuffer = TAbsNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FAbsNodeAudioBuffer)
}

#undef LOCTEXT_NAMESPACE
