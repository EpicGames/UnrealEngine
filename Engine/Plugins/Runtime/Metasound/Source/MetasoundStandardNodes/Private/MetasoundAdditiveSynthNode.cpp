// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FloatArrayMath.h"
#include "DSP/SineWaveTableOsc.h"
#include "MetasoundFacade.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "MetasoundAdditiveSynthNode"

namespace Metasound
{
	namespace AdditiveSynthVertexNames
	{
		METASOUND_PARAM(BaseFrequency, "Base Frequency", "Sinusoid frequency that harmonics are based on, clamped to 0.0 - Nyquist.");
		METASOUND_PARAM(HarmonicMultipliers, "HarmonicMultipliers", "Array of float harmonic multipliers that will be applied to the base frequency. Number of sinusoids used depends on the size of this array. Clamped from 0.0 to a max such that the resulting frequency won't go above Nyquist.");
		METASOUND_PARAM(Amplitudes, "Amplitudes", "Array of sinusoid amplitudes, clamped to 0.0 - 1.0");
		METASOUND_PARAM(Phases, "Phases", "Array of sinusoid phases in degrees, clamped to 0.0 - 360.0");
		METASOUND_PARAM(PanAmounts, "Pan Amounts", "Array of pan amounts (using equal power law; -1.0 is full left, 1.0 is full right).");

		METASOUND_PARAM(LeftAudioOut, "Out Left Audio", "Left output synthesized audio.");
		METASOUND_PARAM(RightAudioOut, "Out Right Audio", "Right output synthesized audio.");
	}

	class FAdditiveSynthOperator : public TExecutableOperator<FAdditiveSynthOperator>
	{
	public:
		using FInputFloatArrayReadRef = TDataReadReference<TArray<float>>;
		using FInputFloatArrayType = TArray<float>;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				const FName OperatorName = "Additive Synth";
				const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_AdditiveSynthNodeDisplayName", "Additive Synth");
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_AdditiveSynthNodeDescription", "Synthesizes audio output given input array of sinusoids.");

				FNodeClassMetadata Info;
				Info.ClassName = { Metasound::StandardNodes::Namespace, OperatorName, FName() };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = NodeDescription;
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("AdditiveSynthesisKeyword", "Synthesis") };

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace AdditiveSynthVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(BaseFrequency), 440.0f),
					TInputDataVertex<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(HarmonicMultipliers)),
					TInputDataVertex<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Amplitudes)),
					TInputDataVertex<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(Phases)),
					TInputDataVertex<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(PanAmounts))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(LeftAudioOut)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(RightAudioOut))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace AdditiveSynthVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FFloatReadRef InputBaseFrequency = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(BaseFrequency), InParams.OperatorSettings);
			FInputFloatArrayReadRef InputHarmonicMultipliers = InputData.GetOrCreateDefaultDataReadReference<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME(HarmonicMultipliers), InParams.OperatorSettings);
			FInputFloatArrayReadRef InputAmplitudes = InputData.GetOrCreateDefaultDataReadReference<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME(Amplitudes), InParams.OperatorSettings);
			FInputFloatArrayReadRef InputPhases = InputData.GetOrCreateDefaultDataReadReference<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME(Phases), InParams.OperatorSettings);
			FInputFloatArrayReadRef InputPanAmounts = InputData.GetOrCreateDefaultDataReadReference<FInputFloatArrayType>(METASOUND_GET_PARAM_NAME(PanAmounts), InParams.OperatorSettings);

			return MakeUnique<FAdditiveSynthOperator>(InParams.OperatorSettings, InputBaseFrequency, InputHarmonicMultipliers, InputAmplitudes, InputPhases, InputPanAmounts);
		}

		FAdditiveSynthOperator(const FOperatorSettings& InSettings,
			const FFloatReadRef& InputBaseFrequency,
			const FInputFloatArrayReadRef& InputHarmonicMultipliers,
			const FInputFloatArrayReadRef& InputAmplitudes,
			const FInputFloatArrayReadRef& InputPhases,
			const FInputFloatArrayReadRef& InputPanAmounts)
			: BaseFrequency(InputBaseFrequency)
			, HarmonicMultipliers(InputHarmonicMultipliers)
			, Amplitudes(InputAmplitudes)
			, Phases(InputPhases)
			, PanAmounts(InputPanAmounts)
			, LeftAudioOut(FAudioBufferWriteRef::CreateNew(InSettings))
			, RightAudioOut(FAudioBufferWriteRef::CreateNew(InSettings))
			, SampleRate(InSettings.GetSampleRate())
		{
			CreateSinusoids();
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AdditiveSynthVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(BaseFrequency), BaseFrequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(HarmonicMultipliers), HarmonicMultipliers);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Amplitudes), Amplitudes);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(Phases), Phases);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(PanAmounts), PanAmounts);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace AdditiveSynthVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LeftAudioOut), LeftAudioOut);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RightAudioOut), RightAudioOut);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			CreateSinusoids();
			LeftAudioOut->Zero();
			RightAudioOut->Zero();
		}

		void Execute()
		{
			int32 NumSamples = LeftAudioOut->Num();

			// Zero out output buffers since we'll sum directly into them
			FMemory::Memzero(LeftAudioOut->GetData(), sizeof(float) * NumSamples);
			FMemory::Memzero(RightAudioOut->GetData(), sizeof(float) * NumSamples);
			// Clear temporary buffers
			if (ScratchBuffer.Num() != NumSamples)
			{
				ScratchBuffer.Reset();
				ScratchBuffer.Init(0, NumSamples);
			}

			int32 NumSinusoids = HarmonicMultipliers->Num();
			
			// Number of elements in parallel arrays doesn't match the current number of oscillators
			if (NumSinusoids != SinusoidData.Num())
			{
				CreateSinusoids();
			}

			// Flag to update frequency (which depends on node base frequency and individual harmonic multiplier) if necessary
			bool UpdateFrequency = false;
			float Frequency = *BaseFrequency;
			// Clamp frequency to Nyquist 
			const float Nyquist = SampleRate / 2.0f;
			Frequency = FMath::Clamp(Frequency, 0.0f, Nyquist);
			if (!FMath::IsNearlyEqual(Frequency, PreviousBaseFrequency))
			{
				PreviousBaseFrequency = Frequency;
				UpdateFrequency = true;
			}

			// For each sinusoid, update parameters and generate audio
			for (int32 SinusoidIndex = 0; SinusoidIndex < NumSinusoids; ++SinusoidIndex)
			{
				Audio::FSineWaveTableOsc& WaveTableOsc = SinusoidData[SinusoidIndex];

				// Update harmonic multiplier and resulting frequency if necessary
				float HarmonicMultiplier = (*HarmonicMultipliers)[SinusoidIndex];
				// Maximum is Nyquist / Frequency because the actual frequency will be HarmonicMultiplier * Frequency
				HarmonicMultiplier = FMath::Clamp(HarmonicMultiplier, 0.0f, Nyquist / Frequency);
				if (!FMath::IsNearlyEqual(HarmonicMultiplier, PreviousHarmonicMultipliers[SinusoidIndex]))
				{
					PreviousHarmonicMultipliers[SinusoidIndex] = HarmonicMultiplier;
					UpdateFrequency = true;
				}
				
				if (UpdateFrequency)
				{
					WaveTableOsc.SetFrequencyHz(Frequency * HarmonicMultiplier);
				}

				// Update phase if necessary, set to 0 if no phases are given
				// If the number of elements in Phases doesn't equal the number of sinusoids, 
				// modulus on the Phases array (i.e. if 2 elements, phases will alternate)
				float Phase = 0.0f;
				if (Phases->Num() == 0)
				{
					if (!FMath::IsNearlyEqual(Phase, PreviousPhases[SinusoidIndex]))
					{
						PreviousPhases[SinusoidIndex] = Phase;
						WaveTableOsc.SetPhase(Phase);
					}
				}
				else
				{
					int32 PhaseModIndex = SinusoidIndex % Phases->Num();
					Phase = FMath::Clamp((*Phases)[PhaseModIndex], 0.0f, 360.0f);
					if (!FMath::IsNearlyEqual(Phase, PreviousPhases[PhaseModIndex]))
					{
						PreviousPhases[PhaseModIndex] = Phase;
						WaveTableOsc.SetPhase(Phase / 360.0f);
					}
				}

				// Get samples from wavetable
				WaveTableOsc.Generate(ScratchBuffer.GetData(), NumSamples);

				// Apply pan
				float LeftPanGain = 1.0f;
				float RightPanGain = 1.0f;
				// If PanAmounts is empty, will default to 0 
				// If the number of elements in PanAmounts doesn't equal the number of sinusoids, 
				// modulus on the PanAmounts array (i.e. if 2 elements, pan amounts will alternate)
				if (PanAmounts->Num() > 0)
				{
					float Fraction = 0.5f * (FMath::Clamp((*PanAmounts)[SinusoidIndex % PanAmounts->Num()], -1.0f, 1.0f) + 1.0f);
					// Using equal power panning law 
					FMath::SinCos(&RightPanGain, &LeftPanGain, 0.5f * PI * Fraction);
				}

				// Clamp given amplitude
				float BaseGain = 1.0f;
				if (Amplitudes->Num() > 0)
				{
					 BaseGain = FMath::Clamp((*Amplitudes)[SinusoidIndex % Amplitudes->Num()], 0.0f, 1.0f);
				}

				// Multiply by gain (both amplitude and pan)
				float NewLeftGain = LeftPanGain * BaseGain;
				float NewRightGain = RightPanGain * BaseGain;

				// Mix into output buffer
				TArrayView<const float> ScratchBufferView(ScratchBuffer.GetData(), NumSamples);
				TArrayView<float> LeftAudioOutView(LeftAudioOut->GetData(), NumSamples);
				TArrayView<float> RightAudioOutView(RightAudioOut->GetData(), NumSamples);
				Audio::ArrayMixIn(ScratchBufferView, LeftAudioOutView, NewLeftGain);
				Audio::ArrayMixIn(ScratchBufferView, RightAudioOutView, NewRightGain);
			}
		}

	private:
		void CreateSinusoids()
		{
			int32 NumSinusoids = HarmonicMultipliers->Num();
			SinusoidData.Reset(NumSinusoids);
			PreviousHarmonicMultipliers.Reset(NumSinusoids);
			PreviousPhases.Reset(NumSinusoids);

			// Cache information so we don't have to update the oscillator every time, pad to NumSinusoids 
			PreviousHarmonicMultipliers = *HarmonicMultipliers;
			PreviousPhases = *Phases;
			if (PreviousPhases.Num() < NumSinusoids)
			{
				PreviousPhases.AddZeroed(NumSinusoids - PreviousPhases.Num());
			}
			PreviousBaseFrequency = *BaseFrequency;

			for (int32 SinusoidIndex = 0; SinusoidIndex < NumSinusoids; ++SinusoidIndex)
			{
				// Create wavetable oscillators for each sinusoid
				Audio::FSineWaveTableOsc WaveTableOsc = Audio::FSineWaveTableOsc();
				float Phase = Phases->Num() > 0 ? (*Phases)[SinusoidIndex % Phases->Num()] : 0.0f;
				WaveTableOsc.Init(SampleRate, (*HarmonicMultipliers)[SinusoidIndex] * (*BaseFrequency), Phase / 360.0f);
				SinusoidData.Add(WaveTableOsc);
			}
		}

		TArray<Audio::FSineWaveTableOsc> SinusoidData;

		FFloatReadRef BaseFrequency;
		FInputFloatArrayReadRef HarmonicMultipliers;
		FInputFloatArrayReadRef Amplitudes;
		FInputFloatArrayReadRef Phases;
		FInputFloatArrayReadRef PanAmounts;

		// Cached copies of sinusoid parameters to avoid unnecessary updates
		TArray<float> PreviousHarmonicMultipliers;
		TArray<float> PreviousPhases;
		float PreviousBaseFrequency = 440.0f;

		// Scratch aligned buffer
		Audio::FAlignedFloatBuffer ScratchBuffer;

		// Audio output
		FAudioBufferWriteRef LeftAudioOut;
		FAudioBufferWriteRef RightAudioOut;

		// Sample rate, will be set in constructor
		float SampleRate = 48000.0f;
	};

	using FAdditiveSynthNode = TNodeFacade<FAdditiveSynthOperator>;

	METASOUND_REGISTER_NODE(FAdditiveSynthNode)
}

#undef LOCTEXT_NAMESPACE
