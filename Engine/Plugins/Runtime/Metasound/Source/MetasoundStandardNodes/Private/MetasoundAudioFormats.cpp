// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundAudioFormats.h"

#include "CoreMinimal.h"
#include "MetasoundArrayNodesRegistration.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundInputNode.h"
#include "MetasoundLiteral.h"
#include "MetasoundOutputNode.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_AudioFormats"

namespace Metasound
{
	/* FMultichannelAudioFormat */

	FMultichannelAudioFormat::FMultichannelAudioFormat()
	:	NumChannels(0)
	{
	}

	FMultichannelAudioFormat::FMultichannelAudioFormat(int32 InNumFrames, int32 InNumChannels)
	:	NumChannels(InNumChannels)
	{
		NumChannels = FMath::Max(0, NumChannels);
		InNumFrames = FMath::Max(0, InNumFrames);

		for (int32 i = 0; i < NumChannels; i++)
		{
			FAudioBufferWriteRef Audio = FAudioBufferWriteRef::CreateNew(InNumFrames);
			Audio->Zero();

			WritableBufferStorage.Add(Audio);
			ReadableBufferStorage.Add(Audio);
		}

		WritableBuffers = WritableBufferStorage;
		ReadableBuffers = ReadableBufferStorage;
	}

	FMultichannelAudioFormat::FMultichannelAudioFormat(const FOperatorSettings& InSettings, int32 InNumChannels)
		: FMultichannelAudioFormat(InSettings.GetNumFramesPerBlock(), InNumChannels)
	{}

	FMultichannelAudioFormat::FMultichannelAudioFormat(TArrayView<const FAudioBufferWriteRef> InWriteRefs)
	:	NumChannels(InWriteRefs.Num())
	{
		if (NumChannels > 0)
		{
			const int32 NumFrames = InWriteRefs[0]->Num();

			for (const FAudioBufferWriteRef& Ref : InWriteRefs)
			{
				checkf(NumFrames == Ref->Num(), TEXT("All buffers must have same number of frames (%d != %d)"), NumFrames, Ref->Num());

				WritableBufferStorage.Add(Ref);
				ReadableBufferStorage.Add(Ref);
			}

			WritableBuffers = WritableBufferStorage;
			ReadableBuffers = ReadableBufferStorage;
		}
	}

	FMultichannelAudioFormat::FMultichannelAudioFormat(TArrayView<const FAudioBufferReadRef> InReadRefs)
	:	NumChannels(InReadRefs.Num())
	{
		if (NumChannels > 0)
		{
			const int32 NumFrames = InReadRefs[0]->Num();

			for (const FAudioBufferReadRef& Ref : InReadRefs)
			{
				checkf(NumFrames == Ref->Num(), TEXT("All buffers must have same number of frames (%d != %d)"), NumFrames, Ref->Num());

				WritableBufferStorage.Add(WriteCast(Ref));
				ReadableBufferStorage.Add(Ref);
			}

			WritableBuffers = WritableBufferStorage;
			ReadableBuffers = ReadableBufferStorage;
		}
	}

	// Special vertex keys for stereo input/output nodes.
	namespace StereoAudioFormatVertexKeys
	{
		METASOUND_PARAM(LeftChannelVertex, "Left", "Left channel audio output.")
		METASOUND_PARAM(RightChannelVertex, "Right", "Right channel audio output.")
	}

	// Specialization of TOutputNode<FStereoAudio> to support direct connection
	// of audio buffers to left/right inputs. 
	template<>
	class METASOUNDSTANDARDNODES_API TOutputNode<FStereoAudioFormat> : public FNode
	{
		// FOutputOperator primarly used to report inputs and outputs. Has no execute function.
		class FOutputOperator : public FNoOpOperator
		{
		public:
			FOutputOperator(const FVertexName& InOutputName, TDataReadReference<FAudioBuffer> InLeft, TDataReadReference<FAudioBuffer> InRight, TDataReadReference<FStereoAudioFormat> InStereo)
			: OutputName(InOutputName)
			, Left(InLeft)
			, Right(InRight)
			, Stereo(InStereo)
			{
			}

			virtual ~FOutputOperator() {}


			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				using namespace StereoAudioFormatVertexKeys;

				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LeftChannelVertex), Left);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RightChannelVertex), Right);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(OutputName, Stereo);
			}

		private:

			FVertexName OutputName;
			TDataReadReference<FAudioBuffer> Left;
			TDataReadReference<FAudioBuffer> Right;
			TDataReadReference<FStereoAudioFormat> Stereo;
		};

		class FOutputOperatorFactory : public IOperatorFactory
		{
		public:
			FOutputOperatorFactory(const FVertexName& InOutputName)
			: OutputName(InOutputName)
			{
			}

			TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				using namespace StereoAudioFormatVertexKeys;

				// Construct stereo from left and right audio buffers. 
				TDataReadReference<FAudioBuffer> Left = InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(LeftChannelVertex), InParams.OperatorSettings);
				TDataReadReference<FAudioBuffer> Right = InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(RightChannelVertex), InParams.OperatorSettings);

				TDataReadReference<FStereoAudioFormat> Stereo = TDataReadReferenceFactory<FStereoAudioFormat>::CreateExplicitArgs(InParams.OperatorSettings, WriteCast(Left), WriteCast(Right));

				return MakeUnique<FOutputOperator>(OutputName, Left, Right, Stereo);
			}

		private:
			FVertexName OutputName;
		};

		static FVertexInterface CreateVertexInterface(const FVertexName& InVertexName)
		{
			using namespace StereoAudioFormatVertexKeys; 

			return FVertexInterface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(LeftChannelVertex)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(RightChannelVertex))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FStereoAudioFormat>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_StereoOutputVertexDescription", "Stereo Output.")})
				)
			);
		}



	public:
		TOutputNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FVertexName& InVertexName)
		:	FNode(InInstanceName, InInstanceID, CreateNodeClassMetadata(InVertexName))
		,	VertexInterface(CreateVertexInterface(InVertexName))
		,	Factory(MakeShared<FOutputOperatorFactory, ESPMode::ThreadSafe>(InVertexName))
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InOutputName)
		{
			FNodeClassMetadata Info;

			Info.ClassName = { "Output", GetMetasoundDataTypeName<FStereoAudioFormat>(), "" };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_OutputNodeDisplayNameFormat", "Output {0}", GetMetasoundDataTypeDisplayText<FStereoAudioFormat>());
			Info.Description = METASOUND_LOCTEXT("Metasound_OutputNodeDescription", "Output from the parent Metasound graph.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = CreateVertexInterface(InOutputName);
			Info.AccessFlags = ENodeClassAccessFlags::Deprecated;

			return Info;
		};

		const FVertexInterface& GetVertexInterface() const override
		{
			return VertexInterface;
		}

		virtual TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override
		{
			return Factory;
		}

	private:
		FVertexInterface VertexInterface;

		TSharedRef<FOutputOperatorFactory, ESPMode::ThreadSafe> Factory;

	};

	// Input node specialization to expose left/right audio buffers 
	template<>
	class METASOUNDSTANDARDNODES_API TInputNode<FStereoAudioFormat> : public FInputNode
	{
		// Noop operator. Used to return inputs / outputs for debugging.
		class FInputOperator : public FNoOpOperator
		{
		public:

			FInputOperator(const FVertexName& InInputName, TDataReadReference<FAudioBuffer> InLeft, TDataReadReference<FAudioBuffer> InRight, TDataReadReference<FStereoAudioFormat> InStereo)
			: InputName(InInputName)
			, Left(InLeft)
			, Right(InRight)
			, Stereo(InStereo)
			{
			}

			virtual ~FInputOperator() {}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(InputName, Stereo);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				using namespace StereoAudioFormatVertexKeys;
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LeftChannelVertex), Left);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RightChannelVertex), Right);
			}

		private:

			FVertexName InputName;
			TDataReadReference<FAudioBuffer> Left;
			TDataReadReference<FAudioBuffer> Right;
			TDataReadReference<FStereoAudioFormat> Stereo;
		};

		class FInputOperatorFactory : public IOperatorFactory
		{
		public:
			FInputOperatorFactory(const FVertexName& InInputName)
			: InputName(InInputName)
			{
			}

			TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				// Split a stereo signal into left/right.
				TDataReadReference<FStereoAudioFormat> Stereo = InParams.InputData.GetOrCreateDefaultDataReadReference<FStereoAudioFormat>(InputName, InParams.OperatorSettings);
				TDataReadReference<FAudioBuffer> Left = Stereo->GetLeft();
				TDataReadReference<FAudioBuffer> Right = Stereo->GetRight();

				return MakeUnique<FInputOperator>(InputName, Left, Right, Stereo);
			}

		private:
			FVertexName InputName;
		};

		static FVertexInterface CreateVertexInterface(const FVertexName& InVertexName)
		{
			using namespace StereoAudioFormatVertexKeys;
			
			return FVertexInterface(
				FInputVertexInterface(
					TInputDataVertex<FStereoAudioFormat>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_StereoInputVertexDescription", "Stereo Input.")})
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(LeftChannelVertex)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(RightChannelVertex))
				)
			);
		}



	public:
		static constexpr bool bCanRegister = true;

		TInputNode(FInputNodeConstructorParams&& InParams)
		:	FInputNode(MoveTemp(InParams), GetMetasoundDataTypeName<FStereoAudioFormat>(), EVertexAccessType::Reference, MakeShared<FInputOperatorFactory>(InParams.VertexName))
		{
		}

		explicit TInputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		: FInputNode(MakeShared<FInputOperatorFactory>(InVertexName), MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InInputName)
		{
			FNodeClassMetadata Info;

			Info.ClassName = { "Input", GetMetasoundDataTypeName<FStereoAudioFormat>(), "" };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_InputNodeDisplayNameFormat", "Input {0}", GetMetasoundDataTypeDisplayText<FStereoAudioFormat>());
			Info.Description = METASOUND_LOCTEXT("Metasound_InputNodeDescription", "Input from the parent Metasound graph.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = CreateVertexInterface(InInputName);
			Info.AccessFlags = ENodeClassAccessFlags::Deprecated;

			return Info;
		};
	};

	// TOutputNode<> specialization for FMonoAudioFormat. Allows an audio buffer
	// to be directly connected to an mono audio output.
	template<>
	class METASOUNDSTANDARDNODES_API TOutputNode<FMonoAudioFormat> : public FNode
	{
		class FOutputOperator : public FNoOpOperator
		{
		public:
			FOutputOperator(const FVertexName& InOutputName, TDataReadReference<FAudioBuffer> InCenter, TDataReadReference<FMonoAudioFormat> InMono)
			: OutputName(InOutputName)
			, Center(InCenter)
			, Mono(InMono)
			{
			}

			virtual ~FOutputOperator() {}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(OutputName, Center);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(OutputName, Mono);
			}

			virtual FExecuteFunction GetExecuteFunction() override
			{
				return nullptr;
			}

			virtual FResetFunction GetResetFunction() override
			{
				return nullptr;
			}

		private:

			FVertexName OutputName;
			TDataReadReference<FAudioBuffer> Center;
			TDataReadReference<FMonoAudioFormat> Mono;
		};

		class FOutputOperatorFactory : public IOperatorFactory
		{
		public:
			FOutputOperatorFactory(const FVertexName& InOutputName)
			: OutputName(InOutputName)
			{
			}

			TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				TDataReadReference<FAudioBuffer> Center = InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(OutputName, InParams.OperatorSettings);

				TDataReadReference<FMonoAudioFormat> Mono = TDataReadReferenceFactory<FMonoAudioFormat>::CreateExplicitArgs(InParams.OperatorSettings, WriteCast(Center));

				return MakeUnique<FOutputOperator>(OutputName, Center, Mono);
			}

		private:
			FVertexName OutputName;
		};

		static FVertexInterface CreateVertexInterface(const FVertexName& InVertexName)
		{
			return FVertexInterface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_CenterMonoOutputVertexDescription", "Center channel audio output.")})
				),
				FOutputVertexInterface(
					TOutputDataVertex<FMonoAudioFormat>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_MonoOutputVertexDescription", "Mono Output.")})
				)
			);
		}



	public:
		TOutputNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FVertexName& InVertexName)
		:	FNode(InInstanceName, InInstanceID, CreateNodeClassMetadata(InVertexName))
		,	VertexInterface(CreateVertexInterface(InVertexName))
		,	Factory(MakeShared<FOutputOperatorFactory, ESPMode::ThreadSafe>(InVertexName))
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InOutputName)
		{
			FNodeClassMetadata Info;

			Info.ClassName = { "Output", GetMetasoundDataTypeName<FMonoAudioFormat>(), "" };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_OutputNodeDisplayNameFormat", "Output {0}", GetMetasoundDataTypeDisplayText<FMonoAudioFormat>());
			Info.Description = METASOUND_LOCTEXT("Metasound_OutputNodeDescription", "Output from the parent Metasound graph.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = CreateVertexInterface(InOutputName);
			Info.AccessFlags = ENodeClassAccessFlags::Deprecated;

			return Info;
		};

		const FVertexInterface& GetVertexInterface() const override
		{
			return VertexInterface;
		}

		TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> GetDefaultOperatorFactory() const override
		{
			return Factory;
		}

	private:
		FVertexInterface VertexInterface;

		TSharedRef<FOutputOperatorFactory, ESPMode::ThreadSafe> Factory;

	};

	// TInputNode<> specializastion for FMonoAudioFormat. Allows an input mono audio
	// format to be exposed as a single buffer.
	template<>
	class METASOUNDSTANDARDNODES_API TInputNode<FMonoAudioFormat> : public FInputNode
	{
		class FInputOperator : public FNoOpOperator
		{
		public:

			FInputOperator(const FVertexName& InInputName, TDataReadReference<FAudioBuffer> InCenter, TDataReadReference<FMonoAudioFormat> InMono)
			: InputName(InInputName)
			, Center(InCenter)
			, Mono(InMono)
			{
			}

			virtual ~FInputOperator() {}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(InputName, Mono);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
			{
				InOutVertexData.BindReadVertex(InputName, Center);
			}

			virtual FExecuteFunction GetExecuteFunction() override
			{
				return nullptr;
			}

			virtual FResetFunction GetResetFunction() override
			{
				return nullptr;
			}

		private:

			FVertexName InputName;
			TDataReadReference<FAudioBuffer> Center;
			TDataReadReference<FMonoAudioFormat> Mono;
		};

		class FInputOperatorFactory : public IOperatorFactory
		{
		public:
			FInputOperatorFactory(const FVertexName& InInputName)
			: InputName(InInputName)
			{
			}

			TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				TDataReadReference<FMonoAudioFormat> Mono = InParams.InputData.GetOrCreateDefaultDataReadReference<FMonoAudioFormat>(InputName, InParams.OperatorSettings);
				TDataReadReference<FAudioBuffer> Center = Mono->GetCenter();

				return MakeUnique<FInputOperator>(InputName, Center, Mono);
			}

		private:
			FVertexName InputName;
		};

		static FVertexInterface CreateVertexInterface(const FVertexName& InVertexName)
		{
			return FVertexInterface(
				FInputVertexInterface(
					TInputDataVertex<FMonoAudioFormat>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_MonoInputVertexDescription", "Mono Input.")})
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(InVertexName, FDataVertexMetadata{METASOUND_LOCTEXT("Metasound_CenterMonoInputVertexDescription", "Center channel audio output.")})
				)
			);
		}

	public:
		static constexpr bool bCanRegister = true;

		TInputNode(FInputNodeConstructorParams&& InParams)
		:	FInputNode(MoveTemp(InParams), GetMetasoundDataTypeName<FMonoAudioFormat>(), EVertexAccessType::Reference, MakeShared<FInputOperatorFactory>(InParams.VertexName))
		{
		}

		explicit TInputNode(const FVertexName& InVertexName, FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
		: FInputNode(MakeShared<FInputOperatorFactory>(InVertexName), MoveTemp(InNodeData), MoveTemp(InClassMetadata))
		{
		}

		static FNodeClassMetadata CreateNodeClassMetadata(const FVertexName& InInputName)
		{
			FNodeClassMetadata Info;

			Info.ClassName = { "Input", GetMetasoundDataTypeName<FMonoAudioFormat>(), "" };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_InputNodeDisplayNameFormat", "Input {0}", GetMetasoundDataTypeDisplayText<FMonoAudioFormat>());
			Info.Description = METASOUND_LOCTEXT("Metasound_InputNodeDescription", "Input from the parent Metasound graph.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = CreateVertexInterface(InInputName);
			Info.AccessFlags = ENodeClassAccessFlags::Deprecated;

			return Info;
		};
	};

	// Disable arrays of audio formats. 
	template<>
	struct TEnableArrayNodes<FMonoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	template<>
	struct TEnableArrayNodes<FStereoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	// Disable transmission of audio formats
	template<>
	struct TEnableTransmissionNodeRegistration<FMonoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	template<>
	struct TEnableTransmissionNodeRegistration<FStereoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	// Disable auto converts using audio format constructors
	template<typename FromDataType>
	struct TEnableAutoConverterNodeRegistration<FromDataType, FMonoAudioFormat>
	{
		static constexpr bool Value = !std::is_arithmetic<FromDataType>::value;
	};

	template<typename FromDataType>
	struct TEnableAutoConverterNodeRegistration<FromDataType, FStereoAudioFormat>
	{
		static constexpr bool Value = !std::is_arithmetic<FromDataType>::value;
	};

	// Disable arrays of audio formats
	template<>
	struct TEnableAutoArrayTypeRegistration<FMonoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	template<>
	struct TEnableAutoArrayTypeRegistration<FStereoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	// Disable constructor inputs/outputs
	template<>
	struct TEnableConstructorVertex<FMonoAudioFormat>
	{
		static constexpr bool Value = false;
	};

	// Disable constructor inputs/outputs
	template<>
	struct TEnableConstructorVertex<FStereoAudioFormat>
	{
		static constexpr bool Value = false;
	};
}

// Data type registration has to happen after TInputNode<> and TOutputNode<> specializations
// so that the registration macro has access to the specializations.
REGISTER_METASOUND_DATATYPE(Metasound::FMonoAudioFormat, "Audio:Mono");
REGISTER_METASOUND_DATATYPE(Metasound::FStereoAudioFormat, "Audio:Stereo");
//REGISTER_METASOUND_DATATYPE(Metasound::FMultichannelAudioFormat, "Audio:Multichannel", ELiteralType::Integer);

#undef LOCTEXT_NAMESPACE // MetasoundStandardNodes
