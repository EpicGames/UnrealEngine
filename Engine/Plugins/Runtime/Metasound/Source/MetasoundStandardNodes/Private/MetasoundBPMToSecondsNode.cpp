// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundBPMToSecondsNode.h"

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	namespace BPMToSecondsVertexNames
	{
		METASOUND_PARAM(InputBPM, "BPM", "Beats Per Minute.")
		METASOUND_PARAM(InputBeatMultiplier, "Beat Multiplier", "The multiplier of the BPM.")
		METASOUND_PARAM(InputDivOfWholeNote, "Divisions of Whole Note", "Divisions of a whole note.")
		METASOUND_PARAM(OutputTimeSeconds, "Seconds", "The output time in seconds.")
	}

	class FBPMToSecondsOperator : public TExecutableOperator<FBPMToSecondsOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FBPMToSecondsOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InBPM, const FFloatReadRef& InBeatMultiplier, const FFloatReadRef& InDivOfWholeNote);


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		void UpdateTime();

		FFloatReadRef BPM;
		FFloatReadRef BeatMultiplier;
		FFloatReadRef DivOfWholeNote;

		// The output seconds
		FTimeWriteRef TimeSeconds;

		// Cached midi note value. Used to catch if the value changes to recompute freq output.
		float PrevBPM = -1.0f;
		float PrevBeatMultiplier = -1.0f;
		float PrevDivOfWholeNote = -1.0f;

		static constexpr float MaxBPM = UE_MAX_FLT / 4;
		static constexpr float MinBPM = 1.f; 
		static constexpr float MaxBeatMultiplier = UE_MAX_FLT / 4;
		static constexpr float MinBeatMultiplier = KINDA_SMALL_NUMBER;
		static constexpr float MaxDivOfWholeNote = UE_MAX_FLT / 4;
		static constexpr float MinDivOfWholeNote = 1.f;
	};

	FBPMToSecondsOperator::FBPMToSecondsOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InBPM, const FFloatReadRef& InBeatMultiplier, const FFloatReadRef& InDivOfWholeNote)
		: BPM(InBPM)
		, BeatMultiplier(InBeatMultiplier)
		, DivOfWholeNote(InDivOfWholeNote)
		, TimeSeconds(TDataWriteReferenceFactory<FTime>::CreateExplicitArgs(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	void FBPMToSecondsOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace BPMToSecondsVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputBPM), BPM);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputBeatMultiplier), BeatMultiplier);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDivOfWholeNote), DivOfWholeNote);
	}

	void FBPMToSecondsOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace BPMToSecondsVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTimeSeconds), TimeSeconds);
	}

	void FBPMToSecondsOperator::UpdateTime()
	{
		float CurrBPM = FMath::Clamp(*BPM, MinBPM, MaxBPM);
		float CurrBeatMultiplier = FMath::Clamp(*BeatMultiplier, MinBeatMultiplier, MaxBeatMultiplier);
		float CurrDivOfWholeNote = FMath::Clamp(*DivOfWholeNote, MinDivOfWholeNote, MaxDivOfWholeNote);

		if (!FMath::IsNearlyEqual(PrevBPM, CurrBPM) || 
			!FMath::IsNearlyEqual(PrevBeatMultiplier, CurrBeatMultiplier) || 
			!FMath::IsNearlyEqual(PrevDivOfWholeNote, CurrDivOfWholeNote))
		{
			PrevBPM = CurrBPM;
			PrevBeatMultiplier = CurrBeatMultiplier;
			PrevDivOfWholeNote = CurrDivOfWholeNote;

			check(CurrBPM > 0.0f);
			check(CurrDivOfWholeNote > 0.0f);
			const float QuarterNoteTime = 60.0f / CurrBPM;
			float NewTimeSeconds = 4.0f * (float)CurrBeatMultiplier * QuarterNoteTime / CurrDivOfWholeNote;
			*TimeSeconds = FTime(NewTimeSeconds);
		}

	}

	void FBPMToSecondsOperator::Reset(const IOperator::FResetParams& InParams)
	{
		UpdateTime();
	}

	void FBPMToSecondsOperator::Execute()
	{
		UpdateTime();
	}

	const FVertexInterface& FBPMToSecondsOperator::GetVertexInterface()
	{
		using namespace BPMToSecondsVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputBPM), 90.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputBeatMultiplier), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDivOfWholeNote), 4.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTimeSeconds))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FBPMToSecondsOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, "BPMToSeconds", "" };
			Info.MajorVersion = 1;
			Info.MinorVersion = 1;
			Info.DisplayName = METASOUND_LOCTEXT("BPMToSecondsNode_DisplayName", "BPM To Seconds");
			Info.Description = METASOUND_LOCTEXT("BPMToSecondsNode_Desc", "Calculates a beat time in seconds from the given BPM, beat multiplier and divisions of a whole note.");
			Info.Author = PluginAuthor;
			Info.CategoryHierarchy = { NodeCategories::Conversions };
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.Keywords = { METASOUND_LOCTEXT("BPMNodeRhythmKeyword", "Rhythm"), METASOUND_LOCTEXT("BPMNodeTempoKeyword", "Tempo") };

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FBPMToSecondsOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace BPMToSecondsVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InBPM = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputBPM), InParams.OperatorSettings);
		FFloatReadRef InBeatMultiplier = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputBeatMultiplier), InParams.OperatorSettings);
		FFloatReadRef InDivOfWholeNote = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDivOfWholeNote), InParams.OperatorSettings);

		return MakeUnique<FBPMToSecondsOperator>(InParams, InBPM, InBeatMultiplier, InDivOfWholeNote);
	}

	FBPMToSecondsNode::FBPMToSecondsNode(const FNodeInitData& InitData)
	: FBPMToSecondsNode(FNodeData(InitData.InstanceName, InitData.InstanceID, FBPMToSecondsOperator::GetNodeInfo().DefaultInterface), MakeShared<const FNodeClassMetadata>(FBPMToSecondsOperator::GetNodeInfo()))
	{
	}

	FBPMToSecondsNode::FBPMToSecondsNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
	: FNodeFacade(MoveTemp(InNodeData), MoveTemp(InClassMetadata), TFacadeOperatorClass<FBPMToSecondsOperator>())
	{
	}

	FNodeClassMetadata FBPMToSecondsNode::CreateNodeClassMetadata()
	{
		return FBPMToSecondsOperator::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FBPMToSecondsNode)
}

#undef LOCTEXT_NAMESPACE
