// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LinkwitzRileyBandSplitter.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Bandsplitter"

namespace Metasound
{
	namespace BandSplitterNode
	{
		METASOUND_PARAM(InputFilterOrder, "Filter Order", "The steepness of the crossover filters.");
		METASOUND_PARAM(InputPhaseCompensate, "Phase Compensate", "Whether or not to phase compensate each band so they can be summed back together correctly.");
	}

	enum class EBandSplitterFilterOrder : int32
	{
		TwoPole,
		FourPole,
		SixPole,
		EightPole
	};

	DECLARE_METASOUND_ENUM(EBandSplitterFilterOrder, EBandSplitterFilterOrder::FourPole, METASOUNDSTANDARDNODES_API,
		FEnumBandSplitterFilterOrder, FEnumBandSplitterFilterOrderInfo, FEnumBandSplitterFilterOrderReadRef, FEnumBandSplitterFilterOrderWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(EBandSplitterFilterOrder, FEnumBandSplitterFilterOrder, "Filter Order")
		DEFINE_METASOUND_ENUM_ENTRY(EBandSplitterFilterOrder::TwoPole, "BandSplitterFilterOrderTwoPoleDescription", "Two Pole", "EnvelopePeakModeMSDescriptionTT", "Envelope follows a running Mean Squared of the audio signal."),
		DEFINE_METASOUND_ENUM_ENTRY(EBandSplitterFilterOrder::FourPole, "BandSplitterFilterOrderFourPoleDescription", "Four Pole", "EnvelopePeakModeRMSDescriptionTT", "Envelope follows a running Root Mean Squared of the audio signal."),
		DEFINE_METASOUND_ENUM_ENTRY(EBandSplitterFilterOrder::SixPole, "BandSplitterFilterOrderSixPoleDescription", "Six Pole", "EnvelopePeakModePeakDescriptionTT", "Envelope follows the peaks in the audio signal."),
		DEFINE_METASOUND_ENUM_ENTRY(EBandSplitterFilterOrder::EightPole, "BandSplitterFilterOrderEightPoleDescription", "Eight Pole", "EnvelopePeakModePeakDescriptionTT", "Envelope follows the peaks in the audio signal."),
	DEFINE_METASOUND_ENUM_END()

	template<uint32 NumBands, uint32 NumChannels>
	class METASOUNDSTANDARDNODES_API TBandSplitterOperator : public TExecutableOperator<TBandSplitterOperator<NumBands, NumChannels>>
	{
	public:
		TBandSplitterOperator(
			const FOperatorSettings& InSettings
			, const TArray<FAudioBufferReadRef>&& InInputBuffers
			, const TArray<FFloatReadRef>&& InCrossoverFrequencies
			, const FEnumBandSplitterFilterOrderReadRef& InFilterOrder
			, const FBoolReadRef& InPhaseCompensate)
			: AudioInputs(InInputBuffers)
			, CrossoverFrequencies(InCrossoverFrequencies)
			, FilterOrder(InFilterOrder)
			, bPhaseCompensate(InPhaseCompensate)
			, Settings(InSettings)
		{
			// Update internal crossover buffer from passed data references
			Crossovers.Reset();
			Crossovers.AddUninitialized(NumBands - 1);
			CopyClampCrossovers();

			bPrevPhaseCompensate = *bPhaseCompensate;
			PrevFilterOrder = GetFilterOrder(*InFilterOrder);

			for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				MultiBandBuffers.AddDefaulted();
				Filters.AddDefaulted();

				MultiBandBuffers[ChannelIndex].SetBands(NumBands);
				Filters[ChannelIndex].Init(1, InSettings.GetSampleRate(), PrevFilterOrder, bPrevPhaseCompensate, Crossovers);

				for (uint32 BandIndex = 0; BandIndex < NumBands; ++BandIndex)
				{
					AudioOutputs.Add(FAudioBufferWriteRef::CreateNew(InSettings));
				}
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			// used if NumChannels == 1
			auto CreateNodeClassMetadataMono = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Band Splitter (Mono, %d)"), NumBands);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("BandSplitterDisplayNamePattern1", "Mono Band Splitter ({0})", NumBands);
				const FText NodeDescription = METASOUND_LOCTEXT("BandSplitterDescription1", "Will split incoming audio into separate frequency bands.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels == 2
			auto CreateNodeClassMetadataStereo = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Band Splitter (Stereo, %d)"), NumBands);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("BandSplitterDisplayNamePattern2", "Stereo Band Splitter ({0})", NumBands);
				const FText NodeDescription = METASOUND_LOCTEXT("BandSplitterDescription2", "Will split incoming audio into separate frequency bands.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels > 2
			auto CreateNodeClassMetadataMultiChan = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Band Splitter (%d-Channel, %d)"), NumChannels, NumBands);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("BandSplitterDisplayNamePattern3", "{0}-channel Band Splitter ({1})", NumChannels, NumBands);
				const FText NodeDescription = METASOUND_LOCTEXT("BandSplitterDescription3", "Will split incoming audio into separate frequency bands.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = (NumChannels == 1) ? CreateNodeClassMetadataMono()
				: (NumChannels == 2) ? CreateNodeClassMetadataStereo() : CreateNodeClassMetadataMultiChan();

			return Metadata;
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace BandSplitterNode;

			for (uint32 Chan = 0; Chan < NumChannels; ++Chan)
			{
				InOutVertexData.BindReadVertex(AudioInputNames[Chan], AudioInputs[Chan]);
			}

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFilterOrder), FilterOrder);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPhaseCompensate), bPhaseCompensate);

			for (uint32 BandIndex = 0; BandIndex < NumBands - 1; ++BandIndex)
			{
				InOutVertexData.BindReadVertex(CrossoverInputNames[BandIndex], CrossoverFrequencies[BandIndex]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			for (uint32 BandIndex = 0; BandIndex < NumBands; ++BandIndex)
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					InOutVertexData.BindReadVertex(GetAudioOutputName(BandIndex, ChannelIndex), AudioOutputs[BandIndex * NumChannels + ChannelIndex]);
				}
			}
		}

		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace BandSplitterNode;

			auto CreateDefaultInterface = []()-> FVertexInterface
			{
				// inputs
				FInputVertexInterface InputInterface;

				// audio channels
				for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
#if WITH_EDITOR
					const FDataVertexMetadata AudioInputMetadata
					{
						GetAudioInputDescription(ChannelIndex),
						GetAudioInputDisplayName(ChannelIndex)
					};
#else 
					const FDataVertexMetadata AudioInputMetadata;
#endif // WITH_EDITOR

					InputInterface.Add(TInputDataVertex<FAudioBuffer>(AudioInputNames[ChannelIndex], AudioInputMetadata));
				}

				InputInterface.Add(TInputDataVertex<FEnumBandSplitterFilterOrder>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFilterOrder)));
				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPhaseCompensate), true));

				// Crossover frequencies
				for (uint32 InputIndex = 0; InputIndex < NumBands - 1; InputIndex++)
				{
#if WITH_EDITOR
					const FDataVertexMetadata CrossoverInputMetadata
					{
						GetCrossoverInputDescription(InputIndex),
						GetCrossoverInputDisplayName(InputIndex)
					};
#else
					const FDataVertexMetadata CrossoverInputMetadata;
#endif // WITH_EDITOR

					InputInterface.Add(TInputDataVertex<float>(CrossoverInputNames[InputIndex], CrossoverInputMetadata, (1 + InputIndex) * 500.0f));
				}

				// outputs
				FOutputVertexInterface OutputInterface;
				for (uint32 OutputIndex = 0; OutputIndex < NumBands; OutputIndex++)
				{
					for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
					{
#if WITH_EDITOR
						const FDataVertexMetadata AudioOutputMetadata
						{
							GetAudioOutputDescription(OutputIndex, ChannelIndex),
							GetAudioOutputDisplayName(OutputIndex, ChannelIndex)
						};
#else 
						const FDataVertexMetadata AudioOutputMetadata;
#endif // WITH_EDITOR 

						OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(GetAudioOutputName(OutputIndex, ChannelIndex), AudioOutputMetadata));
					}
				}

				return FVertexInterface(InputInterface, OutputInterface);
			}; // end lambda: CreateDefaultInterface()

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace BandSplitterNode;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TArray<FAudioBufferReadRef> InputBuffers;
			TArray<FFloatReadRef> InputCrossovers;

			FEnumBandSplitterFilterOrderReadRef FilterOrderIn = InputData.GetOrCreateDefaultDataReadReference<FEnumBandSplitterFilterOrder>(METASOUND_GET_PARAM_NAME(InputFilterOrder), InParams.OperatorSettings);
			FBoolReadRef bPhaseCompensateIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputPhaseCompensate), InParams.OperatorSettings);

			for (uint32 Chan = 0; Chan < NumChannels; Chan++)
			{
				InputBuffers.Add(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(AudioInputNames[Chan], InParams.OperatorSettings));
			}

			for (uint32 Band = 0; Band < NumBands - 1; Band++)
			{
				InputCrossovers.Add(InputData.GetOrCreateDefaultDataReadReference<float>(CrossoverInputNames[Band], InParams.OperatorSettings));
			}

			return MakeUnique<TBandSplitterOperator<NumBands, NumChannels>>(InParams.OperatorSettings, MoveTemp(InputBuffers), MoveTemp(InputCrossovers), FilterOrderIn, bPhaseCompensateIn);
		}

		void UpdateFilterSettings()
		{
			bool bNeedsReinitFilters = false;
			bool bNeedsUpdateCrossovers = CopyClampCrossovers();

			if (*bPhaseCompensate != bPrevPhaseCompensate)
			{
				bNeedsReinitFilters = true;
				bPrevPhaseCompensate = *bPhaseCompensate;
			}

			if (PrevFilterOrder != GetFilterOrder(*FilterOrder))
			{
				bNeedsReinitFilters = true;
				PrevFilterOrder = GetFilterOrder(*FilterOrder);
			}

			if (bNeedsReinitFilters)
			{
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
					Filters[ChannelIndex].Init(1, Settings.GetSampleRate(), PrevFilterOrder, bPrevPhaseCompensate, Crossovers);
				}
			}
			else if (bNeedsUpdateCrossovers)
			{
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
					Filters[ChannelIndex].SetCrossovers(Crossovers);
				}
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			CopyClampCrossovers();

			bPrevPhaseCompensate = *bPhaseCompensate;
			PrevFilterOrder = GetFilterOrder(*FilterOrder);

			for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				MultiBandBuffers[ChannelIndex].Reset();
				Filters[ChannelIndex].Init(1, InParams.OperatorSettings.GetSampleRate(), PrevFilterOrder, bPrevPhaseCompensate, Crossovers);
			}

			for (const TDataWriteReference<FAudioBuffer>& AudioOutput : AudioOutputs)
			{
				AudioOutput->Zero();
			}
		}

		void Execute()
		{
			UpdateFilterSettings();

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				const int32 NumFrames = AudioInputs[ChannelIndex]->Num();

				MultiBandBuffers[ChannelIndex].SetSamples(NumFrames);

				Filters[ChannelIndex].ProcessAudioBuffer(AudioInputs[ChannelIndex]->GetData(), MultiBandBuffers[ChannelIndex], NumFrames);

				for (int32 BandIndex = 0; BandIndex < NumBands; BandIndex++)
				{
					FMemory::Memcpy(AudioOutputs[BandIndex * NumChannels + ChannelIndex]->GetData(), MultiBandBuffers[ChannelIndex][BandIndex], NumFrames * sizeof(float));
				}
			}
		}


	private:
		bool CopyClampCrossovers()
		{
			check(Crossovers.Num() == CrossoverFrequencies.Num());

			bool bDidUpdate = false;
			
			const float MaxFrequency = 0.95f * Settings.GetSampleRate() / 2.f;
			const float MinFrequency = FMath::Min(20.f, MaxFrequency);

			const TDataReadReference<float>* InputFreqs = CrossoverFrequencies.GetData();
			float* CrossoverData = Crossovers.GetData();

			for (int32 i = 0; i < CrossoverFrequencies.Num(); ++i)
			{
				const float NewCrossover = FMath::Clamp(*InputFreqs[i], MinFrequency, MaxFrequency);
				if (NewCrossover != CrossoverData[i])
				{
					bDidUpdate = true;
					CrossoverData[i] = NewCrossover;
				}
			}

			return bDidUpdate;
		}

		
		static TArray<FVertexName> InitializeAudioInputNames()
		{
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumChannels);

			for (int ChanIndex = 0; ChanIndex < NumChannels; ++ChanIndex)
			{
				if (NumChannels == 1)
				{
					Names[ChanIndex] = *FString::Printf(TEXT("In"));
				}
				else if (NumChannels == 2)
				{
					Names[ChanIndex] = *FString::Printf(TEXT("In %s"), (ChanIndex == 0) ? TEXT("L") : TEXT("R"));
				}
				else
				{
					Names[ChanIndex] = *FString::Printf(TEXT("In %i"), ChanIndex);
				}
			}

			return Names;
		}

		static TArray<FVertexName> InitializeCrossoverInputNames()
		{
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumBands);

			for (int InputIndex = 0; InputIndex < NumBands; ++InputIndex)
			{
				Names[InputIndex] = *FString::Printf(TEXT("Crossover %i"), InputIndex);
			}

			return Names;
		}

		static TArray<FVertexName> InitializeAudioOutputNames()
		{
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumBands * NumChannels);

			for (int BandIndex = 0; BandIndex < NumBands; ++BandIndex)
			{
				for (int ChanIndex = 0; ChanIndex < NumChannels; ++ChanIndex)
				{
					if (NumChannels == 1)
					{
						Names[BandIndex * NumChannels + ChanIndex] = *FString::Printf(TEXT("Band %i Out"), BandIndex);
					}
					else if (NumChannels == 2)
					{
						Names[BandIndex * NumChannels + ChanIndex] = *FString::Printf(TEXT("Band %i %s"), BandIndex, (ChanIndex == 0) ? TEXT("L") : TEXT("R"));
					}
					else
					{
						Names[BandIndex * NumChannels + ChanIndex] = *FString::Printf(TEXT("Band %i Out %i"), BandIndex, ChanIndex);
					}
				}
			}

			return Names;
		}


		TArray<FAudioBufferReadRef> AudioInputs;
		TArray<FFloatReadRef> CrossoverFrequencies;

		TArray<float> Crossovers;

		FEnumBandSplitterFilterOrderReadRef FilterOrder;
		FBoolReadRef bPhaseCompensate;

		bool bPrevPhaseCompensate;
		Audio::EFilterOrder PrevFilterOrder;

		TArray<FAudioBufferWriteRef> AudioOutputs;

		TArray<Audio::FLinkwitzRileyBandSplitter> Filters;
		TArray<Audio::FMultibandBuffer> MultiBandBuffers;

		static const inline TArray<FVertexName> AudioInputNames = InitializeAudioInputNames();
		static const inline TArray<FVertexName> CrossoverInputNames = InitializeCrossoverInputNames();
		static const inline TArray<FVertexName> AudioOutputNames = InitializeAudioOutputNames();

		FOperatorSettings Settings;

		static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "BandSplitter", InOperatorName, FName() },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Filters },
				{ },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}

		static const FVertexName GetAudioOutputName(uint32 OutputIndex, uint32 ChannelIndex)
		{
			return AudioOutputNames[OutputIndex * NumChannels + ChannelIndex];
		}

#if WITH_EDITOR
		static const FText GetAudioInputDescription(uint32 ChannelIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("BandSplitterAudioInputDescription", "Audio Channel: {0}", ChannelIndex);
		}

		static const FText GetAudioInputDisplayName(uint32 ChannelIndex)
		{
			if (NumChannels == 1)
			{
				return METASOUND_LOCTEXT("BandsplitterAudioInputIn1", "In");
			}
			else if (NumChannels == 2)
			{
				if (ChannelIndex == 0)
				{
					return METASOUND_LOCTEXT("BandsplitterAudioInputIn2L", "In L");
				}
				else
				{
					return METASOUND_LOCTEXT("BandsplitterAudioInputIn2R", "In R");
				}
			}

			return METASOUND_LOCTEXT_FORMAT("BandsplitterAudioInputIn", "In {0}", ChannelIndex);
		}

		static const FText GetCrossoverInputDisplayName(uint32 InputIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("BandsplitterCrossoverInputDisplayName", "Crossover {0}", InputIndex);
		}

		static const FText GetCrossoverInputDescription(uint32 InputIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("BandSplitterCrossoverInputDescription", "Crossover Input #: {0}", InputIndex);
		}

		static const FText GetAudioOutputDescription(uint32 OutputIndex, uint32 ChannelIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("BandSplitterAudioOutputDescription", "Channel {0} Output for Band {1}", ChannelIndex, OutputIndex);
		}

		static const FText GetAudioOutputDisplayName(uint32 OutputIndex, uint32 ChannelIndex)
		{
			if (NumChannels == 1)
			{
				return METASOUND_LOCTEXT_FORMAT("BandsplitterAudioOutputOut1", "Band {0} Out", OutputIndex);
			}
			else if (NumChannels == 2)
			{
				if (ChannelIndex == 0)
				{
					return METASOUND_LOCTEXT_FORMAT("BandsplitterAudioOutputOut2L", "Band {0} L", OutputIndex);
				}
				else
				{
					return METASOUND_LOCTEXT_FORMAT("BandsplitterAudioOutputOut2R", "Band {0} R", OutputIndex);
				}
			}

			return METASOUND_LOCTEXT_FORMAT("BandsplitterAudioOutputOut", "Band {0} Out {1}", OutputIndex, ChannelIndex);
		}
#endif // WITH_EDITOR

		FORCEINLINE Audio::EFilterOrder GetFilterOrder(EBandSplitterFilterOrder InFilterOrder) const
		{
			using namespace Audio;

			switch (InFilterOrder)
			{
			case EBandSplitterFilterOrder::TwoPole:
				return EFilterOrder::TwoPole;
				break;
			case EBandSplitterFilterOrder::FourPole:
				return EFilterOrder::FourPole;
				break;
			case EBandSplitterFilterOrder::SixPole:
				return EFilterOrder::SixPole;
				break;
			case EBandSplitterFilterOrder::EightPole:
				return EFilterOrder::EightPole;
				break;
			default:
				ensureMsgf(false, TEXT("Unhandled case converting EBandSplitterFilterOrder!"));
				break;
			}

			return EFilterOrder::TwoPole;
		}
	};

	// Node Class
	template<uint32 NumBands, uint32 NumChannels>
	using TBandSplitterNode = TNodeFacade<TBandSplitterOperator<NumBands, NumChannels>>;

#define REGISTER_BANDSPLITTER_NODE(A, B) \
	using FBandSplitterNode_##A ## _ ##B = TBandSplitterNode<A, B>; \
	METASOUND_REGISTER_NODE(FBandSplitterNode_##A ## _ ##B) \

	// monos
	REGISTER_BANDSPLITTER_NODE(2, 1)
	REGISTER_BANDSPLITTER_NODE(3, 1)
	REGISTER_BANDSPLITTER_NODE(4, 1)
	REGISTER_BANDSPLITTER_NODE(5, 1)

	// stereo
	REGISTER_BANDSPLITTER_NODE(2, 2)
	REGISTER_BANDSPLITTER_NODE(3, 2)
	REGISTER_BANDSPLITTER_NODE(4, 2)
	REGISTER_BANDSPLITTER_NODE(5, 2)

} // namespace Metasound

#undef LOCTEXT_NAMESPACE
