// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundFacade.h"
#include "MetasoundPrimitives.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#include "DSP/Filter.h"
#include "DSP/InterpolatedOnePole.h"

#define LOCTEXT_NAMESPACE "MetasoundBasicFilterNodes"


namespace Metasound
{
	// forward declarations:
	class FLadderFilterOperator;
	class FStateVariableFilterOperator;
	class FOnePoleLowPassFilterOperator;
	class FOnePoleHighPassFilterOperator;
	class FBiquadFilterOperator;

#pragma region Parameter Names
	namespace BasicFilterParameterNames
	{
		// inputs
		METASOUND_PARAM(ParamAudioInput, "In", "Audio to be processed by the filter.");
		METASOUND_PARAM(ParamCutoffFrequency, "Cutoff Frequency", "Controls cutoff frequency.");
		METASOUND_PARAM(ParamResonance, "Resonance", "Controls filter resonance.");
		METASOUND_PARAM(ParamBandwidth, "Bandwidth", "Controls bandwidth when applicable to the current filter type.");
		METASOUND_PARAM(ParamGainDb, "Gain", "Gain applied to the band when in Parametric mode (in decibels).");
		METASOUND_PARAM(ParamFilterType, "Type", "Filter type.");
		METASOUND_PARAM(ParamBandStopControl, "Band Stop Control", "Band stop Control (applied to band stop output).");

		// outputs
		METASOUND_PARAM(ParamAudioOutput, "Out", "Audio processed by the filter.");
		METASOUND_PARAM(ParamHighPassOutput, "High Pass Filter", "High pass filter output.");
		METASOUND_PARAM(ParamLowPassOutput, "Low Pass Filter", "Low pass filter output.");
		METASOUND_PARAM(ParamBandPassOutput, "Band Pass", "Band pass filter output.");
		METASOUND_PARAM(ParamBandStopOutput, "Band Stop", "Band stop filter output.");

	} // namespace BasicFilterParameterNames;

	using namespace BasicFilterParameterNames;
#pragma endregion

	static const TArray<FText> LowPassKeywords = { METASOUND_LOCTEXT("LowPassKeyword", "Lowpass"), METASOUND_LOCTEXT("LowPassSpaceKeyword", "Low Pass"), METASOUND_LOCTEXT("LowPassDashKeyword", "Low-pass"), METASOUND_LOCTEXT("LPFKeyword", "lpf")};
	static const TArray<FText> HighPassKeywords = { METASOUND_LOCTEXT("HighPassKeyword", "Highpass"), METASOUND_LOCTEXT("HighPassSpaceKeyword", "High Pass"), METASOUND_LOCTEXT("HighPassDashKeyword", "High-pass"), METASOUND_LOCTEXT("HPFKeyword", "hpf") };
	static const TArray<FText> BandPassKeywords = { METASOUND_LOCTEXT("BandPassKeyword", "Bandpass"), METASOUND_LOCTEXT("BandPassSpaceKeyword", "Band Pass"), METASOUND_LOCTEXT("BandPassDashKeyword", "Band-pass"), METASOUND_LOCTEXT("BPFKeyword", "bpf") };
	static const TArray<FText> NotchKeywords = { METASOUND_LOCTEXT("NotchKeyword", "Notch"), METASOUND_LOCTEXT("NFKeyword", "nf"), METASOUND_LOCTEXT("BandStopKeyword", "BandStop"), METASOUND_LOCTEXT("BandStopSpaceKeyword", "Band Stop"), METASOUND_LOCTEXT("BandStopDashKeyword", "Band-stop"), METASOUND_LOCTEXT("BSFKeyword", "bsf") };
	static const TArray<FText> ParametricEQKeywords = { METASOUND_LOCTEXT("EQKeyword", "EQ"), METASOUND_LOCTEXT("ParametricKeyword", "Parametric") };
	static const TArray<FText> LowShelfKeywords = { METASOUND_LOCTEXT("LowShelfSpaceKeyword", "Low Shelf"), METASOUND_LOCTEXT("LowShelfDashKeyword", "Low-shelf"), METASOUND_LOCTEXT("LSFKeyword", "lsf") };
	static const TArray<FText> HighShelfKeywords = { METASOUND_LOCTEXT("HighShelfSpaceKeyword", "High Shelf"), METASOUND_LOCTEXT("HighShelfDashKeyword", "High-shelf"), METASOUND_LOCTEXT("HSFKeyword", "hsf") };
	static const TArray<FText> AllPassKeywords = { METASOUND_LOCTEXT("AllPassKeyword", "Allpass"), METASOUND_LOCTEXT("AllPassSpaceKeyword", "All Pass"), METASOUND_LOCTEXT("AllPassDashKeyword", "All-pass"), METASOUND_LOCTEXT("APFKeyword", "apf") };
	static const TArray<FText> ButterworthKeywords = { METASOUND_LOCTEXT("ButterworthKeyword", "Butterworth") };

#pragma region Biquad Filter Enum
	DECLARE_METASOUND_ENUM(Audio::EBiquadFilter::Type, Audio::EBiquadFilter::Lowpass,
	METASOUNDSTANDARDNODES_API, FEnumEBiquadFilterType, FEnumBiQuadFilterTypeInfo, FEnumBiQuadFilterReadRef, FEnumBiQuadFilterWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(Audio::EBiquadFilter::Type, FEnumEBiquadFilterType, "BiquadFilterType")
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::Lowpass, "LpDescription", "Low Pass", "LpDescriptionTT", "Low pass Biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::Highpass, "HpDescription", "High Pass", "HpDescriptionTT", "High pass Biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::Bandpass, "BpDescription", "Band Pass", "BpDescriptionTT", "Band pass Biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::Notch, "NotchDescription", "Notch ", "NotchDescriptionTT", "Notch biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::ParametricEQ, "ParaEqDescription", "Parametric EQ", "ParaEqDescriptionTT", "Parametric EQ biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::LowShelf, "LowShelfDescription", "Low Shelf", "LowShelfDescriptionTT", "Low shelf biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::HighShelf, "HighShelfDescription", "High Shelf", "HighShelfDescriptionTT", "High shelf biquad filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::AllPass, "AllPassDescription", "All Pass", "AllPassDescriptionTT", "All pass biquad Filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::ButterworthLowPass, "LowPassButterDescription", "Butterworth Low Pass", "LowPassButterDescriptionTT", "Butterworth Low Pass Biquad Filter."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EBiquadFilter::ButterworthHighPass, "HighPassButterDescription", "Butterworth High Pass", "HighPassButterDescriptionTT", "Butterworth High Pass Biquad Filter.")
		DEFINE_METASOUND_ENUM_END()
#pragma endregion


#pragma region Ladder Filter

	class FLadderFilterOperator : public TExecutableOperator<FLadderFilterOperator>
	{
	private:
		static constexpr float InvalidValue = -1.f;

	public:
		static const FNodeClassMetadata& GetNodeInfo();

		static FVertexInterface DeclareVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), Frequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamResonance), Resonance);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioOutput), AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams);
		void Execute();


	private: // members
		// input pins
		FAudioBufferReadRef AudioInput;
		FFloatReadRef Frequency;
		FFloatReadRef Resonance;

		// cached data
		float PreviousFrequency{ InvalidValue };
		float PreviousResonance{ InvalidValue };

		// output pins
		FAudioBufferWriteRef AudioOutput;

		// data
		const int32 BlockSize;
		const float SampleRate;
		const float MaxCutoffFrequency;

		// dsp
		Audio::FLadderFilter LadderFilter;


	public:
		// ctor
		FLadderFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InFrequency,
			const FFloatReadRef& InResonance
		)
			: AudioInput(InAudioInput)
			, Frequency(InFrequency)
			, Resonance(InResonance)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
			, MaxCutoffFrequency(0.5f * SampleRate)
		{
			// verify our buffer sizes:
			check(AudioOutput->Num() == BlockSize);

			LadderFilter.Init(SampleRate, 1);
		}
	}; // class FLadderFilterOperator

	const FNodeClassMetadata& FLadderFilterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Ladder Filter"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_LadderFilterNodeDisplayName", "Ladder Filter");
			Info.Description = METASOUND_LOCTEXT("Ladder_Filter_NodeDescription", "Ladder filter");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Filters);
			Info.Keywords = LowPassKeywords;

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FLadderFilterOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamCutoffFrequency), 20000.f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamResonance), 1.f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FLadderFilterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(ParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), InParams.OperatorSettings);
		FFloatReadRef ResonanceIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamResonance), InParams.OperatorSettings);

		return MakeUnique<FLadderFilterOperator>(
			InParams.OperatorSettings
			, AudioIn
			, FrequencyIn
			, ResonanceIn
			);
	}

	void FLadderFilterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PreviousFrequency = InvalidValue;
		PreviousResonance = InvalidValue;
		AudioOutput->Zero();
		LadderFilter.Init(SampleRate, 1 /* NumChannels */);
	}

	void FLadderFilterOperator::Execute()
	{
		const float CurrentFrequency = FMath::Clamp(*Frequency, 0.f, MaxCutoffFrequency);
		const float CurrentResonance = FMath::Clamp(*Resonance, 1.0f, 10.0f);

		const bool bNeedsUpdate =
			(!FMath::IsNearlyEqual(PreviousFrequency, CurrentFrequency))
			|| (!FMath::IsNearlyEqual(PreviousResonance, CurrentResonance));

		if (bNeedsUpdate)
		{
			LadderFilter.SetQ(CurrentResonance);
			LadderFilter.SetFrequency(CurrentFrequency);

			LadderFilter.Update();

			PreviousFrequency = CurrentFrequency;
			PreviousResonance = CurrentResonance;
		}

		LadderFilter.ProcessAudio(AudioInput->GetData(), AudioInput->Num(), AudioOutput->GetData());
	}

#pragma endregion


#pragma region State Variable Filter

	class FStateVariableFilterOperator : public TExecutableOperator<FStateVariableFilterOperator>
	{
	private:
		static constexpr float InvalidValue = -1.f;
	public:
		static const FNodeClassMetadata& GetNodeInfo();

		static FVertexInterface DeclareVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), Frequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamResonance), Resonance);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamBandStopControl), BandStopControl);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamLowPassOutput), LowPassOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamHighPassOutput), HighPassOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamBandPassOutput), BandPassOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamBandStopOutput), BandStopOutput);
		}

		void Reset(const IOperator::FResetParams& InParams);
		void Execute();


	private: // members
		// input pins
		FAudioBufferReadRef AudioInput;
		FFloatReadRef Frequency;
		FFloatReadRef Resonance;
		FFloatReadRef BandStopControl;

		// cached data
		float PreviousFrequency{ InvalidValue };
		float PreviousResonance{ InvalidValue };
		float PreviousBandStopControl{ InvalidValue };

		// output pins
		FAudioBufferWriteRef LowPassOutput;
		FAudioBufferWriteRef HighPassOutput;
		FAudioBufferWriteRef BandPassOutput;
		FAudioBufferWriteRef BandStopOutput;

		// data
		const int32 BlockSize;
		float SampleRate;
		float MaxCutoffFrequency;

		// dsp
		Audio::FStateVariableFilter StateVariableFilter;


	public:
		// ctor
		FStateVariableFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InFrequency,
			const FFloatReadRef& InResonance,
			const FFloatReadRef& InBandStopControl
		)
			: AudioInput(InAudioInput)
			, Frequency(InFrequency)
			, Resonance(InResonance)
			, BandStopControl(InBandStopControl)
			, LowPassOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, HighPassOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BandPassOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BandStopOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
			, MaxCutoffFrequency(0.5f * SampleRate)
		{
			// verify our buffer sizes:
			check(LowPassOutput->Num() == BlockSize);
			check(HighPassOutput->Num() == BlockSize);
			check(BandPassOutput->Num() == BlockSize);
			check(BandPassOutput->Num() == BlockSize);

			StateVariableFilter.Init(SampleRate, 1);
		}
	};

	const FNodeClassMetadata& FStateVariableFilterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("State Variable Filter"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_StateVariableFilterNodeDisplayName", "State Variable Filter");
			Info.Description = METASOUND_LOCTEXT("State_Variable_Filter_NodeDescription", "State Variable filter");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Filters);
			Info.Keywords.Append(LowPassKeywords);
			Info.Keywords.Append(HighPassKeywords); 
			Info.Keywords.Append(BandPassKeywords);
			Info.Keywords.Append(NotchKeywords);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FStateVariableFilterOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamCutoffFrequency), 20000.f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamResonance), 0.f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamBandStopControl), 0.f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamLowPassOutput)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamHighPassOutput)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamBandPassOutput)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamBandStopOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FStateVariableFilterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(ParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), InParams.OperatorSettings);
		FFloatReadRef ResonanceIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamResonance), InParams.OperatorSettings);
		FFloatReadRef PassBandGainCompensationIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamBandStopControl), InParams.OperatorSettings);

		return MakeUnique<FStateVariableFilterOperator>(
			InParams.OperatorSettings
			, AudioIn
			, FrequencyIn
			, ResonanceIn
			, PassBandGainCompensationIn
			);
	}

	void FStateVariableFilterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PreviousFrequency = InvalidValue;
		PreviousResonance = InvalidValue;
		PreviousBandStopControl = InvalidValue;

		// output pins
		LowPassOutput->Zero();
		HighPassOutput->Zero();
		BandPassOutput->Zero();
		BandStopOutput->Zero();

		StateVariableFilter.Init(SampleRate, 1 /* NumChannels */);
	}

	void FStateVariableFilterOperator::Execute()
	{
		const float CurrentFrequency = FMath::Clamp(*Frequency, 0.f, MaxCutoffFrequency);
		const float CurrentResonance = FMath::Clamp(*Resonance, 0.f, 10.f);
		const float CurrentBandStopControl = FMath::Clamp(*BandStopControl, 0.f, 1.f);

		bool bNeedsUpdate =
			(!FMath::IsNearlyEqual(PreviousFrequency, CurrentFrequency))
			|| (!FMath::IsNearlyEqual(PreviousResonance, CurrentResonance))
			|| (!FMath::IsNearlyEqual(PreviousBandStopControl, CurrentBandStopControl));

		if (bNeedsUpdate)
		{
			StateVariableFilter.SetQ(CurrentResonance);
			StateVariableFilter.SetFrequency(CurrentFrequency);
			StateVariableFilter.SetBandStopControl(CurrentBandStopControl);

			StateVariableFilter.Update();

			PreviousFrequency = CurrentFrequency;
			PreviousResonance = CurrentResonance;
			PreviousBandStopControl = CurrentBandStopControl;
		}

		StateVariableFilter.ProcessAudio(
			AudioInput->GetData()
			, AudioInput->Num()
			, LowPassOutput->GetData()
			, HighPassOutput->GetData()
			, BandPassOutput->GetData()
			, BandStopOutput->GetData()
		);
	}

#pragma endregion


#pragma region OnePoleLowPass Filter

	class FOnePoleLowPassFilterOperator : public TExecutableOperator<FOnePoleLowPassFilterOperator>
	{
	private:
		static constexpr float InvalidValue = -1.f;
	public:
		static const FNodeClassMetadata& GetNodeInfo();

		static FVertexInterface DeclareVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), Frequency);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioOutput), AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams);
		void Execute();


	private: // members
		// input pins
		FAudioBufferReadRef AudioInput;
		FFloatReadRef Frequency;

		// output pins
		FAudioBufferWriteRef AudioOutput;

		// data
		const int32 BlockSize;
		float SampleRate;

		// dsp
		Audio::FInterpolatedLPF OnePoleLowPassFilter;


	public:
		// ctor
		FOnePoleLowPassFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InFrequency
		)
			: AudioInput(InAudioInput)
			, Frequency(InFrequency)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
		{
			// verify our buffer sizes:
			check(AudioOutput->Num() == BlockSize);

			OnePoleLowPassFilter.Init(SampleRate, 1); // mono
		}
	};

	const FNodeClassMetadata& FOnePoleLowPassFilterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("One-Pole Low Pass Filter"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_OnePoleLpfNodeDisplayName", "One-Pole Low Pass Filter");
			Info.Description = METASOUND_LOCTEXT("One_Pole_Low_Pass_Filter_NodeDescription", "One-Pole Low Pass Filter");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Filters);
			Info.Keywords = LowPassKeywords;

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FOnePoleLowPassFilterOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamCutoffFrequency), 20000.f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FOnePoleLowPassFilterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(ParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), InParams.OperatorSettings);

		return MakeUnique<FOnePoleLowPassFilterOperator>(
			InParams.OperatorSettings
			, AudioIn
			, FrequencyIn
			);
	}

	void FOnePoleLowPassFilterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioOutput->Zero();
		OnePoleLowPassFilter.Init(SampleRate, 1 /* NumChannels */);
	}

	void FOnePoleLowPassFilterOperator::Execute()
	{
		float ClampedFreq = FMath::Clamp(*Frequency, 0.0f, SampleRate);
		OnePoleLowPassFilter.StartFrequencyInterpolation(ClampedFreq, AudioInput->Num());
		OnePoleLowPassFilter.ProcessAudioBuffer(AudioInput->GetData(), AudioOutput->GetData(), AudioInput->Num());
		OnePoleLowPassFilter.StopFrequencyInterpolation();
	}

#pragma endregion


#pragma region OnePoleHighPass Filter

	class FOnePoleHighPassFilterOperator : public TExecutableOperator<FOnePoleHighPassFilterOperator>
	{
	private:
		static constexpr float InvalidValue = -1.f;
	public:
		static const FNodeClassMetadata& GetNodeInfo();

		static FVertexInterface DeclareVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), Frequency);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioOutput), AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams);
		void Execute();


	private: // members
		// input pins
		FAudioBufferReadRef AudioInput;
		FFloatReadRef Frequency;

		// output pins
		FAudioBufferWriteRef AudioOutput;

		// data
		const int32 BlockSize;
		float SampleRate;

		// dsp
		Audio::FInterpolatedHPF OnePoleHighPassFilter;


	public:
		// ctor
		FOnePoleHighPassFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InFrequency
		)
			: AudioInput(InAudioInput)
			, Frequency(InFrequency)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
		{
			// verify our buffer sizes:
			check(AudioOutput->Num() == BlockSize);

			OnePoleHighPassFilter.Init(SampleRate, 1); // mono
		}
	};

	const FNodeClassMetadata& FOnePoleHighPassFilterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("One-Pole High Pass Filter"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_OnePoleHpfNodeDisplayName", "One-Pole High Pass Filter");
			Info.Description = METASOUND_LOCTEXT("One_Pole_High_Pass_Filter_NodeDescription", "One-Pole High Pass Filter");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Filters);
			Info.Keywords = HighPassKeywords;

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FOnePoleHighPassFilterOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamCutoffFrequency), 10.f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FOnePoleHighPassFilterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(ParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), InParams.OperatorSettings);

		return MakeUnique<FOnePoleHighPassFilterOperator>(
			InParams.OperatorSettings
			, AudioIn
			, FrequencyIn
			);
	}

	void FOnePoleHighPassFilterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioOutput->Zero();
		OnePoleHighPassFilter.Init(SampleRate, 1 /* NumChannels */);
	}

	void FOnePoleHighPassFilterOperator::Execute()
	{
		float ClampedFreq = FMath::Clamp(*Frequency, 0.0f, SampleRate);
		OnePoleHighPassFilter.StartFrequencyInterpolation(ClampedFreq, BlockSize);
		OnePoleHighPassFilter.ProcessAudioBuffer(AudioInput->GetData(), AudioOutput->GetData(), AudioInput->Num());
		OnePoleHighPassFilter.StopFrequencyInterpolation();
	}

#pragma endregion


#pragma region Biquad Filter
	class FBiquadFilterOperator : public TExecutableOperator<FBiquadFilterOperator>
	{
	private:
		static constexpr float InvalidValue = -1.f;
		
	public:
		static const FNodeClassMetadata& GetNodeInfo();

		static FVertexInterface DeclareVertexInterface();

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), Frequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamBandwidth), Bandwidth);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamGainDb), FilterGainDb);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamFilterType), FilterType);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamAudioOutput), AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams);
		void Execute();


	private: 
		void ResetInternal();

		// input pins
		FAudioBufferReadRef AudioInput;
		FFloatReadRef Frequency;
		FFloatReadRef Bandwidth;
		FFloatReadRef FilterGainDb;
		FEnumBiQuadFilterReadRef FilterType;

		// cached data
		float PreviousFrequency{ InvalidValue };
		float PreviousBandwidth{ InvalidValue };
		float PreviousFilterGainDb{ InvalidValue };

		// output pins
		FAudioBufferWriteRef AudioOutput;

		// data
		const int32 BlockSize;
		float SampleRate;
		float MaxCutoffFrequency;

		// dsp
		Audio::EBiquadFilter::Type PreviousFilterType;
		Audio::FBiquadFilter BiquadFilter;


	public:
		// ctor
		FBiquadFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InFrequency,
			const FFloatReadRef& InBandwidth,
			const FFloatReadRef& InFilterGainDb,
			const FEnumBiQuadFilterReadRef& InFilterType
		)
			: AudioInput(InAudioInput)
			, Frequency(InFrequency)
			, Bandwidth(InBandwidth)
			, FilterGainDb(InFilterGainDb)
			, FilterType(InFilterType)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, BlockSize(InSettings.GetNumFramesPerBlock())
			, SampleRate(InSettings.GetSampleRate())
			, MaxCutoffFrequency(0.5f * SampleRate)
			, PreviousFilterType(*FilterType)
		{
			ResetInternal();
		}
	};

	const FNodeClassMetadata& FBiquadFilterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Biquad Filter"), StandardNodes::AudioVariant };
			Info.MajorVersion = 2;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_BiquadFilterNodeDisplayName", "Biquad Filter");
			Info.Description = METASOUND_LOCTEXT("Biquad_Filter_NodeDescription", "Biquad filter");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Filters);
			Info.Keywords.Append(LowPassKeywords);
			Info.Keywords.Append(HighPassKeywords);
			Info.Keywords.Append(BandPassKeywords);
			Info.Keywords.Append(NotchKeywords);
			Info.Keywords.Append(ParametricEQKeywords);
			Info.Keywords.Append(AllPassKeywords);
			Info.Keywords.Append(LowShelfKeywords);
			Info.Keywords.Append(HighShelfKeywords);
			Info.Keywords.Append(ButterworthKeywords);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FVertexInterface FBiquadFilterOperator::DeclareVertexInterface()
	{
		static const FDataVertexMetadata GainPinMetaData
		{
			  METASOUND_GET_PARAM_TT(ParamGainDb) // description
			, METASOUND_LOCTEXT("Biquad_Filter_DisplayName", "Gain (dB)") // display name
		};

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamCutoffFrequency), 20000.f),
				// This bandwidth is equivalent to a Q factor of 1/sqrt(2), 
				// and results in a gradual attenuation of filtered frequencies without added resonance around the cutoff frequency 
				// See https://ccrma.stanford.edu/software/stk/classstk_1_1BiQuad.html
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamBandwidth), 1.89997f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(ParamGainDb), GainPinMetaData, 0.f),
				TInputDataVertex<FEnumEBiquadFilterType>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamFilterType))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamAudioOutput))
			)
		);

		return Interface;
	}

	TUniquePtr<IOperator> FBiquadFilterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(ParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamCutoffFrequency), InParams.OperatorSettings);
		FFloatReadRef BandwidthIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamBandwidth), InParams.OperatorSettings);
		FFloatReadRef FilterGainDbIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamGainDb), InParams.OperatorSettings);
		FEnumBiQuadFilterReadRef FilterType = InputData.GetOrCreateDefaultDataReadReference<FEnumEBiquadFilterType>(METASOUND_GET_PARAM_NAME(ParamFilterType), InParams.OperatorSettings);

		return MakeUnique<FBiquadFilterOperator>(
			InParams.OperatorSettings
			, AudioIn
			, FrequencyIn
			, BandwidthIn
			, FilterGainDbIn
			, FilterType
			);
	}

	void FBiquadFilterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		ResetInternal();
	}

	void FBiquadFilterOperator::ResetInternal()
	{
		PreviousFrequency = InvalidValue;
		PreviousBandwidth = InvalidValue;
		PreviousFilterGainDb = InvalidValue;
		PreviousFilterType = *FilterType;

		AudioOutput->Zero();
		BiquadFilter.Init(SampleRate, 1, *FilterType);
	}

	void FBiquadFilterOperator::Execute()
	{
		const float CurrentFrequency = FMath::Clamp(*Frequency, 0.f, MaxCutoffFrequency);
		const float CurrentBandwidth = FMath::Max(*Bandwidth, 0.f);
		const float CurrentFilterGainDb = FMath::Clamp(*FilterGainDb, -90.0f, 20.0f);

		if (!FMath::IsNearlyEqual(PreviousFrequency, CurrentFrequency))
		{
			BiquadFilter.SetFrequency(CurrentFrequency);
			PreviousFrequency = CurrentFrequency;
		}

		if (!FMath::IsNearlyEqual(PreviousBandwidth, CurrentBandwidth))
		{
			BiquadFilter.SetBandwidth(CurrentBandwidth);
			PreviousBandwidth = CurrentBandwidth;
		}

		if (!FMath::IsNearlyEqual(PreviousFilterGainDb, CurrentFilterGainDb))
		{
			BiquadFilter.SetGainDB(CurrentFilterGainDb);
			PreviousFilterGainDb = CurrentFilterGainDb;
		}

		if (*FilterType != PreviousFilterType)
		{
			BiquadFilter.SetType(*FilterType);
			PreviousFilterType = *FilterType;
		}

		BiquadFilter.ProcessAudio(AudioInput->GetData(), AudioInput->Num(), AudioOutput->GetData());
	}


#pragma endregion


#pragma region Node Declarations
	using FLadderFilterNode = TNodeFacade<FLadderFilterOperator>;
	using FStateVariableFilterNode = TNodeFacade<FStateVariableFilterOperator>;
	using FOnePoleLowPassFilterNode = TNodeFacade<FOnePoleLowPassFilterOperator>;
	using FOnePoleHighPassFilterNode = TNodeFacade<FOnePoleHighPassFilterOperator>;
	using FBiquadFilterNode = TNodeFacade<FBiquadFilterOperator>;
#pragma endregion


#pragma region Node Registration
	METASOUND_REGISTER_NODE(FLadderFilterNode);
	METASOUND_REGISTER_NODE(FStateVariableFilterNode);
	METASOUND_REGISTER_NODE(FOnePoleLowPassFilterNode);
	METASOUND_REGISTER_NODE(FOnePoleHighPassFilterNode);
	METASOUND_REGISTER_NODE(FBiquadFilterNode);
#pragma endregion

} // namespace Metasound



#undef LOCTEXT_NAMESPACE //MetasoundBasicFilterNodes
