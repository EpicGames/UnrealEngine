// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Dsp.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/BitCrusher.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_BitcrusherNode"

namespace Metasound
{
	/* Mid-Side Encoder */
	namespace BitcrusherVertexNames
	{
		METASOUND_PARAM(InputAudio, "Audio", "Incoming audio signal to bitcrush.");
		METASOUND_PARAM(InputSampleRate, "Sample Rate", "The sampling frequency to downsample the audio to.");
		METASOUND_PARAM(InputBitDepth, "Bit Depth", "The bit resolution to reduce the audio to.");

		METASOUND_PARAM(OutputAudio, "Audio", "The bitcrushed audio signal.");
	}

	// Operator Class
	class FBitcrusherOperator : public TExecutableOperator<FBitcrusherOperator>
	{
	public:

		FBitcrusherOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InAudio,
			const FFloatReadRef& InCrushedSampleRate,
			const FFloatReadRef& InCrushedBitDepth)
			: AudioInput(InAudio)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, CrushedSampleRate(InCrushedSampleRate)
			, CrushedBitDepth(InCrushedBitDepth)
			, MaxSampleRate(InParams.OperatorSettings.GetSampleRate())
		{
			Reset(InParams);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "Bitcrusher", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("BitcrusherDisplayName", "Bitcrusher"),
					METASOUND_LOCTEXT("BitcrusherDesc", "Downsamples and lowers the bit-depth of an incoming audio signal."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Filters },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace BitcrusherVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSampleRate), 8000.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputBitDepth), 8.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace BitcrusherVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSampleRate), CrushedSampleRate);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputBitDepth), CrushedBitDepth);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace BitcrusherVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace BitcrusherVertexNames;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FFloatReadRef SampleRateIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputSampleRate), InParams.OperatorSettings);
			FFloatReadRef BitDepthIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputBitDepth), InParams.OperatorSettings);

			return MakeUnique<FBitcrusherOperator>(InParams, AudioIn, SampleRateIn, BitDepthIn);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOutput->Zero();

			// Passing in 1 for NumChannels because the node takes one audio input
			Bitcrusher.Init(InParams.OperatorSettings.GetSampleRate(), 1);

			const float CurSampleRate = FMath::Clamp(*CrushedSampleRate, 1.0f, MaxSampleRate);
			const float CurBitDepth = FMath::Clamp(*CrushedBitDepth, 1.0f, Bitcrusher.GetMaxBitDepth());

			Bitcrusher.SetSampleRateCrush(CurSampleRate);
			Bitcrusher.SetBitDepthCrush(CurBitDepth);

			PrevSampleRate = CurSampleRate;
			PrevBitDepth = CurBitDepth;
		}

		void Execute()
		{
			const int32 NumFrames = AudioInput->Num();
			if (NumFrames != AudioOutput->Num())
			{
				return;
			}

			const float CurrSampleRate = FMath::Clamp(*CrushedSampleRate, 1.0f, MaxSampleRate);
			if (!FMath::IsNearlyEqual(PrevSampleRate, CurrSampleRate))
			{
				Bitcrusher.SetSampleRateCrush(CurrSampleRate);
				PrevSampleRate = CurrSampleRate;
			}

			const float CurrBitDepth = FMath::Clamp(*CrushedBitDepth, 1.0f, Bitcrusher.GetMaxBitDepth());
			if (!FMath::IsNearlyEqual(PrevBitDepth, CurrBitDepth))
			{
				Bitcrusher.SetBitDepthCrush(CurrBitDepth);
				PrevBitDepth = CurrBitDepth;
			}

			Bitcrusher.ProcessAudio(AudioInput->GetData(), NumFrames, AudioOutput->GetData());
		}


	private:

		// Audio input and output
		FAudioBufferReadRef AudioInput;
		FAudioBufferWriteRef AudioOutput;
		
		// User-set sample rate the audio will be converted to
		FFloatReadRef CrushedSampleRate;
		// User-set bit depth for each sample
		FFloatReadRef CrushedBitDepth;
		
		// Internal DSP bitcrusher
		Audio::FBitCrusher Bitcrusher;

		// Cached input parameters
		float PrevSampleRate;
		float PrevBitDepth;
		const float MaxSampleRate;
	
	};

	// Node Class
	using FBitcrusherNode = TNodeFacade<FBitcrusherOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FBitcrusherNode)

}

#undef LOCTEXT_NAMESPACE
