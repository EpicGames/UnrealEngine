// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_ClampNode"

namespace Metasound
{
	namespace ClampVertexNames
	{
		METASOUND_PARAM(InputValue, "In", "Input value to clamp.");
		METASOUND_PARAM(InputMinValue, "Min", "The min value to clamp.");
		METASOUND_PARAM(InputMaxValue, "Max", "The max value to clamp.");
		METASOUND_PARAM(OutputValue, "Value", "The clamped value.");
	}

	namespace MetasoundClampNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "Clamp", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ METASOUND_LOCTEXT("ClampCategory", "Math") },
				{ METASOUND_LOCTEXT("ClampKeyword", "Clamp") },
				FNodeDisplayStyle()
			};

			return Metadata;
		}

		template<typename ValueType>
		struct TClamp
		{
			bool bIsSupported = false;
		};

		template<>
		struct TClamp<int32>
		{
			static void GetClamped(int32 InValue, int32 InMin, int32 InMax, int32& OutClamped)
			{
				OutClamped = FMath::Clamp(InValue, InMin, InMax);
			}

			static TDataReadReference<int32> CreateInValue(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static TDataReadReference<int32> CreateInMin(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputMinValue), InParams.OperatorSettings);
			}

			static TDataReadReference<int32> CreateInMax(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputMaxValue), InParams.OperatorSettings);
			}
		};


		template<>
		struct TClamp<float>
		{
			static void GetClamped(float InValue, float InMin, float InMax, float& OutClamped)
			{
				OutClamped = FMath::Clamp(InValue, InMin, InMax);
			}

			static TDataReadReference<float> CreateInValue(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static TDataReadReference<float> CreateInMin(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputMinValue), InParams.OperatorSettings);
			}

			static TDataReadReference<float> CreateInMax(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputMaxValue), InParams.OperatorSettings);
			}
		};

		template<>
		struct TClamp<FAudioBuffer>
		{
			static void GetClamped(const FAudioBuffer& InValue, const FAudioBuffer& InMin, const FAudioBuffer& InMax, FAudioBuffer& OutClamped)
			{
				float* OutClampedBufferPtr = OutClamped.GetData();
				const float* InBufferPtr = InValue.GetData();
				const float* MinBufferPtr = InMin.GetData();
				const float* MaxBufferPtr = InMax.GetData();
				int32 NumSamples = InValue.Num();
				for (int32 i = 0; i < NumSamples; ++i)
				{
					OutClampedBufferPtr[i] = FMath::Clamp(InBufferPtr[i], MinBufferPtr[i], MaxBufferPtr[i]);
				}
			}

			static TDataReadReference<FAudioBuffer> CreateInValue(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static TDataReadReference<FAudioBuffer> CreateInMin(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputMinValue), InParams.OperatorSettings);
			}

			static TDataReadReference<FAudioBuffer> CreateInMax(const FBuildOperatorParams& InParams)
			{
				using namespace ClampVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputMaxValue), InParams.OperatorSettings);
			}
		};
	}

	template<typename ValueType>
	class TClampNodeOperator : public TExecutableOperator<TClampNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ClampVertexNames;
			using namespace MetasoundClampNodePrivate;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValue)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMinValue)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMaxValue))
				),
				FOutputVertexInterface(
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = "Clamp";
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ClampDisplayNamePattern", "Clamp ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("ClampDesc", "Returns the clamped value of the input within the given value range.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundClampNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ClampVertexNames;
			using namespace MetasoundClampNodePrivate;

			TDataReadReference<ValueType> InputValue = TClamp<ValueType>::CreateInValue(InParams);
			TDataReadReference<ValueType> InputMinValue = TClamp<ValueType>::CreateInMin(InParams);
			TDataReadReference<ValueType> InputMaxValue = TClamp<ValueType>::CreateInMax(InParams);

			return MakeUnique<TClampNodeOperator<ValueType>>(InParams.OperatorSettings, InputValue, InputMinValue, InputMaxValue);
		}


		TClampNodeOperator(const FOperatorSettings& InSettings,
			const TDataReadReference<ValueType>& InInputValue,
			const TDataReadReference<ValueType>& InInputMinValue,
			const TDataReadReference<ValueType>& InInputMaxValue)
			: InputValue(InInputValue)
			, InputMinValue(InInputMinValue)
			, InputMaxValue(InInputMaxValue)
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
		{
			GetClamped();
		}

		virtual ~TClampNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ClampVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), InputValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMinValue), InputMinValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMaxValue), InputMaxValue);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ClampVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void GetClamped()
		{
			using namespace MetasoundClampNodePrivate;

			TClamp<ValueType>::GetClamped(*InputValue, *InputMinValue, *InputMaxValue, *OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			GetClamped();
		}

		void Execute()
		{
			GetClamped();
		}

	private:

		TDataReadReference<ValueType> InputValue;
		TDataReadReference<ValueType> InputMinValue;
		TDataReadReference<ValueType> InputMaxValue;
		TDataWriteReference<ValueType> OutputValue;
	};

	/** TClampNode
 *
 *  Returns the clamped value of the input in the specified range.
 */
	template<typename ValueType>
	using TClampNode = TNodeFacade<TClampNodeOperator<ValueType>>;

	using FClampNodeInt32 = TClampNode<int32>;
 	METASOUND_REGISTER_NODE(FClampNodeInt32)

	using FClampNodeFloat = TClampNode<float>;
	METASOUND_REGISTER_NODE(FClampNodeFloat)

	using FClampNodeAudio = TClampNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FClampNodeAudio)

}

#undef LOCTEXT_NAMESPACE //MetaSoundStandardNodes_ClampNode
