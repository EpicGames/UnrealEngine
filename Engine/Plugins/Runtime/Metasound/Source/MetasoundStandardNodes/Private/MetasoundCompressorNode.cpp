// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Dsp.h"
#include "DSP/DynamicsProcessor.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/IntegerDelay.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CompressorNode"

namespace Metasound
{
	/* Mid-Side Encoder */
	namespace CompressorVertexNames
	{
		METASOUND_PARAM(InputIsBypassed, "Bypass", "When true no audio is processed, the input is copied to the output, and the envelope output is zero.");
		METASOUND_PARAM(InputAudio, "Audio", "Incoming audio signal to compress.");
		METASOUND_PARAM(InputRatio, "Ratio", "Amount of gain reduction. 1 = no reduction, higher = more reduction.");
		METASOUND_PARAM(InputThreshold, "Threshold dB", "Amplitude threshold (dB) above which gain will be reduced.");
		METASOUND_PARAM(InputAttackTime, "Attack Time", "How long it takes for audio above the threshold to reach its compressed volume level.");
		METASOUND_PARAM(InputReleaseTime, "Release Time", "How long it takes for audio below the threshold to return to its original volume level.");
		METASOUND_PARAM(InputLookaheadTime, "Lookahead Time", "How much time the compressor has to lookahead and catch peaks. This delays the signal.");
		METASOUND_PARAM(InputKnee, "Knee", "How hard or soft the gain reduction blends from no gain reduction to gain reduction. 0 dB = no blending.");
		METASOUND_PARAM(InputSidechain, "Sidechain", "(Optional) External audio signal to control the compressor with. If empty, uses the input audio signal.");
		METASOUND_PARAM(InputEnvelopeMode, "Envelope Mode", "The envelope-following method the compressor will use for gain detection.");
		METASOUND_PARAM(InputIsAnalog, "Analog Mode", "Enable Analog Mode for the compressor's envelope follower.");
		METASOUND_PARAM(InputIsUpwards, "Upwards Mode", "Enable to switch from a standard downwards compresser to an upwards compressor.");
		METASOUND_PARAM(InputWetDryMix, "Wet/Dry", "Ratio between the processed/wet signal and the unprocessed/dry signal. 0 is full dry, 1 is full wet, and 0.5 is 50/50.");

		METASOUND_PARAM(OutputAudio, "Audio", "The output audio signal.");
		METASOUND_PARAM(OutputEnvelope, "Gain Envelope", "The compressor's gain being applied to the signal.");
	}

	// Operator Class
	class FCompressorOperator : public TExecutableOperator<FCompressorOperator>
	{
	public:

		FCompressorOperator(const FOperatorSettings& InSettings,
			const FBoolReadRef& bInIsBypassed,
			const FAudioBufferReadRef& InAudio,
			const FFloatReadRef& InRatio,
			const FFloatReadRef& InThresholdDb,
			const FTimeReadRef& InAttackTime,
			const FTimeReadRef& InReleaseTime,
			const FTimeReadRef& InLookaheadTime,
			const FFloatReadRef& InKnee,
			const bool& bInUseSidechain,
			const FAudioBufferReadRef& InSidechain,
			const FEnvelopePeakModeReadRef& InEnvelopeMode,
			const FBoolReadRef& bInIsAnalog,
			const FBoolReadRef& bInIsUpwards,
			const FFloatReadRef& InWetDryMix)
			: bIsBypassedInput(bInIsBypassed)
			, AudioInput(InAudio)
			, RatioInput(InRatio)
			, ThresholdDbInput(InThresholdDb)
			, AttackTimeInput(InAttackTime)
			, ReleaseTimeInput(InReleaseTime)
			, LookaheadTimeInput(InLookaheadTime)
			, KneeInput(InKnee)
			, SidechainInput(InSidechain)
			, EnvelopeModeInput(InEnvelopeMode)
			, bIsAnalogInput(bInIsAnalog)
			, bIsUpwardsInput(bInIsUpwards)
			, WetDryMixInput(InWetDryMix)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, EnvelopeOutput(FAudioBufferWriteRef::CreateNew(InSettings))
			, InputDelay(FMath::CeilToInt(InSettings.GetSampleRate() * Compressor.GetMaxLookaheadMsec() / 1000.f) + 1, InSettings.GetSampleRate() * 10.0f / 1000.0f)
			, DelayedInputSignal(InSettings.GetNumFramesPerBlock())
			, bEnvelopeOutputIsZero(false)
			, bUseSidechain(bInUseSidechain)
			, MsToSamples(InSettings.GetSampleRate() / 1000.0f)
			, PrevAttackTime(FMath::Max(FTime::ToMilliseconds(*InAttackTime), 0.0))
			, PrevReleaseTime(FMath::Max(FTime::ToMilliseconds(*InReleaseTime), 0.0))
			, PrevLookaheadTime(FMath::Max(FTime::ToMilliseconds(*InLookaheadTime), 0.0))
			, PrevKneeInput(*KneeInput)
			, PrevPeakMode(*EnvelopeModeInput)
			, bPrevIsUpwardsInput(*bIsUpwardsInput)
			, PrevClampedRatio(GetClampedRatio())
			, PrevThreshold(*ThresholdDbInput)
		{
			Compressor.Init(InSettings.GetSampleRate(), 1);
			Compressor.SetKeyNumChannels(1);
			Compressor.SetRatio(GetClampedRatio());
			Compressor.SetThreshold(*ThresholdDbInput);
			Compressor.SetAttackTime(PrevAttackTime);
			Compressor.SetReleaseTime(PrevReleaseTime);
			Compressor.SetLookaheadMsec(PrevLookaheadTime);
			Compressor.SetKneeBandwidth(*KneeInput);

			if (*bIsUpwardsInput)
			{
				Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::UpwardsCompressor);
			}
			else
			{
				Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
			}

			switch (*EnvelopeModeInput)
			{
			default:
			case EEnvelopePeakMode::MeanSquared:
				Compressor.SetPeakMode(Audio::EPeakMode::MeanSquared);
				break;
			case EEnvelopePeakMode::RootMeanSquared:
				Compressor.SetPeakMode(Audio::EPeakMode::RootMeanSquared);
				break;
			case EEnvelopePeakMode::Peak:
				Compressor.SetPeakMode(Audio::EPeakMode::Peak);
				break;
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "Compressor", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("CompressorDisplayName", "Compressor"),
					METASOUND_LOCTEXT("CompressorDesc", "Lowers the dynamic range of a signal."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Dynamics },
					{ METASOUND_LOCTEXT("SidechainKeyword", "Sidechain"), METASOUND_LOCTEXT("UpwardsKeyword", "Upwards Compressor")},
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace CompressorVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIsBypassed), false),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputRatio), 1.5f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputThreshold), -6.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackTime), 0.01f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseTime), 0.1f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLookaheadTime), 0.01f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputKnee), 10.0f),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSidechain)),
					TInputDataVertex<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEnvelopeMode), (int32)EEnvelopePeakMode::Peak),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIsAnalog), true),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIsUpwards), false),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputWetDryMix), 1.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputEnvelope))
				)
			);

			return Interface;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CompressorVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIsBypassed), bIsBypassedInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRatio), RatioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputThreshold), ThresholdDbInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackTime), AttackTimeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseTime), ReleaseTimeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLookaheadTime), LookaheadTimeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputKnee), KneeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSidechain), SidechainInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEnvelopeMode), EnvelopeModeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIsUpwards), bIsUpwardsInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIsAnalog), bIsAnalogInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputWetDryMix), WetDryMixInput);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CompressorVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputEnvelope), EnvelopeOutput);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CompressorVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FBoolReadRef bIsBypassedIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputIsBypassed), InParams.OperatorSettings);
			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FFloatReadRef RatioIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputRatio), InParams.OperatorSettings);
			FFloatReadRef ThresholdDbIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputThreshold), InParams.OperatorSettings);
			FTimeReadRef AttackTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputAttackTime), InParams.OperatorSettings);
			FTimeReadRef ReleaseTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputReleaseTime), InParams.OperatorSettings);
			FTimeReadRef LookaheadTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputLookaheadTime), InParams.OperatorSettings);
			FFloatReadRef KneeIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputKnee), InParams.OperatorSettings);
			FAudioBufferReadRef SidechainIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputSidechain), InParams.OperatorSettings);
			FEnvelopePeakModeReadRef EnvelopeModeIn = InputData.GetOrCreateDefaultDataReadReference<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME(InputEnvelopeMode), InParams.OperatorSettings);
			FBoolReadRef bIsAnalogIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputIsAnalog), InParams.OperatorSettings);
			FBoolReadRef bIsUpwardsIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputIsUpwards), InParams.OperatorSettings);
			FFloatReadRef WetDryMixIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputWetDryMix), InParams.OperatorSettings);

			bool bIsSidechainConnected = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(InputSidechain));

			return MakeUnique<FCompressorOperator>(InParams.OperatorSettings, bIsBypassedIn, AudioIn, RatioIn, ThresholdDbIn, AttackTimeIn, ReleaseTimeIn, LookaheadTimeIn, KneeIn, bIsSidechainConnected, SidechainIn, EnvelopeModeIn, bIsAnalogIn, bIsUpwardsIn, WetDryMixIn);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			// Flush audio buffers
			AudioOutput->Zero();
			EnvelopeOutput->Zero();
			bEnvelopeOutputIsZero = true;
			InputDelay.Reset();
			DelayedInputSignal.Zero();

			// Cache dynamics timing
			PrevAttackTime = FMath::Max(FTime::ToMilliseconds(*AttackTimeInput), 0.0);
			PrevReleaseTime = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0);
			PrevLookaheadTime = FMath::Max(FTime::ToMilliseconds(*LookaheadTimeInput), 0.0);

			// Initialize compressor
			Compressor.Init(InParams.OperatorSettings.GetSampleRate(), 1);
			Compressor.SetKeyNumChannels(1);
			PrevClampedRatio = GetClampedRatio();
			Compressor.SetRatio(PrevClampedRatio);
			Compressor.SetThreshold(*ThresholdDbInput);
			PrevThreshold = *ThresholdDbInput;
			Compressor.SetAttackTime(PrevAttackTime);
			Compressor.SetReleaseTime(PrevReleaseTime);
			Compressor.SetLookaheadMsec(PrevLookaheadTime);
			Compressor.SetKneeBandwidth(*KneeInput);
			PrevKneeInput = *KneeInput;

			if (*bIsUpwardsInput)
			{
				Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::UpwardsCompressor);
			}
			else
			{
				Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
			}
			bPrevIsUpwardsInput = *bIsUpwardsInput;

			switch (*EnvelopeModeInput)
			{
			default:
			case EEnvelopePeakMode::MeanSquared:
				Compressor.SetPeakMode(Audio::EPeakMode::MeanSquared);
				break;
			case EEnvelopePeakMode::RootMeanSquared:
				Compressor.SetPeakMode(Audio::EPeakMode::RootMeanSquared);
				break;
			case EEnvelopePeakMode::Peak:
				Compressor.SetPeakMode(Audio::EPeakMode::Peak);
				break;
			}
			PrevPeakMode = *EnvelopeModeInput;
		}

		void Execute()
		{
			if (*bIsBypassedInput)
			{
				FMemory::Memcpy(AudioOutput->GetData(), AudioInput->GetData(), AudioInput->Num() * sizeof(float));
				if (!bEnvelopeOutputIsZero)
				{
					EnvelopeOutput->Zero();
					bEnvelopeOutputIsZero = true;
				}
				return;
			}

			/* Update parameters */
			
			// For a compressor, ratio values should be 1 or greater
			float UpdatedClampedRatio = GetClampedRatio();
			if (PrevClampedRatio != UpdatedClampedRatio)
			{
				Compressor.SetRatio(UpdatedClampedRatio);
				PrevClampedRatio = UpdatedClampedRatio;
			}
			if (PrevThreshold != *ThresholdDbInput)
			{
				Compressor.SetThreshold(*ThresholdDbInput);
				PrevThreshold = *ThresholdDbInput;
			}

			// Attack time cannot be negative
			float CurrAttack = FMath::Max(FTime::ToMilliseconds(*AttackTimeInput), 0.0f);
			if (FMath::IsNearlyEqual(CurrAttack, PrevAttackTime) == false)
			{
				Compressor.SetAttackTime(CurrAttack);
				PrevAttackTime = CurrAttack;
			}

			// Release time cannot be negative
			float CurrRelease = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0f);
			if (FMath::IsNearlyEqual(CurrRelease, PrevReleaseTime) == false)
			{
				Compressor.SetReleaseTime(CurrRelease);
				PrevReleaseTime = CurrRelease;
			}

			float CurrLookahead = FMath::Clamp(FTime::ToMilliseconds(*LookaheadTimeInput), 0.0f, Compressor.GetMaxLookaheadMsec());
			if (FMath::IsNearlyEqual(CurrLookahead, PrevLookaheadTime) == false)
			{
				Compressor.SetLookaheadMsec(CurrLookahead);
				PrevLookaheadTime = CurrLookahead;
			}
			InputDelay.SetDelayLengthSamples(CurrLookahead * MsToSamples);

			if (*KneeInput != PrevKneeInput)
			{
				Compressor.SetKneeBandwidth(*KneeInput);
				PrevKneeInput = *KneeInput;
			}

			if (*bIsUpwardsInput != bPrevIsUpwardsInput)
			{
				if (*bIsUpwardsInput)
				{
					Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::UpwardsCompressor);
				}
				else
				{
					Compressor.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
				}
				bPrevIsUpwardsInput = *bIsUpwardsInput;
			}

			if (*EnvelopeModeInput != PrevPeakMode)
			{
				switch (*EnvelopeModeInput)
				{
				default:
				case EEnvelopePeakMode::MeanSquared:
					Compressor.SetPeakMode(Audio::EPeakMode::MeanSquared);
					break;

				case EEnvelopePeakMode::RootMeanSquared:
					Compressor.SetPeakMode(Audio::EPeakMode::RootMeanSquared);
					break;

				case EEnvelopePeakMode::Peak:
					Compressor.SetPeakMode(Audio::EPeakMode::Peak);
					break;
				}
				PrevPeakMode = *EnvelopeModeInput;
			}

			// Apply lookahead delay to dry signal
			InputDelay.ProcessAudio(*AudioInput, TArrayView<float>(DelayedInputSignal));

			const float* InSamples = AudioInput->GetData();
			float* OutSamples      = AudioOutput->GetData();
			const float* InKey     = SidechainInput->GetData();
			float* OutEnvelope     = EnvelopeOutput->GetData();

			Compressor.ProcessAudio(&InSamples, AudioInput->Num(), &OutSamples, bUseSidechain ? &InKey : nullptr, &OutEnvelope);

			bEnvelopeOutputIsZero = false;

			// Calculate Wet/Dry mix
			float NewWetDryMix = FMath::Clamp(*WetDryMixInput, 0.0f, 1.0f);
			// Wet signal
			Audio::ArrayMultiplyByConstantInPlace(*AudioOutput, NewWetDryMix);
			// Add Dry signal
			Audio::ArrayMultiplyAddInPlace(DelayedInputSignal, 1.0f - NewWetDryMix, *AudioOutput);
			
		}

		float GetClampedRatio() const
		{
			return FMath::Max(*RatioInput, 1.0f);
		}

	private:
		// Audio input and output
		FBoolReadRef bIsBypassedInput;
		FAudioBufferReadRef AudioInput;
		FFloatReadRef RatioInput;
		FFloatReadRef ThresholdDbInput;
		FTimeReadRef AttackTimeInput;
		FTimeReadRef ReleaseTimeInput;
		FTimeReadRef LookaheadTimeInput;
		FFloatReadRef KneeInput;
		FAudioBufferReadRef SidechainInput;
		FEnvelopePeakModeReadRef EnvelopeModeInput;
		FBoolReadRef bIsAnalogInput;
		FBoolReadRef bIsUpwardsInput;
		FFloatReadRef WetDryMixInput;

		FAudioBufferWriteRef AudioOutput;
		// The gain being applied to the input buffer
		FAudioBufferWriteRef EnvelopeOutput;

		// Internal DSP Compressor
		Audio::FDynamicsProcessor Compressor;
		// Compressor does not have a wet/dry signal built in, so
		// we need to account for lookahead delay in the input to prevent phase issues.
		Audio::FIntegerDelay InputDelay;
		FAudioBuffer DelayedInputSignal;

		// When bypassed this prevents continual re-zeroing of envelope buffer.
		bool bEnvelopeOutputIsZero;

		// Whether or not to use sidechain input (is false if no input pin is connected to sidechain input)
		bool bUseSidechain;

		// Conversion from milliseconds to samples
		float MsToSamples;

		// Cached variables to minimize updating the underlying
		// DynamicsProcessor when there are no changes. It DOES NOT
		// "early out" if it is told about "new" settngs that actually
		// match its existing settings. 
		float PrevAttackTime;
		float PrevReleaseTime;
		float PrevLookaheadTime;
		float PrevKneeInput;
		EEnvelopePeakMode PrevPeakMode;
		bool bPrevIsUpwardsInput;
		float PrevClampedRatio;
		float PrevThreshold;
	};

	// Node Class
	using FCompressorNode = TNodeFacade<FCompressorOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FCompressorNode)

}

#undef LOCTEXT_NAMESPACE
