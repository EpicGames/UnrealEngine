// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundStandardNodesCategories.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Conversion"

#define METASOUND_REGISTER_CONVERSION(NodeClass, FromDataType, ToDataType) \
	METASOUND_IMPLEMENT_REGISTRATION_ACTION(UE_JOIN(Conversion,NodeClass), (&Metasound::RegisterConversionOperator<FromDataType, ToDataType>), nullptr)

namespace Metasound
{
	namespace ConversionNodeVertexNames
	{
		METASOUND_PARAM(InputValue, "In", "Input value A.");
		METASOUND_PARAM(OutputValue, "Out", "The converted value.");
	}

	namespace MetasoundConversionNodePrivate
	{
		template<typename FromType, typename ToType>
		struct TConversionNodeSpecialization
		{
			bool bSupported = false;
		};

		template<>
		struct TConversionNodeSpecialization<float, FTime>
		{
			static TDataReadReference<float> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(float InValue, FTime& OutValue)
			{
				OutValue = FTime(InValue);
			}
		};

		template<>
		struct TConversionNodeSpecialization<float, FAudioBuffer>
		{
			static TDataReadReference<float> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(float InValue, FAudioBuffer& OutValue)
			{
				int32 NumSamples = OutValue.Num();
				float* OutValuePtr = OutValue.GetData();
				for (int32 i = 0; i < NumSamples; ++i)
				{
					OutValuePtr[i] = InValue;
				}
			}
		};

		template<>
		struct TConversionNodeSpecialization<int32, FTime>
		{
			static TDataReadReference<int32> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(int32 InValue, FTime& OutValue)
			{
				OutValue = FTime((float)InValue);
			}
		};

		template<>
		struct TConversionNodeSpecialization<FTime, float>
		{
			static TDataReadReference<FTime> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(const FTime& InValue, float& OutValue)
			{
				OutValue = InValue.GetSeconds();
			}
		};

		template<>
		struct TConversionNodeSpecialization<FTime, int32>
		{
			static TDataReadReference<FTime> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(const FTime& InValue, int32& OutValue)
			{
				OutValue = (int32)InValue.GetSeconds();
			}
		};

		template<>
		struct TConversionNodeSpecialization<FAudioBuffer, float>
		{
			static TDataReadReference<FAudioBuffer> CreateInputRef(const FVertexInterface& Interface, const FBuildOperatorParams& InParams)
			{
				using namespace ConversionNodeVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			}

			static void GetConvertedValue(const FAudioBuffer& InValue, float& OutValue)
			{
				OutValue = 0.0f;
				int32 NumSamples = InValue.Num();
				const float* InValuePtr = InValue.GetData();
				for (int32 i = 0; i < NumSamples; ++i)
				{
					OutValue += InValuePtr[i];
				}
				OutValue /= NumSamples;
			}
		};
	}

	template<typename FromType, typename ToType>
	class TConversionOperator : public TExecutableOperator<TConversionOperator<FromType, ToType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ConversionNodeVertexNames;

			static FText InputDesc = METASOUND_LOCTEXT_FORMAT("ConvDisplayNamePatternFrom", "Input {0} value.", GetMetasoundDataTypeDisplayText<FromType>());
			static FText OutputDesc = METASOUND_LOCTEXT_FORMAT("ConvDisplayNamePatternTo", "Output {0} value.", GetMetasoundDataTypeDisplayText<ToType>());

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FromType>(METASOUND_GET_PARAM_NAME(InputValue), FDataVertexMetadata{InputDesc})
				),
				FOutputVertexInterface(
					TOutputDataVertex<ToType>(METASOUND_GET_PARAM_NAME(OutputValue), FDataVertexMetadata{OutputDesc})
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeDisplayStyle DisplayStyle;
				DisplayStyle.bShowName = false;
				DisplayStyle.ImageName = TEXT("MetasoundEditor.Graph.Node.Conversion");
				DisplayStyle.bShowInputNames = false;
				DisplayStyle.bShowOutputNames = false;

				const FText& FromTypeText = GetMetasoundDataTypeDisplayText<FromType>();
				const FText& ToTypeText = GetMetasoundDataTypeDisplayText<ToType>();

				const FString& FromTypeString = GetMetasoundDataTypeString<FromType>();
				const FString& ToTypeString = GetMetasoundDataTypeString<ToType>();

				const FName ClassName = *FString::Format(TEXT("Conversion{0}To{1}"), { FromTypeString, ToTypeString });
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ConverterNodeDisplayName", "{0} To {1}", FromTypeText, ToTypeText);

				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, ClassName, "" };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = GetNodeDescription();
				Info.Author = PluginAuthor;
				Info.DisplayStyle = DisplayStyle;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetDefaultInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Conversions);

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MetasoundConversionNodePrivate;
			TDataReadReference<FromType> InputValue = TConversionNodeSpecialization<FromType, ToType>::CreateInputRef(GetDefaultInterface(), InParams);
			return MakeUnique<TConversionOperator<FromType, ToType>>(InParams.OperatorSettings, InputValue);
		}


		TConversionOperator(const FOperatorSettings& InSettings, TDataReadReference<FromType> InInputValue)
			: InputValue(InInputValue)
			, OutputValue(TDataWriteReferenceFactory<ToType>::CreateAny(InSettings))
		{
			using namespace MetasoundConversionNodePrivate;
			TConversionNodeSpecialization<FromType, ToType>::GetConvertedValue(*InputValue, *OutputValue);
		}

		virtual ~TConversionOperator() = default;

		static FText GetNodeDescription() 
		{
			if constexpr (std::is_same<FromType, float>::value && std::is_same<ToType, FAudioBuffer>::value)
			{
				return METASOUND_LOCTEXT("FloatToAudioConverterDescription", "Converts from float to audio buffer with each sample set to the given float value.");
			}
			else if constexpr (std::is_same<FromType, FAudioBuffer>::value && std::is_same<ToType, float>::value)
			{
				return METASOUND_LOCTEXT("AudioToFloatConverterDescription", "Converts from audio buffer to float by averaging sample values over the buffer. ");
			}
			else
			{
				return METASOUND_LOCTEXT_FORMAT("ConverterNodeDesc", "Converts from {0} to {1}.", GetMetasoundDataTypeDisplayText<FromType>(), GetMetasoundDataTypeDisplayText<ToType>());
			}
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ConversionNodeVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), InputValue);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ConversionNodeVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace MetasoundConversionNodePrivate;
			TConversionNodeSpecialization<FromType, ToType>::GetConvertedValue(*InputValue, *OutputValue);
		}

		void Execute()
		{
			using namespace MetasoundConversionNodePrivate;
			TConversionNodeSpecialization<FromType, ToType>::GetConvertedValue(*InputValue, *OutputValue);
		}

	private:

		TDataReadReference<FromType> InputValue;
		TDataWriteReference<ToType> OutputValue;
	};

	/** TConversionNode
	 *
	 *  Generates a random float value when triggered.
	 */
	template<typename FromType, typename ToType>
	using TConversionNode = TNodeFacade<TConversionOperator<FromType, ToType>>;

	using FConversionFloatToTime = TConversionNode<float, FTime>;
	METASOUND_REGISTER_NODE(FConversionFloatToTime)

	using FConversionTimeToFloat = TConversionNode<FTime, float>;
	METASOUND_REGISTER_NODE(FConversionTimeToFloat)

	using FConversionInt32ToTime = TConversionNode<int32, FTime>;
	METASOUND_REGISTER_NODE(FConversionInt32ToTime)

	using FConversionTimeToInt32 = TConversionNode<FTime, int32>;
	METASOUND_REGISTER_NODE(FConversionTimeToInt32)

	using FConversionFloatToAudio = TConversionNode<float, FAudioBuffer>;
	METASOUND_REGISTER_NODE(FConversionFloatToAudio)

	using FConversionAudioToFloat = TConversionNode<FAudioBuffer, float>;
	METASOUND_REGISTER_NODE(FConversionAudioToFloat)


	template<typename TFromType, typename TToType>
	bool RegisterConversionOperator(const Frontend::FModuleInfo& InModuleInfo)
	{
		using FConverterNodeRegistryKey = ::Metasound::Frontend::FConverterNodeRegistryKey;
		using FConverterNodeInfo = ::Metasound::Frontend::FConverterNodeInfo;

		FName FromType = ::Metasound::TDataReferenceTypeInfo<TFromType>::TypeName;
		FName ToType = ::Metasound::TDataReferenceTypeInfo<TToType>::TypeName;

		FConverterNodeRegistryKey RegistryKey = { FromType, ToType };

		const FNodeClassMetadata& Metadata = TConversionOperator<TFromType, TToType>::GetNodeInfo();

		FConverterNodeInfo ConverterNodeInfo =
		{
			FromType,
			ToType,
			Metasound::Frontend::FNodeRegistryKey(Metadata)
		};

		return FMetasoundFrontendRegistryContainer::Get()->RegisterConversionNode(RegistryKey, ConverterNodeInfo);
	}

	METASOUND_REGISTER_CONVERSION(FConversionFloatToTime, float, FTime)
	METASOUND_REGISTER_CONVERSION(FConversionTimeToFloat, FTime, float)
	METASOUND_REGISTER_CONVERSION(FConversionInt32ToTime, int32, FTime)
	METASOUND_REGISTER_CONVERSION(FConversionTimeToInt32, FTime, int32)
	METASOUND_REGISTER_CONVERSION(FConversionFloatToAudio, float, FAudioBuffer)
	METASOUND_REGISTER_CONVERSION(FConversionAudioToFloat, FAudioBuffer, float)
}

#undef METASOUND_REGISTER_CONVERSION
#undef LOCTEXT_NAMESPACE
