// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundAudioBuffer.h"
#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CrossfadeNode"

#define REGISTER_CROSSFADE_NODE(DataType, Number) \
	using FCrossfadeNode##DataType##_##Number = TCrossfadeNode<DataType, Number>; \
	METASOUND_REGISTER_NODE(FCrossfadeNode##DataType##_##Number) \


namespace Metasound
{
	namespace CrossfadeVertexNames
	{
		METASOUND_PARAM(InputCrossfadeValue, "Crossfade Value", "Crossfade value to crossfade across inputs. Output will be the float value between adjacent whole number values.")
		METASOUND_PARAM(OutputTrigger, "Out", "Output value.")

		const FVertexName GetInputName(uint32 InIndex)
		{
			return *FString::Format(TEXT("In {0}"), { InIndex });
		}

		const FText GetInputDescription(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("CrossfadeInputDesc", "Cross fade {0} input.", InIndex);
		}

		const FText GetInputDisplayName(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("CrossfadeInputDisplayName", "In {0}", InIndex);
		}
	}

	template<typename ValueType, uint32 NumInputs>
	class TCrossfadeHelper
	{
	};

	// Partial specialization for float
	template<uint32 NumInputs>
	class TCrossfadeHelper<float, NumInputs>
	{
	public:
		TCrossfadeHelper(int32 NumFramesPerBlock) {}

		void GetCrossfadeOutput(int32 IndexA, int32 IndexB, float Alpha, const TArray<FFloatReadRef>& InCurrentValues, float& OutValue)
		{
			const FFloatReadRef& InA = InCurrentValues[IndexA];
			const FFloatReadRef& InB = InCurrentValues[IndexB];
			OutValue = FMath::Lerp(*InA, *InB, Alpha);
		}
	};

	// Partial specialization for FAudioBuffer
	template<uint32 NumInputs>
	class TCrossfadeHelper<FAudioBuffer, NumInputs>
	{
	public:
		TCrossfadeHelper(int32 InNumFramesPerBlock)
			: NumFramesPerBlock(InNumFramesPerBlock)
		{
			PrevGains.AddZeroed(NumInputs);
			CurrentGains.AddZeroed(NumInputs);
			NeedsMixing.AddZeroed(NumInputs);
		}

		void GetCrossfadeOutput(int32 IndexA, int32 IndexB, float Alpha, const TArray<FAudioBufferReadRef>& InAudioBuffersValues, FAudioBuffer& OutAudioBuffer)
		{
			// Determine the gains
			for (int32 i = 0; i < NumInputs; ++i)
			{
				if (i == IndexA)
				{
					CurrentGains[i] = 1.0f - Alpha;
					NeedsMixing[i] = true;
				}
				else if (i == IndexB)
				{
					CurrentGains[i] = Alpha;
					NeedsMixing[i] = true;
				}
				else
				{
					CurrentGains[i] = 0.0f;

					// If we were already at 0.0f, don't need to do any mixing!
					if (PrevGains[i] == 0.0f)
					{
						NeedsMixing[i] = false;
					}
				}
			}

			// Zero the output buffer so we can mix into it
			OutAudioBuffer.Zero();
			TArrayView<float> OutAudioBufferView(OutAudioBuffer.GetData(), OutAudioBuffer.Num());

			// Now write to the scratch buffers w/ fade buffer fast given the new inputs
			for (int32 i = 0; i < NumInputs; ++i)
			{
				// Only need to do anything on an input if either curr or prev is non-zero
				if (NeedsMixing[i])
				{
					// Copy the input to the output
					const FAudioBufferReadRef& InBuff = InAudioBuffersValues[i];
					TArrayView<const float> BufferView((*InBuff).GetData(), NumFramesPerBlock);
					const float* BufferPtr = (*InBuff).GetData();

					// mix in and fade to the target gain values
					Audio::ArrayMixIn(BufferView, OutAudioBufferView, PrevGains[i], CurrentGains[i]);
				}
			}

			// Copy the CurrentGains to PrevGains
			PrevGains = CurrentGains;
		}

	private:
		int32 NumFramesPerBlock = 0;
		TArray<float> PrevGains;
		TArray<float> CurrentGains;
		TArray<bool> NeedsMixing;
	};

	template<typename ValueType, uint32 NumInputs>
	class TCrossfadeOperator : public TExecutableOperator<TCrossfadeOperator<ValueType, NumInputs>>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace CrossfadeVertexNames;

			auto CreateDefaultInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;

				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputCrossfadeValue)));

				for (uint32 i = 0; i < NumInputs; ++i)
				{
					const FDataVertexMetadata InputMetadata
					{
						GetInputDescription(i),
						GetInputDisplayName(i)
					};

					InputInterface.Add(TInputDataVertex<ValueType>(GetInputName(i), InputMetadata));
				}

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTrigger)));

				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				FName OperatorName = *FString::Printf(TEXT("Trigger Route (%s, %d)"), *DataTypeName.ToString(), NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("CrossfadeDisplayNamePattern", "Crossfade ({0}, {1})", GetMetasoundDataTypeDisplayText<ValueType>(), NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("CrossfadeDescription", "Crossfades inputs to outputs.");
				FVertexInterface NodeInterface = GetVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { "Crossfade", OperatorName, DataTypeName },
					1, // Major Version
					0, // Minor Version
					NodeDisplayName,
					NodeDescription,
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Envelopes },
					{ },
					FNodeDisplayStyle()
				};
				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CrossfadeVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FFloatReadRef CrossfadeValue = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputCrossfadeValue), InParams.OperatorSettings);

			TArray<TDataReadReference<ValueType>> InputValues;
			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputValues.Add(InputData.GetOrCreateDefaultDataReadReference<ValueType>(GetInputName(i), InParams.OperatorSettings));
			}

			return MakeUnique<TCrossfadeOperator<ValueType, NumInputs>>(InParams.OperatorSettings, CrossfadeValue, MoveTemp(InputValues));
		}


		TCrossfadeOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InCrossfadeValue, TArray<TDataReadReference<ValueType>>&& InInputValues)
			: CrossfadeValue(InCrossfadeValue)
			, InputValues(MoveTemp(InInputValues))
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
			, Crossfader(InSettings.GetNumFramesPerBlock())
		{
			PerformCrossfadeOutput();
		}

		virtual ~TCrossfadeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CrossfadeVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCrossfadeValue), CrossfadeValue);

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InOutVertexData.BindReadVertex(GetInputName(i), InputValues[i]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CrossfadeVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTrigger), OutputValue);
		}

		void PerformCrossfadeOutput()
		{
			// Clamp the cross fade value based on the number of inputs
			float CurrentCrossfadeValue = FMath::Clamp(*CrossfadeValue, 0.0f, (float)(NumInputs - 1));

			// Only update the cross fade state if anything has changed
			if (!FMath::IsNearlyEqual(CurrentCrossfadeValue, PrevCrossfadeValue))
			{
				PrevCrossfadeValue = CurrentCrossfadeValue;
				IndexA = (int32)FMath::Floor(CurrentCrossfadeValue);
				IndexB = FMath::Clamp(IndexA + 1, 0.0f, (float)(NumInputs - 1));
				Alpha = CurrentCrossfadeValue - (float)IndexA;
			}

			// Need to call this each block in case inputs have changed
			Crossfader.GetCrossfadeOutput(IndexA, IndexB, Alpha, InputValues, *OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			PerformCrossfadeOutput();
		}

		void Execute()
		{
			PerformCrossfadeOutput();
		}

	private:
		FFloatReadRef CrossfadeValue;
		TArray<TDataReadReference<ValueType>> InputValues;
		TDataWriteReference<ValueType> OutputValue;

		float PrevCrossfadeValue = -1.0f;
		int32 IndexA = 0;
		int32 IndexB = 0;
		float Alpha = 0.0f;
		TCrossfadeHelper<ValueType, NumInputs> Crossfader;
	};

	template<typename ValueType, uint32 NumInputs>
	using TCrossfadeNode = TNodeFacade<TCrossfadeOperator<ValueType, NumInputs>>;

	REGISTER_CROSSFADE_NODE(float, 2);
	REGISTER_CROSSFADE_NODE(float, 3);
	REGISTER_CROSSFADE_NODE(float, 4);
	REGISTER_CROSSFADE_NODE(float, 5);
	REGISTER_CROSSFADE_NODE(float, 6);
	REGISTER_CROSSFADE_NODE(float, 7);
	REGISTER_CROSSFADE_NODE(float, 8);

	REGISTER_CROSSFADE_NODE(FAudioBuffer, 2);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 3);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 4);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 5);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 6);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 7);
	REGISTER_CROSSFADE_NODE(FAudioBuffer, 8);
}

#undef LOCTEXT_NAMESPACE
