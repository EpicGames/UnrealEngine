// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"


#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_DebugLogNode"

namespace Metasound
{
	namespace MetasoundPrintLogNodePrivate
	{
		//Creates Metadata for the Print Log Node
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "Print Log", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Debug },
				{ },
				FNodeDisplayStyle()
			};

			return Metadata;
		}
	}

	//Getters for the name of each parameter used in Print Log
	namespace PrintLogVertexNames
	{
		METASOUND_PARAM(InputTrigger, "Trigger", "Trigger to write the set value to the log.")
		METASOUND_PARAM(InputLabel, "Label", "The label to attach to the value that will be logged.")
		METASOUND_PARAM(InputValueToLog, "Value To Log", "The value to record to the log when triggered.")
	}


	template<typename PrintLogType>
	class TPrintLogOperator : public TExecutableOperator<TPrintLogOperator<PrintLogType>>
	{
		public:
			using FArrayDataReadReference = TDataReadReference<PrintLogType>;

			static const FVertexInterface& GetDefaultInterface()
			{
				using namespace PrintLogVertexNames;
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(

						TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)),
						TInputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLabel)),
						TInputDataVertex<PrintLogType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValueToLog))
					),
					FOutputVertexInterface(
					)
				);

				return DefaultInterface;
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
				{
					FName DataTypeName = GetMetasoundDataTypeName<PrintLogType>();
					FName OperatorName = TEXT("Print Log");
					FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("PrintLogDisplayNamePattern", "Print Log ({0})", GetMetasoundDataTypeDisplayText<PrintLogType>());
					const FText NodeDescription = METASOUND_LOCTEXT("PrintLogOpDescription", "Used to record values to the log, on trigger");
					FVertexInterface NodeInterface = GetDefaultInterface();

					return MetasoundPrintLogNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
				};

				static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
				return Metadata;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
			{
				using namespace PrintLogVertexNames;

				const FInputVertexInterfaceData& InputData = InParams.InputData;

				FTriggerReadRef Trigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);

				TDataReadReference<FString> Label = InputData.GetOrCreateDefaultDataReadReference<FString>(METASOUND_GET_PARAM_NAME(InputLabel), InParams.OperatorSettings);
				TDataReadReference<PrintLogType> ValueToLog = InputData.GetOrCreateDefaultDataReadReference<PrintLogType>(METASOUND_GET_PARAM_NAME(InputValueToLog), InParams.OperatorSettings);

				return MakeUnique<TPrintLogOperator<PrintLogType>>(InParams, Trigger, Label, ValueToLog);
			}


			TPrintLogOperator(const FBuildOperatorParams& InParams, TDataReadReference<FTrigger> InTrigger, TDataReadReference<FString> InLabelPrintLog, TDataReadReference<PrintLogType> InValueToLogPrintLog)
				: Trigger(InTrigger)
				, Label(InLabelPrintLog)
				, ValueToLog(InValueToLogPrintLog)
			{
				Reset(InParams);
			}

			virtual ~TPrintLogOperator() = default;

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
			{
				using namespace PrintLogVertexNames;
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), Trigger);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLabel), Label);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValueToLog), ValueToLog);
			}

			virtual void BindOutputs(FOutputVertexInterfaceData&) override
			{
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				GraphName = TEXT("");
				InstanceID = INDEX_NONE;
				
				if (InParams.Environment.Contains<FString>(Frontend::SourceInterface::Environment::GraphName))
				{
					FString GraphNameFull = InParams.Environment.GetValue<FString>(Frontend::SourceInterface::Environment::GraphName);
					TArray<FString> ParsedString;
					GraphNameFull.ParseIntoArray(ParsedString, TEXT("."), true);
					if (ParsedString.Num() > 0)
					{
						GraphName = ParsedString.Last();
					}
				}
				
				if (InParams.Environment.Contains<uint64>(Frontend::SourceInterface::Environment::TransmitterID))
				{
					InstanceID = InParams.Environment.GetValue<uint64>(Frontend::SourceInterface::Environment::TransmitterID);
				}
			}

			void Execute()
			{
				if (*Trigger)
				{
					UE_LOGF(LogMetaSound, Display, "[%ls:%llu]: %ls %ls", *GraphName, InstanceID, *(*Label), *LexToString(*ValueToLog));
				}
			}

		private:

			TDataReadReference<FTrigger> Trigger;
			TDataReadReference<FString> Label;
			TDataReadReference<PrintLogType> ValueToLog;

			FString GraphName;
			uint64 InstanceID = INDEX_NONE;
	};

	/** TPrintLogNode
	 *
	 *  Records a value to the log when triggered
	 */
	template<typename PrintLogType>
	using TPrintLogNode = TNodeFacade<TPrintLogOperator<PrintLogType>>;

	using FPrintLogNodeInt32 = TPrintLogNode<int32>;
	METASOUND_REGISTER_NODE(FPrintLogNodeInt32)

	using FPrintLogNodeFloat = TPrintLogNode<float>;
	METASOUND_REGISTER_NODE(FPrintLogNodeFloat)

	using FPrintLogNodeBool = TPrintLogNode<bool>;
	METASOUND_REGISTER_NODE(FPrintLogNodeBool)

	using FPrintLogNodeString = TPrintLogNode<FString>;
	METASOUND_REGISTER_NODE(FPrintLogNodeString)
}

#undef LOCTEXT_NAMESPACE
