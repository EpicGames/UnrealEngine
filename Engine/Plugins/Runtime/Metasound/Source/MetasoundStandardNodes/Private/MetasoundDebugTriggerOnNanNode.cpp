// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerOnNanNode"

namespace Metasound
{

	namespace TriggerOnNanVertexNames
	{
		METASOUND_PARAM(AudioInput, "Audio In", "The Audio Buffer to be evaluated for NANs.");
		METASOUND_PARAM(TriggerOnce, "Trigger Once", "Whether we should trigger for every NAN, or just once for the lifecycle of the Metasound Source");
		
		METASOUND_PARAM(OutputTriggerOnNan, "Trigger Out", "Trigger output on the audio frame containing the Nan");
	}

	class FTriggerOnNanOperator : public TExecutableOperator<FTriggerOnNanOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerOnNanOperator(const FOperatorSettings& InSettings, const FAudioBufferReadRef& InAudioInput, const FBoolReadRef& InTriggerOnce);

		void Reset(const IOperator::FResetParams&);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();

	private:
		// Try to go through gate
		FAudioBufferReadRef AudioInput;
		FBoolReadRef bTriggerOnce;
		FTriggerWriteRef TriggerOutput;

		// Status of the gate
		bool bHasTriggered;
	};

	FTriggerOnNanOperator::FTriggerOnNanOperator(
		const FOperatorSettings& InSettings,
		const FAudioBufferReadRef& InAudioInput,
		const FBoolReadRef& InTriggerOnce)
		: AudioInput(InAudioInput)
		, bTriggerOnce(InTriggerOnce)
		, TriggerOutput(FTriggerWriteRef::CreateNew(InSettings))
		, bHasTriggered(false)
	{
	}

	void FTriggerOnNanOperator::Reset(const IOperator::FResetParams&)
	{
		TriggerOutput->Reset();
		bHasTriggered = false;
	}

	void FTriggerOnNanOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnNanVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AudioInput), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(TriggerOnce), bTriggerOnce);
	}

	void FTriggerOnNanOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnNanVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTriggerOnNan), TriggerOutput);
	}

	void FTriggerOnNanOperator::Execute()
	{
		const bool bShouldTriggerOnce = *bTriggerOnce;
		TriggerOutput->AdvanceBlock();

		const float* AudioPtr = AudioInput->GetData();
		const int32 NumSamples = AudioInput->Num();

		for(int32 i = 0; i < NumSamples; ++i)
		{
			const bool bSuppressTriggers = (bShouldTriggerOnce && bHasTriggered);
			if(!bSuppressTriggers && !FMath::IsFinite(AudioPtr[i]))
			{
				TriggerOutput->TriggerFrame(i);
				bHasTriggered = true;
			}
		}
	}


	TUniquePtr<IOperator> FTriggerOnNanOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerOnNanVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FAudioBufferReadRef AudioInput = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(AudioInput), InParams.OperatorSettings);
		FBoolReadRef bTriggerOnce = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(TriggerOnce), InParams.OperatorSettings);

		return MakeUnique<FTriggerOnNanOperator>(InParams.OperatorSettings, AudioInput, bTriggerOnce);
	}

	const FVertexInterface& FTriggerOnNanOperator::GetVertexInterface()
	{
		using namespace TriggerOnNanVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(AudioInput)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(TriggerOnce))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTriggerOnNan))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerOnNanOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::StandardNodes::Namespace, "Trigger On Nan", FName() };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerOnNanNodeDisplayName", "Trigger On Nan");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerOnNanNodeDescription", "Sends an output trigger the first time the node is triggered, and ignores all others (can be re-opened).");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	using FTriggerOnNanNode = TNodeFacade<FTriggerOnNanOperator>;

	METASOUND_REGISTER_NODE(FTriggerOnNanNode)
}

#undef LOCTEXT_NAMESPACE
