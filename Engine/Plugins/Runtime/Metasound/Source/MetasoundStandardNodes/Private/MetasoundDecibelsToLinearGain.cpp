// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_DecibelsToLinearGain"

namespace Metasound
{

	namespace DecibelsToLinearGainVertexNames
	{
		METASOUND_PARAM(InputDecibelGain, "Decibels", "Input logarithmic (dB) gain.");

		METASOUND_PARAM(OutputLinearGain, "Linear Gain", "Output corresponding linear gain.");
	}


	class FDecibelsToLinearGainOperator : public TExecutableOperator<FDecibelsToLinearGainOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FDecibelsToLinearGainOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InDecibelGain);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The input dB value
		FFloatReadRef DecibelGainInput;

		// The output linear gain
		FFloatWriteRef LinearGainOutput;
	};

	FDecibelsToLinearGainOperator::FDecibelsToLinearGainOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InDecibelGain)
		: DecibelGainInput(InDecibelGain)
		, LinearGainOutput(FFloatWriteRef::CreateNew())
	{
		Reset(InParams);
	}

	void FDecibelsToLinearGainOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace DecibelsToLinearGainVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDecibelGain), DecibelGainInput);
	}

	void FDecibelsToLinearGainOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace DecibelsToLinearGainVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputLinearGain), LinearGainOutput);
	}

	void FDecibelsToLinearGainOperator::Reset(const IOperator::FResetParams& InParams)
	{
		Execute();
	}

	void FDecibelsToLinearGainOperator::Execute()
	{
		*LinearGainOutput = Audio::ConvertToLinear(*DecibelGainInput);
	}

	const FVertexInterface& FDecibelsToLinearGainOperator::GetVertexInterface()
	{
		using namespace DecibelsToLinearGainVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDecibelGain), 0.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputLinearGain))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FDecibelsToLinearGainOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<float>();
			const FName OperatorName = TEXT("Decibels to Linear Gain");
			const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_DecibelsToLinearGainName", "Decibels to Linear Gain");
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_DecibelsToLinearGainDescription", "Converts a logarithmic (dB) gain value to a linear gain value.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Dynamics);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FDecibelsToLinearGainOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace DecibelsToLinearGainVertexNames;
		
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InDecibelGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDecibelGain), InParams.OperatorSettings);

		return MakeUnique<FDecibelsToLinearGainOperator>(InParams, InDecibelGain);
	}

	using FDecibelsToLinearGainNode = TNodeFacade<FDecibelsToLinearGainOperator>;

	METASOUND_REGISTER_NODE(FDecibelsToLinearGainNode)
}

#undef LOCTEXT_NAMESPACE
