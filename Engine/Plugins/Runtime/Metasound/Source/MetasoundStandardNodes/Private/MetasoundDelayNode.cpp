// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Delay.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Delay"


namespace Metasound
{
	namespace DelayNodePrivate
	{
		METASOUND_PARAM(InputResetDelay, "Reset", "Resets the delay buffer.");
		METASOUND_PARAM(InParamAudioInput, "In", "Audio input.")
		METASOUND_PARAM(InParamDelayTime, "Delay Time", "The amount of time to delay the audio.")
		METASOUND_PARAM(InParamDryLevel, "Dry Level", "The dry level of the delay.")
		METASOUND_PARAM(InParamWetLevel, "Wet Level", "The wet level of the delay.")
		METASOUND_PARAM(InParamFeedbackAmount, "Feedback", "Feedback amount.")
		METASOUND_PARAM(InParamMaxDelayTime, "Max Delay Time", "The maximum amount of time to delay the audio.")

		METASOUND_PARAM(OutParamAudio, "Out", "Audio output.")

		static constexpr float MinMaxDelaySeconds = 0.001f;
		static constexpr float MaxMaxDelaySeconds = 1000.f;
		static constexpr float DefaultMaxDelaySeconds = 5.0f;

		template<typename ValueType>
		struct TDelay
		{
			bool bSupported = false;
		};

		template<>
		struct TDelay<FTime>
		{
			static FNodeClassName GetClassName(const FName& InOperatorName)
			{
				// Note the older FTime variant was used StandardNodes::AudioVariant
				return FNodeClassName{ StandardNodes::Namespace, InOperatorName, StandardNodes::AudioVariant };
			}

			static float GetDelayLengthSeconds(TDataReadReference<FTime>& InDelayParam, int32 InFrameIndex = 0)
			{
				return (float)InDelayParam->GetSeconds();
			}

			static TDataReadReference<FTime> CreateInRef(const FBuildOperatorParams& InParams)
			{
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InParamDelayTime), InParams.OperatorSettings);
			}

			static bool IsAudioType() { return false; }
		};

		template<>
		struct TDelay<FAudioBuffer>
		{
			static FNodeClassName GetClassName(const FName& InOperatorName)
			{
				// Even though this is technically the audio variant of the node, we need to pick something different than "audio variant", so pick something very specific
				return FNodeClassName{ "Delay", InOperatorName, "AudioBufferDelayTime" };
			}

			static float GetDelayLengthSeconds(TDataReadReference<FAudioBuffer>& InDelayParam, int32 InFrameIndex = 0)
			{
				if (InFrameIndex < 0 || InFrameIndex >= InDelayParam->Num())
					return 0.0f;

				const float* Data = InDelayParam->GetData();
				// the first sample of the audio buffer is the initial delay length
				return Data[InFrameIndex];
			}

			static TDataReadReference<FAudioBuffer> CreateInRef(const FBuildOperatorParams& InParams)
			{
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InParamDelayTime), InParams.OperatorSettings);
			}

			static bool IsAudioType() { return true; }
		};
	}

	template<typename ValueType>
	class TDelayNodeOperator : public TExecutableOperator<TDelayNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace DelayNodePrivate;

			FDataVertexMetadata MaxDelayTimeMetadata = METASOUND_GET_PARAM_METADATA(InParamMaxDelayTime);
			MaxDelayTimeMetadata.bIsAdvancedDisplay = true;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamAudioInput)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamDelayTime), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamDryLevel), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamWetLevel), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamFeedbackAmount), 0.0f),
					TInputConstructorVertex<FTime>(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), MaxDelayTimeMetadata, DefaultMaxDelaySeconds),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetDelay))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamAudio))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
				{
					FNodeClassMetadata Metadata
					{
						DelayNodePrivate::TDelay<ValueType>::GetClassName(TEXT("Delay")),
						1, // Major Version
						1, // Minor Version
						METASOUND_LOCTEXT_FORMAT("DelayDisplayNamePattern", "Delay ({0})", GetMetasoundDataTypeDisplayText<ValueType>()),
						METASOUND_LOCTEXT("DelayDesc", "Delays an audio buffer by the specified amount."),
						PluginAuthor,
						PluginNodeMissingPrompt,
						GetDefaultInterface(),
						{ NodeCategories::Delays },
						{ },
						FNodeDisplayStyle{}
					};

					return Metadata;
				};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace DelayNodePrivate;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InParamAudioInput), InParams.OperatorSettings);
			TDataReadReference<ValueType> DelayTime = TDelay<ValueType>::CreateInRef(InParams);
			FFloatReadRef DryLevel = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamDryLevel), InParams.OperatorSettings);
			FFloatReadRef WetLevel = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamWetLevel), InParams.OperatorSettings);
			FFloatReadRef Feedback = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamFeedbackAmount), InParams.OperatorSettings);
			FTime MaxDelayTime = InputData.GetOrCreateDefaultValue<FTime>(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), InParams.OperatorSettings);
			FTriggerReadRef TriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputResetDelay), InParams.OperatorSettings);

			return MakeUnique<TDelayNodeOperator<ValueType>>(InParams, AudioIn, DelayTime, DryLevel, WetLevel, Feedback, MaxDelayTime.GetSeconds(), TriggerReset);
		}

		TDelayNodeOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InAudioInput,
			const TDataReadReference<ValueType>& InDelayTime,
			const FFloatReadRef& InDryLevel,
			const FFloatReadRef& InWetLevel,
			const FFloatReadRef& InFeedback,
			float InMaxDelayTimeSeconds,
			const FTriggerReadRef& InTriggerReset)

			: AudioInput(InAudioInput)
			, DelayTime(InDelayTime)
			, DryLevel(InDryLevel)
			, WetLevel(InWetLevel)
			, Feedback(InFeedback)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, TriggerReset(InTriggerReset)
		{
			MaxDelayTimeSeconds = FMath::Clamp(InMaxDelayTimeSeconds, DelayNodePrivate::MinMaxDelaySeconds, DelayNodePrivate::MaxMaxDelaySeconds);

			Reset(InParams);
		}

		virtual ~TDelayNodeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DelayNodePrivate;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamAudioInput), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamDelayTime), DelayTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamDryLevel), DryLevel);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamWetLevel), WetLevel);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamFeedbackAmount), Feedback);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), FTime::FromSeconds(MaxDelayTimeSeconds));
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetDelay), TriggerReset);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DelayNodePrivate;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamAudio), AudioOutput);
		}

		float GetClampedDelayTimeMsec(float InDelayTime) const
		{
			// Clamp the delay time to the max delay allowed
			return 1000.0f * FMath::Clamp(InDelayTime, 0.0f, MaxDelayTimeSeconds);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace DelayNodePrivate;

			FeedbackSample = 0.f;
			PrevDelayTimeMsec = -1.0f;

			// Initialize the delay buffer, but don't set the delay length until first execute
			DelayBuffer.Init(InParams.OperatorSettings.GetSampleRate(), MaxDelayTimeSeconds);
			AudioOutput->Zero();
		}

		void Execute()
		{
			// Wait until our first execute to set the delay length
			// This allows Reset and intialization to propagate through first from inputs
			// Not doing this can cause random pitching as random delay lengths etc interpolate
			// from some unintended initial value 
			if (PrevDelayTimeMsec < 0.0f)
			{
				using namespace DelayNodePrivate;

				float InitialDelayInSeconds = TDelay<ValueType>::GetDelayLengthSeconds(DelayTime);
				PrevDelayTimeMsec = GetClampedDelayTimeMsec(InitialDelayInSeconds);

				DelayBuffer.SetDelayMsec(PrevDelayTimeMsec);
			}

			TriggerReset->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
					ExecuteInternal(StartFrame, EndFrame);
				},
				[&, this](int32 StartFrame, int32 EndFrame)
				{
					FeedbackSample = 0.f;
					DelayBuffer.ResetWithFade();
					ExecuteInternal(StartFrame, EndFrame);
				}
			);
		}

		void ExecuteInternal(int32 StartFrame, int32 EndFrame)
		{
			using namespace DelayNodePrivate;

			const float* InputAudio = AudioInput->GetData();

			float* OutputAudio = AudioOutput->GetData();
			int32 NumFrames = AudioInput->Num();

			// Clamp the feedback amount to make sure it's bounded. Clamp to a number slightly less than 1.0
			float CurrentFeedbackAmount = FMath::Clamp(*Feedback, 0.0f, 1.0f - SMALL_NUMBER);
			float CurrentDryLevel = FMath::Clamp(*DryLevel, 0.0f, 1.0f);
			float CurrentWetLevel = FMath::Clamp(*WetLevel, 0.0f, 1.0f);

			// check if we're doing an audio-rate delay time
			if (TDelay<ValueType>::IsAudioType())
			{
				if (FMath::IsNearlyZero(CurrentFeedbackAmount))
				{
					FeedbackSample = 0.0f;
					for (int32 FrameIndex = StartFrame; FrameIndex < EndFrame; ++FrameIndex)
					{
						DelayBuffer.SetDelayMsec(GetClampedDelayTimeMsec(TDelay<ValueType>::GetDelayLengthSeconds(DelayTime, FrameIndex)));
						OutputAudio[FrameIndex] = CurrentWetLevel * DelayBuffer.ProcessAudioSample(InputAudio[FrameIndex]) + CurrentDryLevel * InputAudio[FrameIndex];
					}
				}
				else
				{
					// There is some amount of feedback so we do the feedback mixing
					for (int32 FrameIndex = StartFrame; FrameIndex < EndFrame; ++FrameIndex)
					{
						DelayBuffer.SetDelayMsec(GetClampedDelayTimeMsec(TDelay<ValueType>::GetDelayLengthSeconds(DelayTime, FrameIndex)));
						OutputAudio[FrameIndex] = CurrentWetLevel * DelayBuffer.ProcessAudioSample(InputAudio[FrameIndex] + FeedbackSample * CurrentFeedbackAmount) + CurrentDryLevel * InputAudio[FrameIndex];
						FeedbackSample = OutputAudio[FrameIndex];
					}
				}
			}
			else // non audio-rate delay
			{
				// Get clamped delay time
				float CurrentInputDelayTime = GetClampedDelayTimeMsec(TDelay<ValueType>::GetDelayLengthSeconds(DelayTime));

				// Check to see if our delay amount has changed
				if (!FMath::IsNearlyEqual(PrevDelayTimeMsec, CurrentInputDelayTime))
				{
					PrevDelayTimeMsec = CurrentInputDelayTime;
					DelayBuffer.SetEasedDelayMsec(PrevDelayTimeMsec);
				}

				if (FMath::IsNearlyZero(CurrentFeedbackAmount))
				{
					FeedbackSample = 0.0f;

					for (int32 FrameIndex = StartFrame; FrameIndex < EndFrame; ++FrameIndex)
					{
						OutputAudio[FrameIndex] = CurrentWetLevel * DelayBuffer.ProcessAudioSample(InputAudio[FrameIndex]) + CurrentDryLevel * InputAudio[FrameIndex];
					}
				}
				else
				{
					// There is some amount of feedback so we do the feedback mixing
					for (int32 FrameIndex = StartFrame; FrameIndex < EndFrame; ++FrameIndex)
					{
						OutputAudio[FrameIndex] = CurrentWetLevel * DelayBuffer.ProcessAudioSample(InputAudio[FrameIndex] + FeedbackSample * CurrentFeedbackAmount) + CurrentDryLevel * InputAudio[FrameIndex];
						FeedbackSample = OutputAudio[FrameIndex];
					}
				}
			}
		}


	private:

		// The input audio buffer
		FAudioBufferReadRef AudioInput;

		// The amount of delay time
		TDataReadReference<ValueType> DelayTime;

		// The dry level
		FFloatReadRef DryLevel;

		// The wet level
		FFloatReadRef WetLevel;

		// The feedback amount
		FFloatReadRef Feedback;

		// The audio output
		FAudioBufferWriteRef AudioOutput;

		// The internal delay buffer
		Audio::FDelay DelayBuffer;

		// The previous delay time
		float PrevDelayTimeMsec = -1.f;

		// Feedback sample
		float FeedbackSample = 0.f;

		// Maximum delay time
		float MaxDelayTimeSeconds = DelayNodePrivate::DefaultMaxDelaySeconds;

		// The reset trigger
		FTriggerReadRef TriggerReset;
	};

	template<typename ValueType>
	using TDelayNode = TNodeFacade<TDelayNodeOperator<ValueType>>;

	using FDelayNode = TDelayNode<FTime>;
	METASOUND_REGISTER_NODE(FDelayNode)

	using FDelayNodeAudio = TDelayNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FDelayNodeAudio)
}

#undef LOCTEXT_NAMESPACE
