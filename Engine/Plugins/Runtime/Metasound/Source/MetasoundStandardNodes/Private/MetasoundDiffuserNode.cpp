// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/LongDelayAPF.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_DiffuserNode"
namespace Metasound
{
	namespace DiffuserNode
	{
		METASOUND_PARAM(InputAudio, "Input Audio", "Incoming Audio");
		METASOUND_PARAM(InputDiffusionDepth, "Depth", "1 - 5: The number of filters to use to diffuse the audio. Will not update while running.");
		METASOUND_PARAM(InputFeedbackGain, "Feedback", "0 - 1: the amount of feedback on each diffuser.");
		METASOUND_PARAM(OutputAudio, "Output Audio", "Diffuse Audio");

		// APF Lengths, as a ratio of samplerate. 
		//These were influenced by the APF lengths used in Flexiverb.cpp, except in ms instead of samples
		static const float APFLengthsMs[] = { 5.3125f, 1.1458f, 11.5833f, 9.1875f, 1.21f, 7.104167f };
		constexpr int32 MaxNumAPFs = 6;

		// max number of samples the APFs will allocate for internal work buffers
		constexpr int32 MaxBufferLengthSamples = 240;
	}

	class METASOUNDSTANDARDNODES_API FDiffuserOperator : public TExecutableOperator<FDiffuserOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Diffuser"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_DiffuserDisplayName", "Diffuser");
				Info.Description = METASOUND_LOCTEXT("Metasound_DiffuserNodeDescription", "Applies diffusion to incoming audio.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy = { NodeCategories::Delays };

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace DiffuserNode;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDiffusionDepth), 4),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFeedbackGain), 0.707f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace DiffuserNode;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FInt32ReadRef DepthIn = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputDiffusionDepth), InParams.OperatorSettings);
			FFloatReadRef FeedbackIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputFeedbackGain), InParams.OperatorSettings);

			return MakeUnique<FDiffuserOperator>(InParams.OperatorSettings, AudioIn, DepthIn, FeedbackIn);
		}

		FDiffuserOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FInt32ReadRef& InDepth,
			const FFloatReadRef& InFeedback)
			: AudioInput(InAudioInput)
			, Depth(InDepth)
			, Feedback(InFeedback)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		{
			using namespace DiffuserNode;

			ClampedDepth = FMath::Clamp(*InDepth, 1, MaxNumAPFs);

			const float FeedbackGain = FMath::Clamp(*InFeedback, 0.0f, 1.0f - KINDA_SMALL_NUMBER);
			const float SampleRatio =  InSettings.GetSampleRate() / 1000.f;

			Delays.Reset(ClampedDepth);

			for (int32 FilterIdx = 0; FilterIdx < ClampedDepth; ++FilterIdx)
			{
				// This is all because FLongDelayAPF has a TUniquePtr<FAlignedBlockBuffer> and can't be copied instantiated directly into an array
				TUniquePtr<Audio::FLongDelayAPF>& NewDelay = GetDelayFromIndex(FilterIdx);
				NewDelay = MakeUnique<Audio::FLongDelayAPF>(FeedbackGain, SampleRatio * APFLengthsMs[FilterIdx], MaxBufferLengthSamples);

				Delays.Add(NewDelay.Get());
			}
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DiffuserNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDiffusionDepth), Depth);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFeedbackGain), Feedback);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DiffuserNode;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			using namespace DiffuserNode;

			for (Audio::FLongDelayAPF* Delay : Delays)
			{
				Delay->Reset();
			}
			AudioOutput->Zero();
		}

		void Execute()
		{
			const int32 NumSamples = AudioInput->Num();

			FMemory::Memcpy(AudioOutput->GetData(), AudioInput->GetData(), NumSamples * sizeof(float));

			const float FeedbackClamped = FMath::Clamp(*Feedback, 0.0f, 1.0f - KINDA_SMALL_NUMBER);

			for (Audio::FLongDelayAPF* Delay : Delays)
			{
				Delay->SetG(FeedbackClamped);
				Delay->ProcessAudio(*AudioOutput);
			}
		}

	private:
		// The input audio buffer
		FAudioBufferReadRef AudioInput;

		// The amount that the wave is shaped
		FInt32ReadRef Depth;

		int32 ClampedDepth = 1;

		// DC offset to apply before gain
		FFloatReadRef Feedback;

		// The audio output
		FAudioBufferWriteRef AudioOutput;

		TArray<Audio::FLongDelayAPF*> Delays;

		// Can't hold the unique ptrs of this directly in an array because of hidden copy constructors
		TUniquePtr<Audio::FLongDelayAPF> Delay_0;
		TUniquePtr<Audio::FLongDelayAPF> Delay_1;
		TUniquePtr<Audio::FLongDelayAPF> Delay_2;
		TUniquePtr<Audio::FLongDelayAPF> Delay_3;
		TUniquePtr<Audio::FLongDelayAPF> Delay_4;
		TUniquePtr<Audio::FLongDelayAPF> Delay_5;

		FORCEINLINE TUniquePtr<Audio::FLongDelayAPF>& GetDelayFromIndex(int32 Idx)
		{
			switch (Idx)
			{
			case 0:
				return Delay_0;
			case 1:
				return Delay_1;
			case 2:
				return Delay_2;
			case 3:
				return Delay_3;
			case 4:
				return Delay_4;
			case 5:
				return Delay_5;
			default:
				ensure(false);
			}

			return Delay_0;
		}
	};

	using FDiffuserNode = TNodeFacade<FDiffuserOperator>;

	METASOUND_REGISTER_NODE(FDiffuserNode)
}

#undef LOCTEXT_NAMESPACE
