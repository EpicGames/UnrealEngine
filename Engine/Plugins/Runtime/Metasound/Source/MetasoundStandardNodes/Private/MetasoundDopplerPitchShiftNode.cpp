// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/TapDelayPitchShifter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodesDopplerPitchShift"

namespace Metasound
{
	namespace DopplerPitchShift
	{
		METASOUND_PARAM(InParamAudioInput, "In", "Input audio buffer.")
		METASOUND_PARAM(InParamPitchShift, "Pitch Shift", "The amount to pitch shift the audio signal, in semitones.")
		METASOUND_PARAM(InParamDelayLength, "Delay Length", "The delay length of the internal delay buffer in milliseconds (10 ms to 100 ms). Changing this can reduce artifacts in certain pitch shift regions.")
		METASOUND_PARAM(OutParamAudio, "Out", "Output audio buffer.")
	}

	class FDopplerPitchShiftOperator : public TExecutableOperator<FDopplerPitchShiftOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FDopplerPitchShiftOperator(const FBuildOperatorParams& InParams, const FAudioBufferReadRef& InAudioInput, const FFloatReadRef& InPitchShift, const FFloatReadRef& InDelayLength);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		FAudioBufferReadRef AudioInput;
		FFloatReadRef PitchShift;
		FFloatReadRef DelayLength;
		FAudioBufferWriteRef AudioOutput;
		Audio::FDelay DelayBuffer;
		Audio::FTapDelayPitchShifter DelayPitchShifter;
	};

	FDopplerPitchShiftOperator::FDopplerPitchShiftOperator(const FBuildOperatorParams& InParams,
		const FAudioBufferReadRef& InAudioInput,
		const FFloatReadRef& InPitchShift,
		const FFloatReadRef& InDelayLength)

		: AudioInput(InAudioInput)
		, PitchShift(InPitchShift)
		, DelayLength(InDelayLength)
		, AudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	void FDopplerPitchShiftOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace DopplerPitchShift;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamAudioInput), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamPitchShift), PitchShift);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamDelayLength), DelayLength);
	}

	void FDopplerPitchShiftOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace DopplerPitchShift;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamAudio), AudioOutput);
	}

	void FDopplerPitchShiftOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioOutput->Zero();
		DelayBuffer.Init(InParams.OperatorSettings.GetSampleRate(), 0.001f * Audio::FTapDelayPitchShifter::MaxDelayLength);
		DelayPitchShifter.Init(InParams.OperatorSettings.GetSampleRate(), *PitchShift, *DelayLength);
	}

	void FDopplerPitchShiftOperator::Execute()
	{
		DelayPitchShifter.SetDelayLength(*DelayLength);
		DelayPitchShifter.SetPitchShift(*PitchShift);
		DelayPitchShifter.ProcessAudio(DelayBuffer, AudioInput->GetData(), AudioInput->Num(), AudioOutput->GetData());
	}

	const FVertexInterface& FDopplerPitchShiftOperator::GetVertexInterface()
	{
		using namespace DopplerPitchShift;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamPitchShift), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamDelayLength), 30.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamAudio))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FDopplerPitchShiftOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, "Delay Pitch Shift", StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("DopperDelayPitchShift_DisplayName", "Delay Pitch Shift");
			Info.Description = METASOUND_LOCTEXT("DopperDelayPitchShift_Description", "Pitch shifts the audio buffer using a delay-based doppler-shift method.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Delays);
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FDopplerPitchShiftOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace DopplerPitchShift;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InParamAudioInput), InParams.OperatorSettings);
		FFloatReadRef PitchShift = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamPitchShift), InParams.OperatorSettings);
		FFloatReadRef DelayLength = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamDelayLength), InParams.OperatorSettings);

		return MakeUnique<FDopplerPitchShiftOperator>(InParams, AudioIn, PitchShift, DelayLength);
	}

	using FDopperPitchShiftNode = TNodeFacade<FDopplerPitchShiftOperator>;

	METASOUND_REGISTER_NODE(FDopperPitchShiftNode)
}

#undef LOCTEXT_NAMESPACE
