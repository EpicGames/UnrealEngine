// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DynamicStateVariableFilter.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_DynamicFilter"

namespace Metasound
{
	namespace DynamicFilterNode
	{
		METASOUND_PARAM(InputAudio, "Audio", "Incoming audio signal.");
		METASOUND_PARAM(InputSidechain, "Sidechain", "(Optional) External audio signal to control the filter with. If empty, uses the input audio signal.");
		METASOUND_PARAM(InputFilterType, "FilterType", "Filter shape to use.");
		METASOUND_PARAM(InputFrequency, "Frequency", "The center frequency of the filter.");
		METASOUND_PARAM(InputQ, "Q", "Filter Q, or resonance, controls the steepness of the filter.");
		METASOUND_PARAM(InputThreshold, "Threshold dB", "Amplitude threshold (dB) above which gain will be reduced.");
		METASOUND_PARAM(InputRatio, "Ratio", "Amount of gain reduction. 1 = no reduction, higher = more reduction.");
		METASOUND_PARAM(InputKnee, "Knee", "How hard or soft the gain reduction blends from no gain reduction to gain reduction. 0 dB, no blending.");
		METASOUND_PARAM(InputRange, "Range", "The maximum gain reduction (dB) allowed. Negative values apply compression, positive values flip it into an expander.");
		METASOUND_PARAM(InputGain, "Gain", "Amount of make-up gain to apply");
		METASOUND_PARAM(InputAttackTime, "AttackTime", "How long it takes for audio above the threshold to reach its compressed volume level.");
		METASOUND_PARAM(InputReleaseTime, "ReleaseTime", "How long it takes for audio below the threshold to return to its original volume level.");
		METASOUND_PARAM(InputEnvelopeMode, "EnvelopeMode", "The envelope-following method the compressor will use for gain detection.");
		METASOUND_PARAM(InputAnalogMode, "AnalogMode", "Enable Analog Mode for the compressor's envelope follower.");

		METASOUND_PARAM(OutputAudio, "Audio", "The output audio signal.");
	}

	DECLARE_METASOUND_ENUM(
		Audio::EDynamicFilterType, 
		Audio::EDynamicFilterType::Bell, 
		METASOUNDSTANDARDNODES_API, 
		FEnumEDynamicFilterType, 
		FEnumEDynamicFilterTypeInfo,
		FEnumEDynamicFilterReadRef,
		FEnumEDynamicFilterWriteRef
	);

	DEFINE_METASOUND_ENUM_BEGIN(Audio::EDynamicFilterType, FEnumEDynamicFilterType, "DynamicFilterType")
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EDynamicFilterType::Bell, "BellDescription", "Bell", "BellTT", "Bell Filter"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EDynamicFilterType::LowShelf, "LowShelfDescription", "Low Shelf", "LowShelfTT", "Low Shelf Filter"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EDynamicFilterType::HighShelf, "HighShelfDescription", "High Shelf", "HighShelfTT", "High Shelf Filter"),
	DEFINE_METASOUND_ENUM_END()

	class FDynamicFilterOperator : public TExecutableOperator<FDynamicFilterOperator>
	{
	public:
		FDynamicFilterOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FAudioBufferReadRef& InSidechainInput,
			const FEnumEDynamicFilterReadRef& InFilterType,
			const FFloatReadRef& InFrequency,
			const FFloatReadRef& InQ,
			const FFloatReadRef& InThresholdDB,
			const FFloatReadRef& InRatio,
			const FFloatReadRef& InKneeDB,
			const FFloatReadRef& InRange,
			const FFloatReadRef& InGainDB,
			const FTimeReadRef& InAttackTime,
			const FTimeReadRef& InReleaseTime,
			const FEnvelopePeakModeReadRef& InEnvelopeMode,
			const FBoolReadRef& InAnalogMode,
			const bool& bInUseSidechain
			)
			: AudioInput(InAudioInput)
			, SidechainInput(InSidechainInput)
			, FilterType(InFilterType)
			, Frequency(InFrequency)
			, Q(InQ)
			, ThresholdDB(InThresholdDB)
			, Ratio(InRatio)
			, KneeDB(InKneeDB)
			, Range(InRange)
			, GainDB(InGainDB)
			, AttackTime(InAttackTime)
			, ReleaseTime(InReleaseTime)
			, EnvelopeMode(InEnvelopeMode)
			, bAnalogMode(InAnalogMode)
			, bUseSidechain(bInUseSidechain)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		{
			Filter.Init(InSettings.GetSampleRate(), 1);
			UpdateFilterSettings();
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "DynamicFilter", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("DynamicFilterDisplayName", "Dynamic Filter"),
					METASOUND_LOCTEXT("DynamicFilterDesc", "Filters a band of audio based on the strength of the input signal."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Filters },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DynamicFilterNode;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSidechain), SidechainInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFilterType), FilterType);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFrequency), Frequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputQ), Q);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputThreshold), ThresholdDB);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRatio), Ratio);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputKnee), KneeDB);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRange), Range);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputGain), GainDB);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAttackTime), AttackTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseTime), ReleaseTime);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEnvelopeMode), EnvelopeMode);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAnalogMode), bAnalogMode);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace DynamicFilterNode;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace DynamicFilterNode;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSidechain)),
					TInputDataVertex<FEnumEDynamicFilterType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFilterType)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFrequency), 1000.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputQ), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputThreshold), -12.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputRatio), 4.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputKnee), 12.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputRange), -60.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(InputGain),
					{ 
						METASOUND_GET_PARAM_TT(InputGain),
						METASOUND_LOCTEXT("InputGainName", "Gain (dB)")
					}, 0.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAttackTime), 0.01f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseTime), 0.1f),
					TInputDataVertex<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEnvelopeMode), (int32)EEnvelopePeakMode::MeanSquared),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAnalogMode), true)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace DynamicFilterNode;

			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FAudioBufferReadRef SidechainIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputSidechain), InParams.OperatorSettings);
			FEnumEDynamicFilterReadRef FilterTypeIn = InputData.GetOrCreateDefaultDataReadReference<FEnumEDynamicFilterType>(METASOUND_GET_PARAM_NAME(InputFilterType), InParams.OperatorSettings);
			FFloatReadRef FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputFrequency), InParams.OperatorSettings);
			FFloatReadRef QIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputQ), InParams.OperatorSettings);
			FFloatReadRef ThresholdIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputThreshold), InParams.OperatorSettings);
			FFloatReadRef RatioIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputRatio), InParams.OperatorSettings);
			FFloatReadRef KneeIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputKnee), InParams.OperatorSettings);
			FFloatReadRef RangeIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputRange), InParams.OperatorSettings);
			FFloatReadRef GainIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputGain), InParams.OperatorSettings);
			FTimeReadRef AttackTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputAttackTime), InParams.OperatorSettings);
			FTimeReadRef ReleaseTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputReleaseTime), InParams.OperatorSettings);
			FEnvelopePeakModeReadRef EnvelopeModeIn = InputData.GetOrCreateDefaultDataReadReference<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME(InputEnvelopeMode), InParams.OperatorSettings);
			FBoolReadRef bAnalogIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputAnalogMode), InParams.OperatorSettings);

			bool bIsSidechainConnected = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(InputSidechain));

			return MakeUnique<FDynamicFilterOperator>(InParams.OperatorSettings, AudioIn, SidechainIn, FilterTypeIn, FrequencyIn, QIn, ThresholdIn, RatioIn, KneeIn, RangeIn, GainIn, AttackTimeIn, ReleaseTimeIn, EnvelopeModeIn, bAnalogIn, bIsSidechainConnected);
		}

		void UpdateFilterSettings()
		{
			Filter.SetFrequency(*Frequency);
			Filter.SetQ(*Q);
			Filter.SetDynamicRange(*Range);
			Filter.SetRatio(*Ratio);
			Filter.SetThreshold(*ThresholdDB);
			Filter.SetAttackTime(FTime::ToMilliseconds(*AttackTime));
			Filter.SetReleaseTime(FTime::ToMilliseconds(*ReleaseTime));
			Filter.SetKnee(*KneeDB);
			Filter.SetGain(*GainDB);
			Filter.SetAnalog(*bAnalogMode);

			Filter.SetFilterType(*FilterType);

			switch (*EnvelopeMode)
			{
			default:
			case EEnvelopePeakMode::MeanSquared:
				Filter.SetEnvMode(Audio::EPeakMode::MeanSquared);
				break;

			case EEnvelopePeakMode::RootMeanSquared:
				Filter.SetEnvMode(Audio::EPeakMode::RootMeanSquared);
				break;

			case EEnvelopePeakMode::Peak:
				Filter.SetEnvMode(Audio::EPeakMode::Peak);
				break;
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOutput->Zero();
			Filter.Reset();
			UpdateFilterSettings();
		}

		void Execute()
		{
			const float* InputAudio = AudioInput->GetData();

			float* OutputAudio = AudioOutput->GetData();
			int32 NumFrames = AudioInput->Num();

			UpdateFilterSettings();

			if (bUseSidechain)
			{
				Filter.ProcessAudio(InputAudio, OutputAudio, SidechainInput->GetData(), NumFrames);
			}
			else
			{
				Filter.ProcessAudio(InputAudio, OutputAudio, NumFrames);
			}
		}

	private:
		FAudioBufferReadRef AudioInput;
		FAudioBufferReadRef SidechainInput;
		FEnumEDynamicFilterReadRef FilterType;
		FFloatReadRef Frequency;
		FFloatReadRef Q;
		FFloatReadRef ThresholdDB;
		FFloatReadRef Ratio;
		FFloatReadRef KneeDB;
		FFloatReadRef Range;
		FFloatReadRef GainDB;
		FTimeReadRef AttackTime;
		FTimeReadRef ReleaseTime;
		FEnvelopePeakModeReadRef EnvelopeMode;
		FBoolReadRef bAnalogMode;

		bool bUseSidechain = false;

		FAudioBufferWriteRef AudioOutput;

		Audio::FDynamicStateVariableFilter Filter;
	};

	// Node Class
	using FDynamicFilterNode = TNodeFacade<FDynamicFilterOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FDynamicFilterNode)
}

#undef LOCTEXT_NAMESPACE
