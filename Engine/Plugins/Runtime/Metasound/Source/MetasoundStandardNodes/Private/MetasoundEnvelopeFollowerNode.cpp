// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnvelopeFollowerNode.h"

#include "Algo/MaxElement.h"
#include "DSP/EnvelopeFollower.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnvelopeFollowerTypes.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_EnvelopeFollower"


namespace Metasound
{
	namespace EnvelopeFollowerVertexNames
	{
		METASOUND_PARAM(InParamEnable, "Enable", "Enable the envelope follower.")
		METASOUND_PARAM(InParamAudioInput, "In", "Audio input.")
		METASOUND_PARAM(InParamAttackTime, "Attack Time", "The attack time of the envelope follower.")
		METASOUND_PARAM(InParamReleaseTime,"Release Time" , "The release time of the envelope follower.")
		METASOUND_PARAM(InParamFollowMode, "Peak Mode", "The following-method of the envelope follower.")
		METASOUND_PARAM(OutParamEnvelope, "Envelope", "The output envelope value of the audio signal.")
		METASOUND_PARAM(OutputAudioEnvelope, "Audio Envelope", "The output envelope value of the audio signal (audio rate).");
	}

	class FEnvelopeFollowerOperator : public TExecutableOperator<FEnvelopeFollowerOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FEnvelopeFollowerOperator(const FBuildOperatorParams& InOperatorSettings,
			const FBoolReadRef& InEnable,
			const FAudioBufferReadRef& InAudioInput,
			const FTimeReadRef& InAttackTime,
			const FTimeReadRef& InReleaseTime,
			const FEnvelopePeakModeReadRef& InEnvelopeMode);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// Whether the enveloper follower is enabled
		FBoolReadRef EnableInput;
		bool OutputNeedsResetOnDisable = true;
		
		// The input audio buffer
		FAudioBufferReadRef AudioInput;

		// The amount of attack time
		FTimeReadRef AttackTimeInput;

		// The amount of release time
		FTimeReadRef ReleaseTimeInput;

		// The Envelope-Following method
		FEnvelopePeakModeReadRef FollowModeInput;

		// The envelope outputs
		FFloatWriteRef EnvelopeFloatOutput;
		FAudioBufferWriteRef EnvelopeAudioOutput;

		// The envelope follower DSP object
		Audio::FEnvelopeFollower EnvelopeFollower;

		double PrevAttackTime = 0.0;
		double PrevReleaseTime = 0.0;
		EEnvelopePeakMode PrevFollowMode = EEnvelopePeakMode::Peak;
	};

	FEnvelopeFollowerOperator::FEnvelopeFollowerOperator(const FBuildOperatorParams& InParams,
		const FBoolReadRef& InEnable,
		const FAudioBufferReadRef& InAudioInput,
		const FTimeReadRef& InAttackTime,
		const FTimeReadRef& InReleaseTime,
		const FEnvelopePeakModeReadRef& InEnvelopeMode)
		: EnableInput(InEnable)
		, AudioInput(InAudioInput)
		, AttackTimeInput(InAttackTime)
		, ReleaseTimeInput(InReleaseTime)
		, FollowModeInput(InEnvelopeMode)
		, EnvelopeFloatOutput(FFloatWriteRef::CreateNew())
		, EnvelopeAudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
	{
		Reset(InParams);
	}

	void FEnvelopeFollowerOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace EnvelopeFollowerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamEnable), EnableInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamAudioInput), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamAttackTime), AttackTimeInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamReleaseTime), ReleaseTimeInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamFollowMode), FollowModeInput);
	}

	void FEnvelopeFollowerOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace EnvelopeFollowerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamEnvelope), EnvelopeFloatOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioEnvelope), EnvelopeAudioOutput);
	}

	void FEnvelopeFollowerOperator::Reset(const IOperator::FResetParams& InParams)
	{
		OutputNeedsResetOnDisable = true;
		
		PrevAttackTime = FMath::Max(FTime::ToMilliseconds(*AttackTimeInput), 0.0);
		PrevReleaseTime = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0);

		Audio::FEnvelopeFollowerInitParams EnvelopeParamsInitParams;
	
		EnvelopeParamsInitParams.SampleRate = InParams.OperatorSettings.GetSampleRate();
		EnvelopeParamsInitParams.NumChannels = 1;
		EnvelopeParamsInitParams.AttackTimeMsec = PrevAttackTime;
		EnvelopeParamsInitParams.ReleaseTimeMsec = PrevReleaseTime;

		EnvelopeFollower.Init(EnvelopeParamsInitParams);
		*EnvelopeFloatOutput = 0.f;
		EnvelopeAudioOutput->Zero();
	}

	void FEnvelopeFollowerOperator::Execute()
	{
		// Skip rendering if disabled
		if (!*EnableInput)
		{
			if (OutputNeedsResetOnDisable)
			{
				EnvelopeAudioOutput->Zero();
				*EnvelopeFloatOutput = 0;
				OutputNeedsResetOnDisable = false;
			}
			
			return;
		}

		OutputNeedsResetOnDisable = true;
		
		// Check for any input changes
		double CurrentAttackTime = FMath::Max(FTime::ToMilliseconds(*AttackTimeInput), 0.0);
		if (!FMath::IsNearlyEqual(CurrentAttackTime, PrevAttackTime))
		{
			PrevAttackTime = CurrentAttackTime;
			EnvelopeFollower.SetAttackTime(CurrentAttackTime);
		}

		double CurrentReleaseTime = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0);
		if (!FMath::IsNearlyEqual(CurrentReleaseTime, PrevReleaseTime))
		{
			PrevReleaseTime = CurrentReleaseTime;
			EnvelopeFollower.SetReleaseTime(CurrentReleaseTime);
		}

		if (PrevFollowMode != *FollowModeInput)
		{
			PrevFollowMode = *FollowModeInput;
			switch (PrevFollowMode)
			{
			case EEnvelopePeakMode::MeanSquared:
			default:
				EnvelopeFollower.SetMode(Audio::EPeakMode::Type::MeanSquared);
				break;
			case EEnvelopePeakMode::RootMeanSquared:
				EnvelopeFollower.SetMode(Audio::EPeakMode::Type::RootMeanSquared);
				break;
			case EEnvelopePeakMode::Peak:
				EnvelopeFollower.SetMode(Audio::EPeakMode::Type::Peak);
				break;
			}
		}

		// Process the audio through the envelope follower
		EnvelopeFollower.ProcessAudio(AudioInput->GetData(), AudioInput->Num(), EnvelopeAudioOutput->GetData());
		if (const float* MaxElement = Algo::MaxElement(EnvelopeFollower.GetEnvelopeValues()))
		{
			*EnvelopeFloatOutput = *MaxElement;
		}
		else
		{
			*EnvelopeFloatOutput = 0.f;
		}
	}

	const FVertexInterface& FEnvelopeFollowerOperator::GetVertexInterface()
	{
		using namespace EnvelopeFollowerVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamEnable), true),
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamAudioInput)),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamAttackTime), 0.01f),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamReleaseTime), 0.1f),
				TInputDataVertex<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamFollowMode), (int32)EEnvelopePeakMode::Peak)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamEnvelope)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioEnvelope))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FEnvelopeFollowerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Envelope Follower"), TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 3;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_EnvelopeFollowerDisplayName", "Envelope Follower");
			Info.Description = METASOUND_LOCTEXT("Metasound_EnvelopeFollowerDescription", "Outputs an envelope from an input audio signal.");
			Info.Author = PluginAuthor;
			Info.CategoryHierarchy = { NodeCategories::Envelopes };
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FEnvelopeFollowerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace EnvelopeFollowerVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FBoolReadRef EnableIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InParamEnable), InParams.OperatorSettings);
		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InParamAudioInput), InParams.OperatorSettings);
		FTimeReadRef AttackTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InParamAttackTime), InParams.OperatorSettings);
		FTimeReadRef ReleaseTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InParamReleaseTime), InParams.OperatorSettings);
		FEnvelopePeakModeReadRef EnvelopeModeIn = InputData.GetOrCreateDefaultDataReadReference<FEnumEnvelopePeakMode>(METASOUND_GET_PARAM_NAME(InParamFollowMode), InParams.OperatorSettings);

		return MakeUnique<FEnvelopeFollowerOperator>(InParams, EnableIn, AudioIn, AttackTime, ReleaseTime, EnvelopeModeIn);
	}

	FEnvelopeFollowerNode::FEnvelopeFollowerNode(const FNodeInitData& InitData)
	: FEnvelopeFollowerNode(FNodeData(InitData.InstanceName, InitData.InstanceID, FEnvelopeFollowerOperator::GetNodeInfo().DefaultInterface), MakeShared<const FNodeClassMetadata>(FEnvelopeFollowerOperator::GetNodeInfo()))
	{
	}

	FEnvelopeFollowerNode::FEnvelopeFollowerNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
	: FNodeFacade(MoveTemp(InNodeData), MoveTemp(InClassMetadata), TFacadeOperatorClass<FEnvelopeFollowerOperator>())
	{
	}

	FNodeClassMetadata FEnvelopeFollowerNode::CreateNodeClassMetadata()
	{
		return FEnvelopeFollowerOperator::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FEnvelopeFollowerNode)
}

#undef LOCTEXT_NAMESPACE
