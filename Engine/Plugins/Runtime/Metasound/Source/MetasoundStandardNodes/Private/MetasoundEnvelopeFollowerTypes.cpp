// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnvelopeFollowerTypes.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_EnvelopeFollower"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(EEnvelopePeakMode, FEnumEnvelopePeakMode, "EnvelopePeakMode")
		DEFINE_METASOUND_ENUM_ENTRY(EEnvelopePeakMode::MeanSquared, "EnvelopePeakModeMSDescription", "MS", "EnvelopePeakModeMSDescriptionTT", "Envelope follows a running Mean Squared of the audio signal."),
		DEFINE_METASOUND_ENUM_ENTRY(EEnvelopePeakMode::RootMeanSquared, "EnvelopePeakModeRMSDescription", "RMS", "EnvelopePeakModeRMSDescriptionTT", "Envelope follows a running Root Mean Squared of the audio signal."),
		DEFINE_METASOUND_ENUM_ENTRY(EEnvelopePeakMode::Peak, "EnvelopePeakModePeakDescription", "Peak", "EnvelopePeakModePeakDescriptionTT", "Envelope follows the peaks in the audio signal."),
		DEFINE_METASOUND_ENUM_END()
}

#undef LOCTEXT_NAMESPACE