// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Dsp.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "Metasound_ConvertQToBandwidth"

namespace Metasound
{
	namespace QToBandwidth
	{
		METASOUND_PARAM(InputQ, "Q", "Q parameter for filter control.");
		METASOUND_PARAM(OutputBandwidth, "Bandwidth", "Resulting bandwidth value.");
	}

	class FConvertQToBandwidthNodeOperator : public TExecutableOperator<FConvertQToBandwidthNodeOperator>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace QToBandwidth;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputQ), 1.f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputBandwidth))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "Convert Filter Q To Bandwidth", "" },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("ConvertQToBandwidthNode_Name", "Filter Q To Bandwidth"),
					METASOUND_LOCTEXT("ConvertQToBandwidthNode_Description", "Converts Filter Q To Bandwidth."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ NodeCategories::Math },
					{ },
					{ }
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace QToBandwidth;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FFloatReadRef FloatInput = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputQ), InParams.OperatorSettings);

			return MakeUnique<FConvertQToBandwidthNodeOperator>(InParams.OperatorSettings, FloatInput);
		}

		FConvertQToBandwidthNodeOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InInValue)
			: InValue(InInValue)
			, OutValue(TDataWriteReferenceFactory<float>::CreateAny(InSettings))
		{
			Execute();
		}

		virtual ~FConvertQToBandwidthNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace QToBandwidth;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputQ), InValue);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace QToBandwidth;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputBandwidth), OutValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			*OutValue = Audio::GetBandwidthFromQ(FMath::Max(*InValue, KINDA_SMALL_NUMBER));
		}

		void Execute()
		{
			*OutValue = Audio::GetBandwidthFromQ(FMath::Max(*InValue, KINDA_SMALL_NUMBER));
		}

	private:
		FFloatReadRef InValue;
		FFloatWriteRef OutValue;
	};

	using FConvertQToBandwidthNode = TNodeFacade<FConvertQToBandwidthNodeOperator>;

	METASOUND_REGISTER_NODE(FConvertQToBandwidthNode)
} 

#undef LOCTEXT_NAMESPACE
