// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BufferVectorOperations.h"
#include "DSP/Flanger.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_FlangerNode"


namespace Metasound
{
	namespace FlangerVertexNames
	{
		METASOUND_PARAM(InputAudio, "In Audio", "The audio input.");
		METASOUND_PARAM(InputModulationRate, "Modulation Rate", "The LFO frequency (rate) that varies the delay time, in Hz. Clamped at blockrate.");
		METASOUND_PARAM(InputModulationDepth, "Modulation Depth", "The LFO amplitude (strength) that scales the delay time.");
		METASOUND_PARAM(InputCenterDelay, "Center Delay", "The center delay amount (in milliseconds).");
		METASOUND_PARAM(InputMixLevel, "Mix Level", "Balance between original and delayed signal (Should be between 0 and 1.0; 0.5 is equal amounts of each and > 0.5 is more delayed signal than non-delayed signal).");
		
		METASOUND_PARAM(OutputAudio, "Out Audio", "Output audio with flanger effect applied.");
	}

	class FFlangerOperator : public TExecutableOperator<FFlangerOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				const FName OperatorName = TEXT("Flanger");
				const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_FlangerNodeDisplayName", "Flanger");
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_FlangerNodeDescription", "Applies a flanger effect to input audio.");

				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = NodeDescription;
				Info.Author = PluginAuthor;
				Info.CategoryHierarchy = { NodeCategories::Filters };
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace FlangerVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)), 
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputModulationRate), 0.5f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputModulationDepth), 0.5f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputCenterDelay), 0.5f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMixLevel), 0.5f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace FlangerVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioInput = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FFloatReadRef ModulationRate = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputModulationRate), InParams.OperatorSettings);
			FFloatReadRef ModulationDepth = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputModulationDepth), InParams.OperatorSettings);
			FFloatReadRef CenterDelay = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputCenterDelay), InParams.OperatorSettings);
			FFloatReadRef MixLevel = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputMixLevel), InParams.OperatorSettings);

			return MakeUnique<FFlangerOperator>(InParams, AudioInput, ModulationRate, ModulationDepth, CenterDelay, MixLevel);
		}

		FFlangerOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InAudioInput,
			const FFloatReadRef& InputModulationRate,
			const FFloatReadRef& InputModulationDepth,
			const FFloatReadRef& InputCenterDelay,
			const FFloatReadRef& InMixLevel)

			: AudioIn(InAudioInput)
			, ModulationRate(InputModulationRate)
			, ModulationDepth(InputModulationDepth)
			, CenterDelay(InputCenterDelay)
			, MixLevel(InMixLevel)
			, AudioOut(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, Flanger()
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace FlangerVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputModulationRate), ModulationRate);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputModulationDepth), ModulationDepth);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCenterDelay), CenterDelay);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMixLevel), MixLevel);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace FlangerVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOut);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOut->Zero();
			Flanger.Init(InParams.OperatorSettings.GetSampleRate());
		}

		void Execute()
		{
			// Update flanger parameters if necessary 
			Flanger.SetModulationRate(*ModulationRate);
			Flanger.SetCenterDelay(*CenterDelay);
			Flanger.SetModulationDepth(*ModulationDepth);
			Flanger.SetMixLevel(*MixLevel);

			int32 NumFrames = AudioIn->Num();

			Flanger.ProcessAudio(*AudioIn, NumFrames, *AudioOut);
		}

	private:
		// The input audio buffer
		FAudioBufferReadRef AudioIn;

		// LFO parameters
		FFloatReadRef ModulationRate;
		FFloatReadRef ModulationDepth;
		FFloatReadRef CenterDelay;

		// Balance between original and delayed signal 
		// (Should be between 0 and 1.0; 
		// 0.5 is equal amounts of each and 
		// > 0.5 is more delayed signal than non-delayed signal)
		FFloatReadRef MixLevel;

		// Audio output
		FAudioBufferWriteRef AudioOut;

		// Flanger DSP object
		Audio::FFlanger Flanger;
	};

	using FFlangerNode = TNodeFacade<FFlangerOperator>;

	METASOUND_REGISTER_NODE(FFlangerNode)
}

#undef LOCTEXT_NAMESPACE
