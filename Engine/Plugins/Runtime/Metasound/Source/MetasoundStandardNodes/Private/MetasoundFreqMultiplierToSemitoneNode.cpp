// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_FreqMultiplierToSemitone"

namespace Metasound
{

	namespace FrequencyMultiplierToSemitoneVertexNames
	{
		METASOUND_PARAM(InputFrequencyMultiplier, "Frequency Multiplier", "Input frequency multiplier.");

		METASOUND_PARAM(OutputSemitone, "Semitones", "Output corresponding difference in semitones.");
	}


	class FFrequencyMultiplierToSemitoneOperator : public TExecutableOperator<FFrequencyMultiplierToSemitoneOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FFrequencyMultiplierToSemitoneOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InFrequencyMultiplier);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The input frequency multiplier
		FFloatReadRef FrequencyMultiplierInput;

		// The output difference in semitones
		FFloatWriteRef SemitoneOutput;

		// Cached Frequency Multiplier value. Used to catch if the value changes to recompute semitone output.
		float PrevFrequencyMultiplier;
	};

	FFrequencyMultiplierToSemitoneOperator::FFrequencyMultiplierToSemitoneOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InFrequencyMultiplier)
		: FrequencyMultiplierInput(InFrequencyMultiplier)
		, SemitoneOutput(FFloatWriteRef::CreateNew(Audio::GetSemitones(*InFrequencyMultiplier)))
		, PrevFrequencyMultiplier(*InFrequencyMultiplier)
	{
	}


	void FFrequencyMultiplierToSemitoneOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace FrequencyMultiplierToSemitoneVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFrequencyMultiplier), FrequencyMultiplierInput);
	}

	void FFrequencyMultiplierToSemitoneOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace FrequencyMultiplierToSemitoneVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputSemitone), SemitoneOutput);
	}

	void FFrequencyMultiplierToSemitoneOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PrevFrequencyMultiplier = *FrequencyMultiplierInput;
		*SemitoneOutput = Audio::GetSemitones(*FrequencyMultiplierInput);
	}

	void FFrequencyMultiplierToSemitoneOperator::Execute()
	{
		// Only do anything if the scale changes
		if (!FMath::IsNearlyEqual(*FrequencyMultiplierInput, PrevFrequencyMultiplier))
		{
			PrevFrequencyMultiplier = *FrequencyMultiplierInput;
			*SemitoneOutput = Audio::GetSemitones(*FrequencyMultiplierInput);
		}
	}

	const FVertexInterface& FFrequencyMultiplierToSemitoneOperator::GetVertexInterface()
	{
		using namespace FrequencyMultiplierToSemitoneVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFrequencyMultiplier), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputSemitone))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FFrequencyMultiplierToSemitoneOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<float>();
			const FName OperatorName = TEXT("Frequency Multiplier to Semitone");
			const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_FrequencyMultiplierToSemitoneName", "Frequency Multiplier to Semitone");
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_FrequencyMultiplierToSemitoneDescription", "Converts a frequency multiplier to the corresponding difference in semitones.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Conversions);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FFrequencyMultiplierToSemitoneOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace FrequencyMultiplierToSemitoneVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InFrequencyMultiplier = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputFrequencyMultiplier), InParams.OperatorSettings);

		return MakeUnique<FFrequencyMultiplierToSemitoneOperator>(InParams.OperatorSettings, InFrequencyMultiplier);
	}

	using FFrequencyMultiplierToSemitoneNode = TNodeFacade<FFrequencyMultiplierToSemitoneOperator>;

	METASOUND_REGISTER_NODE(FFrequencyMultiplierToSemitoneNode)
}

#undef LOCTEXT_NAMESPACE
