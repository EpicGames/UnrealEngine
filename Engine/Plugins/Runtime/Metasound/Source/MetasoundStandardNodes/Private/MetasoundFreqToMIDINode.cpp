// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_FreqToMidiNode"

namespace Metasound
{
	namespace FrequencyToMidiVertexNames
	{
		METASOUND_PARAM(InputFreq, "Frequency In", "Input frequency value in Hz.");
		METASOUND_PARAM(OutputMidi, "Out MIDI", "Output MIDI note value that corresponds to the input frequency value.");
	}

	class FFreqToMidiOperator : public TExecutableOperator<FFreqToMidiOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FFreqToMidiOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InMidiNote);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The input frequency value
		FFloatReadRef FreqInput;

		// The output MIDI note value
		FFloatWriteRef MidiOutput;

		// Cached freq note value. Used to catch if the value changes to recompute Midi output.
		float PrevFreq;
	};

	FFreqToMidiOperator::FFreqToMidiOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InFreq)
		: FreqInput(InFreq)
		, MidiOutput(FFloatWriteRef::CreateNew(Audio::GetMidiFromFrequency(*InFreq)))
		, PrevFreq(*InFreq)
	{
	}

	void FFreqToMidiOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace FrequencyToMidiVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFreq), FreqInput);
	}

	void FFreqToMidiOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace FrequencyToMidiVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputMidi), MidiOutput);
	}

	void FFreqToMidiOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PrevFreq = *FreqInput;
		*MidiOutput = Audio::GetMidiFromFrequency(PrevFreq);
	}

	void FFreqToMidiOperator::Execute()
	{
		// Only do anything if the Midi note changes
		if (!FMath::IsNearlyEqual(*FreqInput, PrevFreq))
		{
			PrevFreq = *FreqInput;

			*MidiOutput = Audio::GetMidiFromFrequency(PrevFreq);
		}
	}

	const FVertexInterface& FFreqToMidiOperator::GetVertexInterface()
	{
		using namespace FrequencyToMidiVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFreq), 440.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputMidi))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FFreqToMidiOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<float>();
			const FName OperatorName = TEXT("Frequency to MIDI");
			const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_FreqToMidiNodeName", "Frequency To MIDI");
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_FreqToMidiNodeDescription", "Converts a frequency (Hz) value to a Midi Note value.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Music);
			Info.Keywords.Add(METASOUND_LOCTEXT("FreqToMIDIPitchKeyword", "Pitch"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FFreqToMidiOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace FrequencyToMidiVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InFreq = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputFreq), InParams.OperatorSettings);

		return MakeUnique<FFreqToMidiOperator>(InParams.OperatorSettings, InFreq);
	}

	using FFreqToMidiNode = TNodeFacade<FFreqToMidiOperator>;

	METASOUND_REGISTER_NODE(FFreqToMidiNode)
}

#undef LOCTEXT_NAMESPACE
