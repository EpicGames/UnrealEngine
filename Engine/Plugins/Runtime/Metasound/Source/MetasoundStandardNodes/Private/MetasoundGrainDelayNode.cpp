// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Delay.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "DSP/GrainDelay.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_GrainDelayNode"

namespace Metasound
{
	namespace GrainDelay
	{
		METASOUND_PARAM(InAudio, "In Audio", "Audio buffer input to be grain delayed.")
		METASOUND_PARAM(InGrainSpawnTrigger, "Grain Spawn", "Spawns a new grain of audio.")
		METASOUND_PARAM(InGrainDelay, "Grain Delay", "The delay of the next spawned grain (in milliseconds) between 0 and the maximum delay.")
		METASOUND_PARAM(InGrainDelayDelta, "Grain Delay Range", "A random grain offset (in milliseconds) before and after the grain delay.")
		METASOUND_PARAM(InGrainDuration, "Grain Duration", "The duration of the next spawned grain (in milliseconds).")
		METASOUND_PARAM(InGrainDurationDelta, "Grain Duration Range", "A random grain duration range (in milliseconds) before and after the grain delay.")
		METASOUND_PARAM(InGrainPitchShift, "Pitch Shift", "A pitch value (in semitones) to change the grain pitch of all rendering grains.")
		METASOUND_PARAM(InGrainPitchShiftDelta, "Pitch Shift Range", "A random pitch shift delta (in semitones) randomly chosen when a grain is spawned.")
		METASOUND_PARAM(InGrainDelayEnvelope, "Grain Envelope", "The type of envelope to use for the grains.")
		METASOUND_PARAM(InGrainMaxCount, "Max Grain Count", "The maximum number of grains to render at a time (between 1 and 100). More grains will cost more CPU and potentially clip.")
		METASOUND_PARAM(InGrainDelayFeedbackAmount, "Feedback Amount", "The amount of feedback of each grain. The grain delay will feed its audio output back into itself.")
		METASOUND_PARAM(InGrainMaxDelayTimeSeconds, "Max Delay Time", "The maximum amount of time to delay the audio.")

		METASOUND_PARAM(OutAudio, "Out Audio", "Output audio buffer which has been grain-delayed.")
	}
	
	DECLARE_METASOUND_ENUM(Audio::Grain::EEnvelope, Audio::Grain::EEnvelope::Gaussian, METASOUNDSTANDARDNODES_API, FEnumGrainDelayEnvelope, FEnumGrainDelayEnvelopeInfo, FGrainDelayEnvelopeReadRef, FEnumGrainDelayEnvelopeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(Audio::Grain::EEnvelope, FEnumGrainDelayEnvelope, "GrainDelayEnvelope")
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::Gaussian, "GrainDelayEnvelopeGaussianDescription", "Gaussian", "GrainDelayEnvelopeGaussianDescriptionTT", "A gaussian envelope."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::Triangle, "GrainDelayEnvelopeTriangleDescription", "Triangle", "GrainDelayEnvelopeTriangleDescriptionTT", "A triangular envelope."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::DownwardTriangle, "GrainDelayEnvelopeDownwardTriangleDescription", "Downward Triangle", "GrainDelayEnvelopeDownwardTriangleDescriptionTT", "A downward triangular envelope."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::UpwardTriangle, "GrainDelayEnvelopeUpwardTriangleDescription", "Upward Triangle", "GrainDelayEnvelopeUpwardTriangleDescriptionTT", "An upward triangular envelope."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::ExponentialDecay, "GrainDelayEnvelopeExponentialDecayDescription", "Exponential Decay", "GrainDelayEnvelopeExponentialDecayDescriptionTT", "An exponential decay envelope."),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::Grain::EEnvelope::ExponentialAttack, "GrainDelayEnvelopeExponentialAttackDescription", "Exponential Attack", "GrainDelayEnvelopeExponentialAttackDescriptionTT", "An exponential attack envelope."),
	DEFINE_METASOUND_ENUM_END()

	class FGrainDelayOperator final : public TExecutableOperator<FGrainDelayOperator>
	{
	public:

		FGrainDelayOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput,
			const FTriggerReadRef& InGrainSpawnTrigger,
			const FFloatReadRef& InGrainDelay,
			const FFloatReadRef& InGrainDelayDelta,
			const FFloatReadRef& InGrainDuration,
			const FFloatReadRef& InGrainDurationDelta,
			const FFloatReadRef& InGrainPitchShift,
			const FFloatReadRef& InGrainPitchShiftDelta,
			const FGrainDelayEnvelopeReadRef& InGrainDelayEnvelope,
			const FInt32ReadRef& InGrainMaxCount,	
			const FFloatReadRef& InGrainDelayFeedbackAmount,
			float InGrainMaxDelaySeconds
		);

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		const FOperatorSettings OperatorSettings;

		// Input references
		FAudioBufferReadRef AudioInput;
		FTriggerReadRef GrainSpawnTrigger;
		FFloatReadRef GrainDelay;
		FFloatReadRef GrainDelayDelta;
		FFloatReadRef GrainDuration;
		FFloatReadRef GrainDurationDelta;
		FFloatReadRef GrainPitchShift;
		FFloatReadRef GrainPitchShiftDelta;
		FGrainDelayEnvelopeReadRef GrainDelayEnvelope;
		FInt32ReadRef GrainMaxCount;
		FFloatReadRef GrainDelayFeedbackAmount;
		float GrainMaxDelayTimeSeconds = 2.0f;

		// Output references
		FAudioBufferWriteRef AudioOutput;

		// Cached values from input refs
		int32 PreviousGrainDelayMsec = 0;
		float PreviousPitchShift = 0.0f;
		Audio::Grain::EEnvelope PreviousGrainEnvelopeType = Audio::Grain::EEnvelope::Gaussian;

		// Grain manager manages rendering grains
		Audio::GrainDelay::FGrainDelay GrainDelayProcessor;
	};
	
	FGrainDelayOperator::FGrainDelayOperator(const FOperatorSettings& InSettings,
											  const FAudioBufferReadRef& InAudioInput,
											  const FTriggerReadRef& InGrainSpawnTrigger,
											  const FFloatReadRef& InGrainDelay,
											  const FFloatReadRef& InGrainDelayDelta,
											  const FFloatReadRef& InGrainDuration,
											  const FFloatReadRef& InGrainDurationDelta,
											  const FFloatReadRef& InGrainPitchShift,
											  const FFloatReadRef& InGrainPitchShiftDelta,
											  const FGrainDelayEnvelopeReadRef& InGrainDelayEnvelope,
											  const FInt32ReadRef& InGrainMaxCount,
											  const FFloatReadRef& InGrainDelayFeedbackAmount,
											  float InGrainMaxDelayTimeSeconds)
		: OperatorSettings(InSettings),
	      AudioInput(InAudioInput),
		  GrainSpawnTrigger(InGrainSpawnTrigger),
		  GrainDelay(InGrainDelay),
		  GrainDelayDelta(InGrainDelayDelta),
		  GrainDuration(InGrainDuration),
		  GrainDurationDelta(InGrainDurationDelta),
		  GrainPitchShift(InGrainPitchShift),
		  GrainPitchShiftDelta(InGrainPitchShiftDelta),
		  GrainDelayEnvelope(InGrainDelayEnvelope),
		  GrainMaxCount(InGrainMaxCount),
		  GrainDelayFeedbackAmount(InGrainDelayFeedbackAmount),
		  GrainMaxDelayTimeSeconds(InGrainMaxDelayTimeSeconds),
	      AudioOutput(FAudioBufferWriteRef::CreateNew(OperatorSettings)),
		  GrainDelayProcessor(OperatorSettings.GetSampleRate(), GrainMaxDelayTimeSeconds)
	{
		PreviousGrainDelayMsec = GrainDelayProcessor.GetGrainDelayClamped(*GrainDelay);
		PreviousGrainEnvelopeType = *InGrainDelayEnvelope;
		PreviousPitchShift = GrainDelayProcessor.GetGrainPitchShiftClamped(*GrainPitchShift);
		
		GrainDelayProcessor.SetGrainEnvelope(*GrainDelayEnvelope);
		GrainDelayProcessor.SetGrainBasePitchShiftRatio(GrainDelayProcessor.GetGrainPitchShiftFrameRatio(PreviousPitchShift));
	}

	const FNodeClassMetadata& FGrainDelayOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace,TEXT("GrainDelayNode"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("GrainDelayNodeDisplayName", "Grain Delay");
			Info.Description = METASOUND_LOCTEXT("Metasound_GrainDelayNodeDescription", "Performs delayed audio granulation on an input audio buffer.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Delays);
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	const FVertexInterface& FGrainDelayOperator::GetVertexInterface()
	{
		using namespace GrainDelay;

		auto CreateDefaultInterface = []()-> FVertexInterface
		{
			FDataVertexMetadata MaxDelayTimeMetadata = METASOUND_GET_PARAM_METADATA(InGrainMaxDelayTimeSeconds);
			MaxDelayTimeMetadata.bIsAdvancedDisplay = true;

			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InAudio)));
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainSpawnTrigger)));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDelay), 128.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDelayDelta), 0.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDuration), 128.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDurationDelta), 0.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainPitchShift), 0.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainPitchShiftDelta), 0.0f));
			InputInterface.Add(TInputDataVertex<FEnumGrainDelayEnvelope>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDelayEnvelope), (int32)Audio::Grain::EEnvelope::Gaussian));
			InputInterface.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainMaxCount), 16));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InGrainDelayFeedbackAmount), 0.0f));
			InputInterface.Add(TInputConstructorVertex<FTime>(METASOUND_GET_PARAM_NAME(InGrainMaxDelayTimeSeconds), MaxDelayTimeMetadata, 2.0f));

			FOutputVertexInterface OutputInterface; 
			OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio)));

			return FVertexInterface(InputInterface, OutputInterface);
		};

		static const FVertexInterface DefaultInterface = CreateDefaultInterface();
		return DefaultInterface;
	}

	TUniquePtr<IOperator> FGrainDelayOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace GrainDelay;

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		
		FAudioBufferReadRef AudioInput = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InAudio), InParams.OperatorSettings);
		FTriggerReadRef GrainSpawnTrigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InGrainSpawnTrigger), InParams.OperatorSettings);
		FFloatReadRef GrainDelay = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainDelay), InParams.OperatorSettings);
		FFloatReadRef GrainDelayDelta = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainDelayDelta), InParams.OperatorSettings);
		FFloatReadRef GrainDuration = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainDuration), InParams.OperatorSettings);
		FFloatReadRef GrainDurationDelta = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainDurationDelta), InParams.OperatorSettings);
		FFloatReadRef GrainPitchShift = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainPitchShift), InParams.OperatorSettings);
		FFloatReadRef GrainPitchShiftDelta = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainPitchShiftDelta), InParams.OperatorSettings);
		FGrainDelayEnvelopeReadRef GrainDelayEnvelope = InputData.GetOrCreateDefaultDataReadReference<FEnumGrainDelayEnvelope>(METASOUND_GET_PARAM_NAME(InGrainDelayEnvelope), InParams.OperatorSettings);
		FInt32ReadRef GrainMaxCount = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InGrainMaxCount), InParams.OperatorSettings);
		FFloatReadRef GrainDelayFeedbackAmount = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InGrainDelayFeedbackAmount), InParams.OperatorSettings);
		FTime GrainMaxDelayTimeSeconds = InputData.GetOrCreateDefaultValue<FTime>(METASOUND_GET_PARAM_NAME(InGrainMaxDelayTimeSeconds), InParams.OperatorSettings);

		return MakeUnique<FGrainDelayOperator>(InParams.OperatorSettings, 
			AudioInput,
			GrainSpawnTrigger,
			GrainDelay,
			GrainDelayDelta,
			GrainDuration,
			GrainDurationDelta,
			GrainPitchShift,
			GrainPitchShiftDelta,
			GrainDelayEnvelope,
			GrainMaxCount,
			GrainDelayFeedbackAmount,
			GrainMaxDelayTimeSeconds.GetSeconds());
	}

	void FGrainDelayOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace GrainDelay;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InAudio), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainSpawnTrigger), GrainSpawnTrigger);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDelay), GrainDelay);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDelayDelta), GrainDelayDelta);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDuration), GrainDuration);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDurationDelta), GrainDurationDelta);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainPitchShift), GrainPitchShift);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainPitchShiftDelta), GrainPitchShiftDelta);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDelayEnvelope), GrainDelayEnvelope);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainMaxCount), GrainMaxCount);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InGrainDelayFeedbackAmount), GrainDelayFeedbackAmount);
		InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(InGrainMaxDelayTimeSeconds), FTime::FromSeconds(GrainMaxDelayTimeSeconds));
	}

	void FGrainDelayOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace GrainDelay;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutAudio), AudioOutput);
	}

	void FGrainDelayOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioOutput->Zero();

		GrainDelayProcessor.Reset();
		PreviousGrainDelayMsec = GrainDelayProcessor.GetGrainDelayClamped(*GrainDelay);
		PreviousGrainEnvelopeType = *GrainDelayEnvelope;
		PreviousPitchShift = GrainDelayProcessor.GetGrainPitchShiftClamped(*GrainPitchShift);
		
		GrainDelayProcessor.SetGrainEnvelope(*GrainDelayEnvelope);
		GrainDelayProcessor.SetGrainBasePitchShiftRatio(GrainDelayProcessor.GetGrainPitchShiftFrameRatio(PreviousPitchShift));
	}

	void FGrainDelayOperator::Execute()
	{
		// Parse triggers and render audio correctly in the audio block
		int32 SpawnTrigIndex = 0;
		int32 NextSpawnFrame = 0;
		const int32 NumSpawnTriggers = GrainSpawnTrigger->NumTriggeredInBlock();

		int32 CurrAudioFrame = 0;
		int32 NextAudioFrame = 0;
		const int32 LastAudioFrame = OperatorSettings.GetNumFramesPerBlock() - 1;
		const int32 NoTrigger = OperatorSettings.GetNumFramesPerBlock() << 1;

		GrainDelayProcessor.SetMaxGrains(FMath::Clamp(*GrainMaxCount, 1, 1000));
		
		// Update the envelope before rendering the audio block
		if (PreviousGrainEnvelopeType != *GrainDelayEnvelope)
		{
			PreviousGrainEnvelopeType = *GrainDelayEnvelope;
			GrainDelayProcessor.SetGrainEnvelope(PreviousGrainEnvelopeType);
		}

		// Update the base pitch shift ratio for the grain manager if it has changed
		const float CurrentPitchShiftClamped = GrainDelayProcessor.GetGrainPitchShiftClamped(*GrainPitchShift);
		if (!FMath::IsNearlyEqual(PreviousPitchShift, CurrentPitchShiftClamped))
		{
			PreviousPitchShift = CurrentPitchShiftClamped;
			GrainDelayProcessor.SetGrainBasePitchShiftRatio(GrainDelayProcessor.GetGrainPitchShiftFrameRatio(PreviousPitchShift));
		}

		const float CurrentDelayLengthMsec = GrainDelayProcessor.GetGrainDelayClamped(*GrainDelay);
		if (!FMath::IsNearlyEqual(CurrentDelayLengthMsec, PreviousGrainDelayMsec))
		{
			PreviousGrainDelayMsec = CurrentDelayLengthMsec;
		}

		const float GrainDelayFeedbackAmountClamped = FMath::Clamp(*GrainDelayFeedbackAmount, 0.0f, 1.0f);
		GrainDelayProcessor.SetFeedbackAmount(GrainDelayFeedbackAmountClamped);
		
		const float NewDurationSeconds = GrainDelayProcessor.GetGrainDurationClamped(*GrainDuration + FMath::RandRange(-0.5f * *GrainDurationDelta, 0.5f * *GrainDurationDelta));

		const float NewDelay = GrainDelayProcessor.GetGrainDelayClamped(CurrentDelayLengthMsec + FMath::RandRange(-0.5f * *GrainDelayDelta, 0.5f * *GrainDelayDelta));

		const float ClampedPitchShiftDelta = GrainDelayProcessor.GetGrainPitchShiftClamped(*GrainPitchShift + FMath::RandRange(-0.5f * *GrainPitchShiftDelta, 0.5f * *GrainPitchShiftDelta));
		const float NewPitchShiftRatioOffset = GrainDelayProcessor.GetGrainPitchShiftFrameRatio(ClampedPitchShiftDelta);

		// Loop until we either hit the end of the frame or the next trigger
		while (NextAudioFrame < LastAudioFrame)
		{
			if (SpawnTrigIndex < NumSpawnTriggers)
			{
				NextSpawnFrame = (*GrainSpawnTrigger)[SpawnTrigIndex];
			}
			else
			{
				NextSpawnFrame = NoTrigger;
			}

			NextAudioFrame = FMath::Min(NextSpawnFrame, LastAudioFrame);

			// no more triggers, rendering to the end of the block
			if (NextAudioFrame == NoTrigger)
			{
				NextAudioFrame = OperatorSettings.GetNumFramesPerBlock();
			}

			// render audio (while loop handles looping audio)
			while (CurrAudioFrame != NextAudioFrame)
			{
				GrainDelayProcessor.SynthesizeAudio(CurrAudioFrame, NextAudioFrame, AudioInput->GetData(), AudioOutput->GetData());
				
				CurrAudioFrame = NextAudioFrame;
			}

			// execute the next spawn trigger
			if (CurrAudioFrame == NextSpawnFrame)
			{
				++SpawnTrigIndex;

				// Spawn a new grain. Note the grain manager handles spawning logic and rate limiting.
				GrainDelayProcessor.SpawnGrain(NewDelay, NewDurationSeconds, NewPitchShiftRatioOffset);
			}
		}
	}
	
	using FGrainDelayNode = TNodeFacade<FGrainDelayOperator>;

	METASOUND_REGISTER_NODE(FGrainDelayNode)
}

#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes_GrainDelayNode
