// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Delay.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundFacade.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_ITDPannerNode"

namespace Metasound
{
	namespace ITDPannerVertexNames
	{
		METASOUND_PARAM(InputAudio, "In", "The input audio to spatialize.")
		METASOUND_PARAM(InputPanAngle, "Angle", "The sound source angle in degrees. 90 degrees is in front, 0 degrees is to the right, 270 degrees is behind, 180 degrees is to the left.")
		METASOUND_PARAM(InputDistanceFactor, "Distance Factor", "The normalized distance factor (0.0 to 1.0) to use for ILD (Inter-aural level difference) calculations. 0.0 is near, 1.0 is far. The further away something is the less there is a difference in levels (gain) between the ears.")
		METASOUND_PARAM(InputHeadWidth, "Head Width", "The width of the listener head to use for ITD calculations in centimeters.")
		METASOUND_PARAM(OutputAudioLeft, "Out Left", "Left channel audio output.")
		METASOUND_PARAM(OutputAudioRight, "Out Right", "Right channel audio output.")
	}

	class FITDPannerOperator : public TExecutableOperator<FITDPannerOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FITDPannerOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput, 
			const FFloatReadRef& InPanningAngle,
			const FFloatReadRef& InDistanceFactor,
			const FFloatReadRef& InHeadWidth);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		void UpdateParams(bool bIsInit);

		FAudioBufferReadRef AudioInput;
		FFloatReadRef PanningAngle;
		FFloatReadRef DistanceFactor;
		FFloatReadRef HeadWidth;

		FAudioBufferWriteRef AudioLeftOutput;
		FAudioBufferWriteRef AudioRightOutput;

		float CurrAngle = 0.0f;
		float CurrX = 0.0f;
		float CurrY = 0.0f;
		float CurrDistanceFactor = 0.0f;
		float CurrHeadWidth = 0.0f;
		float CurrLeftGain = 0.0f;
		float CurrRightGain = 0.0f;
		float CurrLeftDelay = 0.0f;
		float CurrRightDelay = 0.0f;

		float PrevLeftGain = 0.0f;
		float PrevRightGain = 0.0f;

		Audio::FDelay LeftDelay;
		Audio::FDelay RightDelay;
	};

	FITDPannerOperator::FITDPannerOperator(const FOperatorSettings& InSettings,
		const FAudioBufferReadRef& InAudioInput, 
		const FFloatReadRef& InPanningAngle,
		const FFloatReadRef& InDistanceFactor,
		const FFloatReadRef& InHeadWidth)
		: AudioInput(InAudioInput)
		, PanningAngle(InPanningAngle)
		, DistanceFactor(InDistanceFactor)
		, HeadWidth(InHeadWidth)
		, AudioLeftOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		, AudioRightOutput(FAudioBufferWriteRef::CreateNew(InSettings))
	{
		LeftDelay.Init(InSettings.GetSampleRate(), 0.5f);
		RightDelay.Init(InSettings.GetSampleRate(), 0.5f);

		const float EaseFactor = Audio::FExponentialEase::GetFactorForTau(0.1f, InSettings.GetSampleRate());
		LeftDelay.SetEaseFactor(EaseFactor);
		RightDelay.SetEaseFactor(EaseFactor);

		CurrAngle = FMath::Clamp(*PanningAngle, 0.0f, 360.0f);
		CurrDistanceFactor = FMath::Clamp(*DistanceFactor, 0.0f, 1.0f);
		CurrHeadWidth = FMath::Max(*InHeadWidth, 0.0f);

		UpdateParams(true);

		PrevLeftGain = CurrLeftGain;
		PrevRightGain = CurrRightGain;
	}

	void FITDPannerOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace ITDPannerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPanAngle), PanningAngle);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDistanceFactor), DistanceFactor);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputHeadWidth), HeadWidth);
	}

	void FITDPannerOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace ITDPannerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioLeft), AudioLeftOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioRight), AudioRightOutput);
	}

	void FITDPannerOperator::UpdateParams(bool bIsInit)
	{
		// ****************
		// Update the x-y values
		const float CurrRadians = (CurrAngle / 360.0f) * 2.0f * PI;
		FMath::SinCos(&CurrY, &CurrX, CurrRadians);

		// ****************
		// Update ILD gains
		const float HeadRadiusMeters = 0.005f * CurrHeadWidth; // (InHeadWidth / 100.0f) / 2.0f;

		// InX is -1.0 to 1.0, so get it in 0.0 to 1.0 (i.e. hard left, hard right)
		const float Fraction = (CurrX + 1.0f) * 0.5f;

		// Feed the linear pan value into a equal power equation
		float PanLeft;
		float PanRight;
		FMath::SinCos(&PanRight, &PanLeft, 0.5f * PI * Fraction);

		// If distance factor is 1.0 (i.e. far away) this will have equal gain, if distance factor is 0.0 it will be normal equal power pan.
		CurrLeftGain = FMath::Lerp(PanLeft, 0.5f, CurrDistanceFactor);
		CurrRightGain = FMath::Lerp(PanRight, 0.5f, CurrDistanceFactor);

		// *********************
		// Update the ITD delays

		// Use pythagorean theorem to get distances
		const float DistToLeftEar = FMath::Sqrt((CurrY * CurrY) + FMath::Square(HeadRadiusMeters + CurrX));
		const float DistToRightEar = FMath::Sqrt((CurrY * CurrY) + FMath::Square(HeadRadiusMeters - CurrX));

		// Compute delta time based on speed of sound 
		constexpr float SpeedOfSound = 343.0f;
		const float DeltaTimeSeconds = (DistToLeftEar - DistToRightEar) / SpeedOfSound;

		if (DeltaTimeSeconds > 0.0f)
		{
			LeftDelay.SetEasedDelayMsec(1000.0f * DeltaTimeSeconds, bIsInit);
			RightDelay.SetEasedDelayMsec(0.0f, bIsInit);
		}
		else
		{
			LeftDelay.SetEasedDelayMsec(0.0f, bIsInit);
			RightDelay.SetEasedDelayMsec(-1000.0f * DeltaTimeSeconds, bIsInit);
		}
	}
	

	void FITDPannerOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioLeftOutput->Zero();
		AudioRightOutput->Zero();

		LeftDelay.Reset();
		RightDelay.Reset();

		CurrAngle = FMath::Clamp(*PanningAngle, 0.0f, 360.0f);
		CurrDistanceFactor = FMath::Clamp(*DistanceFactor, 0.0f, 1.0f);
		CurrHeadWidth = FMath::Max(*HeadWidth, 0.0f);

		UpdateParams(true);

		PrevLeftGain = CurrLeftGain;
		PrevRightGain = CurrRightGain;
	}

	void FITDPannerOperator::Execute()
	{
		float NewHeadWidth = FMath::Max(*HeadWidth, 0.0f);
		float NewAngle = FMath::Clamp(*PanningAngle, 0.0f, 360.0f);
		float NewDistanceFactor = FMath::Clamp(*DistanceFactor, 0.0f, 1.0f);

		if (!FMath::IsNearlyEqual(NewAngle, CurrHeadWidth) ||
			!FMath::IsNearlyEqual(NewDistanceFactor, CurrAngle) ||
			!FMath::IsNearlyEqual(NewHeadWidth, CurrDistanceFactor))
		{
			CurrHeadWidth = NewHeadWidth;
			CurrAngle = NewAngle;
			CurrDistanceFactor = NewDistanceFactor;

			UpdateParams(false);
		}

		const float* InputBufferPtr = AudioInput->GetData();
		int32 InputSampleCount = AudioInput->Num();
		float* OutputLeftBufferPtr = AudioLeftOutput->GetData();
		float* OutputRightBufferPtr = AudioRightOutput->GetData();
		TArrayView<float> OutputLeftBufferView(AudioLeftOutput->GetData(), InputSampleCount);
		TArrayView<float> OutputRightBufferView(AudioRightOutput->GetData(), InputSampleCount);

		// Feed the input audio into the left and right delays
		for (int32 i = 0; i < InputSampleCount; ++i)
		{
			OutputLeftBufferPtr[i] = LeftDelay.ProcessAudioSample(InputBufferPtr[i]);
			OutputRightBufferPtr[i] = RightDelay.ProcessAudioSample(InputBufferPtr[i]);
		}

		// Now apply the panning
		if (FMath::IsNearlyEqual(PrevLeftGain, CurrLeftDelay))
		{
			Audio::ArrayMultiplyByConstantInPlace(OutputLeftBufferView, PrevLeftGain);
			Audio::ArrayMultiplyByConstantInPlace(OutputRightBufferView, PrevRightGain);
		}
		else
		{
			Audio::ArrayFade(OutputLeftBufferView, PrevLeftGain, CurrLeftGain);
			Audio::ArrayFade(OutputRightBufferView, PrevRightGain, CurrRightGain);

			PrevLeftGain = CurrLeftGain;
			PrevRightGain = CurrRightGain;
		}
	}

	const FVertexInterface& FITDPannerOperator::GetVertexInterface()
	{
		using namespace ITDPannerVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPanAngle), 90.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDistanceFactor), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputHeadWidth), 34.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioRight))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FITDPannerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("ITD Panner"), TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_ITDPannerDisplayName", "ITD Panner");
			Info.Description = METASOUND_LOCTEXT("Metasound_ITDPannerNodeDescription", "Pans an input audio signal using an inter-aural time delay method.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Spatialization);
			Info.Keywords = { METASOUND_LOCTEXT("ITDBinauralKeyword", "Binaural"), METASOUND_LOCTEXT("ITDInterauralKeyword", "Interaural Time Delay")};
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FITDPannerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		using namespace ITDPannerVertexNames;

		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
		FFloatReadRef PanningAngle = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPanAngle), InParams.OperatorSettings);
		FFloatReadRef DistanceFactor = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDistanceFactor), InParams.OperatorSettings);
		FFloatReadRef HeadWidth = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputHeadWidth), InParams.OperatorSettings);

		return MakeUnique<FITDPannerOperator>(InParams.OperatorSettings, AudioIn, PanningAngle, DistanceFactor, HeadWidth);
	}

	using FITDPannerNode = TNodeFacade<FITDPannerOperator>;

	METASOUND_REGISTER_NODE(FITDPannerNode)
}

#undef LOCTEXT_NAMESPACE
