// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFacade.h"

#include "DSP/VolumeFader.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_InterpNode"

namespace Metasound
{
	namespace InterpToVertexNames
	{
		METASOUND_PARAM(InParamTarget, "Target", "Target value.")
		METASOUND_PARAM(InParamInterpTime, "Interp Time", "The time to interpolate from the current value to the target value.")
		METASOUND_PARAM(OutParamValue, "Value", "The current value.")
	}


	class FInterpToOperator : public TExecutableOperator<FInterpToOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FInterpToOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InTargetValue, const FTimeReadRef& InInterpTime);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The target value of the lerp. The output will lerp from it's current value to the output value.
		FFloatReadRef TargetValue;

		// The amount of time to do lerp
		FTimeReadRef InterpTime;

		// The current output value.
		FFloatWriteRef ValueOutput;

		// Volume fader object which performs the interpolating
		Audio::FVolumeFader VolumeFader;

		// The time-delta per block
		float BlockTimeDelta = 0.0f;

		// The previous target value
		float PreviousTargetValue = 0.0f;
	};

	FInterpToOperator::FInterpToOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InTargetValue, const FTimeReadRef& InInterpTime)
		: TargetValue(InTargetValue)
		, InterpTime(InInterpTime)
		, ValueOutput(FFloatWriteRef::CreateNew(*TargetValue))
	{
		Reset(InParams);
	}

	void FInterpToOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace InterpToVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamTarget), TargetValue);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InParamInterpTime), InterpTime);
	}

	void FInterpToOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace InterpToVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutParamValue), ValueOutput);
	}

	void FInterpToOperator::Reset(const IOperator::FResetParams& InParams)
	{
		// Set the fade to start at the value specified in the current value
		VolumeFader.SetVolume(*TargetValue);

		float BlockRate = InParams.OperatorSettings.GetActualBlockRate();
		BlockTimeDelta = 1.0f / BlockRate;

		PreviousTargetValue = *TargetValue;

		*ValueOutput = *TargetValue;

	}

	void FInterpToOperator::Execute()
	{
		// Update the value output with the current value in case it was changed
		if (!FMath::IsNearlyEqual(PreviousTargetValue, *TargetValue))
		{
			PreviousTargetValue = *TargetValue;
			// Start the volume fader on the interp trigger
			float FadeSeconds = (float)InterpTime->GetSeconds();
			VolumeFader.StartFade(*TargetValue, FadeSeconds, Audio::EFaderCurve::Linear);
		}

		// Perform the fading
		if (VolumeFader.IsFading())
		{
			VolumeFader.Update(BlockTimeDelta);
		}

		// Update the fader w/ the current volume
		*ValueOutput = VolumeFader.GetVolume();
	}

	const FVertexInterface& FInterpToOperator::GetVertexInterface()
	{
		using namespace InterpToVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamInterpTime), 0.1f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InParamTarget), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutParamValue))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FInterpToOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("InterpTo"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_InterpDisplayName", "InterpTo");
			Info.Description = METASOUND_LOCTEXT("Metasound_InterpNodeDescription", "Interpolates between the current value and a target value over the specified time.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.CategoryHierarchy = { NodeCategories::Math };
			Info.DefaultInterface = GetVertexInterface();
			Info.Keywords.Add(METASOUND_LOCTEXT("LerpKeyword", "Lerp"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FInterpToOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace InterpToVertexNames; 

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef TargetValue = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InParamTarget), InParams.OperatorSettings);
		FTimeReadRef InterpTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InParamInterpTime), InParams.OperatorSettings);

		return MakeUnique<FInterpToOperator>(InParams, TargetValue, InterpTime);
	}

	using FInterpToNode = TNodeFacade<FInterpToOperator>;

	METASOUND_REGISTER_NODE(FInterpToNode)
}

#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes_InterpNode
