// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "DSP/Dsp.h"
#include "DSP/DynamicsProcessor.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_LimiterNode"

namespace Metasound
{
	/* Mid-Side Encoder */
	namespace LimiterVertexNames
	{
		METASOUND_PARAM(InputAudio, "Audio", "Incoming audio signal to compress.");
		METASOUND_PARAM(InputInGainDb, "Input Gain dB", "Gain to apply to the input before limiting, in decibels. Maximum 100 dB. ");
		METASOUND_PARAM(InputThresholdDb, "Threshold dB", "Amplitude threshold above which gain will be reduced.");
		METASOUND_PARAM(InputReleaseTime, "Release Time", "How long it takes for audio below the threshold to return to its original volume level.");
		METASOUND_PARAM(InputKneeMode, "Knee", "Whether the limiter uses a hard or soft knee.");

		METASOUND_PARAM(OutputAudio, "Audio", "The output audio signal.");
	}

	enum class EKneeMode
	{
		Hard = 0,
		Soft,
	};

	DECLARE_METASOUND_ENUM(EKneeMode, EKneeMode::Hard, METASOUNDSTANDARDNODES_API,
	FEnumKneeMode, FEnumKneeModeInfo, FKneeModeReadRef, FEnumKneeModeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(EKneeMode, FEnumKneeMode, "KneeMode")
		DEFINE_METASOUND_ENUM_ENTRY(EKneeMode::Hard, "KneeModeHardDescription", "Hard", "KneeModeHardDescriptionTT", "Only audio strictly above the threshold is affected by the limiter."),
		DEFINE_METASOUND_ENUM_ENTRY(EKneeMode::Soft, "KneeModeSoftDescription", "Soft", "KneeModeSoftDescriptionTT", "Limiter activates more smoothly near the threshold."),
		DEFINE_METASOUND_ENUM_END()

	// Operator Class
	class FLimiterOperator : public TExecutableOperator<FLimiterOperator>
	{
	public:

		static constexpr float HardKneeBandwitdh = 0.0f;
		static constexpr float SoftKneeBandwitdh = 10.0f;
		static constexpr float MaxInputGain = 100.0f;

		FLimiterOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InAudio,
			const FFloatReadRef& InGainDb,
			const FFloatReadRef& InThresholdDb,
			const FTimeReadRef& InReleaseTime,
			const FKneeModeReadRef& InKneeMode)
			: AudioInput(InAudio)
			, InGainDbInput(InGainDb)
			, ThresholdDbInput(InThresholdDb)
			, ReleaseTimeInput(InReleaseTime)
			, KneeModeInput(InKneeMode)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, Limiter()
			, PrevInGainDb(*InGainDb)
			, PrevThresholdDb(*InThresholdDb)
			, PrevReleaseTime(FMath::Max(FTime::ToMilliseconds(*InReleaseTime), 0.0))
		{
			Reset(InParams);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName{StandardNodes::Namespace, TEXT("Limiter"), StandardNodes::AudioVariant},
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("LimiterDisplayName", "Limiter"),
					METASOUND_LOCTEXT("LimiterDesc", "Prevents a signal from going above a given threshold."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{NodeCategories::Dynamics},
					{},
					FNodeDisplayStyle{}
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace LimiterVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInGainDb), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputThresholdDb), 0.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReleaseTime), 0.1f),
					TInputDataVertex<FEnumKneeMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputKneeMode), (int32)EKneeMode::Hard)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LimiterVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInGainDb), InGainDbInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputThresholdDb), ThresholdDbInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReleaseTime), ReleaseTimeInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputKneeMode), KneeModeInput);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LimiterVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace LimiterVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FFloatReadRef InGainDbIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputInGainDb), InParams.OperatorSettings);
			FFloatReadRef ThresholdDbIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputThresholdDb), InParams.OperatorSettings);
			FTimeReadRef ReleaseTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputReleaseTime), InParams.OperatorSettings);
			FKneeModeReadRef KneeModeIn = InputData.GetOrCreateDefaultDataReadReference<FEnumKneeMode>(METASOUND_GET_PARAM_NAME(InputKneeMode), InParams.OperatorSettings);

			return MakeUnique<FLimiterOperator>(InParams, AudioIn, InGainDbIn, ThresholdDbIn, ReleaseTimeIn, KneeModeIn);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOutput->Zero();

			const float ClampedInGainDb = FMath::Min(*InGainDbInput, MaxInputGain);
			const float ClampedReleaseTime = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0);

			Limiter.Init(InParams.OperatorSettings.GetSampleRate(), 1);
			Limiter.SetProcessingMode(Audio::EDynamicsProcessingMode::Limiter);
			Limiter.SetInputGain(ClampedInGainDb);
			Limiter.SetThreshold(*ThresholdDbInput);
			Limiter.SetAttackTime(0.0f);
			Limiter.SetReleaseTime(ClampedReleaseTime);
			Limiter.SetPeakMode(Audio::EPeakMode::Peak);

			switch (*KneeModeInput)
			{
			default:
			case EKneeMode::Hard:
				Limiter.SetKneeBandwidth(HardKneeBandwitdh);
				break;
			case EKneeMode::Soft:
				Limiter.SetKneeBandwidth(SoftKneeBandwitdh);
				break;
			}

			PrevInGainDb = ClampedInGainDb;
			PrevReleaseTime = ClampedReleaseTime;
			PrevThresholdDb = *ThresholdDbInput;
			PrevKneeMode = *KneeModeInput;
		}

		void Execute()
		{
			/* Update parameters */
			float ClampedInGainDb = FMath::Min(*InGainDbInput, MaxInputGain);
			if (!FMath::IsNearlyEqual(ClampedInGainDb, PrevInGainDb))
			{
				Limiter.SetInputGain(ClampedInGainDb);
				PrevInGainDb = ClampedInGainDb;
			}

			if (!FMath::IsNearlyEqual(*ThresholdDbInput, PrevThresholdDb))
			{
				Limiter.SetThreshold(*ThresholdDbInput);
				PrevThresholdDb = *ThresholdDbInput;
			}
			// Release time cannot be negative
			double CurrRelease = FMath::Max(FTime::ToMilliseconds(*ReleaseTimeInput), 0.0f);
			if (!FMath::IsNearlyEqual(CurrRelease, PrevReleaseTime))
			{
				Limiter.SetReleaseTime(CurrRelease);
				PrevReleaseTime = CurrRelease;
			}

			if (PrevKneeMode != *KneeModeInput)
			{
				switch (*KneeModeInput)
				{
				default:
				case EKneeMode::Hard:
					Limiter.SetKneeBandwidth(HardKneeBandwitdh);
					break;
				case EKneeMode::Soft:
					Limiter.SetKneeBandwidth(SoftKneeBandwitdh);
					break;
				}			
				PrevKneeMode = *KneeModeInput;
			}

			Limiter.ProcessAudio(AudioInput->GetData(), AudioInput->Num(), AudioOutput->GetData());
		}

	private:
		FAudioBufferReadRef AudioInput;
		FFloatReadRef InGainDbInput;
		FFloatReadRef ThresholdDbInput;
		FTimeReadRef ReleaseTimeInput;
		FKneeModeReadRef KneeModeInput;

		FAudioBufferWriteRef AudioOutput;

		// Internal DSP Limiter
		Audio::FDynamicsProcessor Limiter;

		// Cached variables
		float PrevInGainDb;
		float PrevThresholdDb;
		double PrevReleaseTime;
		EKneeMode PrevKneeMode;
	};

	// Node Class
	using FLimiterNode = TNodeFacade<FLimiterOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FLimiterNode)
}

#undef LOCTEXT_NAMESPACE
