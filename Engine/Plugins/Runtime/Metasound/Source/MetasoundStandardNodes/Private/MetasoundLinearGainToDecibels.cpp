// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_LinearGainToDecibels"

namespace Metasound
{
	namespace LinearGainToDecibelsVertexNames
	{
		METASOUND_PARAM(InputLinearGain, "Linear Gain", "Input linear gain.");

		METASOUND_PARAM(OutputDecibels, "Decibels", "Output corresponding logarithmic (dB) gain.");
	}

	class FLinearGainToDecibelsOperator : public TExecutableOperator<FLinearGainToDecibelsOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FLinearGainToDecibelsOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InLinearGain);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The input linear gain value.
		FFloatReadRef LinearGainInput;

		// The output gain in dB.
		FFloatWriteRef DecibelOutput;
	};

	FLinearGainToDecibelsOperator::FLinearGainToDecibelsOperator(const FBuildOperatorParams& InParams, const FFloatReadRef& InLinearGain)
		: LinearGainInput(InLinearGain)
		, DecibelOutput(FFloatWriteRef::CreateNew(0))
	{
		Reset(InParams);
	}

	void FLinearGainToDecibelsOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace LinearGainToDecibelsVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLinearGain), LinearGainInput);
	}

	void FLinearGainToDecibelsOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace LinearGainToDecibelsVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputDecibels), DecibelOutput);
	}

	void FLinearGainToDecibelsOperator::Reset(const IOperator::FResetParams& InParams)
	{
		Execute();
	}

	void FLinearGainToDecibelsOperator::Execute()
	{
		// Linear gain value must be above 0
		const float ClampedInput = FMath::Max(*LinearGainInput, TNumericLimits<float>::Min());
		// Convert input to decibel gain value
		const float UnclampedDBOutput = Audio::ConvertToDecibels(ClampedInput, TNumericLimits<float>::Min());
		// Clamp to extremely low decibel value
		*DecibelOutput = FMath::Max(UnclampedDBOutput, -240.0f);
	}

	const FVertexInterface& FLinearGainToDecibelsOperator::GetVertexInterface()
	{
		using namespace LinearGainToDecibelsVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLinearGain), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputDecibels))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FLinearGainToDecibelsOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<float>();
			const FName OperatorName = TEXT("Linear Gain to Decibels");
			const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_LinearGainToDecibelsName", "Linear Gain to Decibels");
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_LinearGainToDecibelsDescription", "Converts a linear gain value to a logarithmic (dB) gain value.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Dynamics);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FLinearGainToDecibelsOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace LinearGainToDecibelsVertexNames;
		
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InLinearGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputLinearGain), InParams.OperatorSettings);
		
		return MakeUnique<FLinearGainToDecibelsOperator>(InParams, InLinearGain);
	}

	using FLinearGainToDecibelsNode = TNodeFacade<FLinearGainToDecibelsOperator>;

	METASOUND_REGISTER_NODE(FLinearGainToDecibelsNode)
}

#undef LOCTEXT_NAMESPACE
