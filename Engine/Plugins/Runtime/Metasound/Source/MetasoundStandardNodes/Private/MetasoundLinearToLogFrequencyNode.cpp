// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_LinearToLogFreqNode"

namespace Metasound
{
	namespace LinearToLogFrequencyVertexNames
	{
		METASOUND_PARAM(InputValue, "Value", "Linear input value to map to log frequency output. Input and output are clamped to specified domain and range.");
		METASOUND_PARAM(InputDomainMin, "Min Domain", "Min domain for the input value.");
		METASOUND_PARAM(InputDomainMax, "Max Domain", "Max domain for the input value.");
		METASOUND_PARAM(InputRangeMin, "Min Range", "Min positive range for the output frequency (Hz) value.");
		METASOUND_PARAM(InputRangeMax, "Max Range", "Max positive range for the output frequency (Hz) value.");
		METASOUND_PARAM(OutputFreq, "Frequency", "Output frequency value in hertz that is the log frequency of the input value.");
	}

	class FLinearToLogFrequencyOperator : public TExecutableOperator<FLinearToLogFrequencyOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FName OperatorName = TEXT("Linear To Log Frequency");
				FText NodeDisplayName = METASOUND_LOCTEXT("LinearToLogFrequencyName", "Linear To Log Frequency");
				const FText NodeDescription = METASOUND_LOCTEXT("LinearToLogFrequencyDescription", "Converts a linear space input value to log-frequency space.");

				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = NodeDescription;
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Math);
				Info.Keywords.Add(METASOUND_LOCTEXT("LinearToLogPitchKeyword", "Pitch"));

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();

			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace LinearToLogFrequencyVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValue),     0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDomainMin), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDomainMax), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputRangeMin),  20.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputRangeMax),  20000.0f)
					),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputFreq))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace LinearToLogFrequencyVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TDataReadReference<float> InValue = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);
			TDataReadReference<float> InDomainMin = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDomainMin), InParams.OperatorSettings);
			TDataReadReference<float> InDomainMax = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDomainMax), InParams.OperatorSettings);
			TDataReadReference<float> InRangeMin = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputRangeMin), InParams.OperatorSettings);
			TDataReadReference<float> InRangeMax = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputRangeMax), InParams.OperatorSettings);

			return MakeUnique<FLinearToLogFrequencyOperator>(InValue, InDomainMin, InDomainMax, InRangeMin, InRangeMax);
		}

		FLinearToLogFrequencyOperator(
			const TDataReadReference<float>& InValue,
			const TDataReadReference<float>& InDomainMin,
			const TDataReadReference<float>& InDomainMax,
			const TDataReadReference<float>& InRangeMin,
			const TDataReadReference<float>& InRangeMax)
			: Value(InValue)
			, DomainMin(InDomainMin)
			, DomainMax(InDomainMax)
			, RangeMin(InRangeMin)
			, RangeMax(InRangeMax)
			, FreqOutput(FFloatWriteRef::CreateNew(GetOutputValue()))
			, PrevInputValue(*Value)
		{
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LinearToLogFrequencyVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), Value);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDomainMin), DomainMin);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDomainMax), DomainMax);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRangeMin), RangeMin);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputRangeMax), RangeMax);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LinearToLogFrequencyVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputFreq), FreqOutput);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			PrevInputValue = *Value;
			*FreqOutput = GetOutputValue();
		}

		void Execute()
		{
			using namespace LinearToLogFrequencyVertexNames;

			if (!FMath::IsNearlyEqual(*Value, PrevInputValue))
			{
				PrevInputValue = *Value;
				*FreqOutput = GetOutputValue();
			}
		}

	private:
		float GetOutputValue() const
		{
			float RangeMinClamped = FMath::Max(*RangeMin, SMALL_NUMBER);
			float RangeMaxClamped = FMath::Max(*RangeMax, SMALL_NUMBER);
			float Result = Audio::GetLogFrequencyClamped(*Value, { *DomainMin, *DomainMax }, { RangeMinClamped, RangeMaxClamped });
			return Result;
		}

		TDataReadReference<float> Value;
		TDataReadReference<float> DomainMin;
		TDataReadReference<float> DomainMax;
		TDataReadReference<float> RangeMin;
		TDataReadReference<float> RangeMax;

		FFloatWriteRef FreqOutput;

		float PrevInputValue;
	};

	using FLinearToLogFrequencyNode = TNodeFacade<FLinearToLogFrequencyOperator>;

	METASOUND_REGISTER_NODE(FLinearToLogFrequencyNode)
}

#undef LOCTEXT_NAMESPACE
