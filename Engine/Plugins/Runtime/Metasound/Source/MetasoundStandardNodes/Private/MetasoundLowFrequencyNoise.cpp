// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "DSP/Dsp.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_LowFrequencyNoise"

// Low Frequency Noise :
// Similar to an Oscillator has Frequency/Phase. At the start of each phase a random value is chosen between (0..1)
// Between each phase the output is an interpolation between the previous and the next value
// The interpolation method can be either, None (just snaps every new value), Linear (lerp last 2 points), Cubic (cubic between 4 points).
// Same as other oscillators phase can be reset and synced

namespace Metasound
{
	enum class ELowFrequencyNoiseInterpType
	{
		None,
		Linear,
		Cubic,
	};

	DECLARE_METASOUND_ENUM(ELowFrequencyNoiseInterpType, ELowFrequencyNoiseInterpType::Cubic, METASOUNDSTANDARDNODES_API,
		FEnumLowFrequencyNoiseInterpType, FEnumLowFrequencyNoiseInterpTypeInfo, FEnumLowFrequencyNoiseInterpTypeReadRef, FEnumLowFrequencyNoiseInterpTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ELowFrequencyNoiseInterpType, FEnumLowFrequencyNoiseInterpType, "LowFrequencyNoiseInterpType")
		DEFINE_METASOUND_ENUM_ENTRY(ELowFrequencyNoiseInterpType::None, "ELowFrequencyNoiseInterpTypeDescriptionNone",		"None",		"ELowFrequencyNoiseInterpTypeDescriptionNoneTT", "No interpolation"),
		DEFINE_METASOUND_ENUM_ENTRY(ELowFrequencyNoiseInterpType::Linear, "ELowFrequencyNoiseInterpTypeDescriptionLinear",	"Linear",	"ELowFrequencyNoiseInterpTypeDescriptionLinearTT", "Linear interpolation"),
		DEFINE_METASOUND_ENUM_ENTRY(ELowFrequencyNoiseInterpType::Cubic, "ELowFrequencyNoiseInterpTypeDescriptionCubic",	"Cubic",	"ELowFrequencyNoiseInterpTypeDescriptionCubicTT", "Cubic interpolation"),
	DEFINE_METASOUND_ENUM_END()

	namespace LowFrequencyNoiseVertexNames
	{
		METASOUND_PARAM(SyncPin, "Sync", "Reset Phase")
		METASOUND_PARAM(RatePin, "Rate", "Rate of each new value in Hz (clamped at BlockRate, same as LFO).")
		METASOUND_PARAM(RateJitterPin, "Rate Jitter", "How much % to randomly add (+/-) to Rate (0..1)")
		METASOUND_PARAM(StepLimitPin, "Step Limit", "Limit the next (unit) random number in sequence (i.e. 1 no limit, 0.25 max step 25% (+/-) from current value")
		METASOUND_PARAM(SeedPin, "Seed", "Value to seed the Random Number Generator (-1 will use system rand())")
		METASOUND_PARAM(ResetSeedPin, "Reset Seed", "Resets the seed to what's set in the seed input")
		METASOUND_PARAM(InterpTypePin, "Interpolation", "Interpolation algorthm to use (defaults is cubic)")
		METASOUND_PARAM(MinOutputValuePin, "Min Value", "Minimum output value")
		METASOUND_PARAM(MaxOutputValuePin, "Max Value", "Maximum output value")
		METASOUND_PARAM(NormalizedOutputPin, "Normalized", "Value before scaling (0..1)")
		METASOUND_PARAM(ScaledOutputPin, "Out", "Output value scaled (Min Value...Max Value)")
	}

	class FLowFrequencyNoiseOperator : public TExecutableOperator<FLowFrequencyNoiseOperator>
	{
	public:
		// Helper to hide management of the state
		// Multi-tap delay line of current/previous values.
		struct FValueFIFO
		{
			enum { NumValues = 4, CurrentIndex = NumValues / 2};
			float Values[NumValues]={0};
			
			const int32 Num() const
			{
				return NumValues;
			}
			void Push(const float InNewValue)
			{
				// Shuffle down
				for(int32 i=0; i < NumValues - 1; ++i)
				{
					Values[i] = Values[i+1];
				}

				// Add new
				Values[NumValues - 1] = InNewValue;
			}
			// 0 is current, -1 previous, +1 future etc.
			float GetValue(const int32 InOffsetFromCurrent=0) const 
			{
				const int32 Index = CurrentIndex + InOffsetFromCurrent;
				check(Index >=0 && Index < NumValues);
				return Values[Index];
			};		
		};
		
		static const FVertexInterface& GetVertexInterface()
		{			
			using namespace LowFrequencyNoiseVertexNames;

			static const FVertexInterface Interface
			{
				FInputVertexInterface
				{
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(RatePin), 5.f),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(SeedPin), -1),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(ResetSeedPin)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(SyncPin)),
					TInputDataVertex<FEnumLowFrequencyNoiseInterpType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InterpTypePin), static_cast<int32>(ELowFrequencyNoiseInterpType::Cubic)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(RateJitterPin), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(StepLimitPin), 1.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MinOutputValuePin), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MaxOutputValuePin), 1.0f),
				},
				FOutputVertexInterface
				{
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ScaledOutputPin)),
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(NormalizedOutputPin))
				}
			};
			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Lfo Frequency Noise"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_LowFrequencyNoiseNodeDisplayName", "Low Frequency Noise");
				Info.Description = METASOUND_LOCTEXT("Metasound_LowFrequencyNoiseNodeDescription", "Produces random values at a Rate and interpolates between them");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.CategoryHierarchy.Emplace(NodeCategories::RandomUtils);			

				Info.Keywords.Emplace(METASOUND_LOCTEXT("Metasound_LowFrequencyNoiseNodeKeyword_Rand", "Rand"));
								
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		struct FPinReadRefs
		{
			FFloatReadRef Rate;
			FFloatReadRef RateJitter;
			FFloatReadRef StepLimit;
			FInt32ReadRef Seed;
			FEnumLowFrequencyNoiseInterpTypeReadRef InterpType;
			FTriggerReadRef ResetSeed;
			FTriggerReadRef Sync;
			FFloatReadRef MinOutput; 
			FFloatReadRef MaxOutput;
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			using namespace LowFrequencyNoiseVertexNames;

			FPinReadRefs Pins
			{
				  InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(RatePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(RateJitterPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(StepLimitPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(SeedPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<FEnumLowFrequencyNoiseInterpType>(METASOUND_GET_PARAM_NAME(InterpTypePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(ResetSeedPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(SyncPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MinOutputValuePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MaxOutputValuePin), Settings)
			};

			return MakeUnique<FLowFrequencyNoiseOperator>(InParams, MoveTemp(Pins));
		}

		FLowFrequencyNoiseOperator(const FBuildOperatorParams& InParams, FPinReadRefs&& InPins)
			: BlockRate{ InParams.OperatorSettings.GetActualBlockRate() }
			, Pins{ MoveTemp(InPins) }
			, ScaledOutput{ FFloatWriteRef::CreateNew(0.f) }
			, NormalizedOutput{ FFloatWriteRef::CreateNew(0.f) }
		{	
			Reset(InParams);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			Phase = 0.f;
			*ScaledOutput = 0.f;
			*NormalizedOutput = 0.f;

			SetSeedAndFillValues();
		}

		void SetSeedAndFillValues()
		{		
			SetSeed(*Pins.Seed);
		
			// Every time we change the seed, in order for the results to be deterministic 
			// we must also also flush and fill the value/delay line. This will create a discontinuation in the 
			// interpolation, but will be consistent.
			for(int32 i = 0; i < Values.Num(); ++i)
			{
				Values.Push(GenerateRandomValue());
			}

			// The random jitter should be chosen again following the seed change.
			CurrentRandomRate = GenerateRandomRate();
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LowFrequencyNoiseVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RatePin), Pins.Rate);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RateJitterPin), Pins.RateJitter);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(StepLimitPin), Pins.StepLimit);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(SeedPin), Pins.Seed);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ResetSeedPin), Pins.ResetSeed);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(SyncPin), Pins.Sync);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InterpTypePin), Pins.InterpType);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MinOutputValuePin), Pins.MinOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MaxOutputValuePin), Pins.MaxOutput);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LowFrequencyNoiseVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ScaledOutputPin), ScaledOutput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(NormalizedOutputPin), NormalizedOutput);
		}

		void SetSeed(int32 InSeedValue)
		{
			if (InSeedValue != -1)
			{
				RandomStream.Initialize(InSeedValue);
				RandomJitter.Initialize(InSeedValue);
			}
			else
			{
				RandomStream.GenerateNewSeed();
				RandomJitter.GenerateNewSeed();
			}
			CurrentSeed = InSeedValue;
		}

		// Generate a value between 0..1
		float GenerateRandomValue()
		{
			float RandomFraction = RandomStream.GetFraction();
			float PreviousValue = Values.GetValue();				// On init this will be zero.
			
			float LimitClamped = FMath::Clamp(*Pins.StepLimit, 0.f, 1.f);
			float RangeMin = FMath::Max(PreviousValue - LimitClamped, 0.f);
			float RangeMax = FMath::Min(PreviousValue + LimitClamped, 1.f);
			float LimitedRandom = FMath::GetRangeValue(FVector2d(RangeMin, RangeMax), RandomFraction);
			return LimitedRandom;
		}

		float GenerateRandomRate()
		{
			const float Nyquist = BlockRate / 2.f;
			float RandomFraction = RandomJitter.GetFraction();
			float ClampedJitter =  FMath::Clamp(*Pins.RateJitter, 0.f, 1.f);
			float RangeMin = FMath::Max(*Pins.Rate - (*Pins.Rate * ClampedJitter), 0.f);
			float RangeMax = FMath::Min(*Pins.Rate + (*Pins.Rate * ClampedJitter), Nyquist);
			float JitteredRate = FMath::GetRangeValue(FVector2d(RangeMin, RangeMax), RandomFraction);
			return JitteredRate;
		}
		
		// Taken from: http://paulbourke.net/miscellaneous/interpolation/
		// Different from FMath::CubicInterp as its requires no tension points.
		static float CubicInterpolate(
			float Y0, float Y1,
			float Y2, float Y3,
			float MU)
		{
			float MU2 = MU * MU;
			float A0 = Y3 - Y2 - Y0 + Y1;
			float A1 = Y0 - Y1 - A0;
			float A2 = Y2 - Y0;
			float A3 = Y1;

			return (A0 * MU * MU2 + A1 * MU2 + A2 * MU + A3);
		}
		
		float Interpolate() const
		{
			switch(*Pins.InterpType)
			{
				case ELowFrequencyNoiseInterpType::None:
				{
					return Values.GetValue(0);
				}
				case ELowFrequencyNoiseInterpType::Linear:
				{
					return FMath::Lerp(Values.GetValue(0), Values.GetValue(1), Phase);
				}
				default:
				case ELowFrequencyNoiseInterpType::Cubic:
				{
					return CubicInterpolate(Values.GetValue(-2), Values.GetValue(-1), Values.GetValue(0), Values.GetValue(1), Phase);
				}
			}
		}

		void HandleSyncChanges()
		{
			if (Pins.Sync->IsTriggeredInBlock())
			{
				Phase = 0.f;
			}
		}

		void HandleSeedChanges()
		{
			if (CurrentSeed != *Pins.Seed || Pins.ResetSeed->IsTriggeredInBlock())
			{
				SetSeedAndFillValues();
			}
		}

		void HandleRateChanges()
		{
			if (!FMath::IsNearlyEqual(CurrentRate, *Pins.Rate))
			{
				CurrentRate = *Pins.Rate;
				CurrentRandomRate = GenerateRandomRate();
			}
		}

		void UpdateCurrentValue()
		{
			if (Phase > 1.f)
			{
				Values.Push(GenerateRandomValue());
				Phase -= 1.f;
				CurrentRandomRate = GenerateRandomRate();
			}
		}

		void UpdateOutputs()
		{
			// Generate in-between values by interpolating (with a chosen algorithm) 
			float InterpolatedValue = Interpolate();

			// Output the raw value (0..1) for the normalized output.
			*NormalizedOutput = InterpolatedValue;

			// Output the scaled version.
			float OutputValue = FMath::GetMappedRangeValueClamped(
				FVector2f{ 0.f, 1.0f }, 
				FVector2f{ *Pins.MinOutput, *Pins.MaxOutput }, 
			InterpolatedValue );

			*ScaledOutput = OutputValue;
		}

		void UpdatePhase()
		{
			// Similar to an LFO, we calculate our rate as a fraction of the block-rate 
			check(BlockRate > 0.f)
			const float Nyquist = BlockRate / 2.f;
			const float ClampedFreq = FMath::Clamp(CurrentRandomRate, 0.f, Nyquist);
			const float DeltaPhase = ClampedFreq * (1.f / BlockRate);

			Phase += DeltaPhase;
		}

		void HandlePinChanges()
		{
			HandleRateChanges();
			HandleSyncChanges();
			HandleSeedChanges();
		}

		void Execute()
		{
			HandlePinChanges();

			UpdateCurrentValue();
			UpdateOutputs();
			UpdatePhase();
		}

	private:
		float BlockRate = 0.f;
		float Phase = 0.f;
		float CurrentRandomRate = 0.f;
		float CurrentRate = 0.f;
		int32 CurrentSeed = 0;
		FValueFIFO Values;
		FRandomStream RandomStream;
		FRandomStream RandomJitter;
		FPinReadRefs Pins;
		FFloatWriteRef ScaledOutput;		// Scaled output value (min/max)
		FFloatWriteRef NormalizedOutput;	// Unit value (0..1)
	};

	using FRandomLowFrequencyNoiseNode = TNodeFacade<FLowFrequencyNoiseOperator>;

	METASOUND_REGISTER_NODE(FRandomLowFrequencyNoiseNode);
}

#undef LOCTEXT_NAMESPACE

