// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MidiToFreqNode"

namespace Metasound
{
	namespace MidiToFrequencyVertexNames
	{
		METASOUND_PARAM(InputMidi, "MIDI In", "A value representing a MIDI note value.");
		METASOUND_PARAM(OutputFreq, "Out Frequency", "Output frequency value in hertz that corresponds to the input Midi note value.");
	}

	namespace MidiToFrequencyPrivate
	{
		template<typename ValueType>
		struct TMidiToFreqNodeSpecialization
		{
		};


		template<>
		struct TMidiToFreqNodeSpecialization<int32>
		{
			static int32 GetFreqValue(int32 InMidi)
			{
				return Audio::GetFrequencyFromMidi(FMath::Clamp(InMidi, 0, 127));
			}

			static bool IsValueEqual(int32 InValueA, int32 InValueB)
			{
				return InValueA == InValueB; 
			}
		};

		template<>
		struct TMidiToFreqNodeSpecialization<float>
		{
			static float GetFreqValue(float InMidi)
			{
				return Audio::GetFrequencyFromMidi(FMath::Clamp(InMidi, 0.0f, 127.0f));
			}

			static bool IsValueEqual(float InValueA, float InValueB)
			{
				return FMath::IsNearlyEqual(InValueA, InValueB);
			}
		};
	}

	template<typename ValueType>
	class TMidiToFreqOperator : public TExecutableOperator<TMidiToFreqOperator<ValueType>>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		TMidiToFreqOperator(const FBuildOperatorParams& InParams, const TDataReadReference<ValueType>& InMidiNote);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private:
		// The input Midi value
		TDataReadReference<ValueType> MidiNote;

		// The output frequency
		FFloatWriteRef FreqOutput;

		// Cached Midi note value. Used to catch if the value changes to recompute freq output.
		ValueType PrevMidiNote;
	};

	template<typename ValueType>
	TMidiToFreqOperator<ValueType>::TMidiToFreqOperator(const FBuildOperatorParams& InParams, const TDataReadReference<ValueType>& InMidiNote)
		: MidiNote(InMidiNote)
		, FreqOutput(FFloatWriteRef::CreateNew(Audio::GetFrequencyFromMidi(*InMidiNote)))
		, PrevMidiNote(*InMidiNote)
	{
		Reset(InParams);
	}

	template<typename ValueType>
	void TMidiToFreqOperator<ValueType>::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace MidiToFrequencyVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMidi), MidiNote);
	}

	template<typename ValueType>
	void TMidiToFreqOperator<ValueType>::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace MidiToFrequencyVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputFreq), FreqOutput);
	}

	template<typename ValueType>
	void TMidiToFreqOperator<ValueType>::Reset(const IOperator::FResetParams& InParams)
	{
		using namespace MidiToFrequencyPrivate;
		PrevMidiNote = *MidiNote;
		*FreqOutput = TMidiToFreqNodeSpecialization<ValueType>::GetFreqValue(PrevMidiNote);
	}

	template<typename ValueType>
	void TMidiToFreqOperator<ValueType>::Execute()
	{
		using namespace MidiToFrequencyPrivate;

		// Only do anything if the Midi note changes
		if (!TMidiToFreqNodeSpecialization<ValueType>::IsValueEqual(*MidiNote, PrevMidiNote))
		{
			PrevMidiNote = *MidiNote;

			*FreqOutput = TMidiToFreqNodeSpecialization<ValueType>::GetFreqValue(PrevMidiNote);
		}
	}

	template<typename ValueType>
	const FVertexInterface& TMidiToFreqOperator<ValueType>::GetVertexInterface()
	{
		using namespace MidiToFrequencyVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMidi), (ValueType)60.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputFreq))
			)
		);

		return Interface;
	}

	template<typename ValueType>
	const FNodeClassMetadata& TMidiToFreqOperator<ValueType>::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
			const FName OperatorName = TEXT("MIDI To Frequency");
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("RandomGetArrayOpDisplayNamePattern", "MIDI To Frequency ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_MidiToFreqNodeDescription", "Converts a Midi note value to a frequency (hz) value.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Music);
			Info.Keywords.Add(METASOUND_LOCTEXT("MIDIToFreqPitchKeyword", "Pitch"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	template<typename ValueType>
	TUniquePtr<IOperator> TMidiToFreqOperator<ValueType>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace MidiToFrequencyVertexNames;
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		TDataReadReference<ValueType> InMidiNote = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputMidi), InParams.OperatorSettings);

		return MakeUnique<TMidiToFreqOperator>(InParams, InMidiNote);
	}

	template<typename ValueType>
	using TMidiToFreqNode = TNodeFacade<TMidiToFreqOperator<ValueType>>;

	using FMidiToFreqNodeInt32 = TMidiToFreqNode<int32>;
	METASOUND_REGISTER_NODE(FMidiToFreqNodeInt32)

	using FMidiToFreqNodeFloat = TMidiToFreqNode<float>;
	METASOUND_REGISTER_NODE(FMidiToFreqNodeFloat)
}

#undef LOCTEXT_NAMESPACE
