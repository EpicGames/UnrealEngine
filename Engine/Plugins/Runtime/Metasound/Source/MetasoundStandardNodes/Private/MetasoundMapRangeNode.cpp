// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundAudioBuffer.h"
#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MapRangeNode"

namespace Metasound
{
	namespace MapRangeVertexNames
	{
		METASOUND_PARAM(InputValueName, "In", "Input value to map.");
		METASOUND_PARAM(InputInRangeAName, "In Range A", "The min input value range.");
		METASOUND_PARAM(InputInRangeBName, "In Range B", "The max input value range.");
		METASOUND_PARAM(InputOutRangeAName, "Out Range A", "The min output value range.");
		METASOUND_PARAM(InputOutRangeBName, "Out Range B", "The max output value range.");
		METASOUND_PARAM(InputClampedName, "Clamped", "Whether or not to clamp the input to the specified input range.");

		METASOUND_PARAM(OutputValueName, "Out Value", "Mapped output value.");
	}

	namespace MetasoundMapRangeNodePrivate
	{
		using namespace MapRangeVertexNames;

		class FIntRange
		{
		public:
			static FName GetDataTypeName()
			{
				return TEXT("Int32");
			}

			static FText GetNodeName()
			{
				return METASOUND_LOCTEXT("MapRange_Int32Name", "Map Range (Int32)");
			}

			static FVertexInterface GetVertexInterface()
			{
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValueName)),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeAName), 0),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeBName), 100),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeAName), 0),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeBName), 100),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputClampedName), true)
					),
					FOutputVertexInterface(
						TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValueName))
					)
				);
				return DefaultInterface;
			}

			FIntRange(const FOperatorSettings& OperatorSettings, const FInputVertexInterfaceData& InInputData)
				: Value(InInputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputValueName), OperatorSettings))
				, InRangeA(InInputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputInRangeAName), OperatorSettings))
				, InRangeB(InInputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputInRangeBName), OperatorSettings))
				, OutRangeA(InInputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputOutRangeAName), OperatorSettings))
				, OutRangeB(InInputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputOutRangeBName), OperatorSettings))
				, bClamped(InInputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputClampedName), OperatorSettings))
				, OutputValue(TDataWriteReferenceFactory<int32>::CreateAny(OperatorSettings))
			{
			}

			void BindInputs(FInputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValueName), Value);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeAName), InRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeBName), InRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeAName), OutRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeBName), OutRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputClampedName), bClamped);
			}

			void BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValueName), OutputValue);
			}

			void DoMapping()
			{
				if (*bClamped)
				{
					*OutputValue = (int32)FMath::GetMappedRangeValueClamped(FVector2f{ (float)*InRangeA, (float)*InRangeB }, FVector2f{ (float)*OutRangeA, (float)*OutRangeB }, (float)*Value);
				}
				else
				{
					*OutputValue = (int32)FMath::GetMappedRangeValueUnclamped(FVector2f{ (float)*InRangeA, (float)*InRangeB }, FVector2f{ (float)*OutRangeA, (float)*OutRangeB }, (float)*Value);
				}
			}

		private:
			TDataReadReference<int32> Value;
			TDataReadReference<int32> InRangeA;
			TDataReadReference<int32> InRangeB;
			TDataReadReference<int32> OutRangeA;
			TDataReadReference<int32> OutRangeB;
			FBoolReadRef bClamped;
			TDataWriteReference<int32> OutputValue;
		};

		class FFloatRange
		{
		public:
			static FName GetDataTypeName()
			{
				return TEXT("Float");
			}

			static FText GetNodeName()
			{
				return METASOUND_LOCTEXT("MapRange_FloatName", "Map Range (Float)");
			}

			static FVertexInterface GetVertexInterface()
			{
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValueName)),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeAName), 0.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeBName), 1.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeAName), 0.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeBName), 1.0f),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputClampedName), true)
					),
					FOutputVertexInterface(
						TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValueName))
					)
				);
				return DefaultInterface;
			}

			FFloatRange(const FOperatorSettings& OperatorSettings, const FInputVertexInterfaceData& InInputData)
				: Value(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputValueName),		OperatorSettings))
				, InRangeA(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputInRangeAName),	OperatorSettings))
				, InRangeB(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputInRangeBName),	OperatorSettings))
				, OutRangeA(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputOutRangeAName),	OperatorSettings))
				, OutRangeB(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputOutRangeBName),	OperatorSettings))
				, bClamped(InInputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputClampedName),		OperatorSettings))
				, OutputValue(TDataWriteReferenceFactory<float>::CreateAny(OperatorSettings))
			{
			}

			void BindInputs(FInputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValueName),Value);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeAName), InRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeBName), InRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeAName), OutRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeBName), OutRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputClampedName), bClamped);
			}

			void BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValueName), OutputValue);
			}


			void DoMapping()
			{
				if (*bClamped)
				{
					*OutputValue = FMath::GetMappedRangeValueClamped(FVector2f{ *InRangeA, *InRangeB }, FVector2f{ *OutRangeA, *OutRangeB }, *Value);
				}
				else
				{
					*OutputValue = FMath::GetMappedRangeValueUnclamped(FVector2f{ *InRangeA, *InRangeB }, FVector2f{ *OutRangeA, *OutRangeB }, *Value);
				}
			}

		private:
			TDataReadReference<float> Value;
			TDataReadReference<float> InRangeA;
			TDataReadReference<float> InRangeB;
			TDataReadReference<float> OutRangeA;
			TDataReadReference<float> OutRangeB;
			FBoolReadRef bClamped;
			TDataWriteReference<float> OutputValue;
		};

		class FAudioRange
		{
		public:
			static FName GetDataTypeName()
			{
				return TEXT("Audio");
			}

			static FText GetNodeName()
			{
				return METASOUND_LOCTEXT("MapRange_AudioName", "Map Range (Audio)");
			}

			static FVertexInterface GetVertexInterface()
			{
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(
						TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValueName)),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeAName), -1.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInRangeBName), 1.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeAName), -1.0f),
						TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOutRangeBName), 1.0f),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputClampedName), true)
					),
					FOutputVertexInterface(
						TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValueName))
					)
				);

				return DefaultInterface;
			}

			FAudioRange(const FOperatorSettings& OperatorSettings, const FInputVertexInterfaceData& InInputData)
				: Value(InInputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputValueName), OperatorSettings))
				, InRangeA(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputInRangeAName), OperatorSettings))
				, InRangeB(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputInRangeBName), OperatorSettings))
				, OutRangeA(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputOutRangeAName), OperatorSettings))
				, OutRangeB(InInputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputOutRangeBName), OperatorSettings))
				, bClamped(InInputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputClampedName), OperatorSettings))
				, OutputValue(TDataWriteReferenceFactory<FAudioBuffer>::CreateAny(OperatorSettings))
			{
			}


			void BindInputs(FInputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValueName), Value);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeAName), InRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInRangeBName), InRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeAName), OutRangeA);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOutRangeBName), OutRangeB);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputClampedName), bClamped);
			}

			void BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValueName), OutputValue);
			}


			void DoMapping()
			{
				FAudioBuffer& OutBuffer = *OutputValue;
				float* OutBufferPtr = OutBuffer.GetData();
				const FAudioBuffer& InBuffer = *Value;
				const float* InBufferPtr = InBuffer.GetData();
				int32 NumSamples = InBuffer.Num();

				FVector2f InputRange = { *InRangeA, *InRangeB };
				FVector2f OutputRange = { *OutRangeA, *OutRangeB };

				// TODO: SIMD this
				for (int32 i = 0; i < NumSamples; ++i)
				{
					OutBufferPtr[i] = FMath::GetMappedRangeValueClamped(InputRange, OutputRange, InBufferPtr[i]);
				}
			}

		private:
			TDataReadReference<FAudioBuffer> Value;
			TDataReadReference<float> InRangeA;
			TDataReadReference<float> InRangeB;
			TDataReadReference<float> OutRangeA;
			TDataReadReference<float> OutRangeB;
			FBoolReadRef bClamped;
			TDataWriteReference<FAudioBuffer> OutputValue;
		};
	}

	template<typename FMappingClass>
	class TMapRangeOperator : public TExecutableOperator<TMapRangeOperator<FMappingClass>>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FName OperatorName = TEXT("MapRange");
				const FText NodeDescription = METASOUND_LOCTEXT("MapRangeDescription", "Maps an input value in the given input range to the given output range.");
				FVertexInterface NodeInterface = FMappingClass::GetVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { "MapRange", OperatorName, FMappingClass::GetDataTypeName() },
					1, // Major Version
					0, // Minor Version
					FMappingClass::GetNodeName(),
					NodeDescription,
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Math },
					{ },
					FNodeDisplayStyle()
				};
				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			FMappingClass MappingObject(InParams.OperatorSettings, InParams.InputData);

			return MakeUnique<TMapRangeOperator<FMappingClass>>(MappingObject);
		}


		TMapRangeOperator(const FMappingClass& InMappingObject)
			: MappingObject(InMappingObject)
		{
			MappingObject.DoMapping();
		}

		virtual ~TMapRangeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			MappingObject.BindInputs(InOutVertexData);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			MappingObject.BindOutputs(InOutVertexData);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			MappingObject.DoMapping();
		}

		void Execute()
		{
			MappingObject.DoMapping();
		}

	private:
		FMappingClass MappingObject;
	};

	template<typename FMappingClass>
	using TMapRangeNode = TNodeFacade<TMapRangeOperator<FMappingClass>>;

	using FMapRangeNodeInt32 = TMapRangeNode<MetasoundMapRangeNodePrivate::FIntRange>;
 	METASOUND_REGISTER_NODE(FMapRangeNodeInt32)

	using FMapRangeNodeFloat = TMapRangeNode<MetasoundMapRangeNodePrivate::FFloatRange>;
	METASOUND_REGISTER_NODE(FMapRangeNodeFloat)

	using FMapRangeNodeAudio = TMapRangeNode<MetasoundMapRangeNodePrivate::FAudioRange>;
	METASOUND_REGISTER_NODE(FMapRangeNodeAudio)

}

#undef LOCTEXT_NAMESPACE
