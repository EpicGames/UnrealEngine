// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"


#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

#define DEFINE_METASOUND_MATHOP(OpName, DataTypeName, DataClass, Description, Keywords) \
	class F##OpName##DataTypeName##Node \
	: public TNodeFacade<TMathOperator<F##OpName##DataTypeName##Node, ::Metasound::TMathOp##OpName<DataClass, DataClass>, DataClass, DataClass>>\
	{ \
	public: \
		using TNodeFacade::TNodeFacade; \
		static FNodeClassName GetClassName() { return { ::Metasound::StandardNodes::Namespace, TEXT(#OpName), TEXT(#DataTypeName)}; } \
		static FText GetDisplayName() { return MathOpNames::Get##OpName##DisplayName<DataClass>(); } \
		static FText GetDescription() { return Description; } \
		static FName GetImageName() { return "MetasoundEditor.Graph.Node.Math." #OpName; } \
		static TArray<FText> GetKeywords() { return Keywords; } \
	};

#define DEFINE_METASOUND_OPERAND_TYPED_MATHOP(OpName, DataTypeName, DataClass, OperandTypeName, OperandDataClass, Description, Keywords) \
	class F##OpName##DataTypeName##OperandTypeName##Node \
	: public TNodeFacade<TMathOperator<F##OpName##DataTypeName##OperandTypeName##Node, ::Metasound::TMathOp##OpName<DataClass, OperandDataClass>, DataClass, OperandDataClass>> \
	{ \
	public: \
		using TNodeFacade::TNodeFacade; \
		static FNodeClassName GetClassName() { return {::Metasound::StandardNodes::Namespace, TEXT(#OpName), TEXT(#DataTypeName " by " #OperandTypeName)}; } \
		static FText GetDisplayName() { return MathOpNames::Get##OpName##DisplayName<DataClass, OperandDataClass>(); } \
		static FText GetDescription() { return Description; } \
		static FName GetImageName() { return "MetasoundEditor.Graph.Node.Math." #OpName; } \
		static TArray<FText> GetKeywords() { return Keywords; } \
	};

namespace Metasound
{
	namespace MathOpNames
	{
		METASOUND_PARAM(PrimaryOperand, "PrimaryOperand", "Initial operand")
		METASOUND_PARAM(AdditionalOperands, "AdditionalOperands", "Additional operand(s)")

		METASOUND_PARAM(OutMath, "Out", "Math operation result.")
		METASOUND_PARAM(OutAudio, "Out", "Resulting buffer.")

		static const TArray<FText> AddKeywords = { METASOUND_LOCTEXT("AddMathKeyword", "+") };
		static const TArray<FText> SubtractKeywords = { METASOUND_LOCTEXT("SubtractMathKeyword", "-") };
		static const TArray<FText> MultiplyKeywords = { METASOUND_LOCTEXT("MultiplyMathKeyword", "*") };
		static const TArray<FText> DivideKeywords = { METASOUND_LOCTEXT("DivideMathKeyword", "/") };
		static const TArray<FText> PowerKeywords = { METASOUND_LOCTEXT("PowerMathKeyword", "^") };
		static const TArray<FText> ModuloKeywords = { METASOUND_LOCTEXT("ModuloMathKeyword", "%") };
		
		static const TArray<FText> MultiplyAudioFloatKeywords =
			{
				MultiplyKeywords[0],
				METASOUND_LOCTEXT("MultiplyGainKeyword", "gain"), // DSP/Modular synth term
				METASOUND_LOCTEXT("MultiplyVCAKeyword", "vca"),   // Voltage Controlled Amplifier (DAW/Electronics term)
			};

		template<typename DataType>
		const FText GetAddDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("AddNodeDisplayNamePattern1", "Add ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetAddDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("AddNodeDisplayNamePattern2", "Add ({0} to {1})",
				GetMetasoundDataTypeDisplayText<OperandDataType>(),
				GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetSubtractDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("SubtractNodeDisplayNamePattern", "Subtract ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetSubtractDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("SubtractNodeOperandTypedDisplayNamePattern", "Subtract ({0} from {1})",
				GetMetasoundDataTypeDisplayText<OperandDataType>(),
				GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetMultiplyDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("MultiplyNodeDisplayNamePattern", "Multiply ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetMultiplyDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("MultiplyNodeOperandTypedDisplayNamePattern", "Multiply ({0} by {1})",
				GetMetasoundDataTypeDisplayText<DataType>(),
				GetMetasoundDataTypeDisplayText<OperandDataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetDivideDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("DivideNodeDisplayNamePattern", "Divide ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetDivideDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("DivideNodeOperandTypedDisplayNamePattern", "Divide ({0} by {1})",
				GetMetasoundDataTypeDisplayText<DataType>(),
				GetMetasoundDataTypeDisplayText<OperandDataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetModuloDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("ModuloNodeDisplayNamePattern", "Modulo ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetModuloDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("ModuloNodeOperandTypedDisplayNamePattern", "Modulo ({0} by {1})",
				GetMetasoundDataTypeDisplayText<DataType>(),
				GetMetasoundDataTypeDisplayText<OperandDataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetPowerDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("PowerNodeDisplayNamePattern", "Power ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetPowerDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("PowerNodeOperandTypedDisplayNamePattern", "Power ({0} to the power of {1})",
				GetMetasoundDataTypeDisplayText<DataType>(),
				GetMetasoundDataTypeDisplayText<OperandDataType>());
			return DisplayName;
		}

		template<typename DataType>
		const FText GetLogarithmDisplayName()
		{
			static const FText DisplayName = METASOUND_LOCTEXT_FORMAT("LogNodeDisplayNamePattern", "Log ({0})", GetMetasoundDataTypeDisplayText<DataType>());
			return DisplayName;
		}

		template<typename DataType, typename OperandDataType>
		const FText GetLogarithmDisplayName()
		{
			static const FText DisplayName = 
				METASOUND_LOCTEXT_FORMAT("LogarithmNodeOperandTypedDisplayNamePattern", "Log ({0}-Base logarithm of {1})",
				GetMetasoundDataTypeDisplayText<DataType>(),
				GetMetasoundDataTypeDisplayText<OperandDataType>());
			return DisplayName;
		}
	}

	template <typename TDerivedClass, typename TMathOpClass, typename TDataClass, typename TOperandDataClass>
	class TMathOperator : public TExecutableOperator<TMathOperator<TDerivedClass, TMathOpClass, TDataClass, TOperandDataClass>>
	{
	private:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		TMathOpClass InstanceData;

		TDataClassReadRef PrimaryOperandRef;
		TArray<TOperandDataClassReadRef> AdditionalOperandRefs;
		TDataClassWriteRef ValueRef;

	public:
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeDisplayStyle DisplayStyle;
				DisplayStyle.ImageName = TDerivedClass::GetImageName();
				DisplayStyle.bShowName = false;
				DisplayStyle.bShowInputNames = false;
				DisplayStyle.bShowOutputNames = false;

				FNodeClassMetadata Info;
				Info.ClassName = TDerivedClass::GetClassName();
				Info.MajorVersion = 1;
				Info.MinorVersion = 1;
				Info.DisplayName = TDerivedClass::GetDisplayName();
				Info.Description = TDerivedClass::GetDescription();
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = TMathOpClass::GetVertexInterface();
				Info.DisplayStyle = DisplayStyle;
				Info.CategoryHierarchy = { NodeCategories::Math };
				Info.Keywords = TDerivedClass::GetKeywords();

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}


		void Reset(const IOperator::FResetParams& InParams)
		{
			TMathOpClass::Reset(InstanceData, PrimaryOperandRef, AdditionalOperandRefs, ValueRef);
		}

		void Execute()
		{
			TMathOpClass::Calculate(InstanceData, PrimaryOperandRef, AdditionalOperandRefs, ValueRef);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MathOpNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TDataClassReadRef PrimaryOperand = InputData.GetOrCreateDefaultDataReadReference<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), InParams.OperatorSettings);

			// TODO: Support dynamic number of inputs
			TOperandDataClassReadRef Op1 = InputData.GetOrCreateDefaultDataReadReference<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), InParams.OperatorSettings);
			TArray<TOperandDataClassReadRef> AdditionalOperandRefs = { Op1 };

			return MakeUnique<TMathOperator>(InParams, PrimaryOperand, AdditionalOperandRefs);
		}

		TMathOperator(const FBuildOperatorParams& InParams, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands)
			: PrimaryOperandRef(InPrimaryOperand)
			, AdditionalOperandRefs(InAdditionalOperands)
			, ValueRef(TDataWriteReferenceFactory<TDataClass>::CreateAny(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MathOpNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandRef);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandRefs[0]);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MathOpNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutMath), ValueRef);
		}
	};



	// Standard Math Operations
	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpAdd
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddNode_InitialTooltip", "Initial addend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalAddendsMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddNode_AddendsTooltip", "Additional addend(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddNode_OutTooltip", "Math operation result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, static_cast<TDataClass>(0)),
					TInputDataVertex<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalAddendsMetadata, static_cast<TOperandDataClass>(0))
				),
				FOutputVertexInterface(
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}
	
		static void Reset(TMathOpAdd<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpAdd<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				*OutResult += *InAdditionalOperands[i];
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpSubtract
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames; 

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpSubtractNode_MinuendTooltip", "Minuend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpSubtractNode_SubtrahendsTooltip", "Subtrahend(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpSubtractNode_OutTooltip", "Subtraction result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, static_cast<TDataClass>(0)),
					TInputDataVertex<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, static_cast<TOperandDataClass>(0))
				),
				FOutputVertexInterface(
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}
		
		static void Reset(TMathOpSubtract<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}
		
		static void Calculate(TMathOpSubtract<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				*OutResult -= *InAdditionalOperands[i];
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpMultiply
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpMultiplyNode_InitMultiplicandTooltip", "Initial multiplicand.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				 METASOUND_LOCTEXT("MathOpMultiplyNode_MultiplicandsTooltip", "Additional multiplicand(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpMultiplyNode_ResultTooltip", "Multiplication result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, static_cast<TDataClass>(1)),
					TInputDataVertex<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, static_cast<TOperandDataClass>(1))
				),
				FOutputVertexInterface(
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Reset(TMathOpMultiply<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpMultiply<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				*OutResult *= *InAdditionalOperands[i];
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpDivide
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpDivideNode_DividendTooltip", "Dividend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpDivideNode_DivisorsTooltip", "Divisor(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpDivideNode_OutTooltip", "Division result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, static_cast<TDataClass>(1)),
					TInputDataVertex<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, static_cast<TOperandDataClass>(1))
				),
				FOutputVertexInterface(
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Reset(TMathOpDivide<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpDivide& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const TDataClass& OperandValue = *InAdditionalOperands[i];
				if (OperandValue == static_cast<TDataClass>(0))
				{
					// TODO: Error here
					return;
				}

				*OutResult /= OperandValue;
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpModulo
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpModuloNode_DividendTooltip", "Dividend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpModuloNode_DivisorsTooltip", "Divisor(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpModuloNode_OutTooltip", "Modulo result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, static_cast<TDataClass>(1)),
					TInputDataVertex<TOperandDataClass>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, static_cast<TOperandDataClass>(1))
				),
				FOutputVertexInterface(
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Reset(TMathOpModulo<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpModulo& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;

			// TODO: chaining modulo operations doesn't make too much sense... how do we forbid additional operands in some math ops?
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const TDataClass& OperandValue = *InAdditionalOperands[i];
				if (OperandValue == static_cast<TDataClass>(0))
				{
					// TODO: Error here
					return;
				}

				*OutResult %= OperandValue;
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpPower
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpPowerNode_Base", "The base of the power") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpPowerNode_Exponent", "The exponent to take the base to the power of") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
					  };

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpPowerNode_Result", "Returns Base to the Exponent power") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata)
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return DefaultInterface;
		}

		static void Reset(TMathOpPower<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpPower& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const TDataClass& OperandValue = *InAdditionalOperands[i];

				*OutResult = FMath::Pow(*OutResult, OperandValue);
			}
		}
	};

	template <typename TDataClass, typename TOperandDataClass = TDataClass>
	class TMathOpLogarithm
	{
	public:
		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		using TOperandDataClassReadRef = TDataReadReference<TOperandDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				   METASOUND_LOCTEXT("MathOpLogarithmNode_Base", "The base of the logarithm") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpLogarithmNode_Value", "The value to find the logarithm of") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpLogarithmNode_Result", "The logarithm of the inputted value") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata)
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return DefaultInterface;
		}

		static void Reset(TMathOpLogarithm<TDataClass, TOperandDataClass>& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

		static void Calculate(TMathOpLogarithm& InInstanceData, const TDataClassReadRef& InPrimaryOperand, const TArray<TOperandDataClassReadRef>& InAdditionalOperands, TDataClassWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;

			//TODO: Find out how to disallow it from additional inputs, it doesn't really make sense in the context of logarithms
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const TDataClass& OperandValue = *InAdditionalOperands[i];

				*OutResult = FMath::LogX(FMath::Max(SMALL_NUMBER, *OutResult), FMath::Max(SMALL_NUMBER, OperandValue));
			}
		}
	};

	// Specialized Math Operations
	template <>
	class TMathOpAdd<FAudioBuffer>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames; 

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddAudioBufferNameTooltip", "First addend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddAudioBufferAdditionalTooltip", "Additional addends.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpAdd<FAudioBuffer>& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				OutResult->Zero();
				return;
			}

			const int32 NumSamples = InAdditionalOperands[0]->Num();
			if (!ensure(NumSamples == OutResult->Num()))
			{
				// TODO: Error
				OutResult->Zero();
				return;
			}

			FMemory::Memcpy(OutResult->GetData(), InPrimaryOperand->GetData(), sizeof(float) * NumSamples);

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const FAudioBuffer& Operand = *InAdditionalOperands[i];
				if (!ensure(NumSamples == Operand.Num()))
				{
					// TODO: Error
					OutResult->Zero();
					return;
				}

				TArrayView<const float> OperandDataView(InAdditionalOperands[i]->GetData(), NumSamples);
				TArrayView<float> OutDataView(OutResult->GetData(), NumSamples);

				if (NumSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					Audio::ArrayMixIn(OperandDataView, OutDataView);
				}
				else
				{
					const float StartGain = 1.0f;
					const float EndGain = 1.0f;
					Audio::ArrayMixIn(OperandDataView, OutDataView, StartGain, EndGain);
				}
			}
		}

		static void Reset(TMathOpAdd<FAudioBuffer>& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpAdd<FTime>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddTimeTooltip", "First addend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddTimeAdditionalalTooltip", "Additional addends.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpOutTooltip", "Math operation result") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, 0.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpAdd& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FTimeReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				*OutResult += *InAdditionalOperands[i];
			}
		}

		static void Reset(TMathOpAdd& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FTimeReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpAdd<FAudioBuffer, float>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames; 

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddAudioTooltip", "Audio Buffer to add offset(s) to.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpAddAdditionalTooltip", "Float addends of which to offset buffer samples.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpAdd& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			FMemory::Memcpy(OutResult->GetData(), InPrimaryOperand->GetData(), sizeof(float) * OutResult->Num());

			TArrayView<float> OutResultView(OutResult->GetData(), OutResult->Num());

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				Audio::ArrayAddConstantInplace(OutResultView, *InAdditionalOperands[i]);
			}
		}

		static void Reset(TMathOpAdd& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpSubtract<FAudioBuffer>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpBuffersMinuendTooltip", "Initial buffer to act as minuend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalAddendsMetadata
			{
				 METASOUND_LOCTEXT("MathOpSubtractBuffersSubtrahendsTooltip", "Additional buffers to act as subtrahend(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalAddendsMetadata)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpSubtract& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				OutResult->Zero();
				return;
			}

			const int32 NumSamples = InAdditionalOperands[0]->Num();
			if (!ensure(NumSamples == OutResult->Num()))
			{
				// TODO: Error
				OutResult->Zero();
				return;
			}

			FMemory::Memcpy(OutResult->GetData(), InPrimaryOperand->GetData(), sizeof(float) * NumSamples);

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const FAudioBuffer& Operand = *InAdditionalOperands[i];
				if (!ensure(NumSamples == Operand.Num()))
				{
					// TODO: Error
					OutResult->Zero();
					return;
				}

				TArrayView<const float> OperandDataView(InAdditionalOperands[i]->GetData(), NumSamples);
				TArrayView<float> OutDataView(OutResult->GetData(), NumSamples);

				Audio::ArraySubtractInPlace2(OutDataView, OperandDataView);
			}
		}

		static void Reset(TMathOpSubtract& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpSubtract<FTime>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;
			
			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				 METASOUND_LOCTEXT("MathOpTimeNode_MinuendTooltip", "Time minuend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalAddendsMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeNode_SubtrahendsTooltip", "Time subtrahends.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeNode_SubtractOutTooltip", "Resulting time value") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, 0.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalAddendsMetadata, 0.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpSubtract& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FTimeReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			*OutResult = *InPrimaryOperand;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				*OutResult -= *InAdditionalOperands[i];
			}
		}

		static void Reset(TMathOpSubtract& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FTimeReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpMultiply<FAudioBuffer>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAudioMultiplyNode_InitMultiplicandTooltip", "Initial audio to multiply.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalAddendsMetadata
			{
				 METASOUND_LOCTEXT("MathOpAudioMultiplyNode_MultiplicandsTooltip", "Additional audio to multiply sample-by-sample.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalAddendsMetadata)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpMultiply& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				OutResult->Zero();
				return;
			}

			const int32 NumSamples = InPrimaryOperand->Num();
			if (!ensure(NumSamples == OutResult->Num()))
			{
				// TODO: Error
				OutResult->Zero();
				return;
			}

			if (InAdditionalOperands.Num() > 0)
			{
				const FAudioBuffer& Operand = *InAdditionalOperands[0];
				if (!ensure(NumSamples == Operand.Num()))
				{
					// TODO: Error
					OutResult->Zero();
					return;
				}

				TArrayView<const float> OperandADataView(InPrimaryOperand->GetData(), NumSamples);
				TArrayView<const float> OperandBDataView(Operand.GetData(), NumSamples);
				TArrayView<float> OutDataView(OutResult->GetData(), NumSamples);

				Audio::ArrayMultiply(OperandADataView, OperandBDataView, OutDataView);
			}
			else
			{
				FMemory::Memcpy(OutResult->GetData(), InPrimaryOperand->GetData(), sizeof(float) * NumSamples);
			}

			for (int32 i = 1; i < InAdditionalOperands.Num(); ++i)
			{
				const FAudioBuffer& Operand = *InAdditionalOperands[i];
				if (!ensure(NumSamples == Operand.Num()))
				{
					// TODO: Error
					OutResult->Zero();
					return;
				}

				TArrayView<const float> OperandDataView(InAdditionalOperands[i]->GetData(), NumSamples);
				TArrayView<float> OutDataView(OutResult->GetData(), NumSamples);

				Audio::ArrayMultiplyInPlace(OperandDataView, OutDataView);
			}
		}

		static void Reset(TMathOpMultiply& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FAudioBufferReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpMultiply<FAudioBuffer, float>
	{
		bool bInit = false;
		float LastGain = 0.0f;

	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames; 

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpAudioFloatMultiplyNode_FloatTooltip", "Audio multiplicand.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpAudioFloatMultiplyNode_MultiplicandTooltip", "Float multiplicand to apply sample-by-sample to audio. Interpolates over buffer size on value change.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, 1.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpMultiply& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			constexpr float MaxGain = TNumericLimits<float>::Max() / 1024.f;
			constexpr float MinGain = -MaxGain;

			if (!InInstanceData.bInit)
			{
				InInstanceData.bInit = true;
				InInstanceData.LastGain = 1.0f;

				for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
				{
					InInstanceData.LastGain *= FMath::Clamp(*InAdditionalOperands[i], MinGain, MaxGain);
				}
			}

			FMemory::Memcpy(OutResult->GetData(), InPrimaryOperand->GetData(), sizeof(float) * InPrimaryOperand->Num());

			float NewGain = 1.0f;
			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const float Gain = FMath::Clamp(*InAdditionalOperands[i], MinGain, MaxGain);

				Audio::ArrayFade(*OutResult, InInstanceData.LastGain, Gain);

				NewGain *= Gain;
			}

			InInstanceData.LastGain = NewGain;
		}

		static void Reset(TMathOpMultiply& InInstanceData, const FAudioBufferReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FAudioBufferWriteRef& OutResult)
		{
			InInstanceData.bInit = false;
			OutResult->Zero();
		}

	};

	template <>
	class TMathOpMultiply<FTime, float>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames;
			
			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatMultiplyNode_MultiplyFloatTooltip", "Time multiplicand.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatMultiplyNode_MultiplicandsTooltip", "Float multiplicand(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatMultiplyNode_OutTooltip", "Time float multiplication output") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, 1.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpMultiply& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			*OutResult = *InPrimaryOperand;

			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const float OperandValue = *InAdditionalOperands[i];
				*OutResult *= OperandValue;
			}
		}

		static void Reset(TMathOpMultiply& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <>
	class TMathOpDivide<FTime, float>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace MathOpNames; 

			static const FDataVertexMetadata PrimaryOperandMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatDivideNode_DividendFloatTooltip", "Time dividend.") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(PrimaryOperand) // display name
			};

			static const FDataVertexMetadata AdditionalOperandsMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatDivideNode_DivisorsTooltip", "Float divisor(s).") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(AdditionalOperands) // display name
			};

			static const FDataVertexMetadata OutputMetadata
			{
				  METASOUND_LOCTEXT("MathOpTimeFloatDivideNode_OutTooltip", "Time divide-by-float output") // description
				, METASOUND_GET_PARAM_DISPLAYNAME(OutMath) // display name
			};

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(PrimaryOperand), PrimaryOperandMetadata, 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME(AdditionalOperands), AdditionalOperandsMetadata, 1.0f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTime>(METASOUND_GET_PARAM_NAME(OutMath), OutputMetadata)
				)
			);

			return Interface;
		}

		static void Calculate(TMathOpDivide& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			*OutResult = *InPrimaryOperand;

			if (!ensure(!InAdditionalOperands.IsEmpty()))
			{
				return;
			}

			for (int32 i = 0; i < InAdditionalOperands.Num(); ++i)
			{
				const float OperandValue = *InAdditionalOperands[i];
				if (OperandValue == 0.0f)
				{
					// TODO: Error here
					continue;
				}

				*OutResult /= OperandValue;
			}
		}

		static void Reset(TMathOpDivide& InInstanceData, const FTimeReadRef& InPrimaryOperand, const TArray<FFloatReadRef>& InAdditionalOperands, FTimeWriteRef& OutResult)
		{
			Calculate(InInstanceData, InPrimaryOperand, InAdditionalOperands, OutResult);
		}

	};

	template <typename TDerivedClass, typename TMathOpClass, typename TDataClass, typename TOperandDataClass>
	using TMathOpNode = TNodeFacade<TMathOperator<TDerivedClass, TMathOpClass, TDataClass, TOperandDataClass>>;

	// Definitions
	DEFINE_METASOUND_MATHOP(Add, Float, float, METASOUND_LOCTEXT("MathAddFloatNode_Description", "Adds floats."), MathOpNames::AddKeywords)
	DEFINE_METASOUND_MATHOP(Add, Int32, int32, METASOUND_LOCTEXT("MathAddInt32Node_Description", "Adds int32s."), MathOpNames::AddKeywords)
	DEFINE_METASOUND_MATHOP(Add, Audio, FAudioBuffer, METASOUND_LOCTEXT("MathAddBufferNode_Description", "Adds buffers together by sample."), MathOpNames::AddKeywords)
	DEFINE_METASOUND_MATHOP(Add, Time, FTime, METASOUND_LOCTEXT("MathAddTimeNode_Description", "Adds time values."), MathOpNames::AddKeywords)

	DEFINE_METASOUND_OPERAND_TYPED_MATHOP(Add, Audio, FAudioBuffer, Float, float, METASOUND_LOCTEXT("MathAddAudioFloatNode_Description", "Add floats to buffer sample-by-sample."), MathOpNames::AddKeywords)

	DEFINE_METASOUND_MATHOP(Subtract, Float, float, METASOUND_LOCTEXT("MathSubractFloatNode_Description", "Subtracts floats."), MathOpNames::SubtractKeywords)
	DEFINE_METASOUND_MATHOP(Subtract, Int32, int32, METASOUND_LOCTEXT("MathSubractInt32Node_Description", "Subtracts int32s."), MathOpNames::SubtractKeywords)
	DEFINE_METASOUND_MATHOP(Subtract, Audio, FAudioBuffer, METASOUND_LOCTEXT("MathSubtractBufferNode_Description", "Subtracts buffers sample-by-sample."), MathOpNames::SubtractKeywords)
	DEFINE_METASOUND_MATHOP(Subtract, Time, FTime, METASOUND_LOCTEXT("MathSubractTimeNode_Description", "Subtracts time values."), MathOpNames::SubtractKeywords)

	DEFINE_METASOUND_MATHOP(Multiply, Float, float, METASOUND_LOCTEXT("MathMultiplyFloatNode_Description", "Multiplies floats."), MathOpNames::MultiplyKeywords)
	DEFINE_METASOUND_MATHOP(Multiply, Int32, int32, METASOUND_LOCTEXT("MathMultiplyInt32Node_Description", "Multiplies int32s."), MathOpNames::MultiplyKeywords)
	DEFINE_METASOUND_MATHOP(Multiply, Audio, FAudioBuffer, METASOUND_LOCTEXT("MathMultiplyBufferNode_Description", "Multiplies buffers together sample-by-sample."), MathOpNames::MultiplyKeywords)

	DEFINE_METASOUND_OPERAND_TYPED_MATHOP(Multiply, Audio, FAudioBuffer, Float, float, METASOUND_LOCTEXT("MathMultiplyAudioByFloatDescription", "Multiplies buffer by float scalars."), MathOpNames::MultiplyAudioFloatKeywords)
	DEFINE_METASOUND_OPERAND_TYPED_MATHOP(Multiply, Time, FTime, Float, float, METASOUND_LOCTEXT("MathMultiplyTimeNode_Description", "Scales time by floats."), MathOpNames::MultiplyKeywords)

	DEFINE_METASOUND_MATHOP(Divide, Float, float, METASOUND_LOCTEXT("MathDivideFloatNode_Description", "Divide float by another float."), MathOpNames::DivideKeywords)
	DEFINE_METASOUND_MATHOP(Divide, Int32, int32, METASOUND_LOCTEXT("MathDivideInt32Node_Description", "Divide int32 by another int32."), MathOpNames::DivideKeywords)

	DEFINE_METASOUND_OPERAND_TYPED_MATHOP(Divide, Time, FTime, Float, float, METASOUND_LOCTEXT("MathDivideTimeNode_Description", "Divides time by floats."), MathOpNames::DivideKeywords)

	DEFINE_METASOUND_MATHOP(Modulo, Int32, int32, METASOUND_LOCTEXT("MathModulusInt32Node_Description", "Modulo int32 by another int32."), MathOpNames::ModuloKeywords)

	DEFINE_METASOUND_MATHOP(Power, Float, float, METASOUND_LOCTEXT("MathPowerFloatNode_Description", "Raise float to the power of another float."), MathOpNames::PowerKeywords)

	DEFINE_METASOUND_MATHOP(Logarithm, Float, float, METASOUND_LOCTEXT("MathLogarithmFloatNode_Description", "Calculate float-base logarithm of another float."), TArray<FText>())

	METASOUND_REGISTER_NODE(FAddFloatNode)
	METASOUND_REGISTER_NODE(FAddInt32Node)
	METASOUND_REGISTER_NODE(FAddTimeNode)
	METASOUND_REGISTER_NODE(FAddAudioNode)
	METASOUND_REGISTER_NODE(FAddAudioFloatNode)

	METASOUND_REGISTER_NODE(FSubtractFloatNode)
	METASOUND_REGISTER_NODE(FSubtractInt32Node)
	METASOUND_REGISTER_NODE(FSubtractTimeNode)
	METASOUND_REGISTER_NODE(FSubtractAudioNode)

	METASOUND_REGISTER_NODE(FMultiplyAudioNode)
	METASOUND_REGISTER_NODE(FMultiplyAudioFloatNode)
	METASOUND_REGISTER_NODE(FMultiplyFloatNode)
	METASOUND_REGISTER_NODE(FMultiplyInt32Node)
	METASOUND_REGISTER_NODE(FMultiplyTimeFloatNode)

	METASOUND_REGISTER_NODE(FDivideFloatNode)
	METASOUND_REGISTER_NODE(FDivideInt32Node)
	METASOUND_REGISTER_NODE(FDivideTimeFloatNode)

	METASOUND_REGISTER_NODE(FModuloInt32Node)

	METASOUND_REGISTER_NODE(FPowerFloatNode)

	METASOUND_REGISTER_NODE(FLogarithmFloatNode)
}

#undef LOCTEXT_NAMESPACE // MetasoundMathOpNode
