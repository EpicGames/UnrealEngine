// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundNodeRegistrationMacro.h"
#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MaxNode"

namespace Metasound
{
	namespace MaxVertexNames
	{
		METASOUND_PARAM(InputAValue, "A", "Input value A.");
		METASOUND_PARAM(InputBValue, "B", "Input value B.");
		METASOUND_PARAM(OutputValue, "Value", "The max of A and B.");
	}

	namespace MetasoundMaxNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "Max", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Math },
				{ },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}

		template<typename ValueType>
		struct TMax
		{
			bool bSupported = false;
		};

		template<>
		struct TMax<int32>
		{
			static void GetMax(int32 InA, int32 InB, int32& OutMax)
			{
				OutMax = FMath::Max(InA, InB);
			}

			static TDataReadReference<int32> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<int32> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}
		};


		template<>
		struct TMax<float>
		{
			static void GetMax(float InA, float InB, float& OutMax)
			{
				OutMax = FMath::Max(InA, InB);
			}

			static TDataReadReference<float> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<float> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}

			static bool IsAudioBuffer() { return false; }
		};

		template<>
		struct TMax<FAudioBuffer>
		{
			static void GetMax(const FAudioBuffer& InA, const FAudioBuffer& InB, FAudioBuffer& OutMax)
			{
				float* OutMaxBufferPtr = OutMax.GetData();
				const float* APtr = InA.GetData();
				const float* BPtr = InB.GetData();
				int32 NumSamples = InA.Num();
				for (int32 i = 0; i < NumSamples; ++i)
				{
					OutMaxBufferPtr[i] = FMath::Max(APtr[i], BPtr[i]);
				}
			}

			static TDataReadReference<FAudioBuffer> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<FAudioBuffer> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MaxVertexNames;				
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}
		};
	}

	template<typename ValueType>
	class TMaxNodeOperator : public TExecutableOperator<TMaxNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace MaxVertexNames;
			using namespace MetasoundMaxNodePrivate;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAValue)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputBValue))
				),
				FOutputVertexInterface(
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = TEXT("Max");
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("MaxDisplayNamePattern", "Max ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("MaxDesc", "Returns the max of A and B.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundMaxNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MaxVertexNames;
			using namespace MetasoundMaxNodePrivate;

			TDataReadReference<ValueType> InputA = TMax<ValueType>::CreateInRefA(InParams);
			TDataReadReference<ValueType> InputB = TMax<ValueType>::CreateInRefB(InParams);

			return MakeUnique<TMaxNodeOperator<ValueType>>(InParams.OperatorSettings, InputA, InputB);
		}


		TMaxNodeOperator(const FOperatorSettings& InSettings,
			const TDataReadReference<ValueType>& InInputA,
			const TDataReadReference<ValueType>& InInputB)
			: InputA(InInputA)
			, InputB(InInputB)
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
		{
			GetMax();
		}

		virtual ~TMaxNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MaxVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAValue), InputA);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputBValue), InputB);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MaxVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void GetMax()
		{
			using namespace MetasoundMaxNodePrivate;

			TMax<ValueType>::GetMax(*InputA, *InputB, *OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			GetMax();
		}

		void Execute()
		{
			GetMax();
		}

	private:

		TDataReadReference<ValueType> InputA;
		TDataReadReference<ValueType> InputB;
		TDataWriteReference<ValueType> OutputValue;
	};

	/** TMaxNode
	 *
	 *  Returns the min of both inputs
	 */
	template<typename ValueType>
	using TMaxNode = TNodeFacade<TMaxNodeOperator<ValueType>>;

	using FMaxNodeInt32 = TMaxNode<int32>;
 	METASOUND_REGISTER_NODE(FMaxNodeInt32)

	using FMaxNodeFloat = TMaxNode<float>;
	METASOUND_REGISTER_NODE(FMaxNodeFloat)

	using FMaxNodeAudioBuffer = TMaxNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FMaxNodeAudioBuffer)
}

#undef LOCTEXT_NAMESPACE
