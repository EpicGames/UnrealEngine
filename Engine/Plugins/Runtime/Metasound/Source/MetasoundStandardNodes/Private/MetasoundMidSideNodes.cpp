// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Dsp.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "DSP/FloatArrayMath.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MetasoundMidSideNodes"

namespace Metasound
{
	/* Mid-Side Encoder */
	namespace MidSideEncodeVertexNames
	{
		METASOUND_PARAM(InputAudioLeft, "In Left", "The left audio channel to convert.");
		METASOUND_PARAM(InputAudioRight, "In Right", "The right audio channel to convert.");
		METASOUND_PARAM(InputSpreadAmount, "Spread Amount", "Amount of Stereo Spread. 0.0 is no spread, 0.5 is the original signal, and 1.0 is full wide.");
		METASOUND_PARAM(InputEqualPower, "Equal Power", "Whether an equal power relationship between the mid and side signals should be maintained.");

		METASOUND_PARAM(OutputAudioMid, "Out Mid", "The Mid content from the audio signal.");
		METASOUND_PARAM(OutputAudioSide, "Out Side", "The Side content from the audio signal.");
	}

	// Operator Class
	class FMidSideEncodeOperator : public TExecutableOperator<FMidSideEncodeOperator>
	{
	public:

		FMidSideEncodeOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InLeftAudioInput,
			const FAudioBufferReadRef& InRightAudioInput,
			const FFloatReadRef& InSpreadAmount,
			const FBoolReadRef& InEqualPower)
			: AudioInputLeft(InLeftAudioInput)
			, AudioInputRight(InRightAudioInput)
			, SpreadAmount(InSpreadAmount)
			, bEqualPower(InEqualPower)
			, AudioOutputMid(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, AudioOutputSide(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, MidScale(0.0f)
			, SideScale(0.0f)
			, PrevMidScale(0.0f)
			, PrevSideScale(0.0f)
			, PrevSpreadAmount(*SpreadAmount)
			, bPrevEqualPower(*bEqualPower)
		{
			Reset(InParams);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					// This node needs to maintain this node-class-name in order to avoid breaking linkage. 
					// Note that the class name is "Mid-Side Decode" when this is in fact the "Encoder". 
					FNodeClassName { StandardNodes::Namespace, "Mid-Side Decode", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("MidSideEncodeDisplayName", "Mid-Side Encode"),
					METASOUND_LOCTEXT("MidSideEncodeDesc", "Converts a stereo audio signal from Left and Right to Mid and Side channels."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Spatialization},
					{ },
					FNodeDisplayStyle{}
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace MidSideEncodeVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioLeft)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioRight)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSpreadAmount), 0.5f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEqualPower), false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioMid)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioSide))
				)
			);

			return Interface;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MidSideEncodeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioLeft), AudioInputLeft);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioRight), AudioInputRight);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSpreadAmount), SpreadAmount);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEqualPower), bEqualPower);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MidSideEncodeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioMid), AudioOutputMid);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioSide), AudioOutputSide);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MidSideEncodeVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FAudioBufferReadRef LeftAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioLeft), InParams.OperatorSettings);
			FAudioBufferReadRef RightAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioRight), InParams.OperatorSettings);
			FFloatReadRef SpreadAmountIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputSpreadAmount), InParams.OperatorSettings);
			FBoolReadRef bEqualPowerIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputEqualPower), InParams.OperatorSettings);

			return MakeUnique<FMidSideEncodeOperator>(InParams, LeftAudioIn, RightAudioIn, SpreadAmountIn, bEqualPowerIn);
		}

		void Execute()
		{
			/* Update internal variables, if necessary */
			const float ClampedSpread = FMath::Clamp(*SpreadAmount, 0.0f, 1.0f);
			const float bNeedsUpdate = !FMath::IsNearlyEqual(ClampedSpread, PrevSpreadAmount) || bPrevEqualPower != *bEqualPower;

			if (bNeedsUpdate)
			{
				UpdateScale(ClampedSpread);
				PrevSpreadAmount = ClampedSpread;
				bPrevEqualPower = *bEqualPower;
			}

			/* Generate Mid-Side Output */

			// Encode the signal to to MS
			Audio::EncodeMidSide(*AudioInputLeft, *AudioInputRight, *AudioOutputMid, *AudioOutputSide);			
			
			// We should now be in MS mode, now we can apply our gain scalars

			// SideScale never changes quickly without MidScale also changing quickly
			if (FMath::IsNearlyEqual(PrevMidScale, MidScale))
			{
				Audio::ArrayMultiplyByConstantInPlace(*AudioOutputMid, MidScale);
				Audio::ArrayMultiplyByConstantInPlace(*AudioOutputSide, SideScale);
			}
			else
			{
				Audio::ArrayFade(*AudioOutputMid, PrevMidScale, MidScale);
				Audio::ArrayFade(*AudioOutputSide, PrevSideScale, SideScale);
				
				PrevMidScale = MidScale;
				PrevSideScale = SideScale;
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			const float ClampedSpread = FMath::Clamp(*SpreadAmount, 0.0f, 1.0f);
			UpdateScale(ClampedSpread);

			PrevSpreadAmount = ClampedSpread;
			PrevMidScale = MidScale;
			PrevSideScale = SideScale;

			// Encode the signal to to MS
			Audio::EncodeMidSide(*AudioInputLeft, *AudioInputRight, *AudioOutputMid, *AudioOutputSide);			
			Audio::ArrayMultiplyByConstantInPlace(*AudioOutputMid, MidScale);
			Audio::ArrayMultiplyByConstantInPlace(*AudioOutputSide, SideScale);
		}

	private:

		void UpdateScale(float InSpread)
		{
			// Convert to radians between 0.0 and PI/2
			const float SpreadScale = InSpread * 0.5f * PI;

			// Compute equal power relationship between Mid and Side
			FMath::SinCos(&SideScale, &MidScale, SpreadScale);

			// Adjust gain so 0.5f Spread results in a 1.0f to 1.0f gain ratio between Mid and Side
			const float NormalizingFactor = UE_SQRT_2;
			MidScale *= NormalizingFactor;
			SideScale *= NormalizingFactor;

			// Clamp values if not Equal Power
			if (!*bEqualPower)
			{
				MidScale = FMath::Clamp(MidScale, 0.0f, 1.0f);
				SideScale = FMath::Clamp(SideScale, 0.0f, 1.0f);
			}
		}

		// The input audio buffer
		FAudioBufferReadRef AudioInputLeft;
		FAudioBufferReadRef AudioInputRight;

		// Stereo spread amount
		FFloatReadRef SpreadAmount;

		// Whether an equal power relationship is maintained between mid and side channels
		FBoolReadRef bEqualPower;

		// Output audio buffer
		FAudioBufferWriteRef AudioOutputMid;
		FAudioBufferWriteRef AudioOutputSide;

		// Internal variables used to calculate Mid and side channel gain
		float MidScale;
		float SideScale;
		
		// Storing previous scales for interpolated gain
		float PrevMidScale;
		float PrevSideScale;

		// Previous Spread variable; need to update MidScale and SideScale if this changes between Execute() calls
		float PrevSpreadAmount;

		// Previous EqualPower bool value; need to update MidScale and SideScale if this changes between Execute() calls
		bool bPrevEqualPower;
	};

	// Node Class
	using FMidSideEncodeNode = TNodeFacade<FMidSideEncodeOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FMidSideEncodeNode)


	/* Mid-Side Decoder */

	namespace MidSideDecodeVertexNames
	{
		METASOUND_PARAM(InputAudioMid, "In Mid", "The mid audio channel to convert.");
		METASOUND_PARAM(InputAudioSide, "In Side", "The side audio channel to convert.");
		METASOUND_PARAM(InputSpreadAmount, "Spread Amount", "Amount of Stereo Spread. 0.0 is no spread, 0.5 is the original signal, and 1.0 is full wide.");
		METASOUND_PARAM(InputEqualPower, "Equal Power", "Whether an equal power relationship between the mid and side signals should be maintained.");
	
		METASOUND_PARAM(OutputAudioLeft, "Out Left", "The left audio channel which has been processed.");
		METASOUND_PARAM(OutputAudioRight, "Out Right", "The Right audio channel which has been processed.");
	}

	// Operator Class
	class FMidSideDecodeOperator : public TExecutableOperator<FMidSideDecodeOperator>
	{
	public:

		FMidSideDecodeOperator(const FBuildOperatorParams& InParams,
			const FAudioBufferReadRef& InLeftAudioInput,
			const FAudioBufferReadRef& InRightAudioInput,
			const FFloatReadRef& InSpreadAmount,
			const FBoolReadRef& InEqualPower)
			: AudioInputMid(InLeftAudioInput)
			, AudioInputSide(InRightAudioInput)
			, SpreadAmount(InSpreadAmount)
			, bEqualPower(InEqualPower)
			, AudioOutputLeft(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, AudioOutputRight(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
			, MidScale(0.0f)
			, SideScale(0.0f)
			, PrevMidScale(0.0f)
			, PrevSideScale(0.0f)
			, PrevSpreadAmount(*SpreadAmount)
			, bPrevEqualPower(*bEqualPower)
		{
			MidInBuffer.AddUninitialized(InParams.OperatorSettings.GetNumFramesPerBlock());
			FMemory::Memset(MidInBuffer.GetData(), 0, sizeof(float) * MidInBuffer.Num());

			SideInBuffer.AddUninitialized(InParams.OperatorSettings.GetNumFramesPerBlock());
			FMemory::Memset(SideInBuffer.GetData(), 0, sizeof(float) * SideInBuffer.Num());

			Reset(InParams);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					// This node needs to maintain this node-class-name in order to avoid breaking linkage. 
					// Note that the class name is "Mid-Side Encode" when this is in fact the "Decoder". 
					FNodeClassName { StandardNodes::Namespace, "Mid-Side Encode", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("MidSideDecodeDisplayName", "Mid-Side Decode"),
					METASOUND_LOCTEXT("MidSideDecodeDesc", "Decodes a stereo signal with Mid and Side channels to Left and Right channels."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Spatialization },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace MidSideDecodeVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioMid)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioSide)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSpreadAmount), 0.5f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEqualPower), false)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLeft)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioRight))
				)
			);

			return Interface;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MidSideDecodeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioMid), AudioInputMid);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioSide), AudioInputSide);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSpreadAmount), SpreadAmount);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEqualPower), bEqualPower);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MidSideDecodeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioLeft), AudioOutputLeft);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioRight), AudioOutputRight);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MidSideDecodeVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef MidAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioMid), InParams.OperatorSettings);
			FAudioBufferReadRef SideAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioSide), InParams.OperatorSettings);
			FFloatReadRef SpreadAmountIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputSpreadAmount), InParams.OperatorSettings);
			FBoolReadRef bEqualPowerIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputEqualPower), InParams.OperatorSettings);

			return MakeUnique<FMidSideDecodeOperator>(InParams, MidAudioIn, SideAudioIn, SpreadAmountIn, bEqualPowerIn);
		}

		void Execute()
		{
			/* Update internal variables, if necessary */
			const float ClampedSpread = FMath::Clamp(*SpreadAmount, 0.0f, 1.0f);
			const float bNeedsUpdate = !FMath::IsNearlyEqual(ClampedSpread, PrevSpreadAmount) || bPrevEqualPower != *bEqualPower;

			if (bNeedsUpdate)
			{
				UpdateScale(ClampedSpread);
				PrevSpreadAmount = ClampedSpread;
				bPrevEqualPower = *bEqualPower;
			}

			/* Generate Mid-Side Output */

			// SideScale never changes quickly without MidScale also changing quickly
			if (FMath::IsNearlyEqual(PrevMidScale, MidScale))
			{
				Audio::ArrayMultiplyByConstant(*AudioInputMid, MidScale, MidInBuffer);
				Audio::ArrayMultiplyByConstant(*AudioInputSide, SideScale, SideInBuffer);
			}
			else
			{
				MidInBuffer = *AudioInputMid;
				Audio::ArrayFade(MidInBuffer, PrevMidScale, MidScale);
				PrevMidScale = MidScale;

				SideInBuffer = *AudioInputSide;
				Audio::ArrayFade(SideInBuffer, PrevSideScale, SideScale);
				PrevSideScale = SideScale;
			}

			// Convert to L/R Signal
			Audio::DecodeMidSide(MidInBuffer, SideInBuffer, *AudioOutputLeft, *AudioOutputRight);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOutputLeft->Zero();
			AudioOutputRight->Zero();

			const float ClampedSpread = FMath::Clamp(*SpreadAmount, 0.0f, 1.0f);
			UpdateScale(ClampedSpread);
			PrevSpreadAmount = ClampedSpread;

			PrevMidScale = MidScale;
			PrevSideScale = SideScale;
		}

	private:

		void UpdateScale(float InSpread)
		{
			// Convert to radians between 0.0 and PI/2
			const float SpreadScale = InSpread * 0.5f * PI;

			// Compute equal power relationship between Mid and Side
			FMath::SinCos(&SideScale, &MidScale, SpreadScale);

			// Adjust gain so 0.5f Spread results in a 1.0f to 1.0f gain ratio between Mid and Side
			constexpr float NormalizingFactor = UE_SQRT_2;
			MidScale *= NormalizingFactor;
			SideScale *= NormalizingFactor;

			// Clamp values if not Equal Power
			if (!*bEqualPower)
			{
				MidScale = FMath::Clamp(MidScale, 0.0f, 1.0f);
				SideScale = FMath::Clamp(SideScale, 0.0f, 1.0f);
			}
		}

		// The input audio buffer
		FAudioBufferReadRef AudioInputMid;
		FAudioBufferReadRef AudioInputSide;

		// Stereo spread amount
		FFloatReadRef SpreadAmount;

		// Whether an equal power relationship is maintained between mid and side channels
		FBoolReadRef bEqualPower;

		// Output audio buffer
		FAudioBufferWriteRef AudioOutputLeft;
		FAudioBufferWriteRef AudioOutputRight;

		// Internal variables used to calculate Mid and side channel gain
		float MidScale;
		float SideScale;

		// Storing previous scales for interpolated gain
		float PrevMidScale;
		float PrevSideScale;

		// Previous Spread variable; need to update MidScale and SideScale if this changes between Execute() calls
		float PrevSpreadAmount;

		// Previous EqualPower bool value; need to update MidScale and SideScale if this changes between Execute() calls
		bool bPrevEqualPower;

		// Reusable buffers to save performance during Execute().
		Audio::FAlignedFloatBuffer MidInBuffer;
		Audio::FAlignedFloatBuffer SideInBuffer;
	};

	// Node Class
	using FMidSideDecodeNode = TNodeFacade<FMidSideDecodeOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FMidSideDecodeNode)

}

#undef LOCTEXT_NAMESPACE
