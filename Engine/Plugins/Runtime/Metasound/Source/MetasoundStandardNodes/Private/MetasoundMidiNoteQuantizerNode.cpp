// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "DSP/MidiNoteQuantizer.h"
#include "Internationalization/Text.h"

#include "MetasoundFacade.h"
#include "MetasoundPrimitives.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MidiNoteQuantizerNode"

namespace Metasound
{
	// forward declarations
	// ...
		
#pragma region Parameter Names
	namespace MidiNoteQuantizerParameterNames
	{
		// inputs
		METASOUND_PARAM(ParamNoteIn, "Note In", "Midi Note to quantize");
		METASOUND_PARAM(ParamRootNote, "Root Note", "Midi note to treat as the Root (e.g. where 0.0 = C, 1.0 = Db/C#, etc). Values are clamped to positive values.");
		METASOUND_PARAM(ParamScaleDegrees, "Scale Degrees", "Set of notes in ascending order, represeting half steps starting at 0.0f meaning the Root Note. Scale degrees should be the notes in a scale that not including the octave. Can be the output of the Scale To Note Array node.");
		METASOUND_PARAM(ParamScaleRange, "Scale Range In", "The number of semitones in the scale. E.g. a regular diatonic scale will be 12 semitones. Exotic scales could be something else.");

		// outputs
		METASOUND_PARAM(ParamNoteOutput, "Note Out", "Quantized Note");

	} // namespace MidiNoteQuantizerParameterNames

	using namespace MidiNoteQuantizerParameterNames;

#pragma endregion // Parameter Names


#pragma region Operator Declaration
	class FMidiNoteQuantizerOperator: public TExecutableOperator < FMidiNoteQuantizerOperator >
	{
	public:
		using FArrayScaleDegreeReadRef = TDataReadReference<TArray<float>>;
		using ScaleDegreeArrayType = TArray<float>;

		// ctor
		FMidiNoteQuantizerOperator(
			  const FBuildOperatorParams& InParams
			, const FFloatReadRef& InMidiNoteIn
			, const FFloatReadRef& InRootNote
			, const FArrayScaleDegreeReadRef& InScale
			, const FFloatReadRef& InScaleRange
		);

		// node interface
		static const FNodeClassMetadata& GetNodeInfo();
		static FVertexInterface DeclareVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private: // members
		// input pins
		FFloatReadRef MidiNoteIn;
		FFloatReadRef RootNote;
		FArrayScaleDegreeReadRef Scale;
		FFloatReadRef ScaleRange;

		// output pins
		FFloatWriteRef MidiNoteOut;

		// cached values
		float PreviousNoteIn = TNumericLimits<float>::Lowest();
		float PreviousRoot = -1.0f;
		TArray<float> PreviousScale;
		float PreviousNoteOut = 0.0f;
		float PreviousScaleRange = 12.0f;

		static constexpr float MaxNote = 1e12f;
		static constexpr float MinNote = -1e12f;
		static constexpr float MaxRoot = 1e12f;
		static constexpr float MinRoot = 0.f;

	}; // class FMidiNoteQuantizerOperator

#pragma endregion // Operator Declaration


#pragma region Operator Implementation

	// ctor
	FMidiNoteQuantizerOperator::FMidiNoteQuantizerOperator(
		  const FBuildOperatorParams& InParams
		, const FFloatReadRef& InMidiNoteIn
		, const FFloatReadRef& InRootNote
		, const FArrayScaleDegreeReadRef& InScale
		, const FFloatReadRef& InScaleRange
	)
		: MidiNoteIn(InMidiNoteIn)
		, RootNote(InRootNote)
		, Scale(InScale)
		, ScaleRange(InScaleRange)
		, MidiNoteOut(FFloatWriteRef::CreateNew())
	{
		Reset(InParams);
	}


	const FNodeClassMetadata& FMidiNoteQuantizerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("MIDI Note Quantizer"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_MIDI_Note_Quantizer_NodeDisplayName", "MIDI Note Quantizer");
			Info.Description = METASOUND_LOCTEXT("MidiNoteQuantizer_NodeDescription", "Quantizes a MIDI note to the nearset note that matches provided criteria");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Music);
			Info.Keywords.Add(METASOUND_LOCTEXT("QuantizerPitchKeyword", "Pitch"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}


	FVertexInterface FMidiNoteQuantizerOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamNoteIn), 60.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamRootNote), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamScaleRange), 12.0f),
				TInputDataVertex<ScaleDegreeArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamScaleDegrees))
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamNoteOutput))
				)
			);

		return Interface;
	}


	TUniquePtr<IOperator> FMidiNoteQuantizerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FFloatReadRef MidiNoteIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamNoteIn), InParams.OperatorSettings);
		FFloatReadRef RootNoteIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamRootNote), InParams.OperatorSettings);
		FArrayScaleDegreeReadRef InScaleArray = InputData.GetOrCreateDefaultDataReadReference<ScaleDegreeArrayType>(METASOUND_GET_PARAM_NAME(ParamScaleDegrees), InParams.OperatorSettings);
		FFloatReadRef ScaleRangeIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(ParamScaleRange), InParams.OperatorSettings);

		return MakeUnique <FMidiNoteQuantizerOperator>(
			  InParams
			, MidiNoteIn
			, RootNoteIn
			, InScaleArray
			, ScaleRangeIn
			);
	}

	void FMidiNoteQuantizerOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamNoteIn), MidiNoteIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamRootNote), RootNote);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamScaleRange), ScaleRange);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamScaleDegrees), Scale);
	}

	void FMidiNoteQuantizerOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamNoteOutput), MidiNoteOut);
	}

	void FMidiNoteQuantizerOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PreviousNoteIn = FMath::Clamp(*MidiNoteIn, MinNote, MaxNote);
		PreviousRoot = FMath::Clamp(*RootNote, MinRoot, MaxRoot);
		PreviousScale = *Scale;
		PreviousScaleRange = FMath::Max(*ScaleRange, 1.0f);

		if (PreviousScale.IsEmpty())
		{
			PreviousNoteOut = PreviousNoteIn;
		}
		else
		{
			PreviousScale.Sort();
			PreviousNoteOut = Audio::FMidiNoteQuantizer::QuantizeMidiNote(PreviousNoteIn, PreviousRoot, PreviousScale, PreviousScaleRange);
		}
		*MidiNoteOut = PreviousNoteOut;
	}

	void FMidiNoteQuantizerOperator::Execute()
	{
		if ((*Scale).IsEmpty())
		{
			*MidiNoteOut = *MidiNoteIn;
			return;
		}

		// calculate new output and cache values if needed
		const float CurrentNote = FMath::Clamp(*MidiNoteIn, MinNote, MaxNote);
		const float CurrentRoot = FMath::Clamp(*RootNote, MinRoot, MaxRoot);
		const float CurrentScaleRange = FMath::Max(*ScaleRange, 1.0f);;
		if (!FMath::IsNearlyEqual(PreviousNoteIn, CurrentNote)
			|| !FMath::IsNearlyEqual(PreviousRoot, CurrentRoot)
			|| PreviousScale !=  *Scale
			|| !FMath::IsNearlyEqual(PreviousScaleRange, CurrentScaleRange)
			)
		{
			// cache values
			PreviousNoteIn = CurrentNote;
			PreviousRoot = CurrentRoot;
			PreviousScale = *Scale;
			PreviousScaleRange = CurrentScaleRange;

			PreviousScale.Sort();
			PreviousNoteOut = Audio::FMidiNoteQuantizer::QuantizeMidiNote(PreviousNoteIn, PreviousRoot, PreviousScale, PreviousScaleRange);
		}

		// set the output value
		*MidiNoteOut = PreviousNoteOut;
	}

#pragma endregion // Operator Implementation


#pragma region Node Declaration
	using FMidiNoteQuantizerNode = TNodeFacade<FMidiNoteQuantizerOperator>;
#pragma endregion // Node Declaration


#pragma region Node Registration
	METASOUND_REGISTER_NODE(FMidiNoteQuantizerNode);
#pragma endregion // Node Registration

} // namespace Metasound

#undef LOCTEXT_NAMESPACE //MetasoundBasicFilterNodes
