// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "DSP/MidiNoteQuantizer.h"
#include "Internationalization/Text.h"

#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundFacade.h"
#include "MetasoundPrimitives.h"
#include "MetasoundScaleEnums.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundParamHelper.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MidiScaleToArray"

namespace Metasound
{
	DEFINE_METASOUND_ENUM_BEGIN(Audio::EMusicalScale::Scale, FEnumEMusicalScale, "MusicalScale")

	// modes
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Major, "MajorDescription", "Major Scale", "MajorDescriptionTT", "Major (Ionian)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Minor_Dorian, "Minor_DorianDescription", "Minor (Dorian)", "Minor_DorianDescriptionTT", "Dorian Minor"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Phrygian, "PhrygianDescription", "Phrygian ", "PhrygianDescriptionTT", "Phrygian"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Lydian, "LydianDescription", "Lydian", "LydianDescriptionTT", "Lydian (sharp-4)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Dominant7th_Mixolydian, "Dominant7th_MixolydianDescription", "Dominant 7th (Mixolydian)", "Dominant7th_MixolydianDescriptionTT", "Mioxlydian (Dominant 7)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::NaturalMinor_Aeolian, "NaturalMinor_AeolianDescription", "Natural Minor (Aeolian)", "NaturalMinor_AeolianDescriptionTT", "Natural Minor (Aeolian)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::HalfDiminished_Locrian, "HalfDiminished_LocrianDescription", "Half Diminished (Locrian)", "HalfDiminished_LocrianDescriptionTT", "Half-Diminished (Locrian)"),
	// non-diatonic
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Chromatic, "ChromaticDescription", "Chromatic", "ChromaticDescriptionTT", "Chromatic"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::WholeTone, "WholeToneDescription", "Whole-Tone", "WholeToneDescriptionTT", "Whole Tone"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::DiminishedWholeTone, "DiminishedWholeToneDescription", "Diminished Whole-Tone", "DiminishedWholeToneDescriptionTT", "Diminished Whole Tone"),
	// petantonic
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::MajorPentatonic, "MajorPentatonicDescription", "Major Pentatonic ", "MajorPentatonicDescriptionTT", "Major Pentatonic"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::MinorPentatonic, "MinorPentatonicDescription", "Minor Pentatonic ", "MinorPentatonicDescriptionTT", "Minor Pentatonic"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Blues, "BluesDescription", "Blues ", "BluesDescriptionTT", "Blues"),
	// bebop
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Bebop_Major, "Bebop_MajorDescription", "Bebop (Major)", "Bebop_MajorDescriptionTT", "Bebop Major"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Bebop_Minor, "Bebop_MinorDescription", "Bebop (Minor)", "Bebop_MinorDescriptionTT", "Bebop Minor"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Bebop_MinorNumber2, "Bebop_MinorNumber2Description", "Bebop (Minor) #2", "Bebop_MinorNumber2DescriptionTT", "Bebop Minor #2"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Bebop_Dominant, "Bebop_DominantDescription", "Bebop (Dominant)", "Bebop_DominantDescriptionTT", "Bebop Dominant"),
	// common major/minors
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::HarmonicMajor, "HarmonicMajorDescription", "Harmonic Major", "HarmonicMajorDescriptionTT", "Harmonic Major"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::HarmonicMinor, "HarmonicMinorDescription", "Harmonic Minor ", "HarmonicMinorDescriptionTT", "Harmonic Minor"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::MelodicMinor, "MelodicMinorDescription", "Melodic Minor ", "MelodicMinorDescriptionTT", "Melodic Minor"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::SixthModeOfHarmonicMinor, "SixthModeOfHarmonicMinorDescription", "Sixth Mode of Harmonic Minor", "SixthModeOfHarmonicMinorDescriptionTT", "Sixth Mode of Harmonic Minor"),
	// lydian/augmented
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::LydianAugmented, "LydianAugmentedDescription", "Lydian Augmented", "LydianAugmentedDescriptionTT", "Lydian Augmented"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::LydianDominant, "LydianDominantDescription", "Lydian Dominant ", "LydianDominantDescriptionTT", "Lydian Dominant"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Augmented, "AugmentedDescription", "Augmented", "AugmentedDescriptionTT", "Augmented"),
	// diminished
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Diminished, "DiminishedDescription", "Diminished ", "DiminishedDescriptionTT", "Diminished"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Diminished_BeginWithHalfStep, "Diminished_BeginWithHalfStepDescription", "Diminished (Begin With Half-Step)", "Diminished_BeginWithHalfStepDescriptionTT", "Diminished (begins with Half Step)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Diminished_BeginWithWholeStep, "Diminished_BeginWithWholeStepDescription", "Diminished (Begin With Whole-Step", "Diminished_BeginWithWholeStepDescriptionTT", "Diminished (begins with Whole Step)"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::HalfDiminished_LocrianNumber2, "HalfDiminished_LocrianNumber2Description", "Half-Diminished (Locrian #2)", "HalfDiminished_LocrianNumber2DescriptionTT", "Half Diminished Locrian (#2)"),
	// other
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Spanish_or_Jewish, "Spanish_or_JewishDescription", "Spanish or Jewish Scale", "Spanish_or_JewishDescriptionTT", "Spanish/Jewish"),
	DEFINE_METASOUND_ENUM_ENTRY(Audio::EMusicalScale::Scale::Hindu, "HinduDescription", "Hindu ", "HinduDescriptionTT", "Hindu")

	DEFINE_METASOUND_ENUM_END()
#pragma region Parameter Names
	namespace MusicalScaleToNoteArrayParameterNames
	{
		// inputs
		METASOUND_PARAM(ParamScaleDegreesPreset, "Scale Degrees", "Select scale preset");
		METASOUND_PARAM(ParamChordTonesOnly, "Chord Tones Only", "If true, will only return a subset of the scale represeting chord tones. (i.e. scale degrees 1,3,5,7). Will not include chord extensions (i.e. 9, 11, 13).");

		// outputs
		METASOUND_PARAM(ParamNoteArrayOutput, "Scale Array Out", "Array represeting the scale as semitones above the root starting at 0.0. The scale only includes one octave and each note only once.");

	} // namespace MusicalScaleToNoteArrayParameterNames

	using namespace MusicalScaleToNoteArrayParameterNames;

#pragma endregion // Parameter Names


#pragma region Operator Declaration
	class FMusicalScaleToNoteArrayOperator: public TExecutableOperator < FMusicalScaleToNoteArrayOperator >
	{
	public:
		using FArrayScaleDegreeWriteRef = TDataWriteReference<TArray<float>>;
		using ScaleDegreeArrayType = TArray<float>;

		// ctor
		FMusicalScaleToNoteArrayOperator(
			  const FBuildOperatorParams& InParams
			, const FEnumMusicalScaleReadRef& InScale
			, const FBoolReadRef& InChordTonesOnly
		);

		// node interface
		static const FNodeClassMetadata& GetNodeInfo();
		static FVertexInterface DeclareVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Reset(const IOperator::FResetParams& InParams);
		void Execute();

	private: // members
		// input pins
		FEnumMusicalScaleReadRef Scale;
		FBoolReadRef bChordTonesOnly;

		// output pins
		FArrayScaleDegreeWriteRef ScaleArrayOutput;

		// cached values
		Audio::EMusicalScale::Scale PreviousScale = Audio::EMusicalScale::Scale::Count;
		bool bPreviousChordTones = false;

	}; // class FMusicalScaleToNoteArrayOperator

#pragma endregion // Operator Declaration


#pragma region Operator Implementation

	// ctor
	FMusicalScaleToNoteArrayOperator::FMusicalScaleToNoteArrayOperator(
		  const FBuildOperatorParams& InParams
		, const FEnumMusicalScaleReadRef& InScale
		, const FBoolReadRef& InChordTonesOnly
		)
		: Scale(InScale)
		, bChordTonesOnly(InChordTonesOnly)
		, ScaleArrayOutput(FArrayScaleDegreeWriteRef::CreateNew())
	{
		// prime our output array
		Reset(InParams);
	}


	const FNodeClassMetadata& FMusicalScaleToNoteArrayOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Musical Scale To Note Array"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_ScaleToNoteArray_NodeDisplayName", "Scale to Note Array");
			Info.Description = METASOUND_LOCTEXT("MusicalScaleToNoteArray_NodeDescription", "Returns an array represeting the selected scale");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Music);
			Info.Keywords.Add(METASOUND_LOCTEXT("ScaleToArrayPitchKeyword", "Pitch"));
			Info.Keywords.Add(METASOUND_LOCTEXT("ScaleToArrayMIDIKeyword", "MIDI"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}


	FVertexInterface FMusicalScaleToNoteArrayOperator::DeclareVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FEnumEMusicalScale>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamScaleDegreesPreset), (int32)Audio::EMusicalScale::Scale::Major),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamChordTonesOnly), false)
			),
			FOutputVertexInterface(
				TOutputDataVertex<ScaleDegreeArrayType>(METASOUND_GET_PARAM_NAME_AND_METADATA(ParamNoteArrayOutput))
				)
			);

		return Interface;
	}


	TUniquePtr<IOperator> FMusicalScaleToNoteArrayOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// inputs
		FEnumMusicalScaleReadRef Scale = InputData.GetOrCreateDefaultDataReadReference<FEnumEMusicalScale>(METASOUND_GET_PARAM_NAME(ParamScaleDegreesPreset), InParams.OperatorSettings);
		FBoolReadRef ChordTonesOnly = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(ParamChordTonesOnly), InParams.OperatorSettings);

		return MakeUnique <FMusicalScaleToNoteArrayOperator>(
			  InParams
			, Scale
			, ChordTonesOnly
			);
	}

	void FMusicalScaleToNoteArrayOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamScaleDegreesPreset), Scale);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamChordTonesOnly), bChordTonesOnly);
	}

	void FMusicalScaleToNoteArrayOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(ParamNoteArrayOutput), ScaleArrayOutput);
	}

	void FMusicalScaleToNoteArrayOperator::Reset(const IOperator::FResetParams& InParams)
	{
		PreviousScale = *Scale;
		bPreviousChordTones = *bChordTonesOnly;

		*ScaleArrayOutput = Audio::FMidiNoteQuantizer::ScaleDegreeSetMap[PreviousScale].GetScaleDegreeSet(bPreviousChordTones);
	}

	void FMusicalScaleToNoteArrayOperator::Execute()
	{
		// calculate new output and cache values if needed
		if ((PreviousScale != *Scale) || (bPreviousChordTones != *bChordTonesOnly))
		{
			// cache values
			PreviousScale = *Scale;
			bPreviousChordTones = *bChordTonesOnly;

			*ScaleArrayOutput = Audio::FMidiNoteQuantizer::ScaleDegreeSetMap[PreviousScale].GetScaleDegreeSet(bPreviousChordTones);
		}
	}

#pragma endregion // Operator Implementation


#pragma region Node Declaration
	using FMusicalScaleToNoteArrayNode = TNodeFacade<FMusicalScaleToNoteArrayOperator>;
#pragma endregion // Node Declaration


#pragma region Node Registration
	METASOUND_REGISTER_NODE(FMusicalScaleToNoteArrayNode);
#pragma endregion // Node Registration

} // namespace Metasound

#undef LOCTEXT_NAMESPACE //MetasoundBasicFilterNodes
