// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "Internationalization/Text.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MinNode"

namespace Metasound
{
	namespace MinVertexNames
	{
		METASOUND_PARAM(InputAValue, "A", "Input value A.");
		METASOUND_PARAM(InputBValue, "B", "Input value B.");
		METASOUND_PARAM(OutputValue, "Value", "The min of A and B.");
	}

	namespace MetasoundMinNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "Min", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Math },
				{ },
				FNodeDisplayStyle()
			};

			return Metadata;
		}

		template<typename ValueType>
		struct TMin
		{
			bool bIsSupported = false;
		};

		template<>
		struct TMin<int32>
		{
			static void GetMin(int32 InA, int32 InB, int32& OutMin)
			{
				OutMin = FMath::Min(InA, InB);
			}

			static TDataReadReference<int32> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<int32> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}
		};


		template<>
		struct TMin<float>
		{
			static void GetMin(float InA, float InB, float& OutMin)
			{
				OutMin = FMath::Min(InA, InB);
			}

			static TDataReadReference<float> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<float> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}
		};


		template<>
		struct TMin<FAudioBuffer>
		{
			static void GetMin(const FAudioBuffer& InA, const FAudioBuffer& InB, FAudioBuffer& OutMax)
			{
				float* OutMaxBufferPtr = OutMax.GetData();
				const float* APtr = InA.GetData();
				const float* BPtr = InB.GetData();
				for (int32 i = 0; i < InA.Num(); ++i)
				{
					OutMaxBufferPtr[i] = FMath::Min(APtr[i], BPtr[i]);
				}
			}

			static TDataReadReference<FAudioBuffer> CreateInRefA(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAValue), InParams.OperatorSettings);
			}

			static TDataReadReference<FAudioBuffer> CreateInRefB(const FBuildOperatorParams& InParams)
			{
				using namespace MinVertexNames;
				const FInputVertexInterfaceData& InputData = InParams.InputData;
				return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputBValue), InParams.OperatorSettings);
			}
		};

	}

	template<typename ValueType>
	class TMinNodeOperator : public TExecutableOperator<TMinNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace MinVertexNames;
			using namespace MetasoundMinNodePrivate;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAValue)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputBValue))
				),
				FOutputVertexInterface(
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = TEXT("Min");
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("MinDisplayNamePattern", "Min ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("MinDesc", "Returns the min of A and B.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundMinNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			TDataReadReference<ValueType> InputA = MetasoundMinNodePrivate::TMin<ValueType>::CreateInRefA(InParams);
			TDataReadReference<ValueType> InputB = MetasoundMinNodePrivate::TMin<ValueType>::CreateInRefB(InParams);

			return MakeUnique<TMinNodeOperator<ValueType>>(InParams.OperatorSettings, InputA, InputB);
		}


		TMinNodeOperator(const FOperatorSettings& InSettings,
			const TDataReadReference<ValueType>& InInputA,
			const TDataReadReference<ValueType>& InInputB)
			: InputA(InInputA)
			, InputB(InInputB)
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
		{
			GetMin();
		}

		virtual ~TMinNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MinVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAValue), InputA);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputBValue), InputB);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MinVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void GetMin()
		{
			using namespace MetasoundMinNodePrivate;

			TMin<ValueType>::GetMin(*InputA, *InputB, *OutputValue);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			GetMin();
		}

		void Execute()
		{
			GetMin();
		}

	private:

		TDataReadReference<ValueType> InputA;
		TDataReadReference<ValueType> InputB;
		TDataWriteReference<ValueType> OutputValue;
	};

	/** TMinNode
	 *
	 *  Returns the min of both inputs
	 */
	template<typename ValueType>
	using TMinNode = TNodeFacade<TMinNodeOperator<ValueType>>;

	using FMinNodeInt32 = TMinNode<int32>;
	METASOUND_REGISTER_NODE(FMinNodeInt32)

	using FMinNodeFloat = TMinNode<float>;
	METASOUND_REGISTER_NODE(FMinNodeFloat)

	using FMinNodeAudio = TMinNode<FAudioBuffer>;
	METASOUND_REGISTER_NODE(FMinNodeAudio)
}

#undef LOCTEXT_NAMESPACE
