// Copyright Epic Games, Inc. All Rights Reserved.
#include "DSP/Dsp.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MixerNode"


namespace Metasound
{
	template<uint32 NumInputs, uint32 NumChannels>
	class TAudioMixerNodeOperator : public TExecutableOperator<TAudioMixerNodeOperator<NumInputs, NumChannels>>
	{
	public:
		// ctor
		TAudioMixerNodeOperator(const FBuildOperatorParams& InParams, const TArray<TOptional<FAudioBufferReadRef>>& InInputBuffers, const TArray<FFloatReadRef>& InGainValues)
			: Gains(InGainValues)
			, Inputs (InInputBuffers)
		{
			// create write refs
			for (uint32 Index = 0; Index < NumChannels; ++Index)
			{
				Outputs.Add(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings));
			}

			// Init previous gains to current values
			PrevGains.Reset();
			PrevGains.AddUninitialized(NumInputs);

			Reset(InParams);
		}

		// dtor
		virtual ~TAudioMixerNodeOperator() = default;

		static const FVertexInterface& GetDefaultInterface()
		{
			auto CreateDefaultInterface = []()-> FVertexInterface
			{
				// inputs
				FInputVertexInterface InputInterface;
				for (uint32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
				{
					// audio channels
					for (uint32 ChanIndex = 0; ChanIndex < NumChannels; ++ChanIndex)
					{
#if WITH_EDITOR
						const FDataVertexMetadata AudioInputMetadata
						{
							GetAudioInputDescription(InputIndex, ChanIndex),
							GetAudioInputDisplayName(InputIndex, ChanIndex)
						};
#else 
						const FDataVertexMetadata AudioInputMetadata;
#endif // WITH_EDITOR
						InputInterface.Add(TInputDataVertex<FAudioBuffer>(GetAudioInputName(InputIndex, ChanIndex), AudioInputMetadata));
					}

					// gain scalar
#if WITH_EDITOR
					FDataVertexMetadata GainPinMetaData
					{
						GetGainInputDescription(InputIndex),
						GetGainInputDisplayName(InputIndex)
					};
#else 
					FDataVertexMetadata GainPinMetaData;
#endif // WITH_EDITOR
					TInputDataVertex<float>GainVertexModel(GainInputNames[InputIndex], GainPinMetaData, 1.0f);

					InputInterface.Add(GainVertexModel);
				}

				// outputs
				FOutputVertexInterface OutputInterface;
				for (uint32 i = 0; i < NumChannels; ++i)
				{
#if WITH_EDITOR
					const FDataVertexMetadata AudioOutputMetadata
					{
						GetAudioOutputDescription(i),
						GetAudioOutputDisplayName(i)
					};
#else 
					const FDataVertexMetadata AudioOutputMetadata;
#endif // WITH_EDITOR
					OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(AudioOutputNames[i], AudioOutputMetadata));
				}

				return FVertexInterface(InputInterface, OutputInterface);
			}; // end lambda: CreateDefaultInterface()

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			// used if NumChannels == 1
			auto CreateNodeClassMetadataMono = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Audio Mixer (Mono, %d)"), NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("MonoMixer", "Mono Mixer ({0})", NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("MixerDescription1", "Will scale input channels by their corresponding gain value and sum them together.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels == 2
			auto CreateNodeClassMetadataStereo = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Audio Mixer (Stereo, %d)"), NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("StereoMixer", "Stereo Mixer ({0})", NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("MixerDescription2", "Will scale input channels by their corresponding gain value and sum them together.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels > 2
			auto CreateNodeClassMetadataMultiChan = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Audio Mixer (%d-Channel, %d)"), NumChannels, NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("NChannelMixer", "{0}-channel Mixer ({1})", NumChannels, NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("MixerDescription3", "Will scale input audio by their corresponding gain value and sum them together.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = (NumChannels == 1)? CreateNodeClassMetadataMono()
														: (NumChannels == 2)? CreateNodeClassMetadataStereo() : CreateNodeClassMetadataMultiChan();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			TArray<TOptional<FAudioBufferReadRef>> InputBuffers;
			TArray<FFloatReadRef> InputGains;

			for (uint32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					const FVertexName& AudioInputName = GetAudioInputName(InputIndex, ChannelIndex);
					if (InputData.IsVertexBound(AudioInputName))
					{
						InputBuffers.Add(InputData.GetDataReadReference<FAudioBuffer>(AudioInputName));
					}
					else
					{
						InputBuffers.AddDefaulted();
					}
				}

				InputGains.Add(InputData.GetOrCreateDefaultDataReadReference<float>(GainInputNames[InputIndex], InParams.OperatorSettings));
			}

			return MakeUnique<TAudioMixerNodeOperator<NumInputs, NumChannels>>(InParams, MoveTemp(InputBuffers), MoveTemp(InputGains));
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			for (uint32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					const FVertexName& AudioInputName = GetAudioInputName(InputIndex, ChannelIndex);
					int32 InputBufferIndex = InputIndex * NumChannels + ChannelIndex;
					if (FAudioBufferReadRef* InputPtr = Inputs[InputBufferIndex].GetPtrOrNull())
					{
						InOutVertexData.BindReadVertex(AudioInputName, *InputPtr);
					}
					else if (InOutVertexData.IsVertexBound(AudioInputName))
					{
						Inputs[InputBufferIndex] = InOutVertexData.GetDataReadReference<FAudioBuffer>(AudioInputName);
					}
				}

				InOutVertexData.BindReadVertex(GainInputNames[InputIndex], Gains[InputIndex]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			for (uint32 i = 0; i < NumChannels; ++i)
			{
				InOutVertexData.BindReadVertex(AudioOutputNames[i], Outputs[i]);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			for (uint32 i = 0; i < NumInputs; ++i)
			{
				PrevGains[i] = *Gains[i];
			}

			Execute();
		}

		void Execute()
		{
			// initialize the output using the first set of input channels
			float NextGain = *Gains[0];
			float PrevGain = PrevGains[0];

			// initialize the output buffers w/ the first set of input buffers 
			for (uint32 Idx = 0; Idx < NumChannels; ++Idx)
			{
				TArrayView<float> OutputView = *Outputs[Idx];
				if (const FAudioBufferReadRef *InputPtr = Inputs[Idx].GetPtrOrNull())
				{
					TArrayView<const float> InputView = **InputPtr;
					check(InputView.Num() == OutputView.Num());
					Audio::ArrayFade(InputView, PrevGain, NextGain, OutputView);
				}
				else
				{
					FMemory::Memzero(OutputView.GetData(), OutputView.Num() * sizeof(float));
				}
			}

			PrevGains[0] = NextGain;

			// mix in each following input
			for (uint32 InputIndex = 1; InputIndex < NumInputs; ++InputIndex)
			{
				NextGain = *Gains[InputIndex];
				PrevGain = PrevGains[InputIndex];

				// for each channel of audio
				for (uint32 ChanIndex = 0; ChanIndex < NumChannels; ++ChanIndex)
				{
					int32 InputBufferIndex = InputIndex * NumChannels + ChanIndex;
					if (const FAudioBufferReadRef* InputPtr = Inputs[InputBufferIndex].GetPtrOrNull())
					{
						Audio::ArrayMixIn(**InputPtr, *Outputs[ChanIndex], PrevGain, NextGain);
					}
				}

				PrevGains[InputIndex] = NextGain;
			}
		}


	private:
		static TArray<FVertexName> InitializeAudioInputNames()
		{
			TStringBuilder<32> InputNameStr;
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumInputs * NumChannels);

			for (int InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
			{
				for (int ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					InputNameStr << "In " << InputIndex;

					if (NumChannels == 1)
					{
					}
					else if (NumChannels == 2)
					{
						InputNameStr << ' ' << ((!ChannelIndex)? 'L' : 'R');
					}
					else
					{
						InputNameStr << ", " << ChannelIndex;
					}

					Names[InputIndex * NumChannels + ChannelIndex] = *InputNameStr;
					InputNameStr.Reset();
				}
			}

			return Names;
		}

		static TArray<FVertexName> InitializeAudioOutputNames()
		{
			TStringBuilder<32> AudioOutNameStr;
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumChannels);

			for (int ChanIndex = 0; ChanIndex < NumChannels; ++ChanIndex)
			{
				AudioOutNameStr << "Out";

				if (NumChannels == 1)
				{
				}
				else if (NumChannels == 2)
				{
					AudioOutNameStr << ' ' << ((!ChanIndex)? 'L' : 'R');
				}
				else
				{
					AudioOutNameStr << ' ' << ChanIndex;
				}
			
				Names[ChanIndex] = *AudioOutNameStr;
				AudioOutNameStr.Reset();
			}

			return Names;
		}

		static TArray<FVertexName> InitializeGainInputNames()
		{
			TStringBuilder<32> GainNameStr;
			TArray<FVertexName> Names;
			Names.AddUninitialized(NumInputs);

			for (int GainIndex = 0; GainIndex < NumInputs; ++GainIndex)
			{
				GainNameStr << "Gain " << GainIndex;
				Names[GainIndex] = * GainNameStr;

				GainNameStr.Reset();
			}

			return Names;
		}

		static const FVertexName& GetAudioInputName(uint32 InputIndex, uint32 ChannelIndex)
		{
			return AudioInputNames[InputIndex * NumChannels + ChannelIndex];
		}


		static inline const TArray<FVertexName> AudioInputNames = InitializeAudioInputNames();
		static inline const TArray<FVertexName> AudioOutputNames = InitializeAudioOutputNames();
		static inline const TArray<FVertexName> GainInputNames = InitializeGainInputNames();

		TArray<FFloatReadRef> Gains;
		TArray<TOptional<FAudioBufferReadRef>> Inputs;
		TArray<FAudioBufferWriteRef> Outputs;

		TArray<float> PrevGains;

		static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "AudioMixer", InOperatorName, FName() },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Mix },
				{ METASOUND_LOCTEXT("Metasound_AudioMixerKeyword", "Mixer") },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}


#if WITH_EDITOR
		static const FText GetAudioInputDescription(uint32 InputIndex, uint32 ChannelIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioInputDescription", "Audio Input #: {0}, Channel: {1}", InputIndex, ChannelIndex);
		}

		static const FText GetAudioInputDisplayName(uint32 InputIndex, uint32 ChannelIndex)
		{
			if (NumChannels == 1)
			{
				return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioInput1In", "In {0}", InputIndex);
			}
			else if (NumChannels == 2)
			{
				if (ChannelIndex == 0)
				{
					return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioInput2InL", "In {0} L", InputIndex);
				}
				else
				{
					return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioInput2InR", "In {0} R", InputIndex);
				}
			}
			return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioInputIn", "In {0}, {0}", InputIndex, ChannelIndex);
		}

		static const FText GetGainInputDisplayName(uint32 InputIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("AudioMixerGainInputDisplayName", "Gain {0} (Lin)", InputIndex);
		}

		static const FText GetGainInputDescription(uint32 InputIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("AudioMixerGainInputDescription", "Gain Input #: {0}", InputIndex);
		}

		static const FText GetAudioOutputDisplayName(uint32 ChannelIndex)
		{
			if (NumChannels == 1)
			{
				return METASOUND_LOCTEXT("AudioMixerAudioOutput1Out", "Out");
			}
			else if (NumChannels == 2)
			{
				if (ChannelIndex == 0)
				{
					return METASOUND_LOCTEXT("AudioMixerAudioOutput2OutL", "Out L");
				}
				else
				{
					return METASOUND_LOCTEXT("AudioMixerAudioOutput2OutR", "Out R");
				}
			}
			return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioOutputOut", "Out {0}", ChannelIndex);
		}

		static const FText GetAudioOutputDescription(uint32 ChannelIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("AudioMixerAudioOutputDescription", "Summed output for channel: {0}", ChannelIndex);
		}
#endif // WITH_EDITOR
	}; // class TAudioMixerNodeOperator


	template<uint32 NumInputs, uint32 NumChannels>
	using TAudioMixerNode = TNodeFacade<TAudioMixerNodeOperator<NumInputs, NumChannels>>;

	#define REGISTER_AUDIOMIXER_NODE(A, B) \
		using FAudioMixerNode_##A ## _ ##B = TAudioMixerNode<A, B>; \
		METASOUND_REGISTER_NODE(FAudioMixerNode_##A ## _ ##B) \


	// mono
	REGISTER_AUDIOMIXER_NODE(2, 1)
	REGISTER_AUDIOMIXER_NODE(3, 1)
	REGISTER_AUDIOMIXER_NODE(4, 1)
	REGISTER_AUDIOMIXER_NODE(5, 1)
	REGISTER_AUDIOMIXER_NODE(6, 1)
	REGISTER_AUDIOMIXER_NODE(7, 1)
	REGISTER_AUDIOMIXER_NODE(8, 1)

	// stereo
 	REGISTER_AUDIOMIXER_NODE(2, 2)
	REGISTER_AUDIOMIXER_NODE(3, 2)
	REGISTER_AUDIOMIXER_NODE(4, 2)
	REGISTER_AUDIOMIXER_NODE(5, 2)
	REGISTER_AUDIOMIXER_NODE(6, 2)
	REGISTER_AUDIOMIXER_NODE(7, 2)
	REGISTER_AUDIOMIXER_NODE(8, 2)

	// test
//	REGISTER_AUDIOMIXER_NODE(8, 6)

} // namespace Metasound

#undef LOCTEXT_NAMESPACE // "MetasoundStandardNodes_MixerNode"
