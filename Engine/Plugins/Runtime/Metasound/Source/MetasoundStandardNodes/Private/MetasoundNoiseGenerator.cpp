// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundNoiseGenerator.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "DSP/Noise.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_GeneratorNodes"

namespace Metasound
{
	enum class ENoiseType
	{
		Pink,
		White
	};

	DECLARE_METASOUND_ENUM(ENoiseType, ENoiseType::Pink, METASOUNDSTANDARDNODES_API,
		FEnumNoiseType, FEnumNoiseTypeInfo, FEnumNoiseTypeReadRef, FEnumNoiseTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ENoiseType, FEnumNoiseType, "NoiseType")
		DEFINE_METASOUND_ENUM_ENTRY(ENoiseType::Pink, "PinkDescription", "Pink Noise", "PinkDescriptionTT", "Pink noise or 1/f noise, spectral density is inversely proportional to the frequency of the signal"),
		DEFINE_METASOUND_ENUM_ENTRY(ENoiseType::White, "WhiteDescription", "White Noise", "WhiteDescriptionTT", "A random signal having equal intensity at different frequencies"),
	DEFINE_METASOUND_ENUM_END()

	namespace NoiseGeneratorVertexNames
	{
		METASOUND_PARAM(OutAudio, "Audio", "Audio output.")
		METASOUND_PARAM(InputType, "Type", "Type of Noise to Generate.")
		METASOUND_PARAM(InputSeed, "Seed", "Seed for seeding the Random Number Generator, -1 (default) will use current time.")
	}

	class FNoiseOperator : public IOperator
	{
	public:
		static constexpr int32 DefaultSeed = INDEX_NONE;

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
		static const FNodeClassMetadata& GetNodeInfo();
		static FVertexInterface DeclareVertexInterface();

		FNoiseOperator(const FOperatorSettings& InSettings, FInt32ReadRef&& InSeedReadRef, FEnumNoiseTypeReadRef&& InNoiseTypeReadRef);
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		virtual IOperator::FPostExecuteFunction GetPostExecuteFunction() override { return nullptr; }

	protected:
		FInt32ReadRef Seed;
		FEnumNoiseTypeReadRef NoiseType;
		FAudioBufferWriteRef Out;
		int32 OldSeed;

		template<typename T>
		static T MakeGenerator(int32 InSeed)
		{
			// If we the default seed, use the default noise generator constructor.
			if (InSeed == DefaultSeed)
			{
				return T{};
			}
			else
			{
				return T{ InSeed };
			}
		}

		template<typename T>
		void ResetNoiseOperator(T& InOutGenerator)
		{
			InOutGenerator = MakeGenerator<T>(*Seed);
			OldSeed = *Seed;
		}

		template<typename T>
		FORCEINLINE void CheckAndReseed(T& InOutGenerator)
		{
			// Reseed?
			int32 NewSeed = *Seed;
			if (OldSeed != NewSeed)
			{
				InOutGenerator = MakeGenerator<T>(NewSeed);
				OldSeed = NewSeed;
			}
		}

		template<typename T>
		FORCEINLINE void Generate(T& InGenerator)
		{
			float* WritePtr = Out->GetData();
			for (int32 i = Out->Num(); i > 0; --i)
			{
				*WritePtr++ = InGenerator.Generate();
			}
		}
	};

	struct FNoiseOperator_White final : public FNoiseOperator
	{
		Audio::FWhiteNoise Generator;

		FNoiseOperator_White(const FOperatorSettings& InSettings, FInt32ReadRef&& InSeed, FEnumNoiseTypeReadRef&& InNoiseTypeReadRef)
			: FNoiseOperator{ InSettings, MoveTemp(InSeed), MoveTemp(InNoiseTypeReadRef) }
			, Generator{ MakeGenerator<Audio::FWhiteNoise>(*Seed) }
		{}

		void Reset(const IOperator::FResetParams& InParams)
		{
			ResetNoiseOperator(Generator);
			Out->Zero();
		}

		void Execute()
		{
			// Reseed if necessary.
			CheckAndReseed(Generator);

			// Generate a block.
			Generate(Generator);
		}

		virtual FResetFunction GetResetFunction() override { return &FNoiseOperator_White::ResetFunction; }
		static void ResetFunction(IOperator* InOperator, const IOperator::FResetParams& InParams) { static_cast<FNoiseOperator_White*>(InOperator)->Reset(InParams); }

		static void ExecuteFunction(IOperator* InOperator) { static_cast<FNoiseOperator_White*>(InOperator)->Execute(); }
		FExecuteFunction GetExecuteFunction() override { return &FNoiseOperator_White::ExecuteFunction; }

	};

	struct FNoiseOperator_Pink final : public FNoiseOperator
	{
		using FNoiseOperator::FNoiseOperator;
		Audio::FPinkNoise Generator;

		FNoiseOperator_Pink(const FOperatorSettings& InSettings, FInt32ReadRef&& InSeed, FEnumNoiseTypeReadRef&& InNoiseTypeReadRef)
			: FNoiseOperator{ InSettings, MoveTemp(InSeed), MoveTemp(InNoiseTypeReadRef) }
			, Generator{ MakeGenerator<Audio::FPinkNoise>(*Seed) }
		{}

		static void ExecuteFunction(IOperator* InOperator) { static_cast<FNoiseOperator_Pink*>(InOperator)->Execute(); }
		FExecuteFunction GetExecuteFunction() override { return &FNoiseOperator_Pink::ExecuteFunction; }

		virtual FResetFunction GetResetFunction() override { return &FNoiseOperator_Pink::ResetFunction; }
		static void ResetFunction(IOperator* InOperator, const IOperator::FResetParams& InParams) { static_cast<FNoiseOperator_Pink*>(InOperator)->Reset(InParams); }

		void Reset(const IOperator::FResetParams& InParams)
		{
			ResetNoiseOperator(Generator);
			Out->Zero();
		}

		void Execute()
		{
			// Reseed if necessary.
			CheckAndReseed(Generator);

			// Generate a block.
			Generate(Generator);
		}
	};
	
	FNoiseOperator::FNoiseOperator(const FOperatorSettings& InSettings, FInt32ReadRef&& InSeedReadRef, FEnumNoiseTypeReadRef&& InNoiseTypeReadRef)
		: Seed{ MoveTemp(InSeedReadRef) }
		, NoiseType{ MoveTemp(InNoiseTypeReadRef) }
		, Out{ FAudioBufferWriteRef::CreateNew(InSettings) }
		, OldSeed(*Seed)
	{}


	void FNoiseOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace NoiseGeneratorVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSeed), Seed);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputType), NoiseType);
	}

	void FNoiseOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace NoiseGeneratorVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutAudio), Out);
	}

	FVertexInterface FNoiseOperator::DeclareVertexInterface()
	{
		using namespace NoiseGeneratorVertexNames; 

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSeed), FNoiseOperator::DefaultSeed),
				TInputDataVertex<FEnumNoiseType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputType), (int32)ENoiseType::Pink)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
			)
		);
		return Interface;
	}

	const FNodeClassMetadata& FNoiseOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Noise"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_NoiseNodeDisplayNameX", "Noise");
			Info.Description = METASOUND_LOCTEXT("Metasound_NoiseNodeDescription", "Noise Generator that produces different types of noise");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = DeclareVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
			return Info;
		};
		static const FNodeClassMetadata Info = InitNodeInfo();
		return Info;
	}

	FNoiseNode::FNoiseNode(const FVertexName& InName, const FGuid& InInstanceID, int32)
	: FNoiseNode(FNodeInitData{InName, InInstanceID})
	{}

	FNoiseNode::FNoiseNode(const FNodeInitData& InInitData)
	: FNoiseNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, FNoiseOperator::GetNodeInfo().DefaultInterface), MakeShared<const FNodeClassMetadata>(FNoiseOperator::GetNodeInfo()))
	{}

	FNoiseNode::FNoiseNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata)
	: FNodeFacade(MoveTemp(InNodeData), MoveTemp(InClassMetadata), TFacadeOperatorClass<FNoiseOperator>())
	{
	}

	FNodeClassMetadata FNoiseNode::CreateNodeClassMetadata()
	{
		return FNoiseOperator::GetNodeInfo();
	}

	TUniquePtr<IOperator> FNoiseOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace NoiseGeneratorVertexNames; 

		const FOperatorSettings& Settings = InParams.OperatorSettings;
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		// Static property pin, only used for factory.
		FEnumNoiseTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumNoiseType>(METASOUND_GET_PARAM_NAME(InputType), Settings);

		// Seed.
		FInt32ReadRef Seed = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSeed), Settings);

		switch (*Type)
		{
		default:
		case ENoiseType::White:
			return MakeUnique<FNoiseOperator_White>(InParams.OperatorSettings, MoveTemp(Seed), MoveTemp(Type));
		case ENoiseType::Pink:
			return MakeUnique<FNoiseOperator_Pink>(InParams.OperatorSettings, MoveTemp(Seed), MoveTemp(Type));
		}
	}

	METASOUND_REGISTER_NODE(FNoiseNode);
}
#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes

