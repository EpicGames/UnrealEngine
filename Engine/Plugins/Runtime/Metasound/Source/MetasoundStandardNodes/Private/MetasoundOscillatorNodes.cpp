// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundOscillatorNodes.h"

#include "DSP/Dsp.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOscillators.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_OscillatorNodes"

namespace Metasound
{			
#pragma region Common

	namespace OscillatorCommonVertexNames
	{
		METASOUND_PARAM(EnabledPin, "Enabled", "Enable the oscillator.")
		METASOUND_PARAM(BiPolarPin, "Bi Polar", "If the output is Bi-Polar (-1..1) or Uni-Polar (0..1)")
		METASOUND_PARAM(FrequencyModPin, "Modulation","Modulation Frequency Input (for doing FM)")
		METASOUND_PARAM(OscBaseFrequencyPin, "Frequency", "Base Frequency of Oscillator in Hz.")
		METASOUND_PARAM(OscPhaseResetPin, "Sync", "Phase Reset")
		METASOUND_PARAM(PhaseOffsetPin, "Phase Offset", "Phase Offset In Degrees (0..360)")
		METASOUND_PARAM(GlideFactorPin, "Glide", "The amount of glide to use when changing frequencies. 0.0 = no glide, 1.0 = lots of glide.")
		METASOUND_PARAM(AudioOutPin, "Audio", "The output audio")
		static const FLazyName TypePinName("Type");
	}

	// Base class of Oscillator factories which holds common the interface.
	class FOscilatorFactoryBase : public IOperatorFactory
	{
	public:
		// Common to all Oscillators.
		static FVertexInterface GetCommmonVertexInterface()
		{
			using namespace OscillatorCommonVertexNames; 

			static const FVertexInterface Interface
			{
				FInputVertexInterface{
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(EnabledPin), true),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(BiPolarPin), true),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OscBaseFrequencyPin), 440.f),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(FrequencyModPin)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OscPhaseResetPin)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PhaseOffsetPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(GlideFactorPin), 0.f)
				},
				FOutputVertexInterface{
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(AudioOutPin))
				}
			};
			return Interface;
		}
	};

	// Common set of construct params for each of the Oscillators.
	struct FOscillatorOperatorConstructParams
	{
		const FOperatorSettings& Settings;
		FBoolReadRef Enabled;
		FFloatReadRef BaseFrequency;
		FFloatReadRef PhaseOffset; 
		FTriggerReadRef PhaseReset;
		FFloatReadRef GlideFactor;
		FBoolReadRef BiPolar;
	};

	// Base Oscillator Operator CRTP.
	// Expects your Derived class to implement a Generate(int32 InStartFrame, int32 InEndFrame, float InFreq)
	template<typename GeneratorPolicy, typename Derived>
	class TOscillatorOperatorBase : public TExecutableOperator<TOscillatorOperatorBase<GeneratorPolicy, Derived>>
	{		
	public:
		TOscillatorOperatorBase(const FOscillatorOperatorConstructParams& InConstructParams)
			: Generator{*InConstructParams.PhaseOffset}
			, SampleRate(InConstructParams.Settings.GetSampleRate())
			, Nyquist(InConstructParams.Settings.GetSampleRate() / 2.0f)
			, Enabled(InConstructParams.Enabled)
			, BaseFrequency(InConstructParams.BaseFrequency)
			, PhaseReset(InConstructParams.PhaseReset)
			, PhaseOffset(InConstructParams.PhaseOffset)
			, GlideFactor(InConstructParams.GlideFactor)
			, BiPolar(InConstructParams.BiPolar)
			, AudioBuffer(FAudioBufferWriteRef::CreateNew(InConstructParams.Settings))
		{
			check(AudioBuffer->Num() == InConstructParams.Settings.GetNumFramesPerBlock());
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace OscillatorCommonVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(EnabledPin), Enabled);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OscBaseFrequencyPin), BaseFrequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), PhaseOffset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OscPhaseResetPin), PhaseReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(GlideFactorPin), GlideFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(BiPolarPin), BiPolar);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace OscillatorCommonVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AudioOutPin), AudioBuffer);
		}

		void ResetPhase(float InPhaseInDegrees)
		{
			float LinearPhase = FMath::Clamp(InPhaseInDegrees, 0.f, 360.f) / 360.f;
			
			// Recreate the generator type with the phase requested.
			Generator = GeneratorPolicy{ LinearPhase };
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			ResetPhase(*PhaseOffset);
			AudioBuffer->Zero();
		}

		void Execute()
		{
			// Clamp frequencies into Nyquist range.
			const float ClampedFreq = FMath::Clamp(*BaseFrequency, -Nyquist, Nyquist);
			const float ClampedGlideEase = Audio::GetLogFrequencyClamped(*GlideFactor, { 0.0f, 1.0f }, { 1.0f, 0.0001f });
			AudioBuffer->Zero();

			Derived* Self = static_cast<Derived*>(this);
			PhaseReset->ExecuteBlock
			(
				[Self, ClampedFreq, ClampedGlideEase](int32 InFrameStart, int32 InFrameEnd)
				{
					Self->Generate(InFrameStart, InFrameEnd, ClampedFreq, ClampedGlideEase);
				},
				[Self, ClampedFreq, ClampedGlideEase](int32 InFrameStart, int32 InFrameEnd)
				{
					Self->ResetPhase(*Self->PhaseOffset);
					Self->Generate(InFrameStart, InFrameEnd, ClampedFreq, ClampedGlideEase);
				}
			);
		}

	protected:
		GeneratorPolicy Generator;
		float SampleRate;
		float Nyquist;

		FBoolReadRef Enabled;
		FFloatReadRef BaseFrequency;
		FTriggerReadRef PhaseReset;
		FFloatReadRef PhaseOffset;
		FFloatReadRef GlideFactor;
		FBoolReadRef BiPolar;

		FAudioBufferWriteRef AudioBuffer;
	};

	// Generic Oscillator operator for NON-FM Operators. 
	template<typename GeneratorPolicy, typename TGeneratorEnumReadRefType>
	class TOscillatorOperator final : public TOscillatorOperatorBase<GeneratorPolicy, TOscillatorOperator<GeneratorPolicy, TGeneratorEnumReadRefType>>
	{
		using Super = TOscillatorOperatorBase<GeneratorPolicy, TOscillatorOperator<GeneratorPolicy, TGeneratorEnumReadRefType>>;
	public:
		TOscillatorOperator(const FOscillatorOperatorConstructParams& InConstructParams, const TGeneratorEnumReadRefType& InGeneratorType)
			: Super(InConstructParams)
			, GeneratorType(InGeneratorType)
		{}

		void Generate(int32 InStartFrame, int32 InEndFrame, float InClampedFreq, float InClampedGlideEase)
		{		
			int32 NumFrames = InEndFrame - InStartFrame;

			if (*this->Enabled && NumFrames > 0)
			{
				this->Generator(
				{
					this->SampleRate, 
					InClampedFreq,
					InClampedGlideEase,
					0.f,
					*this->BiPolar,
					MakeArrayView(this->AudioBuffer->GetData() + InStartFrame, NumFrames), // Not aligned.
				});
			}			
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace OscillatorCommonVertexNames;
			Super::BindInputs(InOutVertexData);
			InOutVertexData.BindReadVertex(TypePinName, GeneratorType);
		}

	private:
		TGeneratorEnumReadRefType GeneratorType;
	};

	// Generic Oscillator operator for FM Operators.
	template<typename TGeneratorPolicy, typename TGeneratorEnumReadRefType>
	class TOscillatorOperatorFM final : public TOscillatorOperatorBase<TGeneratorPolicy, TOscillatorOperatorFM<TGeneratorPolicy, TGeneratorEnumReadRefType>>
	{
		using Super = TOscillatorOperatorBase<TGeneratorPolicy, TOscillatorOperatorFM<TGeneratorPolicy, TGeneratorEnumReadRefType>>;
	public:
		TOscillatorOperatorFM(const FOscillatorOperatorConstructParams& InCommonParams, const FAudioBufferReadRef& InFmData, const TGeneratorEnumReadRefType& InGeneratorType)
			: Super(InCommonParams)
			, Fm(InFmData)
			, GeneratorType(InGeneratorType)
		{}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace OscillatorCommonVertexNames;
			Super::BindInputs(InOutVertexData);
			InOutVertexData.BindReadVertex(TypePinName, GeneratorType);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FrequencyModPin), Fm);
		}

		void Generate(int32 InStartFrame, int32 InEndFrame, float InClampedFreq, float InClampedGlideEase)
		{
			int32 NumFrames = InEndFrame - InStartFrame;
			if (*this->Enabled && NumFrames > 0)
			{
				this->Generator(
				{
					this->SampleRate, 
					InClampedFreq,
					InClampedGlideEase,
					0.f,
					*this->BiPolar,
					MakeArrayView(this->AudioBuffer->GetData() + InStartFrame, NumFrames), // Not aligned.
					MakeArrayView(this->Fm->GetData() + InStartFrame, NumFrames) // Not aligned.
				});
			}
		}		

	private:
		FAudioBufferReadRef Fm;
		TGeneratorEnumReadRefType GeneratorType;
	};

	FOscilatorNodeBase::FOscilatorNodeBase(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FNodeClassMetadata& InInfo, const TSharedRef<IOperatorFactory, ESPMode::ThreadSafe>& InFactory, float InDefaultFrequency, float InDefaultGlideFactor, bool bInDefaultEnablement)
		: FBasicNode(FNodeData(InInstanceName, InInstanceID, InInfo.DefaultInterface), MakeShared<FNodeClassMetadata>(InInfo))
		, Factory(InFactory)
		, DefaultFrequency(InDefaultFrequency)
		, DefaultGlideFactor(InDefaultGlideFactor)
		, bDefaultEnablement(bInDefaultEnablement)
	{}

	FOscilatorNodeBase::FOscilatorNodeBase(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata, TSharedRef<IOperatorFactory> InFactory)
	: FBasicNode(MoveTemp(InNodeData), MoveTemp(InClassMetadata))
	, Factory(MoveTemp(InFactory))
	{
	}
	
#pragma endregion Common

#pragma region Sine

	enum class ESineGenerationType
	{
		Rotation,
		Sinf,
		Bhaskara,
		Wavetable,
	};

	DECLARE_METASOUND_ENUM(ESineGenerationType, ESineGenerationType::Wavetable, METASOUNDSTANDARDNODES_API,
		FEnumSineGenerationType, FEnumSineGenerationTypeInfo, FEnumSineGenerationTypeReadRef, FEnumSineGenerationTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ESineGenerationType, FEnumSineGenerationType, "SineGenerationType")
		DEFINE_METASOUND_ENUM_ENTRY(ESineGenerationType::Rotation, "RotationDescription", "2D Rotation", "RotationDescriptionTT", "Rotates around the unit circle generate the sine. Note: Glide and audio rate FM modulation is not supported with the 2D rotator."),
		DEFINE_METASOUND_ENUM_ENTRY(ESineGenerationType::Sinf, "SinfDescription", "Pure Math", "SinfDescriptionTT", "Uses the standard math library (Sinf) to generate the sine (most expensive)"),
		DEFINE_METASOUND_ENUM_ENTRY(ESineGenerationType::Bhaskara, "BhaskaraDescription", "Bhaskara", "BhaskaraDescriptionTT", "Sine approximation using Bhaskara technique discovered in 7th century"),
		DEFINE_METASOUND_ENUM_ENTRY(ESineGenerationType::Wavetable, "WavetableDescription", "Wavetable", "WavetableDescriptionTT", "Uses a wavetable to generate the sine"),
	DEFINE_METASOUND_ENUM_END()

	namespace SineOscilatorVertexNames
	{
		METASOUND_PARAM(SineType, "Type", "Type of the Sinewave Generator")
	}

	class FSineOscilatorNode::FFactory : public FOscilatorFactoryBase
	{
	public:
		FFactory() = default;
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Sine"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 1;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_SineNodeDisplayName", "Sine");
				Info.Description = METASOUND_LOCTEXT("Metasound_SineNodeDescription", "Emits an audio signal of a sinusoid.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("OscSineKeyword", "Osc"), METASOUND_LOCTEXT("FMSineKeyword", "FM"), METASOUND_LOCTEXT("SineSynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SineOscilatorVertexNames; 

			auto MakeInterface = []() -> FVertexInterface
			{
				FVertexInterface Interface = GetCommmonVertexInterface();
				Interface.GetInputInterface().Add(
					TInputDataVertex<FEnumSineGenerationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(SineType), static_cast<int32>(ESineGenerationType::Wavetable))
				);
				return Interface;
			};
			static const FVertexInterface Interface = MakeInterface();
			return Interface;
		}
		
		TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
		{
			const FSineOscilatorNode& SineNode = static_cast<const FSineOscilatorNode&>(InParams.Node);
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			using namespace Generators;
			using namespace OscillatorCommonVertexNames; 
			using namespace SineOscilatorVertexNames;
			
			FOscillatorOperatorConstructParams OpParams
			{
				Settings,
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(OscBaseFrequencyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(OscPhaseResetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(BiPolarPin), Settings)
			};

			// TODO: Make this a static prop. For now its a pin.
			
			// Check to see if we have an FM input connected.
			bool bHasFM = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin));
			FEnumSineGenerationTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumSineGenerationType>(METASOUND_GET_PARAM_NAME(SineType), Settings);
			if (bHasFM)
			{

				// FM Oscillators.
				FAudioBufferReadRef FmBuffer = InputData.GetDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin));
				switch (*Type)
				{
				default:
				case ESineGenerationType::Sinf: return MakeUnique<TOscillatorOperatorFM<FSinfWithFm, FEnumSineGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
				case ESineGenerationType::Bhaskara: return MakeUnique<TOscillatorOperatorFM<FBhaskaraWithFm, FEnumSineGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
				case ESineGenerationType::Wavetable: return MakeUnique<TOscillatorOperatorFM<FSineWaveTableWithFm, FEnumSineGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
				}
			}
			else //HasFM
			{
				switch (*Type)
				{
				default:
				case ESineGenerationType::Sinf: return MakeUnique<TOscillatorOperator<FSinf, FEnumSineGenerationTypeReadRef>>(OpParams, Type);
				case ESineGenerationType::Rotation: return MakeUnique<TOscillatorOperator<F2DRotatorGenerateBlock, FEnumSineGenerationTypeReadRef>>(OpParams, Type);
				case ESineGenerationType::Bhaskara: return MakeUnique<TOscillatorOperator<FBhaskara, FEnumSineGenerationTypeReadRef>>(OpParams, Type);
				case ESineGenerationType::Wavetable: return MakeUnique<TOscillatorOperator<FSineWaveTable, FEnumSineGenerationTypeReadRef>>(OpParams, Type);
				}
			} // HasFM
		}
	};

	FSineOscilatorNode::FSineOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefautlGlideFactor, bool bInDefaultEnablement)
		: FOscilatorNodeBase(InInstanceName, InInstanceID, FFactory::GetNodeInfo(), MakeShared<FFactory, ESPMode::ThreadSafe>(), InDefaultFrequency, InDefautlGlideFactor, bInDefaultEnablement )
	{}

	FSineOscilatorNode::FSineOscilatorNode(const FNodeInitData& InInitData)
	: FSineOscilatorNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, FFactory::GetNodeInfo().DefaultInterface), MakeShared<const FNodeClassMetadata>(FFactory::GetNodeInfo()))
	{}

	FSineOscilatorNode::FSineOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	: FOscilatorNodeBase(MoveTemp(InNodeData), MoveTemp(InMetadata), MakeShared<FFactory>())
	{
	}

	FNodeClassMetadata FSineOscilatorNode::CreateNodeClassMetadata()
	{
		return FFactory::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FSineOscilatorNode);

#pragma endregion Sine

#pragma region Saw

	enum class ESawGenerationType
	{
		PolySmooth,
		Trivial,
		Wavetable
	};

	DECLARE_METASOUND_ENUM(ESawGenerationType, ESawGenerationType::PolySmooth, METASOUNDSTANDARDNODES_API,
		FEnumSawGenerationType, FEnumSawGenerationTypeInfo, FSawGenerationTypeReadRef, FEnumSawGenerationTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ESawGenerationType, FEnumSawGenerationType, "SawGenerationType")
		DEFINE_METASOUND_ENUM_ENTRY(ESawGenerationType::PolySmooth, "SawPolySmoothDescription", "Poly Smooth", "PolySmoothDescriptionTT", "PolySmooth (i.e. BLEP)"),
		DEFINE_METASOUND_ENUM_ENTRY(ESawGenerationType::Trivial, "SawTrivialDescription", "Trivial", "TrivialDescriptionTT", "The most basic raw implementation"),
		//DEFINE_METASOUND_ENUM_ENTRY(ESawGenerationType::Wavetable, "SawWavetableDescription", "Wavetable", "SawWavetableDescriptionTT", "Use a Wavetable iterpolation to generate the Waveform")
	DEFINE_METASOUND_ENUM_END()
	
	namespace SawOscilatorVertexNames
	{
		METASOUND_PARAM(SawType, "Type", "Type of the Saw Generator")
	}

	class FSawOscilatorNode::FFactory : public FOscilatorFactoryBase
	{
	public:
		FFactory() = default;

		TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Saw"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_SawNodeDisplayName", "Saw");
				Info.Description = METASOUND_LOCTEXT("Metasound_SawNodeDescription", "Emits a Saw wave");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("OscSawKeyword", "Osc"), METASOUND_LOCTEXT("FMSawKeyword", "FM"), METASOUND_LOCTEXT("SawSynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SawOscilatorVertexNames;

			auto MakeInterface = []() -> FVertexInterface
			{
				FVertexInterface Interface = GetCommmonVertexInterface();
				Interface.GetInputInterface().Add(
					TInputDataVertex<FEnumSawGenerationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(SawType))
				);
				return Interface;
			};
			static const FVertexInterface Interface = MakeInterface();
			return Interface;
		}
	};

	TUniquePtr<Metasound::IOperator> FSawOscilatorNode::FFactory::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FSawOscilatorNode& Node = static_cast<const FSawOscilatorNode&>(InParams.Node);
		const FInputVertexInterfaceData& InputData = InParams.InputData;
		const FOperatorSettings& Settings = InParams.OperatorSettings;
		using namespace Generators;
		using namespace OscillatorCommonVertexNames;
		using namespace SawOscilatorVertexNames;

		FSawGenerationTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumSawGenerationType>(METASOUND_GET_PARAM_NAME(SawType), Settings);

		FOscillatorOperatorConstructParams OpParams
		{
			Settings,
			InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
			InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(OscBaseFrequencyPin), Settings),
			InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), Settings),
			InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(OscPhaseResetPin), Settings),
			InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
			InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(BiPolarPin), Settings)
		};

		bool bHasFM = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin));

		if (bHasFM)
		{
			FAudioBufferReadRef FmBuffer = InputData.GetDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin));
			switch (*Type)
			{
			default:
			case ESawGenerationType::Trivial: return MakeUnique<TOscillatorOperatorFM<FSawWithFm, FSawGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
			case ESawGenerationType::PolySmooth: return MakeUnique<TOscillatorOperatorFM<FSawPolysmoothWithFm, FSawGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
			}
		}
		else
		{
			switch (*Type)
			{
			default:
			case ESawGenerationType::Trivial: return MakeUnique<TOscillatorOperator<FSaw, FSawGenerationTypeReadRef>>(OpParams, Type);
			case ESawGenerationType::PolySmooth: return MakeUnique<TOscillatorOperator<FSawPolysmooth, FSawGenerationTypeReadRef>>(OpParams, Type);
			}
		}
	}

	FSawOscilatorNode::FSawOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefaultGlideFactor, bool bInDefaultEnablement)
		: FOscilatorNodeBase(InInstanceName, InInstanceID, FFactory::GetNodeInfo(), MakeShared<FFactory, ESPMode::ThreadSafe>(), InDefaultFrequency, InDefaultGlideFactor, bInDefaultEnablement)
	{}

	FSawOscilatorNode::FSawOscilatorNode(const FNodeInitData& InInitData)
	: FSawOscilatorNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, FFactory::GetNodeInfo().DefaultInterface), MakeShared<FNodeClassMetadata>(FFactory::GetNodeInfo()))
	{}

	FSawOscilatorNode::FSawOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	: FOscilatorNodeBase(MoveTemp(InNodeData), MoveTemp(InMetadata), MakeShared<FFactory>())
	{
	}

	FNodeClassMetadata FSawOscilatorNode::CreateNodeClassMetadata()
	{
		return FFactory::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FSawOscilatorNode);

#pragma endregion Saw

#pragma region Square

	enum class ESquareGenerationType
	{
		PolySmooth,
		Trivial,
		Wavetable
	};

	DECLARE_METASOUND_ENUM(ESquareGenerationType, ESquareGenerationType::PolySmooth, METASOUNDSTANDARDNODES_API,
	FEnumSquareGenerationType, FEnumSquareGenerationTypeInfo, FSquareGenerationTypeReadRef, FEnumSquareGenerationTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ESquareGenerationType, FEnumSquareGenerationType, "SquareGenerationType")
		DEFINE_METASOUND_ENUM_ENTRY(ESquareGenerationType::PolySmooth, "SquarePolySmoothDescription", "Poly Smooth", "PolySmoothDescriptionTT", "PolySmooth (i.e. BLEP)"),
		DEFINE_METASOUND_ENUM_ENTRY(ESquareGenerationType::Trivial, "SquareTrivialDescription", "Trivial", "SquareTrivialDescriptionTT", "The most basic raw implementation. Does not obey pulse width."),
		//DEFINE_METASOUND_ENUM_ENTRY(ESquareGenerationType::Wavetable, "SquareWavetableDescription", "Wavetable", "SquareWavetableDescriptionTT", "Use a Wavetable interpolation to generate the Waveform")
	DEFINE_METASOUND_ENUM_END()

	namespace SquareOscillatorVertexNames
	{
		METASOUND_PARAM(SquarePulseWidthPin, "Pulse Width", "The Width of the square part of the wave")
		METASOUND_PARAM(SquareTypePin, "Type", "The generator type to make the squarewave")
	}
	
	template<typename GeneratorPolicy>
	class FSquareOperator final : public TOscillatorOperatorBase<GeneratorPolicy, FSquareOperator<GeneratorPolicy>>
	{
		using Super = TOscillatorOperatorBase<GeneratorPolicy, FSquareOperator<GeneratorPolicy>>;
	public:
		FSquareOperator(const FOscillatorOperatorConstructParams& InConstructParams, const FFloatReadRef& InPulseWidth, const FSquareGenerationTypeReadRef& InGeneratorType)
			: Super(InConstructParams)
			, PulseWidth(InPulseWidth)
			, GeneratorType(InGeneratorType)
		{}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SquareOscillatorVertexNames;
			using namespace OscillatorCommonVertexNames;

			Super::BindInputs(InOutVertexData);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(SquarePulseWidthPin), PulseWidth);
			InOutVertexData.BindReadVertex(TypePinName, GeneratorType);
		}

		void Generate(int32 InStartFrame, int32 InEndFrame, float InClampedFreq, float InClampedGlideEase)
		{
			int32 NumFrames = InEndFrame - InStartFrame;
			float ClampedPulseWidth = FMath::Clamp(*PulseWidth, 0.0f, 1.0f);
			if (*this->Enabled && NumFrames > 0)
			{
				this->Generator(
				{
					this->SampleRate,
					InClampedFreq,
					InClampedGlideEase,
					ClampedPulseWidth,
					*this->BiPolar, 
					MakeArrayView(this->AudioBuffer->GetData() + InStartFrame, NumFrames), // Not aligned.
				});
			}
		}

	private:
		FFloatReadRef PulseWidth;
		FSquareGenerationTypeReadRef GeneratorType;
	};

	template<typename GeneratorPolicy>
	class FSquareOperatorFM final : public TOscillatorOperatorBase<GeneratorPolicy, FSquareOperatorFM<GeneratorPolicy>>
	{
		using Super = TOscillatorOperatorBase<GeneratorPolicy, FSquareOperatorFM<GeneratorPolicy>>;
	public:
		FSquareOperatorFM(const FOscillatorOperatorConstructParams& InConstructParams, const FFloatReadRef& InPulseWidth, const FAudioBufferReadRef& InFm, const FSquareGenerationTypeReadRef& InGeneratorType)
			: Super(InConstructParams)
			, PulseWidth(InPulseWidth)
			, FM(InFm)
			, GeneratorType(InGeneratorType)
		{
			check(InFm->GetData());
			check(InConstructParams.Settings.GetNumFramesPerBlock() == InFm->Num());
		}
		void Generate(int32 InStartFrame, int32 InEndFrame, float InClampedFreq, float InClampedGlideEase)
		{
			int32 NumFrames = InEndFrame - InStartFrame;
			float ClampedPulseWidth = FMath::Clamp(*PulseWidth, 0.0f, 1.0f);
			if (*this->Enabled && NumFrames > 0)
			{
				this->Generator(
				{
					this->SampleRate,
					InClampedFreq,
					InClampedGlideEase,
					ClampedPulseWidth,
					*this->BiPolar,
					MakeArrayView(this->AudioBuffer->GetData() + InStartFrame, NumFrames), // Not aligned.
					MakeArrayView(this->FM->GetData() + InStartFrame, NumFrames), // Not aligned.
				});
			}
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SquareOscillatorVertexNames;
			using namespace OscillatorCommonVertexNames;

			Super::BindInputs(InOutVertexData);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(SquarePulseWidthPin), PulseWidth);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FrequencyModPin), FM);
			InOutVertexData.BindReadVertex(TypePinName, GeneratorType);
		}

	private:
		FFloatReadRef PulseWidth;
		FAudioBufferReadRef FM;
		FSquareGenerationTypeReadRef GeneratorType;
	};


	class FSquareOscilatorNode::FFactory : public FOscilatorFactoryBase
	{
	public:
		FFactory() = default;

		TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
		{
			using namespace SquareOscillatorVertexNames;

			const FSquareOscilatorNode& Node = static_cast<const FSquareOscilatorNode&>(InParams.Node);
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			using namespace Generators;
			using namespace OscillatorCommonVertexNames; 

			FOscillatorOperatorConstructParams OpParams
			{
				Settings,
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(OscBaseFrequencyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(OscPhaseResetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(BiPolarPin), Settings)
			};
			
			FSquareGenerationTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumSquareGenerationType>(METASOUND_GET_PARAM_NAME(SquareTypePin), Settings);

			bool bHasFM = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin));
			FFloatReadRef PulseWidth = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(SquarePulseWidthPin), Settings);

			if (bHasFM)
			{
				FAudioBufferReadRef FmBuffer = InputData.GetDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin));
				switch (*Type)
				{
				default:				
				case ESquareGenerationType::Trivial: return MakeUnique<FSquareOperatorFM<FSquareWithFm>>(OpParams, PulseWidth, FmBuffer, Type);
				case ESquareGenerationType::PolySmooth: return MakeUnique<FSquareOperatorFM<FSquarePolysmoothWithFm>>(OpParams, PulseWidth, FmBuffer, Type);
				}
			}
			else
			{
				switch (*Type)
				{
				default:
				case ESquareGenerationType::Trivial: return MakeUnique<FSquareOperator<FSquare>>(OpParams, PulseWidth, Type);
				case ESquareGenerationType::PolySmooth: return MakeUnique<FSquareOperator<FSquarePolysmooth>>(OpParams, PulseWidth, Type);
				}
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Square"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_SquareNodeDisplayName", "Square");
				Info.Description = METASOUND_LOCTEXT("Metasound_SquareNodeDescription", "Emits a Square wave");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("OscSquareKeyword", "Osc"), METASOUND_LOCTEXT("FMSquareKeyword", "FM"), METASOUND_LOCTEXT("SquareSynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SquareOscillatorVertexNames; 

			auto MakeInterface = []() -> FVertexInterface
			{
				FVertexInterface Interface = GetCommmonVertexInterface();
				Interface.GetInputInterface().Add(TInputDataVertex<FEnumSquareGenerationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(SquareTypePin)));
				Interface.GetInputInterface().Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(SquarePulseWidthPin), 0.5f));
				return Interface;
			};
			static const FVertexInterface Interface = MakeInterface();
			return Interface;
		}
	private:
	};
	
	FSquareOscilatorNode::FSquareOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefaultGlideFactor, bool bInDefaultEnablement)
		: FOscilatorNodeBase(InInstanceName, InInstanceID, FFactory::GetNodeInfo(), MakeShared<FFactory, ESPMode::ThreadSafe>(), InDefaultFrequency, InDefaultGlideFactor, bInDefaultEnablement)
	{}

	FSquareOscilatorNode::FSquareOscilatorNode(const FNodeInitData& InInitData)
	: FSquareOscilatorNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, FFactory::GetNodeInfo().DefaultInterface), MakeShared<FNodeClassMetadata>(FFactory::GetNodeInfo()))
	{}

	FSquareOscilatorNode::FSquareOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	: FOscilatorNodeBase(MoveTemp(InNodeData), MoveTemp(InMetadata), MakeShared<FFactory>())
	{
	}

	FNodeClassMetadata FSquareOscilatorNode::CreateNodeClassMetadata()
	{
		return FFactory::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FSquareOscilatorNode)
#pragma endregion Square

#pragma region Triangle

	enum class ETriangleGenerationType
	{
		PolySmooth,
		Trivial,
		Wavetable
	};

	DECLARE_METASOUND_ENUM(ETriangleGenerationType, ETriangleGenerationType::PolySmooth, METASOUNDSTANDARDNODES_API,
	FEnumTriangleGenerationType, FEnumTriangleGenerationTypeInfo, FTriangleGenerationTypeReadRef, FEnumTriangleGenerationTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(ETriangleGenerationType, FEnumTriangleGenerationType, "TriangleGenerationType")
		DEFINE_METASOUND_ENUM_ENTRY(ETriangleGenerationType::PolySmooth, "TrianglePolySmoothDescription", "Poly Smooth", "PolySmoothDescriptionTT", "PolySmooth (i.e. BLEP)"),
		DEFINE_METASOUND_ENUM_ENTRY(ETriangleGenerationType::Trivial, "TriangleTrivialDescription", "Trivial", "TriangleTrivialDescriptionTT", "The most basic raw implementation"),
		//DEFINE_METASOUND_ENUM_ENTRY(ETriangleGenerationType::Wavetable, "TriangleWavetableDescription", "Wavetable", "TriangleWavetableDescriptionTT", "Use a Wavetable iterpolation to generate the Waveform")
	DEFINE_METASOUND_ENUM_END()

	namespace TriangleOscilatorVertexNames
	{
		METASOUND_PARAM(TriangeTypePin, "Type", "The generator type to make the triangle wave")
	}

	class FTriangleOscilatorNode::FFactory : public FOscilatorFactoryBase
	{
	public:
		FFactory() = default;

		TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
		{
			const FTriangleOscilatorNode& Node = static_cast<const FTriangleOscilatorNode&>(InParams.Node);
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			using namespace Generators;
			using namespace OscillatorCommonVertexNames;
			using namespace TriangleOscilatorVertexNames;
			
			FTriangleGenerationTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumTriangleGenerationType>(METASOUND_GET_PARAM_NAME(TriangeTypePin), Settings);
		
			FOscillatorOperatorConstructParams OpParams
			{
				Settings,
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(OscBaseFrequencyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(OscPhaseResetPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(BiPolarPin), Settings)
			};

			bool bHasFM = InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin));
			
			if (bHasFM)
			{
				FAudioBufferReadRef FmBuffer = InputData.GetDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin));
				switch (*Type)
				{
				default:
				case ETriangleGenerationType::PolySmooth: return MakeUnique<TOscillatorOperatorFM<FTrianglePolysmoothWithFm, FTriangleGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
				case ETriangleGenerationType::Trivial: return MakeUnique<TOscillatorOperatorFM<FTriangleWithFm, FTriangleGenerationTypeReadRef>>(OpParams, FmBuffer, Type);
				}
			}
			else
			{
				switch (*Type)
				{
				default:
				case ETriangleGenerationType::PolySmooth: return MakeUnique<TOscillatorOperator<FTrianglePolysmooth, FTriangleGenerationTypeReadRef>>(OpParams, Type);
				case ETriangleGenerationType::Trivial: return MakeUnique<TOscillatorOperator<FTriangle, FTriangleGenerationTypeReadRef>>(OpParams, Type);
				}
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("Triangle"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriangleNodeDisplayName", "Triangle");
				Info.Description = METASOUND_LOCTEXT("Metasound_TriangleNodeDescription", "Emits a Triangle wave");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = {METASOUND_LOCTEXT("OscTriKeyword", "Osc"), METASOUND_LOCTEXT("FMTriKeyword", "FM"), METASOUND_LOCTEXT("TriSynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace TriangleOscilatorVertexNames; 

			auto MakeInterface = []() -> FVertexInterface
			{
				FVertexInterface Interface = GetCommmonVertexInterface();
				Interface.GetInputInterface().Add(
					TInputDataVertex<FEnumTriangleGenerationType>(METASOUND_GET_PARAM_NAME_AND_METADATA(TriangeTypePin))
				);
				return Interface;
			};
			static const FVertexInterface Interface = MakeInterface();
			return Interface;
		}
	};
	
	FTriangleOscilatorNode::FTriangleOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefaultGlideFactor, bool bInDefaultEnablement)
		: FOscilatorNodeBase(InInstanceName, InInstanceID, FFactory::GetNodeInfo(), MakeShared<FFactory, ESPMode::ThreadSafe>(), InDefaultFrequency, InDefaultGlideFactor, bInDefaultEnablement)
	{

	}

	FTriangleOscilatorNode::FTriangleOscilatorNode(const FNodeInitData& InInitData)
	: FTriangleOscilatorNode(FNodeData(InInitData.InstanceName, InInitData.InstanceID, FFactory::GetNodeInfo().DefaultInterface), MakeShared<FNodeClassMetadata>(FFactory::GetNodeInfo()))
	{}

	FTriangleOscilatorNode::FTriangleOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	: FOscilatorNodeBase(MoveTemp(InNodeData), MoveTemp(InMetadata), MakeShared<FFactory>())
	{
	}

	FNodeClassMetadata FTriangleOscilatorNode::CreateNodeClassMetadata()
	{
		return FFactory::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FTriangleOscilatorNode);
#pragma endregion Triangle

#pragma region LFO

	DEFINE_METASOUND_ENUM_BEGIN(ELfoWaveshapeType, FEnumLfoWaveshapeType, "LfoWaveshapeType")
		DEFINE_METASOUND_ENUM_ENTRY(ELfoWaveshapeType::Sine, "LfoWaveShapeSineDescription", "Sine", "LfoWaveShapeSineDescriptionTT", "Sinewave Low Frequency Oscillator"),
		DEFINE_METASOUND_ENUM_ENTRY(ELfoWaveshapeType::Saw, "LfoWaveShapeSawDescription", "Saw", "LfoWaveShapeSawDescriptionTT", "Sawtooth Low Frequency Oscillator"),
		DEFINE_METASOUND_ENUM_ENTRY(ELfoWaveshapeType::Triangle, "LfoWaveShapeTriangleDescription", "Triangle", "LfoWaveShapeTriangleDescriptionTT", "Triangle shape Frequency Oscillator"),
		DEFINE_METASOUND_ENUM_ENTRY(ELfoWaveshapeType::Square, "LfoWaveShapeSquareDescription", "Square", "LfoWaveShapeSquareDescriptionTT", "Square shape Low Frequency Oscillator")
	DEFINE_METASOUND_ENUM_END()

	namespace LfoVertexNames
	{
		// Common pins
		METASOUND_PARAM(WaveshapePin, "Shape", "Waveshape of the LFO")
		METASOUND_PARAM(LfoOutPin, "Out", "Output of the LFO (blockrate)")
		METASOUND_PARAM(LfoBaseFrequencyPin, "Frequency", "Frequency of LFO (Hz), clamped at blockrate")

		METASOUND_PARAM(MinOutputValuePin, "Min Value", "The minimum output value.")
		METASOUND_PARAM(MaxOutputValuePin, "Max Value", "The maximum output value.")
		METASOUND_PARAM(LfoPhaseResetPin, "Sync", "Phase Reset (block rate only)")
		METASOUND_PARAM(PhaseOffsetPin, "Phase Offset", "Phase Offset In Degrees (0..360)")
		METASOUND_PARAM(LfoPulseWidthPin, "Pulse Width", "Pulse Width (0..1)")
	}

	static const TArray<FText> LFOKeywords = { METASOUND_LOCTEXT("SineKeyword", "Sine"), METASOUND_LOCTEXT("SquareKeyword", "Square"), METASOUND_LOCTEXT("TriangleKeyword", "Triangle"), METASOUND_LOCTEXT("SawKeyword", "Saw"), METASOUND_LOCTEXT("OscLFOKeyword", "Oscillator"), METASOUND_LOCTEXT("ModulateKeyword", "Modulate")};

	// Blockrate All-Purpose Oscillator
	class FLfoOperator : public TExecutableOperator<FLfoOperator>
	{
	public:
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace LfoVertexNames; 

			static const FVertexInterface Interface
			{
				FInputVertexInterface{
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(LfoBaseFrequencyPin), 5.f),
					TInputDataVertex<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveshapePin), (int32)ELfoWaveshapeType::Sine),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MinOutputValuePin), -1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MaxOutputValuePin), 1.0f),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(LfoPhaseResetPin)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PhaseOffsetPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(LfoPulseWidthPin), 0.5f)
				},
				FOutputVertexInterface{
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(LfoOutPin))
				}
			};
			return Interface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("LFO"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_LfoNodeDisplayName", "LFO");
				Info.Description = METASOUND_LOCTEXT("Metasound_LfoNodeDescription", "Low frequency oscillator < blockrate");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = LFOKeywords;
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FLfoNode& Node = static_cast<const FLfoNode&>(InParams.Node);
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const FOperatorSettings& Settings = InParams.OperatorSettings;

			using namespace LfoVertexNames;

			return MakeUnique<FLfoOperator>(
				  Settings
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(LfoBaseFrequencyPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME(WaveshapePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MinOutputValuePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MaxOutputValuePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(LfoPhaseResetPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(LfoPulseWidthPin), Settings)
			);
		}

		FLfoOperator(const FOperatorSettings& InSettings, FFloatReadRef&& InFrequency, FEnumLfoWaveshapeTypeReadRef&& InType, FFloatReadRef&& InMinValue, FFloatReadRef&& InMaxValue,
			FTriggerReadRef&& InPhaseReset, FFloatReadRef&& InPhaseOffset, FFloatReadRef&& InPulseWidth)
			: BlockRate{InSettings.GetActualBlockRate()}
			, Phase{0.0f}
			, Frequency{MoveTemp(InFrequency)}
			, Waveshape{MoveTemp(InType)}
			, MinValue{MoveTemp(InMinValue)}
			, MaxValue{MoveTemp(InMaxValue)}
			, PhaseReset{MoveTemp(InPhaseReset)}
			, PhaseOffset{MoveTemp(InPhaseOffset)}
			, PulseWidth{MoveTemp(InPulseWidth)}
			, Output{FFloatWriteRef::CreateNew(0.f)}
		{
			ResetPhase();
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LfoVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LfoBaseFrequencyPin), Frequency);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(WaveshapePin), Waveshape);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MinOutputValuePin), MinValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MaxOutputValuePin), MaxValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(PhaseOffsetPin), PhaseOffset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LfoPhaseResetPin), PhaseReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LfoPulseWidthPin), PulseWidth);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace LfoVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LfoOutPin), Output);
		}

		void ResetPhase()
		{
			float ClampedDegrees = FMath::Clamp(*PhaseOffset, 0.f, 360.f);
			Phase = ClampedDegrees / 360.f;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			ResetPhase();
			*Output = 0.f;
		}

		void Execute()
		{
			using namespace Generators;
		
			// Prevent LFO going faster than the block rate Nyquist
			const float Nyquist = BlockRate / 2.f;
			const float ClampedFreq = FMath::Clamp(*Frequency, 0.f, Nyquist);
			const float DeltaPhase = ClampedFreq * (1.f / BlockRate);
			const float ClampPulseWidth = FMath::Clamp(*PulseWidth, 0.f, 1.f);

			FGeneratorArgs Args{ BlockRate, ClampedFreq, 0.0f, ClampPulseWidth };

			// We are not sample accurate.
			if (PhaseReset->IsTriggeredInBlock()) 
			{
				ResetPhase();
			}
			
			// Wrap phase. (0..1)
			Wrap(Phase);

			float Value = 0.f;
			switch (*Waveshape)
			{
				case ELfoWaveshapeType::Sine:
				{
					Value = SineGenerator(Phase, DeltaPhase, Args);
					break;
				}
				case ELfoWaveshapeType::Saw:
				{
					Value = SawGenerator(Phase, DeltaPhase, Args);
					break;
				}
				case ELfoWaveshapeType::Triangle:
				{
					Value = TriangleGenerator(Phase, DeltaPhase, Args);
					break;
				}
				case ELfoWaveshapeType::Square:
				{
					Value = SquareGenerator(Phase, DeltaPhase, Args);
					break;
				}
				default:
				{
					checkNoEntry();
					break;
				}
			}

			Value = FMath::GetMappedRangeValueClamped(FVector2f{ -1.0f, 1.0f }, FVector2f{ *MinValue, *MaxValue }, Value);

			*Output = Value;

			Phase += DeltaPhase;
		}

	private:
		float BlockRate = 0.f;
		float Phase = 0.f;
		Generators::FWrapPhase Wrap;
		FFloatReadRef Frequency;
		FEnumLfoWaveshapeTypeReadRef Waveshape;
		FFloatReadRef MinValue;
		FFloatReadRef MaxValue;
		FTriggerReadRef PhaseReset;
		FFloatReadRef PhaseOffset;
		FFloatReadRef PulseWidth;
		FFloatWriteRef Output;

		Generators::FSawPolySmoothGenerator SawGenerator;
		Generators::FSineWaveTableGenerator SineGenerator;
		Generators::FTriangleGenerator TriangleGenerator;
		Generators::FSquarePolysmoothGenerator SquareGenerator;

	};

	FLfoNode::FLfoNode(const FNodeInitData& InInitData)
		: FNodeFacade(InInitData.InstanceName, InInitData.InstanceID, TFacadeOperatorClass<FLfoOperator>())
	{}

	FLfoNode::FLfoNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	: FNodeFacade(MoveTemp(InNodeData), MoveTemp(InMetadata), TFacadeOperatorClass<FLfoOperator>())
	{
	}

	FNodeClassMetadata FLfoNode::CreateNodeClassMetadata()
	{
		return FLfoOperator::GetNodeInfo();
	}

	METASOUND_REGISTER_NODE(FLfoNode);

#pragma endregion LFO
}
#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes

