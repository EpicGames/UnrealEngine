// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Dsp.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundPrimitives.h"

namespace Metasound
{
	enum class ELfoWaveshapeType
	{
		Sine,
		Saw,
		Triangle,
		Square,
	};
	
	DECLARE_METASOUND_ENUM(ELfoWaveshapeType, ELfoWaveshapeType::Sine, METASOUNDSTANDARDNODES_API,
		FEnumLfoWaveshapeType, FEnumLfoWaveshapeTypeInfo, FEnumLfoWaveshapeTypeReadRef, FEnumLfoWaveshapeTypeWriteRef);

	// Contained here are a small selection of block generators and sample generators.
	// They are templatized to allow quick construction of new oscillators, but aren't considered
	// optimal (as they contain many branches) and they should be only as a default 
	// until Vectorized versions replace them.
	namespace Generators
	{
		// Standard params passed to the Generate Block templates below
		struct FGeneratorArgs
		{
			float SampleRate = 0.f;
			float FrequencyHz = 0.f;
			float GlideEaseFactor = 0.0f;
			float PulseWidth = 0.f;
			bool BiPolar = true;
			TArrayView<float> AlignedBuffer;
			TArrayView<const float> FM;
		};

		// Functor used when we need to wrap the phase of some of the oscillators. Sinf for example does not care
		struct FWrapPhase
		{
			FORCEINLINE void operator()(float& InPhase) const
			{
				if (InPhase >= 1.f)
				{
					InPhase -= FMath::TruncToFloat(InPhase);
				}

				if (InPhase < 0.0f)
				{
					InPhase -= FMath::TruncToFloat(InPhase) + 1.f;
				}
			}
		};

		// Functor that does nothing to the phase and lets it climb indefinitely 
		struct FPhaseLetClimb
		{
			FORCEINLINE void operator()(const float&) const {};
		};

		// Smooth out the edges of the saw based on its current frequency
		// using a polynomial to smooth it at the discontinuity. This 
		// limits aliasing by avoiding the infinite frequency at the discontinuity.
		FORCEINLINE float PolySmoothSaw(const float InPhase, const float InPhaseDelta)
		{	
			float Output = 0.0f;
			float AbsolutePhaseDelta = FMath::Abs(InPhaseDelta);

			// The current phase is on the left side of discontinuity
			if (InPhase > 1.0f - AbsolutePhaseDelta)
			{
				const float Dist = (InPhase - 1.0f) / AbsolutePhaseDelta;
				Output = -Dist * Dist - 2.0f * Dist - 1.0f;
			}
			// The current phase is on the right side of the discontinuity
			else if (InPhase < AbsolutePhaseDelta)
			{
				// Distance into polynomial
				const float Dist = InPhase / AbsolutePhaseDelta;
				Output = Dist * Dist - 2.0f * Dist + 1.0f;
			}
			
			return Output;
		}

		// Functor that does the a generic generate block with a supplied Oscillator.
		// The PhaseWrap functor controls how the phase is accumulated and wrapped
		template<
			typename Oscillator,
			typename PhaseWrap = FWrapPhase
		>
		struct TGenerateBlock
		{
			float Phase = 0.f;
			Oscillator Osc;
			PhaseWrap Wrap;
			float CurrentFreq = -1.f;
		
			void operator()(const FGeneratorArgs& InArgs)
			{
				int32 RemainingSamplesInBlock = InArgs.AlignedBuffer.Num();
				float* Out = InArgs.AlignedBuffer.GetData();
				const float OneOverSampleRate = 1.f / InArgs.SampleRate;
				
				// TODO: break this into separate calls because there is a lot of code duplication to prevent per sample branching 
				if (InArgs.BiPolar)
				{
					// Constant Freq between this an last update? 
					if (FMath::IsNearlyEqual(InArgs.FrequencyHz, CurrentFreq) || CurrentFreq < 0.f || FMath::IsNearlyEqual(InArgs.GlideEaseFactor, 1.0f))
					{
						CurrentFreq = InArgs.FrequencyHz;
						const float DeltaPhase = InArgs.FrequencyHz * OneOverSampleRate;
						while (RemainingSamplesInBlock > 0)
						{
							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							*Out++ = Output;

							Phase += DeltaPhase;

							RemainingSamplesInBlock--;
						}
					}
					else
					{
						while (RemainingSamplesInBlock > 0)
						{
							Wrap(Phase);

							const float DeltaPhase = CurrentFreq * OneOverSampleRate;

							float Output = Osc(Phase, DeltaPhase, InArgs);
							*Out++ = Output;

							Phase += DeltaPhase;

							// lerp frequency based on the glide factor
							CurrentFreq = CurrentFreq + InArgs.GlideEaseFactor * (InArgs.FrequencyHz - CurrentFreq);

							RemainingSamplesInBlock--;
						}
					}
				}
				else // unipolar
				{
					// Constant Freq between this an last update? 
					if (FMath::IsNearlyEqual(InArgs.FrequencyHz, CurrentFreq) || CurrentFreq < 0.f || FMath::IsNearlyEqual(InArgs.GlideEaseFactor, 1.0f))
					{
						CurrentFreq = InArgs.FrequencyHz;
						const float DeltaPhase = InArgs.FrequencyHz * OneOverSampleRate;
						while (RemainingSamplesInBlock > 0)
						{
							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							Output = Audio::GetUnipolar(Output);
							*Out++ = Output;

							Phase += DeltaPhase;

							RemainingSamplesInBlock--;
						}
					}
					else
					{
						while (RemainingSamplesInBlock > 0)
						{
							Wrap(Phase);

							const float DeltaPhase = CurrentFreq * OneOverSampleRate;

							float Output = Osc(Phase, DeltaPhase, InArgs);
							Output = Audio::GetUnipolar(Output);
							*Out++ = Output;

							Phase += DeltaPhase;

							// lerp frequency based on the glide factor
							CurrentFreq = CurrentFreq + InArgs.GlideEaseFactor * (InArgs.FrequencyHz - CurrentFreq);

							RemainingSamplesInBlock--;
						}
					}
				}
			}
		};

		// Functor that does the a generic generate block with a supplied Oscillator and applies an FM signal
		// to the frequency per sample.
		template<
			typename Oscillator,
			typename PhaseWrap = FWrapPhase
		>
		struct TGenerateBlockFM
		{
			float Phase = 0.f;
			Oscillator Osc;
			PhaseWrap Wrap;

			float CurrentFreq = -1.f;

			void operator()(const FGeneratorArgs& InArgs)
			{
				float Nyquist = InArgs.SampleRate / 2.0f;
				int32 RemainingSamplesInBlock = InArgs.AlignedBuffer.Num();
				float* Out = InArgs.AlignedBuffer.GetData();
				const float* FM = InArgs.FM.GetData();
				const float OneOverSampleRate = 1.f / InArgs.SampleRate;

				// TODO: break this into separate calls because there is a lot of code duplication to prevent per sample branching 
				if (InArgs.BiPolar)
				{
					// Constant Base Freq between this and last update? 
					if (FMath::IsNearlyEqual(InArgs.FrequencyHz, CurrentFreq) || CurrentFreq < 0.f || FMath::IsNearlyEqual(InArgs.GlideEaseFactor, 1.0f))
					{
						while (RemainingSamplesInBlock > 0)
						{
							const float PerSampleFreq = FMath::Clamp(InArgs.FrequencyHz + *FM++, -Nyquist, Nyquist);
							const float DeltaPhase = PerSampleFreq * OneOverSampleRate;

							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							*Out++ = Output;

							Phase += DeltaPhase;

							RemainingSamplesInBlock--;
						}

						CurrentFreq = InArgs.FrequencyHz;
					}
					else
					{
						while (RemainingSamplesInBlock > 0)
						{
							const float ModulatedFreqSum = FMath::Clamp(CurrentFreq + *FM++, -Nyquist, Nyquist);
							const float DeltaPhase = ModulatedFreqSum * OneOverSampleRate;

							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							*Out++ = Output;

							Phase += DeltaPhase;

							// lerp frequency based on the glide factor
							CurrentFreq = CurrentFreq + InArgs.GlideEaseFactor * (InArgs.FrequencyHz - CurrentFreq);

							RemainingSamplesInBlock--;
						}
					}
				}
				else // unipolar
				{
					// Constant Base Freq between this and last update? 
					if (FMath::IsNearlyEqual(InArgs.FrequencyHz, CurrentFreq) || CurrentFreq < 0.f || FMath::IsNearlyEqual(InArgs.GlideEaseFactor, 1.0f))
					{
						while (RemainingSamplesInBlock > 0)
						{
							const float PerSampleFreq = InArgs.FrequencyHz + *FM++;
							const float DeltaPhase = PerSampleFreq * OneOverSampleRate;

							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							Output = Audio::GetUnipolar(Output);
							*Out++ = Output;

							Phase += DeltaPhase;

							RemainingSamplesInBlock--;
						}

						CurrentFreq = InArgs.FrequencyHz;
					}
					else
					{
						while (RemainingSamplesInBlock > 0)
						{
							const float ModulatedFreqSum = CurrentFreq + *FM++;
							const float DeltaPhase = ModulatedFreqSum * OneOverSampleRate;

							Wrap(Phase);

							float Output = Osc(Phase, DeltaPhase, InArgs);
							Output = Audio::GetUnipolar(Output);
							*Out++ = Output;

							Phase += DeltaPhase;

							// lerp frequency based on the glide factor
							CurrentFreq = CurrentFreq + InArgs.GlideEaseFactor * (InArgs.FrequencyHz - CurrentFreq);

							RemainingSamplesInBlock--;
						}
					}
				}
			}
		};	
		
		// Sine types and generators.
		struct FSinfGenerator
		{
			FORCEINLINE float operator()(float InPhase, float, const FGeneratorArgs&)
			{
				return FMath::Sin(InPhase * UE_TWO_PI);
			}
		};
		struct FBhaskaraGenerator
		{
			FORCEINLINE float operator()(float InPhase, float, const FGeneratorArgs&) const
			{
				const float PhaseRadians = InPhase * UE_TWO_PI;

				float InBhaskaraDomain = (PhaseRadians < 0) ? PhaseRadians + UE_PI : PhaseRadians - UE_PI;
				return Audio::FastSin3(InBhaskaraDomain); // Expects [-PI, PI] 
			}
		};
		struct FSineWaveTableGenerator
		{
			static const TArray<float>& GetWaveTable()
			{
				auto MakeSineTable = []() -> const TArray<float>
				{
					int32 TableSize = 4096;
					// Generate the table
					TArray<float> WaveTable;
					WaveTable.AddUninitialized(TableSize);
					float* WaveTableData = WaveTable.GetData();
					for (int32 i = 0; i < TableSize; ++i)
					{
						float Phase = (float)i / TableSize;
						WaveTableData[i] = FMath::Sin(Phase * UE_TWO_PI);
					}
					return WaveTable;
				};

				static const TArray<float> SineWaveTable = MakeSineTable();
				return SineWaveTable;
			}

			float operator()(float InPhase, float, const FGeneratorArgs&) const
			{
				const TArray<float>& WaveTable = GetWaveTable();
				int32 NumEntries = WaveTable.Num();
				float FractionalPhase = FMath::Wrap(InPhase, 0.0f, 1.0f);
				int32 TableIndex = (int32)(FractionalPhase * NumEntries + 0.5f);
				if (TableIndex >= NumEntries)
				{
					TableIndex = 0;
				}
				return WaveTable[TableIndex];
			}
		};

		struct F2DRotatorGenerateBlock
		{
			Audio::FSinOsc2DRotation Rotator;
			F2DRotatorGenerateBlock(float InStartingLinearPhase)
				: Rotator{ UE_TWO_PI * InStartingLinearPhase }
			{}
			void operator()(const FGeneratorArgs& Args)
			{
				Rotator.GenerateBuffer(Args.SampleRate, Args.FrequencyHz, Args.AlignedBuffer.GetData(), Args.AlignedBuffer.Num());
				if (!Args.BiPolar)
				{
					Audio::ConvertBipolarBufferToUnipolar(Args.AlignedBuffer.GetData(), Args.AlignedBuffer.Num());
				}
			}
		};		
		
		using FSinf						= TGenerateBlock<FSinfGenerator>;
		using FSinfWithFm				= TGenerateBlockFM<FSinfGenerator>;
		
		using FBhaskara					= TGenerateBlock<FBhaskaraGenerator>;
		using FBhaskaraWithFm			= TGenerateBlockFM<FBhaskaraGenerator>;

		using FSineWaveTable			= TGenerateBlock<FSineWaveTableGenerator>;
		using FSineWaveTableWithFm		= TGenerateBlockFM<FSineWaveTableGenerator>;

		// Saws.
		struct FSawGenerator
		{
			FORCEINLINE float operator()(float InPhase, float, const FGeneratorArgs&)
			{
				return -1.f + (2.f * InPhase);
			}
		};
		struct FSawPolySmoothGenerator
		{
			FORCEINLINE float operator()(float InPhase, float InPhaseDelta, const FGeneratorArgs&)
			{
				// Two-sided wave-shaped sawtooth
				constexpr float OneOverTanhOnePointFive = 1.10479139f;
				float Output = Audio::GetBipolar(InPhase);
				Output = Audio::FastTanh(1.5f * Output) * OneOverTanhOnePointFive;
				Output += PolySmoothSaw(InPhase, InPhaseDelta);
				return Output;
			}
		};

		using FSaw							= TGenerateBlock<FSawGenerator>;
		using FSawWithFm					= TGenerateBlockFM<FSawGenerator>;

		using FSawPolysmooth				= TGenerateBlock<FSawPolySmoothGenerator>;
		using FSawPolysmoothWithFm			= TGenerateBlockFM<FSawPolySmoothGenerator>;

		// Square.
		struct FSquareGenerator
		{
			FORCEINLINE float operator()(float InPhase, float InPhaseDelta, const FGeneratorArgs& InArgs)
			{
				const float PulseWidthToUse = (InPhaseDelta >= 0) ? InArgs.PulseWidth : 1 - InArgs.PulseWidth;
                
                return InPhase >= PulseWidthToUse ? 1.f : -1.f;
			}
		};
		struct FSquarePolysmoothGenerator
		{
			float operator()(float InPhase, float InPhaseDelta, const FGeneratorArgs& InArgs)
			{
				// Taken piecemeal from Osc.cpp
				// Lots of branches.

				// First generate a smoothed sawtooth
				float SquareSaw1 = Audio::GetBipolar(InPhase);
				SquareSaw1 += PolySmoothSaw(InPhase, InPhaseDelta);

				// Create a second sawtooth that is phase-shifted based on the pulsewidth
				float NewPhase = 0.0f;
				if (InPhaseDelta > 0.0f)
				{
					NewPhase = InPhase + InArgs.PulseWidth;
					if (NewPhase >= 1.0f)
					{
						NewPhase -= 1.0f;
					}
				}
				else
				{
					NewPhase = InPhase - InArgs.PulseWidth;
					if (NewPhase <= 0.0f)
					{
						NewPhase += 1.0f;
					}
				}

				float SquareSaw2 = Audio::GetBipolar(NewPhase);
				SquareSaw2 += PolySmoothSaw(NewPhase, InPhaseDelta);

				// Subtract 2 saws, then apply DC correction 
				// Simplified version of 
				// float Output = 0.5f * SquareSaw1 - 0.5f * SquareSaw2;
				// Output = 2.0f * (Output + InArgs.PulseWidth) - 1.0f;
				if (InPhaseDelta > 0.0)
				{
					return SquareSaw1 - SquareSaw2 + 2.0f * (InArgs.PulseWidth - 0.5f);
				}
				else
				{
					return SquareSaw1 - SquareSaw2 + 2.0f * (0.5f - InArgs.PulseWidth);
				}
			}
		};

		using FSquare						= TGenerateBlock<FSquareGenerator>;
		using FSquareWithFm					= TGenerateBlockFM<FSquareGenerator>;

		using FSquarePolysmooth				= TGenerateBlock<FSquarePolysmoothGenerator>;
		using FSquarePolysmoothWithFm		= TGenerateBlockFM<FSquarePolysmoothGenerator>;

		// Triangle.
		struct FTriangleGenerator
		{
			FORCEINLINE float operator()(float InPhase, float InPhaseDelta, const FGeneratorArgs& InArgs)
			{
				constexpr float OneOverFastAsinHalfPi = 1.f / 1.5707963050f;
				const float PhaseRadians = (InPhase * UE_TWO_PI);
				return FMath::FastAsin(Audio::FastSin3(PhaseRadians - UE_PI)) * OneOverFastAsinHalfPi;
			}
		};

		using FTriangle						= TGenerateBlock<FTriangleGenerator>;
		using FTriangleWithFm				= TGenerateBlockFM<FTriangleGenerator>;

		// TODO: make a true polysmooth version
		using FTrianglePolysmooth = TGenerateBlock<FTriangleGenerator>;
		using FTrianglePolysmoothWithFm = TGenerateBlockFM<FTriangleGenerator>;
	}
}
