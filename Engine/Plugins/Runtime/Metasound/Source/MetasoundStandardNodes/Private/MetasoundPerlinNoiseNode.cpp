// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "DSP/PerlinNoise.h"
#include "Math/UnrealMathUtility.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_PerlinNoise"

// Control/Audio rate Perlin "Value" Noise: 
// Take any value "X" and produces deterministic smooth noise.
// Each "Octave" adds another layer of noise at higher frequency and lower amplitude 
// If you have multiple nodes and which different results for each, use the offset pin to start at different location (similar to a seed).

namespace Metasound
{		
	namespace PerlinNoiseVertexNames
	{
		METASOUND_PARAM(XPin, "X", "Input value to the Perlin function. (Will use internal clock, in seconds, if not connected)")	
		METASOUND_PARAM(OctavesPin, "Layers", "Number of layers (or Octaves) of noise to sum")
		METASOUND_PARAM(OffsetPin, "Seed", "Seed (or offset) into the noise. (-1 will use system rand())")
		METASOUND_PARAM(MinValuePin, "Min Value", "Minimum output value")
		METASOUND_PARAM(MaxValuePin, "Max Value", "Maximum output value")
		METASOUND_PARAM(OutputScaledPin, "Output", "Scaled output based on Min and Max values")
		METASOUND_PARAM(OutputNormalizedPin, "Normalized", "Normalized output from Perlin function (-1...1)")
	}

	// For sanitizing inputs against so we can still sum them in range.
	static float SanitizeFloatInput(const float In, const float InMaxMultiple)
	{
		return FMath::Clamp(In, FLT_MIN / InMaxMultiple, FLT_MAX / InMaxMultiple);
	}

	template<typename TDataClass, typename TThisType>
	class TBasePerlinNoiseOperator : public TExecutableOperator<TThisType>
	{
	public:	
		static const int32 MaxOctaves = 8; // Careful here as this can be expensive, each octave adds a full layer of noise

		using TDataClassReadRef = TDataReadReference<TDataClass>;
		using TDataClassWriteRef = TDataWriteReference<TDataClass>;

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace PerlinNoiseVertexNames;

			static const FVertexInterface Interface
			{
				FInputVertexInterface
				{
					TInputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME_AND_METADATA(XPin)),
					TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OctavesPin), 1),
					TInputConstructorVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OffsetPin), -1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MinValuePin), -1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MaxValuePin), 1.0f),
				},
				FOutputVertexInterface
				{
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputScaledPin)),
					TOutputDataVertex<TDataClass>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputNormalizedPin)),
				}
			};
			return Interface;
		}	

		static const FNodeClassMetadata& GetNodeInfo()
		{
			// Common stuff
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = TThisType::CreateNodeInfo();
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.CategoryHierarchy.Emplace(NodeCategories::RandomUtils);
				Info.Keywords.Emplace(METASOUND_LOCTEXT("Metasound_PerlinNoiseNodeKeywordRandom", "Random"));
				Info.Keywords.Emplace(METASOUND_LOCTEXT("Metasound_PerlinNoiseNodeKeywordRand", "Rand"));
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		struct FPinReadRefs
		{
			TDataClassReadRef X;
			FInt32ReadRef Octaves;
			FFloatReadRef MinValue;
			FFloatReadRef MaxValue;
			bool bXConnected = false;
			float Offset = 0.f;
		};

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{		
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			using namespace PerlinNoiseVertexNames;
			FPinReadRefs Pins
			{
				  InputData.GetOrCreateDefaultDataReadReference<TDataClass>(METASOUND_GET_PARAM_NAME(XPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(OctavesPin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MinValuePin), Settings)
				, InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MaxValuePin), Settings)
				, InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(XPin))
				, InputData.GetOrCreateDefaultValue<float>(METASOUND_GET_PARAM_NAME(OffsetPin), Settings)
			};

			return MakeUnique<TThisType>(InParams, MoveTemp(Pins));
		}

		TBasePerlinNoiseOperator(const FBuildOperatorParams& InParams, FPinReadRefs&& InPins)
			: Pins{ MoveTemp(InPins) }
			, OutputScaled{ TThisType::CreateOutputWriteRef(InParams.OperatorSettings) }
			, OutputNormalized{ TThisType::CreateOutputWriteRef(InParams.OperatorSettings) }
		{
			Reset(InParams);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			// If Offset is -1, use system rand as a "seed"
			if (FMath::IsNearlyEqual(Pins.Offset, -1.f))
			{
				Pins.Offset = FMath::FRand();
			}

			SampleRate = InParams.OperatorSettings.GetSampleRate();
			NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();

			check(SampleRate > 0);
			check(NumFramesPerBlock > 0);

			TThisType::ResetOutputWriteRef(OutputScaled);
			TThisType::ResetOutputWriteRef(OutputNormalized);
		}
		
		void Bind(FVertexInterfaceData& InVertexData) const override
		{

			FOutputVertexInterfaceData& Outputs = InVertexData.GetOutputs();
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace PerlinNoiseVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(XPin), Pins.X);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(OffsetPin), Pins.Offset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OctavesPin), Pins.Octaves);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MinValuePin), Pins.MinValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MaxValuePin), Pins.MaxValue);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace PerlinNoiseVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputScaledPin), OutputScaled);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputNormalizedPin), OutputNormalized);
		}

		void AccumulateTime()
		{
			NumFramesElapsed += NumFramesPerBlock;
		}
		float GetElapsedTimeInSeconds() const 
		{
			return static_cast<float>(NumFramesElapsed / SampleRate);
		}

	protected:
		float SampleRate = 0;
		int32 NumFramesPerBlock = 0;
		int32 NumFramesElapsed = 0;
		FPinReadRefs Pins;
		TDataClassWriteRef OutputScaled;
		TDataClassWriteRef OutputNormalized;
	};

	class FPerlinControlRate : public TBasePerlinNoiseOperator<float, FPerlinControlRate>
	{
	public:
		using BaseClass = TBasePerlinNoiseOperator<float, FPerlinControlRate>;
		using BaseClass::TBasePerlinNoiseOperator;
		using BaseClass::CreateOperator;

		static FFloatWriteRef CreateOutputWriteRef(const FOperatorSettings& InSettings) { return FFloatWriteRef::CreateNew(0.f); }
		static void ResetOutputWriteRef(FFloatWriteRef& WriteRef) { *WriteRef = 0; };

		static FNodeClassMetadata CreateNodeInfo()
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Perlin Noise (float)"), StandardNodes::AudioVariant };
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_PerlinNoiseFloatNodeDisplayName", "Perlin Noise (float)");
			Info.Description = METASOUND_LOCTEXT("Metasound_PerlinNoiseFloatNodeDescription", "Generates 1D Perlin \"Value noise\" at control rate");
			return Info;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			BaseClass::Reset(InParams);
		}

		void Execute()
		{
			// Use input pin if its connected, otherwise use elapsed time in seconds.
			const float X = Pins.bXConnected ?
				*Pins.X :
				GetElapsedTimeInSeconds();

			// Sanitize input. 
			int32 OctavesClamped = FMath::Clamp(*Pins.Octaves, 1, MaxOctaves);
			
			// Make sure we can sum safely the X+Offset without overflow. So limit to FLT_Max/2
			float OffsetClamped = SanitizeFloatInput(Pins.Offset, 2.f);
			float XClamped = SanitizeFloatInput(X, 2.f);
			
			// Value noise will scale up this value to 2^Octaves frequency, so clamp to that range.
			float MaxFreq = (float)(1UL << OctavesClamped);
			float Sum = OffsetClamped + XClamped;
			float SumClamped = SanitizeFloatInput(Sum, MaxFreq);
			
			// Generate noise (this is multiple layers of Perlin noise at different frequencies).
			const float ValueNoise = Audio::PerlinValueNoise1D(SumClamped, OctavesClamped);

			// Output the scaled version.
			const float ScaledValue = FMath::GetMappedRangeValueClamped(
				FVector2f{ -1.0f, 1.0f },
				FVector2f{ *Pins.MinValue, *Pins.MaxValue },
				ValueNoise);

			*OutputScaled = ScaledValue;
			*OutputNormalized = ValueNoise;

			AccumulateTime();
		}
	};

	class FPerlinAudioRate : public TBasePerlinNoiseOperator<FAudioBuffer, FPerlinAudioRate>
	{
	public:
		using BaseClass = TBasePerlinNoiseOperator<FAudioBuffer, FPerlinAudioRate>;
		using BaseClass::CreateOperator;

		static FAudioBufferWriteRef CreateOutputWriteRef(const FOperatorSettings& InSettings) { return FAudioBufferWriteRef::CreateNew(InSettings); }
		static void ResetOutputWriteRef(FAudioBufferWriteRef& WriteRef) { WriteRef->Zero(); };

		static FNodeClassMetadata CreateNodeInfo()
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Perlin Noise (audio)"), StandardNodes::AudioVariant };
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_PerlinNoiseAudioNodeDisplayName", "Perlin Noise (audio)");
			Info.Description = METASOUND_LOCTEXT("Metasound_PerlinNoiseAudioNodeDescription", "Generates 1D Perlin \"Value noise\" at audio rate");
			return Info;
		}

		FPerlinAudioRate(const FBuildOperatorParams& InParams, FPinReadRefs&& InPins)
			: BaseClass{ InParams, MoveTemp(InPins) }
			, TimeBuffer{ InPins.bXConnected ? 0 : InParams.OperatorSettings.GetNumFramesPerBlock() }	// Create a scratch buffer if X is not connected.
		{
		}

		void GenerateTimeArray()
		{
			// Not integrating here, for fear of discontinuities with floating point error
			float* TimePtr = TimeBuffer.GetData();
			const float SampleRateReciprocal = 1.f / SampleRate;
			const int32 StopTime = NumFramesElapsed + NumFramesPerBlock;
			for (int32 i = NumFramesElapsed; i < StopTime; ++i)
			{
				*TimePtr++ = static_cast<float>(i * SampleRateReciprocal);
			}
		}

		void GenerateScaledOutput()
		{
			float* ScaledPtr = OutputScaled->GetData();
			const float* NormalizedPtr = OutputNormalized->GetData();
			const FVector2f MinMax{ *Pins.MinValue, *Pins.MaxValue };
			const FVector2f NormRange{ -1.0f, 1.0f };

			for (int32 i = 0; i < NumFramesPerBlock; ++i)
			{
				// Output the scaled version.
				*ScaledPtr++ = FMath::GetMappedRangeValueClamped(NormRange, MinMax, *NormalizedPtr++);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			BaseClass::Reset(InParams);
		}

		void Execute()
		{
			// Use input pin if its connected, otherwise generate an array of increasing time
			const FAudioBuffer* XBuffer = Pins.X.Get();
			if (!Pins.bXConnected)
			{
				GenerateTimeArray();
				XBuffer = &TimeBuffer;
			}
		
			int32 OctavesClamped = FMath::Clamp(*Pins.Octaves,1,MaxOctaves);

			Audio::PerlinValueNoise1DBuffer(
				*XBuffer,
				Pins.Offset, 
				OctavesClamped, 
				*OutputNormalized
			);
			
			GenerateScaledOutput();
			AccumulateTime();
		}

	private:
		FAudioBuffer TimeBuffer;
	};
	
	using FControlRatePerlinNode = TNodeFacade<FPerlinControlRate>;
	using FAudioRatePerlinNode = TNodeFacade<FPerlinAudioRate>;

	METASOUND_REGISTER_NODE(FControlRatePerlinNode);
	METASOUND_REGISTER_NODE(FAudioRatePerlinNode);
}

#undef LOCTEXT_NAMESPACE
