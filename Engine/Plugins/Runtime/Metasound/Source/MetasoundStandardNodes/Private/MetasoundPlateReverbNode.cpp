// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/LateReflectionsFast.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_PlateReverb"

namespace Metasound
{
	namespace PlateReverb
	{
		namespace Inputs
		{
			METASOUND_PARAM(Bypass, "Bypass", "Toggle to bypass the effect and send audio through unaltered.")
			METASOUND_PARAM(InAudioLeft, "In Left", "Left channel audio input.")
			METASOUND_PARAM(InAudioRight, "In Right", "Right channel audio input.")
			METASOUND_PARAM(DryLevel, "Dry Level", "The level of the dry signal (linear).")
			METASOUND_PARAM(WetLevel, "Wet Level", "The level of the wet signal (linear).")
			METASOUND_PARAM(LateReflectionsDelay, "Delay (ms)", "Pre-delay before late reflections")
			METASOUND_PARAM(LateReflectionsGainDb, "Gain (dB)", "Initial attenuation of audio after it leaves the predelay")
			METASOUND_PARAM(LateReflectionsBandwidth, "Bandwidth", "Frequency bandwidth of audio going into input diffusers. 0.999 is full bandwidth")
			METASOUND_PARAM(LateReflectionsDiffusion, "Diffusion", "Amount of input diffusion (larger value results in more diffusion)")
			METASOUND_PARAM(LateReflectionsDampening, "Dampening", "The amount of high-frequency dampening in plate feedback paths")
			METASOUND_PARAM(LateReflectionsDecay, "Decay", "The amount of decay in the feedback path. Lower value is larger reverb time.")
			METASOUND_PARAM(LateReflectionsDensity, "Density", "The amount of diffusion in decay path. Larger values is a more dense reverb.")
		}

		namespace Outputs
		{
			METASOUND_PARAM(OutAudioLeft, "Out Left", "Left channel audio output.")
			METASOUND_PARAM(OutAudioRight, "Out Right", "Right channel audio output.")
		}
	}

	class FPlateReverbOperator final : public TExecutableOperator<FPlateReverbOperator>
	{
	public:
		struct FInputs
		{
			FBoolReadRef Bypass;
			FAudioBufferReadRef AudioLeft;
			FAudioBufferReadRef AudioRight;
			FFloatReadRef DryLevel;
			FFloatReadRef WetLevel;
			FFloatReadRef LateReflectionsDelay;
			FFloatReadRef LateReflectionsGainDb;
			FFloatReadRef LateReflectionsBandwidth;
			FFloatReadRef LateReflectionsDiffusion;
			FFloatReadRef LateReflectionsDampening;
			FFloatReadRef LateReflectionsDecay;
			FFloatReadRef LateReflectionsDensity;
		};
		
		struct FOutputs
		{
			FAudioBufferWriteRef AudioLeft;
			FAudioBufferWriteRef AudioRight;
		};
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "Plate Reverb", "Stereo" },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("PlateReverbStereoDisplayName", "Plate Reverb (Stereo)"),
					METASOUND_LOCTEXT("PlateReverbDesc", "Plate reverb with configurable early and late reflections."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetVertexInterface(),
					{ NodeCategories::Reverbs },
					{},
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			static const Audio::FLateReflectionsFastSettings Defaults;
			
			static const FVertexInterface Interface
			{
				FInputVertexInterface
				{
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Inputs::Bypass), false),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Inputs::InAudioLeft)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Inputs::InAudioRight)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Inputs::DryLevel), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Inputs::WetLevel), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsDelay), Defaults.LateDelayMsec),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsGainDb), Defaults.LateGainDB),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsBandwidth), Defaults.Bandwidth),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsDiffusion), Defaults.Diffusion),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsDampening), Defaults.Dampening),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsDecay), Defaults.Decay),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(PlateReverb::Inputs::LateReflectionsDensity), Defaults.Density)
				},
				FOutputVertexInterface
				{
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Outputs::OutAudioLeft)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(PlateReverb::Outputs::OutAudioRight))
				}
			};

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			FInputs Inputs
			{
				InParams.InputData.GetOrCreateDefaultDataReadReference<bool>(PlateReverb::Inputs::BypassName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(PlateReverb::Inputs::InAudioLeftName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(PlateReverb::Inputs::InAudioRightName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::DryLevelName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::WetLevelName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsDelayName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsGainDbName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsBandwidthName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsDiffusionName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsDampeningName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsDecayName, InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(PlateReverb::Inputs::LateReflectionsDensityName, InParams.OperatorSettings)
			};

			return MakeUnique<FPlateReverbOperator>(InParams, MoveTemp(Inputs));
		}

		static constexpr int32 MaxReverbBufferSize = 512;

		FPlateReverbOperator(const FBuildOperatorParams& BuildParams, FInputs&& Inputs)
			: Inputs(MoveTemp(Inputs))
			, Outputs({ FAudioBufferWriteRef::CreateNew(BuildParams.OperatorSettings), FAudioBufferWriteRef::CreateNew(BuildParams.OperatorSettings) })
			, Reverb(BuildParams.OperatorSettings.GetSampleRate(), MaxReverbBufferSize)
		{
			Reset(BuildParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::BypassName, Inputs.Bypass);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::InAudioLeftName, Inputs.AudioLeft);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::InAudioRightName, Inputs.AudioRight);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::DryLevelName, Inputs.DryLevel);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::WetLevelName, Inputs.WetLevel);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsDelayName, Inputs.LateReflectionsDelay);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsGainDbName, Inputs.LateReflectionsGainDb);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsBandwidthName, Inputs.LateReflectionsBandwidth);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsDiffusionName, Inputs.LateReflectionsDiffusion);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsDampeningName, Inputs.LateReflectionsDampening);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsDecayName, Inputs.LateReflectionsDecay);
			InOutVertexData.BindReadVertex(PlateReverb::Inputs::LateReflectionsDensityName, Inputs.LateReflectionsDensity);
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(PlateReverb::Outputs::OutAudioLeftName, Outputs.AudioLeft);
			InOutVertexData.BindReadVertex(PlateReverb::Outputs::OutAudioRightName, Outputs.AudioRight);
		}

		void Reset(const FResetParams& Params)
		{
			WorkBuffer.SetNumUninitialized(Params.OperatorSettings.GetNumFramesPerBlock());
			Reverb.FlushAudio();
		}
		
		void Execute()
		{
			if (*Inputs.Bypass != WasBypassed)
			{
				// Flush the reverb if we just bypassed
				if (*Inputs.Bypass)
				{
					Reverb.FlushAudio();
				}

				WasBypassed = *Inputs.Bypass;
			}

			// Pass through audio if bypassed
			if (*Inputs.Bypass)
			{
				FMemory::Memcpy(Outputs.AudioLeft->GetData(), Inputs.AudioLeft->GetData(), Inputs.AudioLeft->Num() * sizeof(float));
				FMemory::Memcpy(Outputs.AudioRight->GetData(), Inputs.AudioRight->GetData(), Inputs.AudioRight->Num() * sizeof(float));
				return;
			}

			// Sum to mono. This happens in the late reflections code when you pass in interleaved, stereo audio.
			// Doing it here and scaling below avoids the extra interleave memory and time.
			Audio::ArraySum(*Inputs.AudioLeft, *Inputs.AudioRight, WorkBuffer);
			
			const float CurrentWetLevel = FMath::Clamp(*Inputs.WetLevel, 0.0f, 1.0f);

			// Apply the wet gain to the input to preserve the reverb tail.
			// We scale by half because that's what happens when passing in stereo audio to the late reflections.
			if (LastWetLevel >= 0.0f && !FMath::IsNearlyEqual(CurrentWetLevel, LastWetLevel))
			{
				const float From = LastWetLevel * 0.5f;
				const float To = CurrentWetLevel * 0.5f;
				Audio::ArrayFade(WorkBuffer, From, To);
			}
			else
			{
				Audio::ArrayMultiplyByConstantInPlace(WorkBuffer, CurrentWetLevel * 0.5f);
			}
			
			LastWetLevel = CurrentWetLevel;

			// Update the reverb settings
			UpdateSettingsIfChanged();
			
			// Process
			Reverb.ProcessAudio(WorkBuffer, 1, *Outputs.AudioLeft, *Outputs.AudioRight);

			// Mix in the dry signal
			const float DryLevel = FMath::Clamp(*Inputs.DryLevel, 0.0f, 1.0f);
			Audio::ArrayMixIn(*Inputs.AudioLeft, *Outputs.AudioLeft, DryLevel);
			Audio::ArrayMixIn(*Inputs.AudioRight, *Outputs.AudioRight, DryLevel);
		}

	private:
		void UpdateSettingsIfChanged()
		{
			bool SettingsChanged = false;

			if (*Inputs.LateReflectionsDelay != CurrentSettings.LateDelayMsec)
			{
				CurrentSettings.LateDelayMsec = *Inputs.LateReflectionsDelay;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsGainDb != CurrentSettings.LateGainDB)
			{
				CurrentSettings.LateGainDB = *Inputs.LateReflectionsGainDb;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsBandwidth != CurrentSettings.Bandwidth)
			{
				CurrentSettings.Bandwidth = *Inputs.LateReflectionsBandwidth;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsDiffusion != CurrentSettings.Diffusion)
			{
				CurrentSettings.Diffusion = *Inputs.LateReflectionsDiffusion;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsDampening != CurrentSettings.Dampening)
			{
				CurrentSettings.Dampening = *Inputs.LateReflectionsDampening;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsDecay != CurrentSettings.Decay)
			{
				CurrentSettings.Decay = *Inputs.LateReflectionsDecay;
				SettingsChanged = true;
			}
			if (*Inputs.LateReflectionsDensity != CurrentSettings.Density)
			{
				CurrentSettings.Density = *Inputs.LateReflectionsDensity;
				SettingsChanged = true;
			}

			if (SettingsChanged)
			{
				Reverb.SetSettings(CurrentSettings);
			}
		}
		
		FInputs Inputs;
		FOutputs Outputs;
		
		Audio::FLateReflectionsFast Reverb;
		Audio::FLateReflectionsFastSettings CurrentSettings;
		Audio::FAlignedFloatBuffer WorkBuffer;

		bool WasBypassed;
		float LastWetLevel{ -1 };
	};

	using FPlateReverbNode = TNodeFacade<FPlateReverbOperator>;

	METASOUND_REGISTER_NODE(FPlateReverbNode)
}

#undef LOCTEXT_NAMESPACE
