// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundRandomNode.h"

#include "Internationalization/Text.h"
#include "MetasoundNodeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	// Mac Clang require linkage for constexpr
	template<typename ValueType>
	constexpr int32 Metasound::TRandomNodeOperator<ValueType>::DefaultSeed;

 	using FRandomNodeInt32 = TRandomNode<int32>;
 	METASOUND_REGISTER_NODE(FRandomNodeInt32)
 
 	using FRandomNodeFloat = TRandomNode<float>;
 	METASOUND_REGISTER_NODE(FRandomNodeFloat)
 
	using FRandomNodeBool = TRandomNode<bool>;
	METASOUND_REGISTER_NODE(FRandomNodeBool)

	using FRandomNodeTime = TRandomNode<FTime>;
	METASOUND_REGISTER_NODE(FRandomNodeTime)
}

#undef LOCTEXT_NAMESPACE
