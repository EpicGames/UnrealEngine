// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FloatArrayMath.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_MetasoundRingModNodes"

namespace Metasound
{
	/* RingMod Processor */
	namespace RingModVertexNames
	{
		METASOUND_PARAM(InputAudioCarrier, "In Carrier", "The carrier audio signal.")
		METASOUND_PARAM(InputAudioModulator, "In Modulator", "The modulator audio signal.")

		METASOUND_PARAM(OutputAudio, "Out Audio", "The modulated audio signal.")
	}

	// Operator Class
	class FRingModOperator : public TExecutableOperator<FRingModOperator>
	{
	public:

		FRingModOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InputAudioCarrier,
			const FAudioBufferReadRef& InputAudioModulator)
			: AudioInputCarrier(InputAudioCarrier)
			, AudioInputModulator(InputAudioModulator)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		{
			AudioOutput->Zero();
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = DeclareVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, "RingMod", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("RingModDisplayName2", "Ring Modulator"),
					METASOUND_LOCTEXT("RingModDesc", "Modulates a carrier signal."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Filters },
					{ },
					FNodeDisplayStyle{}
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			using namespace RingModVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioCarrier)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudioModulator))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace RingModVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioCarrier), AudioInputCarrier);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudioModulator), AudioInputModulator);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace RingModVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace RingModVertexNames;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FAudioBufferReadRef CarrierAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioCarrier), InParams.OperatorSettings);
			FAudioBufferReadRef ModulatorAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudioModulator), InParams.OperatorSettings);

			return MakeUnique<FRingModOperator>(InParams.OperatorSettings, CarrierAudioIn, ModulatorAudioIn);
		}

		void Execute()
		{
			*AudioOutput = *AudioInputCarrier;
			
			// @TODO Add Array Multiply Ain, Bin, Cout
			Audio::ArrayMultiplyInPlace(*AudioInputModulator, *AudioOutput);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOutput->Zero();
		}

	private:

		// The input audio buffer
		FAudioBufferReadRef AudioInputCarrier;
		FAudioBufferReadRef AudioInputModulator;
		
		// Output audio buffer
		FAudioBufferWriteRef AudioOutput;
	};

	// Node Class
	using FRingModNode = TNodeFacade<FRingModOperator>;

	// Register node
	METASOUND_REGISTER_NODE(FRingModNode)
}

#undef LOCTEXT_NAMESPACE
