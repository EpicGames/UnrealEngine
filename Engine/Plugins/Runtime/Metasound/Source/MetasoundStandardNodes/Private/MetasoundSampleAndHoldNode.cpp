// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Delay.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_SampleAndHold"

namespace Metasound
{
	namespace SampleAndHoldVertexNames
	{
		METASOUND_PARAM(InputTriggerSampleAndHold, "Sample And Hold", "Trigger to sample and hold the input audio.");
		METASOUND_PARAM(InputAudio, "In", "The audio input to sample.");

		METASOUND_PARAM(OutputOnSampleAndHold, "On Sample And Hold", "Triggers when the input sample and hold is triggered.");
		METASOUND_PARAM(OutputAudio, "Out", "Sampled output value.");
	}

	class FSampleAndHoldOperator : public TExecutableOperator<FSampleAndHoldOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FVertexInterface NodeInterface = GetVertexInterface();

				FNodeClassMetadata Metadata
				{
					FNodeClassName { "SampleAndHold", "Sample And Hold", StandardNodes::AudioVariant },
					1, // Major Version
					0, // Minor Version
					METASOUND_LOCTEXT("SampleAndHoldDisplayName", "Sample And Hold"),
					METASOUND_LOCTEXT("SampleAndHoldDesc", "Will output a single value of the input audio signal when triggered."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					NodeInterface,
					{ NodeCategories::Filters },
					{ },
					FNodeDisplayStyle()
				};

				return Metadata;
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SampleAndHoldVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTriggerSampleAndHold)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSampleAndHold)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace SampleAndHoldVertexNames;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef InputTriggerSampleAndHold = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTriggerSampleAndHold), InParams.OperatorSettings);
			FAudioBufferReadRef InputAudio = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);

			return MakeUnique<FSampleAndHoldOperator>(InParams, InputTriggerSampleAndHold, InputAudio);
		}

		FSampleAndHoldOperator(const FBuildOperatorParams& InParams, const FTriggerReadRef& InTriggerSampleAndHold, const FAudioBufferReadRef& InAudioInput)
			: AudioInput(InAudioInput)
			, TriggerSampleAndHold(InTriggerSampleAndHold)
			, AudioOutput(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings))
		{
			Reset(InParams);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SampleAndHoldVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTriggerSampleAndHold), TriggerSampleAndHold);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SampleAndHoldVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnSampleAndHold), TriggerSampleAndHold);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOutput);
		}

		void Execute()
		{
			TriggerSampleAndHold->ExecuteBlock(
				// On Pre-Trigger, continue outputting the held value
				[this](int32 StartFrame, int32 EndFrame)
				{
					SetToHold(StartFrame, EndFrame);
				},
				// On Trigger, get a new hold value, then output for the rest of the block
				[this](int32 StartFrame, int32 EndFrame)
				{
					// Get a new value to sample and hold
					HoldValue = AudioInput->GetData()[StartFrame];
					SetToHold(StartFrame, EndFrame);
				}
			);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			HoldValue = AudioInput->GetData()[0];
			SetToHold(0, AudioOutput->Num());
		}

	private:

		void SetToHold(int32 StartFrame, int32 EndFrame)
		{
			check(AudioOutput->Num() >= EndFrame);
			float* OutputBufferPtr = AudioOutput->GetData();
			float* CurrOutputPtr = &OutputBufferPtr[StartFrame];

			const int32 NumSamples = EndFrame - StartFrame;

			// non-SIMD version
			for (int32 i = 0; i < NumSamples; ++i)
			{
				CurrOutputPtr[i] = HoldValue;
			}
		}

		FAudioBufferReadRef AudioInput;
		FTriggerReadRef TriggerSampleAndHold;

		FAudioBufferWriteRef AudioOutput;

		float HoldValue = 0.0f;
	};

	using FSampleAndHoldNode = TNodeFacade<FSampleAndHoldOperator>;

	METASOUND_REGISTER_NODE(FSampleAndHoldNode)
}

#undef LOCTEXT_NAMESPACE
