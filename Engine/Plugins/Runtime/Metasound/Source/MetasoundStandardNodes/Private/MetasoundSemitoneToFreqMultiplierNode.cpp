// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "DSP/Dsp.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_SemitoneToFrequencyMultiplier"

namespace Metasound
{

	namespace SemitoneToFrequencyMultiplierVertexNames
	{
		METASOUND_PARAM(InputSemitone, "Semitones", "Input difference in semitones.");

		METASOUND_PARAM(OutputFrequencyMultiplier, "Frequency Multiplier", "Output corresponding frequency multiplier.");
	}


	class FSemitoneToFrequencyMultiplierOperator : public TExecutableOperator<FSemitoneToFrequencyMultiplierOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FSemitoneToFrequencyMultiplierOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InSemitone);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// The input difference in semitones
		FFloatReadRef SemitoneInput;

		// The output frequency multiplier
		FFloatWriteRef FrequencyMultiplierOutput;

		// Cached semitone value. Used to catch if the value changes to recompute Frequency Multiplier output.
		float PrevSemitone;
	};

	FSemitoneToFrequencyMultiplierOperator::FSemitoneToFrequencyMultiplierOperator(const FOperatorSettings& InSettings, const FFloatReadRef& InSemitone)
		: SemitoneInput(InSemitone)
		, FrequencyMultiplierOutput(FFloatWriteRef::CreateNew(Audio::GetFrequencyMultiplier(*InSemitone)))
		, PrevSemitone(*InSemitone)
	{
	}


	void FSemitoneToFrequencyMultiplierOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace SemitoneToFrequencyMultiplierVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSemitone), SemitoneInput);
	}

	void FSemitoneToFrequencyMultiplierOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace SemitoneToFrequencyMultiplierVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputFrequencyMultiplier), FrequencyMultiplierOutput);
	}

	void FSemitoneToFrequencyMultiplierOperator::Execute()
	{
		const float CurrSemitone = *SemitoneInput;

		// Only do anything if the input changes
		if (!FMath::IsNearlyEqual(CurrSemitone, PrevSemitone))
		{
			PrevSemitone = CurrSemitone;
			*FrequencyMultiplierOutput = Audio::GetFrequencyMultiplier(CurrSemitone);
		}
	}

	void FSemitoneToFrequencyMultiplierOperator::Reset(const IOperator::FResetParams& InParams)
	{
		*FrequencyMultiplierOutput = Audio::GetFrequencyMultiplier(*SemitoneInput);
		PrevSemitone = *SemitoneInput;
	}

	const FVertexInterface& FSemitoneToFrequencyMultiplierOperator::GetVertexInterface()
	{
		using namespace SemitoneToFrequencyMultiplierVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSemitone), 0.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputFrequencyMultiplier))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FSemitoneToFrequencyMultiplierOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<float>();
			const FName OperatorName = TEXT("Semitone to Frequency Multiplier");
			const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_SemitoneToFrequencyMultiplierName", "Semitone to Frequency Multiplier");
			const FText NodeDescription = METASOUND_LOCTEXT("Metasound_SemitoneToFrequencyMultiplierDescription", "Converts a number of semitones to the corresponding frequency multiplier.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Conversions);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FSemitoneToFrequencyMultiplierOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace SemitoneToFrequencyMultiplierVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FFloatReadRef InSemitone = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputSemitone), InParams.OperatorSettings);

		return MakeUnique<FSemitoneToFrequencyMultiplierOperator>(InParams.OperatorSettings, InSemitone);
	}

	using FSemitoneToFrequencyMultiplierNode = TNodeFacade<FSemitoneToFrequencyMultiplierOperator>;

	METASOUND_REGISTER_NODE(FSemitoneToFrequencyMultiplierNode)
}

#undef LOCTEXT_NAMESPACE
