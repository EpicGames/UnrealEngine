// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundStandardNodesCategories.h"

#include "MetasoundNodeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound::NodeCategories
{
	const FText Debug = { METASOUND_LOCTEXT("Metasound_DebugCategory", "Debug") };
	const FText Delays = { METASOUND_LOCTEXT("Metasound_EffectsCategory", "Delays") };
	const FText Dynamics = { METASOUND_LOCTEXT("Metasound_DynamicsCategory", "Dynamics") };
	const FText Envelopes = { METASOUND_LOCTEXT("Metasound_EnvelopesCategory", "Envelopes") };
	const FText Filters = { METASOUND_LOCTEXT("Metasound_FiltersCategory", "Filters") };
	const FText Generators = { METASOUND_LOCTEXT("Metasound_GeneratorsCategory", "Generators") };
	const FText Io = { METASOUND_LOCTEXT("Metasound_IoCategory", "External IO") };
	const FText Math = { METASOUND_LOCTEXT("Metasound_MathCategory", "Math") };
	const FText Mix = { METASOUND_LOCTEXT("Metasound_MixCategory", "Mix") };
	const FText Music = { METASOUND_LOCTEXT("Metasound_MusicCategory", "Music") };
	const FText RandomUtils = { METASOUND_LOCTEXT("Metasound_RandomCategory", "Random") };
	const FText Spatialization = { METASOUND_LOCTEXT("Metasound_SpatializationCategory", "Spatialization") };
	const FText Trigger = { METASOUND_LOCTEXT("Metasound_TriggerCategory", "Triggers") };
	const FText WaveTables = { METASOUND_LOCTEXT("Metasound_WaveTableCategory", "WaveTables") };
	const FText Reverbs = { METASOUND_LOCTEXT("Metasound_ReverbCategory", "Reverbs") };
} // Metasound::NodeCategories

#undef LOCTEXT_NAMESPACE