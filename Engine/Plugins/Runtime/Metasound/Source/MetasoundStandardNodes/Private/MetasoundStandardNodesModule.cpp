// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendNodeClassRegistry.h"
#include "MetasoundFrontendNodeMigration.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundFrontendNodeUpdateTransform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


namespace Metasound 
{
	namespace StandardNodesModulePrivate
	{
		const FMetasoundFrontendClassName BiquadFilterName(StandardNodes::Namespace, TEXT("Biquad Filter"), StandardNodes::AudioVariant);
		const Frontend::FNodeClassRegistryKey BiquadFilter_2_0_Key(
				/*ClassType*/ EMetasoundFrontendClassType::External,
				/*ClassName*/ BiquadFilterName,
				/*Version*/ FMetasoundFrontendVersionNumber(2, 0));
	}
	
	class FMetasoundStandardNodesModule : public IModuleInterface
	{
		virtual void StartupModule() override
		{
			using namespace Frontend;
			METASOUND_REGISTER_ITEMS_IN_MODULE

#if 0
			// Example of node migration
			INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();
			NodeRegistry.RegisterNodeMigration(
				FNodeMigrationInfo
				{
					"5.7",
					FNodeClassName {"Dummy", "Dummy", "Dummy"},
					1,
					0,
					UE_STRINGIZE(METASOUND_PLUGIN),
					UE_STRINGIZE(METASOUND_MODULE),
					"MetasoundExperimental",
					"MetasoundRuntime"
				});
#endif
#if WITH_EDITORONLY_DATA
			// Node update transform
			INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();
			NodeRegistry.RegisterCustomNodeUpdateTransform(
				StandardNodesModulePrivate::BiquadFilterName, /*Version*/ 1,
				MakeShared<FMaintainDefaultsNodeUpdateTransform>(StandardNodesModulePrivate::BiquadFilter_2_0_Key)
			);
#endif // WITH_EDITORONLY_DATA
		}

		virtual void ShutdownModule() override
		{
			using namespace Frontend;
			METASOUND_UNREGISTER_ITEMS_IN_MODULE
			

#if 0
			// Example of node migration
			INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();
			NodeRegistry.UnregisterNodeMigration(
				FNodeMigrationInfo
				{
					"5.7",
					FNodeClassName {"Dummy", "Dummy", "Dummy"},
					1,
					0,
					UE_STRINGIZE(METASOUND_PLUGIN),
					UE_STRINGIZE(METASOUND_MODULE),
					"MetasoundExperimental",
					"MetasoundRuntime"
				});
#endif
#if WITH_EDITORONLY_DATA
			// Node update transform
			INodeClassRegistry& NodeRegistry = INodeClassRegistry::GetChecked();
			NodeRegistry.UnregisterCustomNodeUpdateTransform(StandardNodesModulePrivate::BiquadFilterName, /*Version*/1);
#endif // WITH_EDITORONLY_DATA
		}
	};
}

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(Metasound::FMetasoundStandardNodesModule, MetasoundStandardNodes);

