// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundStandardNodesNames.h"


namespace Metasound
{
	namespace StandardNodes
	{
		const FLazyName Namespace = "UE";
		const FLazyName AudioVariant = "Audio";
	}
}
