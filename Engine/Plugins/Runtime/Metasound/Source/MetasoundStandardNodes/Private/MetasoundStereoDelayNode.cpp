// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/Delay.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_StereoDelayNode"

namespace Metasound
{
	namespace StereoDelay
	{
		METASOUND_PARAM(InputResetDelay, "Reset", "Resets the delay buffer.");
		METASOUND_PARAM(InAudioInputLeft, "In Left", "Left channel audio input.")
		METASOUND_PARAM(InAudioInputRight, "In Right", "Right channel audio input.")
		METASOUND_PARAM(InDelayMode, "Delay Mode", "Delay mode.")
		METASOUND_PARAM(InDelayTime, "Delay Time", "The amount of time to delay the audio.")
		METASOUND_PARAM(InDelayRatio, "Delay Ratio", "Delay spread for left and right channels. Allows left and right channels to have differential delay amounts. Useful for stereo channel decorrelation")
		METASOUND_PARAM(InDryLevel, "Dry Level", "The dry level of the delay.")
		METASOUND_PARAM(InWetLevel, "Wet Level", "The wet level of the delay.")
		METASOUND_PARAM(InFeedbackAmount, "Feedback", "Feedback amount.")
		METASOUND_PARAM(InParamMaxDelayTime, "Max Delay Time", "The maximum amount of time to delay the audio.")
		METASOUND_PARAM(OutAudioLeft, "Out Left", "Left channel audio output.")
		METASOUND_PARAM(OutAudioRight, "Out Right", "Right channel audio output.")

		static constexpr float MinDelaySeconds = 0.001f;
		static constexpr float MaxDelaySeconds = 1000.f;
		static constexpr float DefaultMaxDelaySeconds = 2.5f;
	}

	enum class EStereoDelayMode
	{
		Normal = 0,
		Cross,
		PingPong,
	};

	DECLARE_METASOUND_ENUM(EStereoDelayMode, EStereoDelayMode::Normal, METASOUNDSTANDARDNODES_API,
		FEnumStereoDelayMode, FEnumStereoDelayModeInfo, FStereoDelayModeReadRef, FEnumStereoDelayModeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(EStereoDelayMode, FEnumStereoDelayMode, "StereoDelayMode")
		DEFINE_METASOUND_ENUM_ENTRY(EStereoDelayMode::Normal, "StereoDelayModeNormalDescription", "Normal", "StereoDelayModeNormalDescriptionTT", "Left input mixes with left delay output and feeds to left output."),
		DEFINE_METASOUND_ENUM_ENTRY(EStereoDelayMode::Cross, "StereoDelayModeCrossDescription", "Cross", "StereoDelayModeCrossDescriptionTT", "Left input mixes with right delay output and feeds to right output."),
		DEFINE_METASOUND_ENUM_ENTRY(EStereoDelayMode::PingPong, "StereoDelayModePingPongDescription", "Ping Pong", "StereoDelayModePingPongDescriptionTT", "Left input mixes with left delay output and feeds to right output."),
		DEFINE_METASOUND_ENUM_END()

	class FStereoDelayOperator : public TExecutableOperator<FStereoDelayOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FStereoDelayOperator(
			const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InLeftAudioInput, 
			const FAudioBufferReadRef& InRightAudioInput,
			const FStereoDelayModeReadRef& InStereoDelayMode,
			const FTimeReadRef& InDelayTime,
			const FFloatReadRef& InDelayRatio,
			const FFloatReadRef& InDryLevel, 
			const FFloatReadRef& InWetLevel, 
			const FFloatReadRef& InFeedback,
			float InMaxDelayTimeSeconds,
			const FTriggerReadRef& InTriggerReset);


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		float GetInputDelayTimeMsecClamped() const;
		float GetInputDelayRatioClamped() const;

		// The input audio buffer
		FAudioBufferReadRef LeftAudioInput;
		FAudioBufferReadRef RightAudioInput;

		// Which stereo delay mode to render the audio delay with
		FStereoDelayModeReadRef StereoDelayMode;

		// The amount of delay time
		FTimeReadRef DelayTime;

		// The stereo delay ratio
		FFloatReadRef DelayRatio;

		// The the dry level
		FFloatReadRef DryLevel;

		// The the wet level
		FFloatReadRef WetLevel;

		// The feedback amount
		FFloatReadRef Feedback;

		// The audio output
		FAudioBufferWriteRef LeftAudioOutput;
		FAudioBufferWriteRef RightAudioOutput;

		// The internal delay buffer
		Audio::FDelay LeftDelayBuffer;
		Audio::FDelay RightDelayBuffer;

		// The current delay time and delay ratio
		float PrevDelayTimeMsec;
		float PrevDelayRatio;

		// Maximum delay time
		float MaxDelayTimeSeconds = StereoDelay::DefaultMaxDelaySeconds;

		// The reset trigger
		FTriggerReadRef TriggerReset;
	};

	FStereoDelayOperator::FStereoDelayOperator(
		const FOperatorSettings& InSettings, 
		const FAudioBufferReadRef& InLeftAudioInput, 
		const FAudioBufferReadRef& InRightAudioInput,
		const FStereoDelayModeReadRef& InStereoDelayMode,
		const FTimeReadRef& InDelayTime,
		const FFloatReadRef& InDelayRatio,
		const FFloatReadRef& InDryLevel,
		const FFloatReadRef& InWetLevel, 
		const FFloatReadRef& InFeedback,
		float InMaxDelayTimeSeconds,
		const FTriggerReadRef& InTriggerReset)

		: LeftAudioInput(InLeftAudioInput)
		, RightAudioInput(InRightAudioInput)
		, StereoDelayMode(InStereoDelayMode)
		, DelayTime(InDelayTime)
		, DelayRatio(InDelayRatio)
		, DryLevel(InDryLevel)
		, WetLevel(InWetLevel)
		, Feedback(InFeedback)
		, LeftAudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		, RightAudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		, PrevDelayTimeMsec(0.0f)
		, PrevDelayRatio(GetInputDelayRatioClamped())
		, MaxDelayTimeSeconds(FMath::Clamp(InMaxDelayTimeSeconds, StereoDelay::MinDelaySeconds, StereoDelay::MaxDelaySeconds))
		, TriggerReset(InTriggerReset)
	{
		// Depends on MaxDelayTimeSeconds being set first.
		PrevDelayTimeMsec = GetInputDelayTimeMsecClamped();
		
		LeftDelayBuffer.Init(InSettings.GetSampleRate(), MaxDelayTimeSeconds);
		LeftDelayBuffer.SetDelayMsec(PrevDelayTimeMsec * (1.0f + PrevDelayRatio));
		RightDelayBuffer.Init(InSettings.GetSampleRate(), MaxDelayTimeSeconds);
		RightDelayBuffer.SetDelayMsec(PrevDelayTimeMsec * (1.0f - PrevDelayRatio));
	}


	void FStereoDelayOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace StereoDelay;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InAudioInputLeft), LeftAudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InAudioInputRight), RightAudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InDelayMode), StereoDelayMode);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InDelayTime), DelayTime);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InDelayRatio), DelayRatio);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InDryLevel), DryLevel);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InWetLevel), WetLevel);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InFeedbackAmount), Feedback);
		InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), FTime::FromSeconds(MaxDelayTimeSeconds));
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetDelay), TriggerReset);
	}

	void FStereoDelayOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace StereoDelay;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutAudioLeft), LeftAudioOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutAudioRight), RightAudioOutput);
	}

	float FStereoDelayOperator::GetInputDelayTimeMsecClamped() const
	{
		// Clamp the delay time to the max delay allowed
		return 1000.0f * FMath::Clamp((float)DelayTime->GetSeconds(), 0.0f, MaxDelayTimeSeconds);
	}

	float FStereoDelayOperator::GetInputDelayRatioClamped() const
	{
		return FMath::Clamp(*DelayRatio, -1.0f, 1.0f);
	}

	void FStereoDelayOperator::Execute()
	{
		TriggerReset->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				LeftDelayBuffer.Reset();
				RightDelayBuffer.Reset();
			}
		);

		// Get clamped delay time
		float CurrentInputDelayTime = GetInputDelayTimeMsecClamped();
		float CurrentDelayRatio = GetInputDelayRatioClamped();

		// Check to see if our delay amount has changed
		if (!FMath::IsNearlyEqual(PrevDelayTimeMsec, CurrentInputDelayTime) || !FMath::IsNearlyEqual(PrevDelayRatio, CurrentDelayRatio))
		{
			PrevDelayTimeMsec = CurrentInputDelayTime;
			PrevDelayRatio = CurrentDelayRatio;
			LeftDelayBuffer.SetEasedDelayMsec(PrevDelayTimeMsec * (1.0f + PrevDelayRatio));
			RightDelayBuffer.SetEasedDelayMsec(PrevDelayTimeMsec * (1.0f - PrevDelayRatio));
		}

		const float* LeftInput = LeftAudioInput->GetData();
		const float* RightInput = RightAudioInput->GetData();

		float* LeftOutput = LeftAudioOutput->GetData();
		float* RightOutput = RightAudioOutput->GetData();

		int32 NumFrames = LeftAudioInput->Num();

		// Clamp the feedback amount to make sure it's bounded. Clamp to a number slightly less than 1.0
		float FeedbackAmount = FMath::Clamp(*Feedback, 0.0f, 1.0f - SMALL_NUMBER);
		float CurrentDryLevel = FMath::Clamp(*DryLevel, 0.0f, 1.0f);
		float CurrentWetLevel = FMath::Clamp(*WetLevel, 0.0f, 1.0f);

		if (FMath::IsNearlyZero(FeedbackAmount))
		{
			// if pingpong
			switch (*StereoDelayMode)
			{
				// Normal feeds left to left and right to right
				case EStereoDelayMode::Normal:
				{
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
					{
						LeftOutput[FrameIndex] = CurrentWetLevel * LeftDelayBuffer.ProcessAudioSample(LeftInput[FrameIndex]) + CurrentDryLevel * LeftInput[FrameIndex];
						RightOutput[FrameIndex] = CurrentWetLevel * RightDelayBuffer.ProcessAudioSample(RightInput[FrameIndex]) + CurrentDryLevel * RightInput[FrameIndex];
					}
				}
				break;

				// No-feedback Cross and ping-pong feeds right input to left and left input to right
				case EStereoDelayMode::Cross:
				case EStereoDelayMode::PingPong:
				{
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
					{
						// Ping pong feeds right to left and left to right
						LeftOutput[FrameIndex] = CurrentWetLevel * LeftDelayBuffer.ProcessAudioSample(RightInput[FrameIndex]) + CurrentDryLevel * LeftInput[FrameIndex];
						RightOutput[FrameIndex] = CurrentWetLevel * RightDelayBuffer.ProcessAudioSample(LeftInput[FrameIndex]) + CurrentDryLevel * RightInput[FrameIndex];
					}
				}
				break;
			}
		}
		else
		{
			// TODO: support different delay cross-modes via enum, currently default to pingpong
			switch (*StereoDelayMode)
			{
				case EStereoDelayMode::Normal:
				{
						
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
					{
						float LeftDelayIn = LeftInput[FrameIndex] + FeedbackAmount * LeftDelayBuffer.Read();
						float RightDelayIn = RightInput[FrameIndex] + FeedbackAmount * RightDelayBuffer.Read();

						LeftOutput[FrameIndex] = CurrentWetLevel * LeftDelayBuffer.ProcessAudioSample(LeftDelayIn) + CurrentDryLevel * LeftInput[FrameIndex];
						RightOutput[FrameIndex] = CurrentWetLevel * RightDelayBuffer.ProcessAudioSample(RightDelayIn) + CurrentDryLevel * RightInput[FrameIndex];
					}
				}
				break;

				case EStereoDelayMode::Cross:
				{
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
					{
						float LeftDelayIn = RightInput[FrameIndex] + FeedbackAmount * LeftDelayBuffer.Read();
						float RightDelayIn = LeftInput[FrameIndex] + FeedbackAmount * RightDelayBuffer.Read();

						LeftOutput[FrameIndex] = CurrentWetLevel * LeftDelayBuffer.ProcessAudioSample(LeftDelayIn) + CurrentDryLevel * LeftInput[FrameIndex];
						RightOutput[FrameIndex] = CurrentWetLevel * RightDelayBuffer.ProcessAudioSample(RightDelayIn) + CurrentDryLevel * RightInput[FrameIndex];
					}
				}
				break;

				case EStereoDelayMode::PingPong:
				{
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
					{
						float LeftDelayIn = RightInput[FrameIndex] + FeedbackAmount * RightDelayBuffer.Read();
						float RightDelayIn = LeftInput[FrameIndex] + FeedbackAmount * LeftDelayBuffer.Read();

						LeftOutput[FrameIndex] = CurrentWetLevel * LeftDelayBuffer.ProcessAudioSample(LeftDelayIn) + CurrentDryLevel * LeftInput[FrameIndex];
						RightOutput[FrameIndex] = CurrentWetLevel * RightDelayBuffer.ProcessAudioSample(RightDelayIn) + CurrentDryLevel * RightInput[FrameIndex];
					}
				}
				break;
			}
		}
	}

	void FStereoDelayOperator::Reset(const IOperator::FResetParams& InParams)
	{
		LeftAudioOutput->Zero();
		RightAudioOutput->Zero();

		PrevDelayTimeMsec = GetInputDelayTimeMsecClamped();
		PrevDelayRatio = GetInputDelayRatioClamped();

		LeftDelayBuffer.Reset();
		LeftDelayBuffer.SetDelayMsec(PrevDelayTimeMsec * (1.0f + PrevDelayRatio));
		RightDelayBuffer.Reset();
		RightDelayBuffer.SetDelayMsec(PrevDelayTimeMsec * (1.0f - PrevDelayRatio));
	}

	const FVertexInterface& FStereoDelayOperator::GetVertexInterface()
	{
		using namespace StereoDelay;

		FDataVertexMetadata MaxDelayTimeMetadata = METASOUND_GET_PARAM_METADATA(InParamMaxDelayTime);
		MaxDelayTimeMetadata.bIsAdvancedDisplay = true;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InAudioInputLeft)),
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InAudioInputRight)),
				TInputDataVertex<FEnumStereoDelayMode>(METASOUND_GET_PARAM_NAME_AND_METADATA(InDelayMode), (int32)EStereoDelayMode::PingPong),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InDelayTime), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InDelayRatio), 0.1f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InDryLevel), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InWetLevel), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InFeedbackAmount), 0.0f),
				TInputConstructorVertex<FTime>(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), MaxDelayTimeMetadata, DefaultMaxDelaySeconds),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetDelay))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudioLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudioRight))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FStereoDelayOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Stereo Delay"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_StereoDelayDisplayName", "Stereo Delay");
			Info.Description = METASOUND_LOCTEXT("Metasound_StereoDelayNodeDescription", "Delays a stereo audio buffer by the specified amount.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Delays);
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FStereoDelayOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace StereoDelay; 
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FAudioBufferReadRef LeftAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InAudioInputLeft), InParams.OperatorSettings);
		FAudioBufferReadRef RightAudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InAudioInputRight), InParams.OperatorSettings);
		FStereoDelayModeReadRef StereoDelayMode = InputData.GetOrCreateDefaultDataReadReference<FEnumStereoDelayMode>(METASOUND_GET_PARAM_NAME(InDelayMode), InParams.OperatorSettings);
		FTimeReadRef DelayTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InDelayTime), InParams.OperatorSettings);
		FFloatReadRef DelayRatio = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InDelayRatio), InParams.OperatorSettings);
		FFloatReadRef DryLevel = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InDryLevel), InParams.OperatorSettings);
		FFloatReadRef WetLevel = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InWetLevel), InParams.OperatorSettings);
		FFloatReadRef Feedback = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InFeedbackAmount), InParams.OperatorSettings);
		FTime MaxDelayTime = InputData.GetOrCreateDefaultValue<FTime>(METASOUND_GET_PARAM_NAME(InParamMaxDelayTime), InParams.OperatorSettings);
		FTriggerReadRef TriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputResetDelay), InParams.OperatorSettings);

		return MakeUnique<FStereoDelayOperator>(InParams.OperatorSettings, LeftAudioIn, RightAudioIn, StereoDelayMode, DelayTime, DelayRatio, DryLevel, WetLevel, Feedback, MaxDelayTime.GetSeconds(), TriggerReset);
	}

	using FStereoDelayNode = TNodeFacade<FStereoDelayOperator>;

	METASOUND_REGISTER_NODE(FStereoDelayNode)
}

#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes_DelayNode
