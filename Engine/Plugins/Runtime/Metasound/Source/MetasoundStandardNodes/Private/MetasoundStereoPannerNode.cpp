// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundAudioBuffer.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundFacade.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_StereoPanner"

namespace Metasound
{
	namespace StereoPannerVertexNames
	{
		METASOUND_PARAM(InputAudio, "In", "The input audio to pan.")
		METASOUND_PARAM(InputPanAmount, "Pan Amount", "The amount of pan. -1.0 is full left, 1.0 is full right.")
		METASOUND_PARAM(InputPanningLaw, "Panning Law", "Which panning law should be used for the stereo panner.")
		METASOUND_PARAM(OutputAudioLeft, "Out Left", "Left channel audio output.")
		METASOUND_PARAM(OutputAudioRight, "Out Right", "Right channel audio output.")
	}

	enum class EPanningLaw
	{
		EqualPower = 0,
		Linear
	};

	DECLARE_METASOUND_ENUM(EPanningLaw, EPanningLaw::EqualPower, METASOUNDSTANDARDNODES_API,
	FEnumPanningLaw, FEnumPanningLawInfo, FPanningLawReadRef, FPanningLawWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(EPanningLaw, FEnumPanningLaw, "PanningLaw")
		DEFINE_METASOUND_ENUM_ENTRY(EPanningLaw::EqualPower, "PanningLawEqualPowerName", "Equal Power", "PanningLawEqualPowerTT", "The power of the audio signal is constant while panning."),
		DEFINE_METASOUND_ENUM_ENTRY(EPanningLaw::Linear, "PanningLawLinearName", "Linear", "PanningLawLinearTT", "The amplitude of the audio signal is constant while panning."),
	DEFINE_METASOUND_ENUM_END()

	class FStereoPannerOperator : public TExecutableOperator<FStereoPannerOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FStereoPannerOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioInput, 
			const FFloatReadRef& InPanningAmount,
			const FPanningLawReadRef& InPanningLaw);


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		float GetInputDelayTimeMsec() const;
		void ComputePanGains(float InPanningAmmount, float& OutLeftGain, float& OutRightGain) const;

		// The input audio buffer
		FAudioBufferReadRef AudioInput;

		// The amount of delay time
		FFloatReadRef PanningAmount;

		// The the dry level
		FPanningLawReadRef PanningLaw;

		// The audio output
		FAudioBufferWriteRef AudioLeftOutput;
		FAudioBufferWriteRef AudioRightOutput;

		float PrevPanningAmount = 0.0f;
		float PrevLeftPan = 0.0f;
		float PrevRightPan = 0.0f;
	};

	FStereoPannerOperator::FStereoPannerOperator(const FOperatorSettings& InSettings,
		const FAudioBufferReadRef& InAudioInput, 
		const FFloatReadRef& InPanningAmount,
		const FPanningLawReadRef& InPanningLaw)
		: AudioInput(InAudioInput)
		, PanningAmount(InPanningAmount)
		, PanningLaw(InPanningLaw)
		, AudioLeftOutput(FAudioBufferWriteRef::CreateNew(InSettings))
		, AudioRightOutput(FAudioBufferWriteRef::CreateNew(InSettings))
	{
		PrevPanningAmount = FMath::Clamp(*PanningAmount, -1.0f, 1.0f);

		ComputePanGains(PrevPanningAmount, PrevLeftPan, PrevRightPan);
	}


	void FStereoPannerOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace StereoPannerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPanAmount), PanningAmount);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPanningLaw), PanningLaw);
	}

	void FStereoPannerOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace StereoPannerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioLeft), AudioLeftOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudioRight), AudioRightOutput);
	}

	void FStereoPannerOperator::ComputePanGains(float InPanningAmmount, float& OutLeftGain, float& OutRightGain) const
	{
		// Convert [-1.0, 1.0] to [0.0, 1.0]
		float Fraction = 0.5f * (InPanningAmmount + 1.0f);

		if (*PanningLaw == EPanningLaw::EqualPower)
		{
			// Compute the left and right amount with one math call
			FMath::SinCos(&OutRightGain, &OutLeftGain, 0.5f * PI * Fraction);
		}
		else
		{
			OutLeftGain = Fraction;
			OutRightGain = 1.0f - Fraction;
		}
	}


	void FStereoPannerOperator::Execute()
	{
		float CurrentPanningAmount = FMath::Clamp(*PanningAmount, -1.0f, 1.0f);

		const float* InputBufferPtr = AudioInput->GetData();
		int32 InputSampleCount = AudioInput->Num();
		float* OutputLeftBufferPtr = AudioLeftOutput->GetData();
		float* OutputRightBufferPtr = AudioRightOutput->GetData();

		TArrayView<const float> InputBufferView(AudioInput->GetData(), InputSampleCount);
		TArrayView<float> OutputLeftBufferView(AudioLeftOutput->GetData(), InputSampleCount);
		TArrayView<float> OutputRightBufferView(AudioRightOutput->GetData(), InputSampleCount);

		if (FMath::IsNearlyEqual(PrevPanningAmount, CurrentPanningAmount))
		{
			Audio::ArrayMultiplyByConstant(InputBufferView, PrevLeftPan, OutputLeftBufferView);
			Audio::ArrayMultiplyByConstant(InputBufferView, PrevRightPan, OutputRightBufferView);
		}
		else 
		{
			// The pan amount has changed so recompute it
			float CurrentLeftPan;
			float CurrentRightPan;
			ComputePanGains(CurrentPanningAmount, CurrentLeftPan, CurrentRightPan);

			// Copy the input to the output buffers
			FMemory::Memcpy(OutputLeftBufferPtr, InputBufferPtr, InputSampleCount * sizeof(float));
			FMemory::Memcpy(OutputRightBufferPtr, InputBufferPtr, InputSampleCount * sizeof(float));

			// Do a fast fade on the buffers from the prev left/right gains to current left/right gains
			Audio::ArrayFade(OutputLeftBufferView, PrevLeftPan, CurrentLeftPan);
			Audio::ArrayFade(OutputRightBufferView, PrevRightPan, CurrentRightPan);

			// lerp through the buffer to the target panning amount
			PrevPanningAmount = *PanningAmount;
			PrevLeftPan = CurrentLeftPan;
			PrevRightPan = CurrentRightPan;
		}
	}

	void FStereoPannerOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioLeftOutput->Zero();
		AudioRightOutput->Zero();

		PrevPanningAmount = FMath::Clamp(*PanningAmount, -1.0f, 1.0f);
		ComputePanGains(PrevPanningAmount, PrevLeftPan, PrevRightPan);
	}

	const FVertexInterface& FStereoPannerOperator::GetVertexInterface()
	{
		using namespace StereoPannerVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPanAmount), 0.0f),
				TInputDataVertex<FEnumPanningLaw>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPanningLaw), (int32)EPanningLaw::EqualPower)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioLeft)),
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudioRight))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FStereoPannerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Stereo Panner"), TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_StereoPannerDisplayName", "Stereo Panner");
			Info.Description = METASOUND_LOCTEXT("Metasound_StereoPannerNodeDescription", "Pans an input audio signal to left and right outputs.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Spatialization);
			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FStereoPannerOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		using namespace StereoPannerVertexNames;

		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
		FFloatReadRef PanningAmount = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPanAmount), InParams.OperatorSettings);
		FPanningLawReadRef PanningLaw = InputData.GetOrCreateDefaultDataReadReference<FEnumPanningLaw>(METASOUND_GET_PARAM_NAME(InputPanningLaw), InParams.OperatorSettings);

		return MakeUnique<FStereoPannerOperator>(InParams.OperatorSettings, AudioIn, PanningAmount, PanningLaw);
	}

	using FStereoPannerNode = TNodeFacade<FStereoPannerOperator>;

	METASOUND_REGISTER_NODE(FStereoPannerNode)
}

#undef LOCTEXT_NAMESPACE
