// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/Filter.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Subharmonizer"

namespace Metasound
{
	namespace SubharmonizerVertexNames
	{
		METASOUND_PARAM(InputAudio, "Audio In", "Audio to process");
		METASOUND_PARAM(InputFrequency, "Filter Frequency", "Cutoff Frequency for Lowpass Input and Output Filter to reduce noise, additional Highpass Filter at half frequency.");
		METASOUND_PARAM(InputLinearGain, "Gain (Lin)", "Output Gain Scalar.")
		METASOUND_PARAM(InputSubharmonic, "Subharmonic", "Which Subharmonic to output, input clamped between 1 and 6.");
		METASOUND_PARAM(InputOctaverMode, "Octaver Mode", "When true, output is sub-octaves; when false, output is subharmonics.")

		METASOUND_PARAM(OutputAudio, "Audio Out", "Processed audio signal");
	}

	namespace SubharmonizerNodePrivate
	{
		/*
		*  Subharmonizer processor. Supports suboctave mode and harmonic selection. Uses zero-crossing counting.
		*/
		struct FSubharmonizer
		{
			// Audio processor
			void Process(const float* InBuffer, const int32 NumFrames, float* BufferOut);

			// Parameter update
			void Update(const int32 InSubharmonic, const bool bOctaverModeIn);

			// Reset processor
			void Reset();

		private:
			// Bitmask used to count subharmonics
			uint8 SubharmonicBitmask = 2;

			// Subharmonic base value for modulo op
			uint8 SubharmonicBase = 4;

			// Current subharmonic set for output
			uint8 Subharmonic = 1;

			// Tracking for sign state when in subharmonic mode
			uint8 CurrentSignState = 0;

			// Tracking most recent sign value
			bool bLastSign = false;

			// Whether or not to output sub-octaves instead of subharmonics
			bool bOctaverMode = false;

			// Tracking which side of zero we're currently on
			bool bPolarity = false;
		};
	}

class FSubharmonizerOperator : public TExecutableOperator<FSubharmonizerOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetDefaultInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FSubharmonizerOperator(const FOperatorSettings& InSettings,
			const FAudioBufferReadRef& InAudioIn,
			const FFloatReadRef& InFrequencyIn,
			const FFloatReadRef& InGainDBIn,
			const FInt32ReadRef& InSubharmonicIn,
			const FBoolReadRef& InOctaverModeIn
			);

		virtual ~FSubharmonizerOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void UpdateParams();

		void Reset(const IOperator::FResetParams& InParams);
		
		void Execute();

	private:
		// Internal reset function
		void InternalReset();

		// Input Pin Refs
		FAudioBufferReadRef	AudioIn;
		FFloatReadRef FrequencyIn;
		FFloatReadRef LinearGainIn;
		FInt32ReadRef SubharmonicIn;
		FBoolReadRef OctaverModeIn;

		// Output Pin Ref
		FAudioBufferWriteRef AudioOut;

		// Environment data
		float SampleRate = 48000.0f;
		int32 NumFramesPerBlock = 0;

		// Max filter frequency/nyquist frequency
		float MaxFilterFrequency = 24000.f;

		// Clamped Minimum frequency
		const float MinFilterFrequency = 20.f;
		

		// Cached Input
		float PreviouisFilterFrequency = 120.f;
		
		// Default cutoff frequency values
		float LPFFilterFrequency = 120.f;
		float HPFFilterFrequency = 60.f;

		// Clamped minimum frequency
		const float InputHighPassFilterFrequency = 30.f;

		float LinearGain = 2.f;
		float PreviousLinearGain = 2.f;

		int32 InputSubharmonic = 1;
		int32 PreviousInputSubharmonic = 1;

		// Clamped ranges
		const int32 MinInputSubharmonic = 1;
		const int32 MaxInputSubharmonic = 6;

		bool bOctaverMode = false;
		
		// Processors
		SubharmonizerNodePrivate::FSubharmonizer Subharmonizer;
		
		// Input LPF: Removes unnecessary input noise
		Audio::FBiquadFilter InputLowPassFilter;

		// Output LPF: Smoothing filter for output from pulse wave generated by Subharmonizer
		Audio::FBiquadFilter OutputLowPassFilter;

		// Output HPF: Helps to remove low-frequency noise
		Audio::FBiquadFilter OutputHighPassFilter;
		
		// Envelope follower to help shape output according to input
		Audio::FEnvelopeFollower EnvelopeFollower;

		// Scratch Data
		Audio::FAlignedFloatBuffer ScratchBufferA;
		Audio::FAlignedFloatBuffer ScratchBufferB;
		Audio::FAlignedFloatBuffer EnvelopeBuffer;
		Audio::FAlignedFloatBuffer GainInterpolationBuffer;
	};
	
	const FVertexInterface& FSubharmonizerOperator::GetDefaultInterface()
	{
		using namespace SubharmonizerVertexNames;

		auto CreateDefaultInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;
				InputInterface.Add(TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputFrequency), 80.0f));
				InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLinearGain), 1.f));
				InputInterface.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputSubharmonic), 1));
				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputOctaverMode), false));

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio)));

				return FVertexInterface(InputInterface, OutputInterface);
			};

		static const FVertexInterface Interface = CreateDefaultInterface();

		return Interface;
	}

	void SubharmonizerNodePrivate::FSubharmonizer::Process(const float* InBuffer, const int32 NumFrames, float* BufferOut)
	{
		// Process loop in Octaver Mode
		if (bOctaverMode)
		{
			// Create mask for modulo op
			const int32 SubharmonicBaseMask = SubharmonicBase - 1;

			// Process creates a 1.f or -1.f output value based on either flipping bits or using the results of a modulus operator
			for (int32 i = 0; i < NumFrames; ++i)
			{
				// The sign from this sample
				const bool bNewSign = InBuffer[i] > 0.f;

				// Is the sign is different, we increment our "counters"
				if (bLastSign != bNewSign)
				{
					// Update sign state based on counter
					++CurrentSignState;

					CurrentSignState = CurrentSignState & SubharmonicBaseMask;
				}

				// Select correct output value based on sign and bitmask state
				BufferOut[i] = ((CurrentSignState & SubharmonicBitmask) ? 1.f : -1.f);

				// Cache sign value for next sample evaluation
				bLastSign = bNewSign;
			}
		}
		// Process loop when not in Octaver Mode
		else
		{
			// Process creates a 1.f or -1.f output value based on either flipping bits or using the results of a modulus operator
			for (int32 i = 0; i < NumFrames; ++i)
			{
				// The sign from this sample
				const bool bNewSign = InBuffer[i] > 0.f;

				// Is the sign is different, we increment our "counters"
				if (bLastSign != bNewSign)
				{
					// Update sign state based on counter
					CurrentSignState = FMath::Modulo(++CurrentSignState, SubharmonicBase);

					// If not using octaver mode, base sign change on returning to 0 after mod op
					if (CurrentSignState == 0)
					{
						bPolarity = !bPolarity;
					}
				}

				// Select correct output value based on polarity
				BufferOut[i] = bPolarity ? 1.f : -1.f;
				
				// Cache sign value for next sample evaluation
				bLastSign = bNewSign;
			}
		}
		

	}

	void SubharmonizerNodePrivate::FSubharmonizer::Update(const int32 InSubharmonic, const bool bOctaverModeIn)
	{
		// Cache updated values
		Subharmonic = InSubharmonic;
		bOctaverMode = bOctaverModeIn;
		
		// Set bitmask
		SubharmonicBitmask = 1 << Subharmonic;

		// Adjust Base based on octaver mode or not
		if(bOctaverMode)
		{
			SubharmonicBase = 1 << FMath::Min(Subharmonic + 1, 7);
		}
		else
		{
			SubharmonicBase = FMath::Min(Subharmonic + 1, 7);
		}
		
	}

	void SubharmonizerNodePrivate::FSubharmonizer::Reset()
	{
		// Revert to starting values
		Subharmonic = 2;
		bLastSign = false;
		CurrentSignState = 0;
		bOctaverMode = false;
	}

	const FNodeClassMetadata& FSubharmonizerOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Subharmonizer"));
				FText NodeDisplayName = METASOUND_LOCTEXT("SubharmonizerDisplayNamePattern", "Subharmonizer");
				const FText NodeDescription = METASOUND_LOCTEXT("SubharmonizerDescription", "Generates a subharmonic of the input audio signal.");
				TArray<const FText> NodeKeywords
					{
						METASOUND_LOCTEXT("SubharmonizerKeyword_Subharmonic", "Subharmonic"),
						METASOUND_LOCTEXT("SubharmonizerKeyword_Octaver", "Octaver"),
						METASOUND_LOCTEXT("SubharmonizerKeyword_Bass", "Bass")
				};
			
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = NodeDescription;
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetDefaultInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords.Append(NodeKeywords);

				return Info;
			};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FSubharmonizerOperator::CreateOperator(const FBuildOperatorParams& InParams,
		FBuildResults& OutResults)
	{
		using namespace SubharmonizerVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		
		FAudioBufferReadRef	AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
		FFloatReadRef	FrequencyIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputFrequency), InParams.OperatorSettings);
		FFloatReadRef	LinearGainIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputLinearGain), InParams.OperatorSettings);
		FInt32ReadRef	SubharmonicIn = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSubharmonic), InParams.OperatorSettings);
		FBoolReadRef	OctaverModeIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputOctaverMode), InParams.OperatorSettings);

		return MakeUnique<FSubharmonizerOperator>(InParams.OperatorSettings, 
			AudioIn,
			FrequencyIn,
			LinearGainIn,
			SubharmonicIn,
			OctaverModeIn
			);
	}

	FSubharmonizerOperator::FSubharmonizerOperator(const FOperatorSettings& InSettings,
		const FAudioBufferReadRef& InAudioIn,
		const FFloatReadRef& InFrequencyIn,
		const FFloatReadRef& InLinearGainIn,
		const FInt32ReadRef& InSubharmonicIn,
		const FBoolReadRef& InOctaverModeIn
		)
			: AudioIn(InAudioIn)
			, FrequencyIn(InFrequencyIn)
			, LinearGainIn(InLinearGainIn)
			, SubharmonicIn(InSubharmonicIn)
			, OctaverModeIn(InOctaverModeIn)
			, AudioOut(TDataWriteReferenceFactory<FAudioBuffer>::CreateExplicitArgs(InSettings))
	{
		NumFramesPerBlock = InSettings.GetNumFramesPerBlock();
		SampleRate = InSettings.GetSampleRate();

		InternalReset();
	}

	void FSubharmonizerOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace SubharmonizerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFrequency), FrequencyIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLinearGain), LinearGainIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSubharmonic), SubharmonicIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOctaverMode), OctaverModeIn);
}

	void FSubharmonizerOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace SubharmonizerVertexNames;

		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOut);
	}

	void FSubharmonizerOperator::UpdateParams()
	{
		// Cache latest input values
		LPFFilterFrequency = FMath::Clamp(*FrequencyIn, MinFilterFrequency, MaxFilterFrequency);
		HPFFilterFrequency = FMath::Max(MinFilterFrequency, LPFFilterFrequency * 0.5);
		InputSubharmonic = FMath::Clamp(*SubharmonicIn,MinInputSubharmonic, MaxInputSubharmonic);
		LinearGain = FMath::Max(*LinearGainIn, 0.f);

		// Update Frequency values
		if(!FMath::IsNearlyEqual(PreviouisFilterFrequency, LPFFilterFrequency))
		{
			InputLowPassFilter.SetFrequency(LPFFilterFrequency);
			OutputLowPassFilter.SetFrequency(LPFFilterFrequency / FMath::Pow(2.f, *SubharmonicIn));
			OutputHighPassFilter.SetFrequency(HPFFilterFrequency / FMath::Pow(2.f, *SubharmonicIn));

			PreviouisFilterFrequency = LPFFilterFrequency;
		}

		// Update gain value
		if(!FMath::IsNearlyEqual(LinearGain, *LinearGainIn))
		{
			LinearGain = FMath::Max(*LinearGainIn, 0.f);
		}

		// Update subharmonics and mode
		if(PreviousInputSubharmonic != InputSubharmonic || bOctaverMode != *OctaverModeIn)
		{
			bOctaverMode = *OctaverModeIn;
			Subharmonizer.Update(InputSubharmonic, bOctaverMode);

			// Adjust internal frequency values to target new spectral output range
			if(bOctaverMode)
			{
				OutputLowPassFilter.SetFrequency(LPFFilterFrequency / FMath::Pow(2.f, InputSubharmonic));
				OutputHighPassFilter.SetFrequency(HPFFilterFrequency / FMath::Pow(2.f, InputSubharmonic));			
			}
			else
			{
				OutputLowPassFilter.SetFrequency(LPFFilterFrequency / FMath::Max(InputSubharmonic, 1));
				OutputLowPassFilter.SetFrequency(LPFFilterFrequency / FMath::Max(InputSubharmonic + 1, 1));
			}
			
			// Cache for tracking
			PreviousInputSubharmonic = InputSubharmonic;
		}
	}

	void FSubharmonizerOperator::Reset(const IOperator::FResetParams& InParams)
	{
		// Cache environment
		NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();
		SampleRate = InParams.OperatorSettings.GetSampleRate();

		InternalReset();
	}

	void FSubharmonizerOperator::InternalReset()
	{
		// Max filter frequency/nyquist frequency
		MaxFilterFrequency = SampleRate * 0.5;

		// Init scratch buffers
		ScratchBufferA.Init(0.f,NumFramesPerBlock);
		ScratchBufferB.Init(0.f,NumFramesPerBlock);
		EnvelopeBuffer.Init(0.f, NumFramesPerBlock);
		GainInterpolationBuffer.Init(0.f, NumFramesPerBlock);

		// Constrain input values
		LinearGain = FMath::Max(*LinearGainIn, 0.f);

		// Cache to Previous
		PreviousLinearGain = LinearGain;
		
		// Reset processor
		Subharmonizer.Reset();

		// Clamp subharmonic input
		InputSubharmonic = FMath::Clamp(*SubharmonicIn, MinInputSubharmonic, MaxInputSubharmonic);
		PreviousInputSubharmonic = InputSubharmonic;
		
		// Cache input
		bOctaverMode = *OctaverModeIn;
		Subharmonizer.Update(InputSubharmonic, bOctaverMode);

		// Clamp input values 
		LPFFilterFrequency = FMath::Clamp(*FrequencyIn, MinFilterFrequency, MaxFilterFrequency);

		// Set HPF (low noise filter) cutoff to half that of the the output LPF smoothing filter
		HPFFilterFrequency = FMath::Max(MinFilterFrequency, LPFFilterFrequency * 0.5);
		PreviouisFilterFrequency = LPFFilterFrequency;

		// Init input (high frequency noise reduction) filter
		InputLowPassFilter.Init(SampleRate, 1, Audio::EBiquadFilter::ButterworthLowPass, LPFFilterFrequency);
		
		// Init output smoothing filter (smooths pulse wave output from subharmonizer)
		OutputLowPassFilter.Init(SampleRate, 1, Audio::EBiquadFilter::ButterworthLowPass, LPFFilterFrequency / FMath::Pow(2.f, *SubharmonicIn));

		// Init output high pass filter (reduces low frequency rumble/noise on output)
		OutputHighPassFilter.Init(SampleRate, 1, Audio::EBiquadFilter::ButterworthHighPass, HPFFilterFrequency / FMath::Pow(2.f, *SubharmonicIn));

		// Init output
		AudioOut->Zero();
		
		// Init envelope follower parameters
		const Audio::FEnvelopeFollowerInitParams EnvelopeFollowerInitParams(
			SampleRate,
			1,
			10.f,
			100.f,
			Audio::EPeakMode::Peak,
			true,
			300.f);
		
		// Init envelope follower
		EnvelopeFollower.Init(EnvelopeFollowerInitParams);
		
		// Call update
		UpdateParams();
	}

	void FSubharmonizerOperator::Execute()
	{
		UpdateParams();
		
		const int32 Frames = AudioIn->Num();

		// Calculate Envelope
		EnvelopeFollower.ProcessAudio(AudioIn->GetData(), Frames, EnvelopeBuffer.GetData());
		
		// Process Input Lowpass Filter
		InputLowPassFilter.ProcessAudio(AudioIn->GetData(), Frames, ScratchBufferA.GetData());

		// Process Subharmonizer
		Subharmonizer.Process(ScratchBufferA.GetData(), Frames, ScratchBufferB.GetData());
		
		// Process Output Smoothing Filter
		OutputLowPassFilter.ProcessAudio(ScratchBufferB.GetData(), Frames, ScratchBufferA.GetData());

		// Process Highpass Filter
		OutputHighPassFilter.ProcessAudio(ScratchBufferA.GetData(), Frames, ScratchBufferB.GetData());

		// Apply Envelope
		Audio::ArrayMultiply(ScratchBufferB, EnvelopeBuffer, ScratchBufferA);

		// Apply Linear Gain Interpolation
		Audio::ArrayFade(ScratchBufferA, PreviousLinearGain, LinearGain, *AudioOut);

		// Update previous gain
		PreviousLinearGain = LinearGain;
	}

	// Node registration
	using FSubharmonizerNode = TNodeFacade<FSubharmonizerOperator>;
	METASOUND_REGISTER_NODE(FSubharmonizerNode);
} // namespace Metasound

#undef LOCTEXT_NAMESPACE