// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDefines.h"
#include "Containers/Union.h"
#include "Misc/TVariant.h"
#include "DSP/Dsp.h"
#include "DSP/DynamicsProcessor.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOscillators.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_SuperOscillatorNode"

namespace Metasound
{			
	namespace Generators
	{
		// Standard params passed to the Generate Block templates below
		struct FVoiceGeneratorArgs
		{
			float SampleRate = 0.f;
			float FrequencyHz = 0.f;
			float DetuneMultiplier = 0.f;
			float Entropy = 0.f;
			float GlideEaseFactor = 0.0f;
			float PulseWidth = 0.f;
			float BlendGain = 0.f;
			float StereoWidth = 0.f;
			TArrayView<float> AlignedBuffer;
			TArrayView<const float> Fm;
		};

		struct FVoiceGeneratorInitArgs
		{
			const float SampleRate = 0.f;
			const float Phase = 0.f;
			const float DetuneRatio = 0.f;
			const float PanRatio = 0.f;
			const float Entropy = 0.f;
		};
		
		// wraps a TGenerateBlock with extra members it needs to be an indiviual voice with its own pan/phase/pitch
		template<typename GeneratorType>
		struct TVoiceGenerator
		{
			// [-1, 1] - how strongly this voice should be detuned relative to the current max detune amount
			float DetuneRatio = 0.f; 

			// [-1, 1] - where this voice should be panned to if at full stereo width
			float PanRatio = 0.f; 

			// [-1, 1] - where on either side of our target values we should shift if at full entropy
			float EntropyTarget = 0.f;
			
			GeneratorType Generator;
			
			TVoiceGenerator() = default;
			
			explicit TVoiceGenerator(const FVoiceGeneratorInitArgs& InitArgs)
				: DetuneRatio(InitArgs.DetuneRatio)
				, PanRatio(InitArgs.PanRatio)
				, EntropyTarget(InitArgs.Entropy)
			{
				Generator.Phase = InitArgs.Phase;
			}

			void Process(const FVoiceGeneratorArgs& InArgs)
			{
				const float Entropy = 1.f + InArgs.Entropy * EntropyTarget;
				const float FreqOffset = FMath::Pow(FMath::Pow(InArgs.DetuneMultiplier, Entropy), DetuneRatio);
				const float TargetFreq = InArgs.FrequencyHz * FreqOffset;
				
				FGeneratorArgs Args =
				{
					InArgs.SampleRate,
					TargetFreq,
					InArgs.GlideEaseFactor,
					InArgs.PulseWidth,
					true,
					InArgs.AlignedBuffer,
					InArgs.Fm
				};
				
				Generator(Args);
			}
		};
			
		using FSuperOsc	= TVariant<TVoiceGenerator<FSineWaveTable>, TVoiceGenerator<FSineWaveTableWithFm>,
						TVoiceGenerator<FTrianglePolysmooth>,TVoiceGenerator<FTrianglePolysmoothWithFm>,
						TVoiceGenerator<FSquarePolysmooth>, TVoiceGenerator<FSquarePolysmoothWithFm>,
						TVoiceGenerator<FSawPolysmooth>, TVoiceGenerator<FSawPolysmoothWithFm>>;
	
		#define MAX_SUPEROSCILLATOR_VOICES 16
		using FGeneratorArray = TArray<FSuperOsc, TInlineAllocator<MAX_SUPEROSCILLATOR_VOICES>>;

		template<typename OscillatorType>
		static void FillGeneratorsArray(FGeneratorArray& GeneratorArray, const int32 NumVoicesToInit, const float SampleRate)
		{
			GeneratorArray.Reset(NumVoicesToInit);

			TArray<FVoiceGeneratorInitArgs, TInlineAllocator<MAX_SUPEROSCILLATOR_VOICES>> VoiceArgs;

			VoiceArgs.Add({ SampleRate });

			// alternate +/- offset values starting outside from 1 and working in
			const float NumOffSetsNeeded = FMath::CeilToInt32((float)NumVoicesToInit * 0.5f);
			for (int32 OffsetIdx = 0; OffsetIdx < NumOffSetsNeeded; OffsetIdx++)
			{
				const float Offset = (NumOffSetsNeeded - OffsetIdx) / NumOffSetsNeeded;

				// flip pan every other voice to more evenly distribute +/- detunes to the left and right
				const float PanSign = (OffsetIdx % 2 == 0) ? 1.f : -1.f;

				FVoiceGeneratorInitArgs PosArgs =
				{
					SampleRate,
					Offset,
					Offset,
					Offset * PanSign,
					FMath::FRandRange(-1.f, 1.f)
				};

				VoiceArgs.Add(PosArgs);

				FVoiceGeneratorInitArgs NegArgs =
				{
					SampleRate,
					Offset + 0.5f,
					-Offset,
					-Offset * PanSign,
					FMath::FRandRange(-1.f, 1.f)
				};
				
				VoiceArgs.Add(NegArgs);
			}

			check(VoiceArgs.Num() >= NumVoicesToInit);
			
			for (int32 VoiceIdx = 0; VoiceIdx < NumVoicesToInit; VoiceIdx++)
			{
				FSuperOsc Osc = FSuperOsc(TInPlaceType<OscillatorType>(), VoiceArgs[VoiceIdx]);
				
				GeneratorArray.Add(Osc); 
			}
		}
	}

	namespace SuperOscillatorVertexNames
	{
		METASOUND_PARAM(AudioOutPin, "Audio", "The output audio")
		METASOUND_PARAM(AudioLeftPin, "Left", "Left channel output audio")
		METASOUND_PARAM(AudioRightPin, "Right", "Right channel output audio")

		METASOUND_PARAM(BaseFrequencyPin, "Frequency", "Base Frequency of Oscillator in Hz.")
		METASOUND_PARAM(DetuneVolumePin, "Blend", "Volume in decibels of detuned voices, relative to the primary voice")
		METASOUND_PARAM(EnabledPin, "Enabled", "Enable the oscillator.")
		METASOUND_PARAM(LimitOutputPin, "Limit Output", "Enables an internal limiter to keep output volume in check.")
		METASOUND_PARAM(EntropyPin, "Entropy", "[0,1] Controls how evenly the voices are distributed in pitch")
		METASOUND_PARAM(FrequencyModPin, "Modulation","Modulation Frequency Input (for doing FM)")
		METASOUND_PARAM(GlideFactorPin, "Glide", "The amount of glide to use when changing frequencies. 0.0 = no glide, 1.0 = lots of glide.")
		METASOUND_PARAM(MaxDetunePin, "Detune", "Max pitch offset of any Oscillators in Semitones, up to four octaves. Only oscillators 2+ are detuned")
		METASOUND_PARAM(NumVoicesPin, "Voices", "[1,16] The number of Oscillators")
		METASOUND_PARAM(PulseWidthPin, "Pulse Width", "The Width of the square part of the wave. Only used for square waves.")
		METASOUND_PARAM(StereoWidthPin, "Width", "[0,1] Stereo Width of the oscillators")
		METASOUND_PARAM(WaveTypePin, "Type", "Shape of the Oscillator")
	}

	// Common set of construct params for each of the Oscillators.
	struct FSuperOscillatorOperatorConstructParams
	{
		const FOperatorSettings& Settings;
		FBoolReadRef Enabled;
		FBoolReadRef Limiter;
		int32 NumVoices;
		FFloatReadRef BaseFrequency;
		FFloatReadRef DetunedVoiceVolumeDb;
		FFloatReadRef MaxDetuneSemitones;
		FFloatReadRef Entropy;
		FFloatReadRef GlideFactor;
		FFloatReadRef PulseWidth;
		bool bUseModulation;
		FAudioBufferReadRef Fm;
		FEnumLfoWaveshapeType OscType;
	};
	
	class FSuperOscillatorOperator : public TExecutableOperator<FSuperOscillatorOperator>
	{		
	public:
		FSuperOscillatorOperator(const FSuperOscillatorOperatorConstructParams& InConstructParams)
			: SampleRate(InConstructParams.Settings.GetSampleRate())
			, Nyquist(InConstructParams.Settings.GetSampleRate() / 2.0f)
			, bEnabled(InConstructParams.Enabled)
			, bLimit(InConstructParams.Limiter)
			, NumVoices(FMath::Clamp(InConstructParams.NumVoices, 1, 16))
			, BaseFrequency(InConstructParams.BaseFrequency)
			, MaxDetune(InConstructParams.MaxDetuneSemitones)
			, Entropy(InConstructParams.Entropy)
			, DetuneDb(InConstructParams.DetunedVoiceVolumeDb)
			, GlideFactor(InConstructParams.GlideFactor)
			, PulseWidth(InConstructParams.PulseWidth)
			, bUseModulation(InConstructParams.bUseModulation)
			, Fm(InConstructParams.Fm)
			, OscType(InConstructParams.OscType)
		{

			ScratchBuffer.SetNumZeroed(InConstructParams.Settings.GetNumFramesPerBlock());
			
			InitializeGenerators();
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override 
		{
			using namespace SuperOscillatorVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(EnabledPin), bEnabled);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(LimitOutputPin), bLimit);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(BaseFrequencyPin), BaseFrequency);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(NumVoicesPin), NumVoices);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(MaxDetunePin), MaxDetune);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(DetuneVolumePin), DetuneDb);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(GlideFactorPin), GlideFactor);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(PulseWidthPin), PulseWidth);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(FrequencyModPin), Fm);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(EntropyPin), Entropy);
			InOutVertexData.SetValue(METASOUND_GET_PARAM_NAME(WaveTypePin), OscType);
		}
		
		virtual void FillOutputs(const float Gain = 1.f, const float LinearPan = 0.f) {}
		virtual void ZeroOutputs() {}
		virtual float GetStereoWidth() const { return 0.f; }
		virtual void ResetLimiters() = 0;
		
		void SetLimiterSettings(Audio::FDynamicsProcessor& InLimiter) const
		{
			InLimiter.Init(SampleRate, 1);
			InLimiter.SetLookaheadMsec(5.0f);
			InLimiter.SetAttackTime(20.0f);
			InLimiter.SetReleaseTime(200.0f);
			InLimiter.SetThreshold(-6.0f);
			InLimiter.SetRatio(10.f);
			InLimiter.SetOutputGain(0.0f);
			InLimiter.SetAnalogMode(true);
			InLimiter.SetPeakMode(Audio::EPeakMode::Peak);
			InLimiter.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
		}

		virtual void LimitOutput() {}
		
		void Execute()
		{
			using namespace Generators;
			
			if (Generators.IsEmpty())
			{
				return;
			}
			
			const float ClampedFreq = FMath::Clamp(*BaseFrequency, -Nyquist, Nyquist);
			const float ClampedEntropy = FMath::Clamp(*Entropy, 0.f, 1.f);
			const float ClampedGlideEase = Audio::GetLogFrequencyClamped(*GlideFactor, { 0.0f, 1.0f }, { 1.0f, 0.0001f });

			// allow +- 4 octaves of detune
			constexpr float DetuneClamp = 12.f * 4.f;
			const float DetuneRatio = Audio::GetFrequencyMultiplier(FMath::Clamp(*MaxDetune, -DetuneClamp, DetuneClamp));
			const float DetuneChannelGain = Audio::ConvertToLinear(FMath::Clamp(*DetuneDb, MIN_VOLUME_DECIBELS, 24.f));
			const float ClampedPulsueWidth = FMath::Clamp(*PulseWidth, 0.01f, 0.99f);
			
			const int32 NumFrames = ScratchBuffer.Num();
			ScratchBuffer.SetNumZeroed(ScratchBuffer.Num());

			ZeroOutputs();
			
			if (*bEnabled && NumFrames > 0)
			{
				const FVoiceGeneratorArgs Args = 
				{
					SampleRate,
					ClampedFreq,
					DetuneRatio,
					ClampedEntropy,
					ClampedGlideEase,
					ClampedPulsueWidth,
					DetuneChannelGain,
					GetStereoWidth(),
					MakeArrayView(ScratchBuffer.GetData(), NumFrames),
					MakeArrayView(Fm->GetData(), NumFrames)
				};

				const auto GenerateFunc = [&Args](auto& Osc) -> float
				{
					Osc.Process(Args);
					return Osc.PanRatio;
				};

				bool bFirstVoiceProcessed = false;

				for (FSuperOsc& Generator : Generators)
				{
					const float Gain = bFirstVoiceProcessed ? Args.BlendGain : 1.f;
					const float PanRatio = Visit(GenerateFunc, Generator);

					FillOutputs(Gain, PanRatio * Args.StereoWidth);

					bFirstVoiceProcessed = true;
				}
				
				if (*bLimit)
				{
					LimitOutput();
				}
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			ZeroOutputs();
			ResetLimiters();

			InitializeGenerators();
		}

	protected:

		void InitializeGenerators()
		{
			using namespace Generators;
			// init generators
			if (bUseModulation)
			{
				// fm generators
				switch (OscType)
				{
				case ELfoWaveshapeType::Sine:
					FillGeneratorsArray<TVoiceGenerator<FSineWaveTableWithFm>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Triangle:
					FillGeneratorsArray<TVoiceGenerator<FTrianglePolysmoothWithFm>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Square:
					FillGeneratorsArray<TVoiceGenerator<FSquarePolysmoothWithFm>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Saw:
					FillGeneratorsArray<TVoiceGenerator<FSawPolysmoothWithFm>>(Generators, NumVoices, SampleRate);
					break;
				default: 
					check(!"Unsupported Wave Shape in FSuperOscillatorOperator");
					return;
				}
			}
			else
			{
				//non-fm generators
				switch (OscType)
				{
				case ELfoWaveshapeType::Sine:
					FillGeneratorsArray<TVoiceGenerator<FSineWaveTable>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Triangle:
					FillGeneratorsArray<TVoiceGenerator<FTrianglePolysmooth>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Square:
					FillGeneratorsArray<TVoiceGenerator<FSquarePolysmooth>>(Generators, NumVoices, SampleRate);
					break;
				case ELfoWaveshapeType::Saw:
					FillGeneratorsArray<TVoiceGenerator<FSawPolysmooth>>(Generators, NumVoices, SampleRate);
					break;
				default: 
					check(!"Unsupported Wave Shape in FSuperOscillatorOperator");
					return;
				}
			}
		}

		float SampleRate = 1.f;
		float Nyquist = 1.f;

		// apply this gain whether we are limiting or not, to avoid large volume jumps when toggling the limiter
		static constexpr float PreLimiterGain = 0.707f;

		FBoolReadRef bEnabled;
		FBoolReadRef bLimit;
		int32 NumVoices = 0;
		FFloatReadRef BaseFrequency;
		FFloatReadRef MaxDetune;
		FFloatReadRef Entropy;
		FFloatReadRef DetuneDb;
		FFloatReadRef GlideFactor;
		FFloatReadRef PulseWidth;

		bool bUseModulation = false;
		FAudioBufferReadRef Fm;
		FEnumLfoWaveshapeType OscType;

		Audio::FAlignedFloatBuffer ScratchBuffer;

		TArray<Generators::FSuperOsc, TInlineAllocator<MAX_SUPEROSCILLATOR_VOICES>> Generators;
	};

	class FSuperOscillatorOperatorMono : public FSuperOscillatorOperator
	{
		using Super = FSuperOscillatorOperator;
	public:
		FSuperOscillatorOperatorMono(const FSuperOscillatorOperatorConstructParams& InConstructParams)
			: FSuperOscillatorOperator(InConstructParams)
			, AudioBuffer(FAudioBufferWriteRef::CreateNew(InConstructParams.Settings))
		{
			SetLimiterSettings(Limiter);
		}
		
		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { StandardNodes::Namespace, TEXT("SuperOscillatorMono"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_SuperOscillatorMonoNodeDisplayName", "SuperOscillator (Mono)");
				Info.Description = METASOUND_LOCTEXT("Metasound_SuperOscillatorNodeDescription", "Generates audio using multiple internal oscillators.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("OscKeyword", "Osc"), METASOUND_LOCTEXT("FMKeyword", "FM"), METASOUND_LOCTEXT("SynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SuperOscillatorVertexNames; 

			static const FVertexInterface Interface
			{
				FInputVertexInterface{
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(EnabledPin), true),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(LimitOutputPin), true),
					TInputConstructorVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(NumVoicesPin), 3),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(BaseFrequencyPin), 440.f),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(FrequencyModPin)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MaxDetunePin), -0.25f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(EntropyPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(DetuneVolumePin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(GlideFactorPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PulseWidthPin), 0.5f),
					TInputConstructorVertex<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTypePin))
				},
				FOutputVertexInterface{
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(AudioOutPin))
				}
			};
			
			return Interface;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			const FOperatorSettings& Settings = InParams.OperatorSettings;

			using namespace SuperOscillatorVertexNames; 
			FSuperOscillatorOperatorConstructParams OpParams
			{
				Settings,
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(LimitOutputPin), Settings),
				InputData.GetOrCreateDefaultValue<int32>(METASOUND_GET_PARAM_NAME(NumVoicesPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(BaseFrequencyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(DetuneVolumePin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MaxDetunePin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(EntropyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PulseWidthPin), Settings),
				InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin)),
				InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin), Settings),
				InputData.GetOrCreateDefaultValue<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME(WaveTypePin), Settings)
			};
			
			return MakeUnique<FSuperOscillatorOperatorMono>(OpParams);
		}

		virtual void ZeroOutputs() override
		{
			AudioBuffer->Zero();
		}
		
		virtual void FillOutputs(const float Gain = 1.f, const float LinearPan = 0.f) final override
		{
			Audio::ArrayMixIn(ScratchBuffer, *AudioBuffer, Gain * PreLimiterGain);
		}

		virtual void LimitOutput() override
		{
			Limiter.ProcessAudio(AudioBuffer->GetData(), AudioBuffer->Num(), AudioBuffer->GetData());
		}

		virtual void ResetLimiters() override
		{
			SetLimiterSettings(Limiter);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			Super::BindInputs(InOutVertexData);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SuperOscillatorVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AudioOutPin), AudioBuffer);
		}
		
		FAudioBufferWriteRef AudioBuffer;
		
		Audio::FDynamicsProcessor Limiter;
	};
	
	class FSuperOscillatorOperatorStereo : public FSuperOscillatorOperator
	{
		using Super = FSuperOscillatorOperator;
	public:
		FSuperOscillatorOperatorStereo(const FSuperOscillatorOperatorConstructParams& InConstructParams, FFloatReadRef InStereoWidth)
			: FSuperOscillatorOperator(InConstructParams)
			, StereoWidth(InStereoWidth)
			, AudioLeft(FAudioBufferWriteRef::CreateNew(InConstructParams.Settings))
			, AudioRight(FAudioBufferWriteRef::CreateNew(InConstructParams.Settings))
		{
			SetLimiterSettings(LimiterLeft);
			SetLimiterSettings(LimiterRight);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info = FNodeClassMetadata::GetEmpty();
				Info.ClassName = { StandardNodes::Namespace, TEXT("SuperOscillatorStereo"), StandardNodes::AudioVariant };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = METASOUND_LOCTEXT("Metasound_SuperOscillatorNodeStereoDisplayName", "SuperOscillator (Stereo)");
				Info.Description = METASOUND_LOCTEXT("Metasound_SuperOscillatorNodeDescription", "Generates audio using multiple internal oscillators.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Generators);
				Info.Keywords = { METASOUND_LOCTEXT("OscKeyword", "Osc"), METASOUND_LOCTEXT("FMKeyword", "FM"), METASOUND_LOCTEXT("SynthesisKeyword", "Synthesis") };
				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SuperOscillatorVertexNames; 

			static const FVertexInterface Interface
			{
				FInputVertexInterface{
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(EnabledPin), true),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(LimitOutputPin), true),
					TInputConstructorVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(NumVoicesPin), 3),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(BaseFrequencyPin), 440.f),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(FrequencyModPin)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(MaxDetunePin), -0.25f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(EntropyPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(DetuneVolumePin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(GlideFactorPin), 0.f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(PulseWidthPin), 0.5f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(StereoWidthPin), 0.5f),
					TInputConstructorVertex<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME_AND_METADATA(WaveTypePin))
				},
				FOutputVertexInterface{
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(AudioLeftPin)),
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(AudioRightPin))
				}
			};
			
			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			const FOperatorSettings& Settings = InParams.OperatorSettings;

			using namespace SuperOscillatorVertexNames; 
			FSuperOscillatorOperatorConstructParams OpParams
			{
				Settings,
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(EnabledPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(LimitOutputPin), Settings), 
				InputData.GetOrCreateDefaultValue<int32>(METASOUND_GET_PARAM_NAME(NumVoicesPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(BaseFrequencyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(DetuneVolumePin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(MaxDetunePin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(EntropyPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(GlideFactorPin), Settings),
				InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(PulseWidthPin), Settings),
				InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(FrequencyModPin)),
				InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(FrequencyModPin), Settings),
				InputData.GetOrCreateDefaultValue<FEnumLfoWaveshapeType>(METASOUND_GET_PARAM_NAME(WaveTypePin), Settings)
			};

			FFloatReadRef StereoWidthInput = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(StereoWidthPin), Settings);
			
			return MakeUnique<FSuperOscillatorOperatorStereo>(OpParams, StereoWidthInput);
		}

		virtual void ZeroOutputs() override
		{
			AudioLeft->Zero();
			AudioRight->Zero();
		}

		virtual float GetStereoWidth() const override { return FMath::Clamp(*StereoWidth, 0.f, 1.f); }
		
		virtual void FillOutputs(const float Gain = 1.f, const float LinearPan = 0.f) final override
		{
			float PanLeftGain;
			float PanRightGain;
			
			Audio::GetStereoPan(LinearPan, PanLeftGain, PanRightGain);

			Audio::ArrayMixIn(ScratchBuffer, *AudioLeft, Gain * PanLeftGain * PreLimiterGain);
			Audio::ArrayMixIn(ScratchBuffer, *AudioRight, Gain * PanRightGain * PreLimiterGain);
		}

		virtual void ResetLimiters() override
		{
			SetLimiterSettings(LimiterLeft);
			SetLimiterSettings(LimiterRight);
		}

		virtual void LimitOutput() override
		{
			LimiterLeft.ProcessAudio(AudioLeft->GetData(), AudioLeft->Num(), AudioLeft->GetData());
			LimiterRight.ProcessAudio(AudioRight->GetData(), AudioRight->Num(), AudioRight->GetData());
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			Super::BindInputs(InOutVertexData);
			
			using namespace SuperOscillatorVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(StereoWidthPin), StereoWidth);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SuperOscillatorVertexNames;
			
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AudioLeftPin), AudioLeft);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AudioRightPin), AudioRight);
		}

		FFloatReadRef StereoWidth;

		FAudioBufferWriteRef AudioLeft;
		FAudioBufferWriteRef AudioRight;

		Audio::FDynamicsProcessor LimiterLeft;
		Audio::FDynamicsProcessor LimiterRight;
	};
	
	using FSuperOscillatorNodeMono = TNodeFacade<FSuperOscillatorOperatorMono>;

	using FSuperOscillatorNodeStereo = TNodeFacade<FSuperOscillatorOperatorStereo>;

	METASOUND_REGISTER_NODE(FSuperOscillatorNodeMono);
	METASOUND_REGISTER_NODE(FSuperOscillatorNodeStereo);
} // namespace Metasound

#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes
