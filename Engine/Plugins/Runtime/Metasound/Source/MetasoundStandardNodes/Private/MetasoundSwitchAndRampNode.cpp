// Copyright Epic Games, Inc. All Rights Reserved.
// Background on switch-and-ramp: http://msp.ucsd.edu/techniques/v0.11/book-html/node63.html

#include "DSP/BufferVectorOperations.h"
#include "Internationalization/Text.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_SwitchAndRampNode"

namespace Metasound
{
	namespace SwitchAndRampVertexNames
	{
		METASOUND_PARAM(InputTrigger, "Trigger", "Trigger at the moment of the discontinuity that should be smoothed.");
		METASOUND_PARAM(InputAudio, "In Audio", "The audio input.");
		METASOUND_PARAM(InputSmoothTime, "Smooth Time", "The amount of time to smooth the discontinuity over.");

		METASOUND_PARAM(OutputAudio, "Out Audio", "Output audio with the discontinuity smoothed.");
	}

	class FSwitchAndRampOperator : public TExecutableOperator<FSwitchAndRampOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				const FName OperatorName = TEXT("SwitchAndRamp");
				const FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_SwitchAndRampNodeDisplayName", "Switch And Ramp");
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_SwitchAndRampNodeDescription", "Uses the switch-and-ramp technique to smooth out pops in audio, when triggered at the moment that a pop would occur.");

				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = NodeDescription;
				Info.Author = PluginAuthor;
				Info.CategoryHierarchy = { NodeCategories::Filters };
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetVertexInterface();

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static const FVertexInterface& GetVertexInterface()
		{
			using namespace SwitchAndRampVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)),
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSmoothTime), 0.01f)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputAudio))
				)
			);

			return Interface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace SwitchAndRampVertexNames;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
			FAudioBufferReadRef AudioInput = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			FTimeReadRef SmoothTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputSmoothTime), InParams.OperatorSettings);

			return MakeUnique<FSwitchAndRampOperator>(InParams.OperatorSettings, TriggerIn, AudioInput, SmoothTime);
		}

		FSwitchAndRampOperator(const FOperatorSettings& InSettings,
			const FTriggerReadRef& InTriggerIn,
			const FAudioBufferReadRef& InAudioInput,
			const FTimeReadRef& InputSmoothTime)

			: TriggerIn(InTriggerIn)
			, AudioIn(InAudioInput)
			, SmoothTime(InputSmoothTime)
			, AudioOut(FAudioBufferWriteRef::CreateNew(InSettings))
			, SampleRate(InSettings.GetSampleRate())
		{
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SwitchAndRampVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio), AudioIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSmoothTime), SmoothTime);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace SwitchAndRampVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputAudio), AudioOut);
		}

		void Execute()
		{
			// Lambda is used for both pre and post trigger frames
			auto RunRampLambda = [this](int32 StartFrame, int32 EndFrame)
			{
				const float* InputAudio = AudioIn->GetData();
				float* OutputAudio = AudioOut->GetData();

				//apply the ramp offset, if it's still running
				int32 RampSamplesLeft = NumRampSamples - RampSampleIndex;
				int32 LastRampFrame = FMath::Min(StartFrame + RampSamplesLeft, EndFrame);
				for (int32 i = StartFrame; i < LastRampFrame; ++i)
				{
					float Adjustment = 0;
					if (RampSampleIndex < NumRampSamples)
					{
						float RampFraction = (float)RampSampleIndex / NumRampSamples;
						Adjustment = (1 - RampFraction) * DiscontinuityAmount;
						++RampSampleIndex;
					}
					OutputAudio[i] = InputAudio[i] + Adjustment;
				}

				//copy the rest of the audio as passthrough
				int32 StartCopyFrame = FMath::Max(StartFrame, LastRampFrame);
				if (StartCopyFrame < EndFrame)
				{
					FMemory::Memcpy(&OutputAudio[StartCopyFrame], &InputAudio[StartCopyFrame], sizeof(float) * (EndFrame - StartCopyFrame));
				}

				MostRecentOutputValue = OutputAudio[EndFrame - 1];
			};

			TriggerIn->ExecuteBlock(
				RunRampLambda,
				// OnTrigger
				[this, &RunRampLambda](int32 StartFrame, int32 EndFrame)
				{
					// Only run ramp if there are samples to render out. Multiple triggers
					// on a single frame result empty frame ranges (e.g. StartFrame == EndFrame)
					// and the underlying DSP expects at least 1 renderable audio frame.
					if (EndFrame > StartFrame)
					{
						float RampFromValue = MostRecentOutputValue;
						float RampToValue = AudioIn->GetData()[StartFrame];
						DiscontinuityAmount = RampFromValue - RampToValue;

						float SmoothTimeSeconds = SmoothTime->GetSeconds();
						NumRampSamples = FMath::Max(1, SampleRate * SmoothTimeSeconds);
						RampSampleIndex = 0;

						RunRampLambda(StartFrame, EndFrame);
					}
				}
			);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			AudioOut->Zero();
			MostRecentOutputValue = 0.0f;
			DiscontinuityAmount = 0.0f;
			NumRampSamples = 0;
			RampSampleIndex = 0;
		}

	private:
		FTriggerReadRef TriggerIn;
		
		// The input audio buffer
		FAudioBufferReadRef AudioIn;
		
		FTimeReadRef SmoothTime;

		// Audio output
		FAudioBufferWriteRef AudioOut;

		float SampleRate = 0.0f;
		float MostRecentOutputValue = 0.0f;
		float DiscontinuityAmount = 0.0f;
		int32 NumRampSamples = 0;
		int32 RampSampleIndex = 0;
	};

	using FSwitchAndRampNode = TNodeFacade<FSwitchAndRampOperator>;

	METASOUND_REGISTER_NODE(FSwitchAndRampNode)
}

#undef LOCTEXT_NAMESPACE
