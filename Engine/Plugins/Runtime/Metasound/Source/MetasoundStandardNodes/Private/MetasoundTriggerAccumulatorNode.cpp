// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundTriggerAccumulatorNode.h"

#include "Internationalization/Text.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundVertex.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerAccumulator"

#define REGISTER_TRIGGER_ACCUMULATOR_NODE(Number) \
	using FTriggerAccumulatorNode_##Number = TTriggerAccumulatorNode<Number>; \
	METASOUND_REGISTER_NODE(FTriggerAccumulatorNode_##Number) \

namespace Metasound
{
	namespace MetasoundTriggerAccumulatorNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "TriggerAccumulator", InOperatorName, FName() },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Trigger },
				{ },
				FNodeDisplayStyle()
			};

			return Metadata;
		}
	}

	REGISTER_TRIGGER_ACCUMULATOR_NODE(1)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(2)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(3)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(4)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(5)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(6)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(7)
	REGISTER_TRIGGER_ACCUMULATOR_NODE(8)
}

#undef LOCTEXT_NAMESPACE
