// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundParamHelper.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerAny"

#define REGISTER_TRIGGER_ANY_NODE(Number) \
	using FTriggerAnyNode_##Number = TTriggerAnyNode<Number>; \
	METASOUND_REGISTER_NODE(FTriggerAnyNode_##Number) \


namespace Metasound
{
	namespace MetasoundTriggerAnyNodePrivate
	{
		METASOUNDSTANDARDNODES_API FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "TriggerAny", InOperatorName, FName() },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Trigger },
				{ },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}
	}

	namespace TriggerAnyVertexNames
	{
		METASOUND_PARAM(OutputTrigger, "Out", "Triggered when any of the input triggers have been triggered. ")

		METASOUNDSTANDARDNODES_API const FVertexName GetInputTriggerName(uint32 InIndex)
		{
			return *FString::Format(TEXT("In {0}"), { InIndex });
		}

		METASOUNDSTANDARDNODES_API const FText GetInputTriggerDescription(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("TriggerAnyInputTriggerDesc", "Trigger {0} input. The output trigger is hit when any of the input triggers are hit.", InIndex);
		}

		METASOUNDSTANDARDNODES_API const FText GetInputTriggerDisplayName(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("TriggerAnyInputTriggerDisplayName", "In {0}", InIndex);
		}
	}

	template<uint32 NumInputs>
	class TTriggerAnyOperator : public TExecutableOperator<TTriggerAnyOperator<NumInputs>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace TriggerAnyVertexNames;

			auto CreateDefaultInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;

				for (uint32 i = 0; i < NumInputs; ++i)
				{
					const FDataVertexMetadata InputTriggerMetadata
					{
						  GetInputTriggerDescription(i) // description
						, GetInputTriggerDisplayName(i) // display name
					};

					InputInterface.Add(TInputDataVertex<FTrigger>(GetInputTriggerName(i), InputTriggerMetadata));
				}

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTrigger)));

				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Trigger Any (%d)"), NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerAnyDisplayNamePattern", "Trigger Any ({0})", NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("TriggerAnyDescription", "Will trigger output on any of the input triggers.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundTriggerAnyNodePrivate::CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace TriggerAnyVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TArray<FTriggerReadRef> InputTriggers;

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers.Add(InputData.GetOrCreateDefaultDataReadReference<FTrigger>(GetInputTriggerName(i), InParams.OperatorSettings));
			}

			return MakeUnique<TTriggerAnyOperator<NumInputs>>(InParams.OperatorSettings, MoveTemp(InputTriggers));
		}

		TTriggerAnyOperator(const FOperatorSettings& InSettings, const TArray<FTriggerReadRef>&& InInputTriggers)
			: InputTriggers(InInputTriggers)
			, OutputTrigger(FTriggerWriteRef::CreateNew(InSettings))
		{
		}

		virtual ~TTriggerAnyOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerAnyVertexNames;
			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InOutVertexData.BindReadVertex(GetInputTriggerName(i), InputTriggers[i]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerAnyVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTrigger), OutputTrigger);
		}

		virtual FDataReferenceCollection GetInputs() const override
		{
			// This should never be called. Bind(...) is called instead. This method
			// exists as a stop-gap until the API can be deprecated and removed.
			checkNoEntry();
			return {};
		}

		virtual FDataReferenceCollection GetOutputs() const override
		{
			// This should never be called. Bind(...) is called instead. This method
			// exists as a stop-gap until the API can be deprecated and removed.
			checkNoEntry();
			return {};
		}

		void Execute()
		{
			OutputTrigger->AdvanceBlock();

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers[i]->ExecuteBlock(
					[&](int32 StartFrame, int32 EndFrame)
					{
					},
					[this, i](int32 StartFrame, int32 EndFrame)
					{
						OutputTrigger->TriggerFrame(StartFrame);
					}
					);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			OutputTrigger->Reset();
		}

	private:

		TArray<FTriggerReadRef> InputTriggers;
		FTriggerWriteRef OutputTrigger;
	};

	/** TTriggerAnyNode
	*
	*  Will output a trigger whenever any of its input triggers are set.
	*/
	template<uint32 NumInputs>
	using TTriggerAnyNode = TNodeFacade<TTriggerAnyOperator<NumInputs>>;

	REGISTER_TRIGGER_ANY_NODE(2)
	REGISTER_TRIGGER_ANY_NODE(3)
	REGISTER_TRIGGER_ANY_NODE(4)
	REGISTER_TRIGGER_ANY_NODE(5)
	REGISTER_TRIGGER_ANY_NODE(6)
	REGISTER_TRIGGER_ANY_NODE(7)
	REGISTER_TRIGGER_ANY_NODE(8)
}

#undef LOCTEXT_NAMESPACE
