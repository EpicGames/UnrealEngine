// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerCoinNode"

namespace Metasound
{

	namespace TriggerCoinVertexNames
	{
		METASOUND_PARAM(InputTrigger, "Trigger", "Input trigger.");
		METASOUND_PARAM(InputReset, "Reset", "Trigger to reset the random sequence with the supplied seed. Useful to get randomized repetition.");
		METASOUND_PARAM(InputSeed, "Seed", "The seed value to use for the random node. Set to -1 to use a random seed.");
		METASOUND_PARAM(InputProbability, "Probability", "Probability that the output trigger will get called. 1.0 is always heads, 0.0 is always tails, and 0.5 is a 50% chance for either.");
		METASOUND_PARAM(OutputTrueTrigger, "Heads", "The first possible output trigger, chosen with Probability chance.");
		METASOUND_PARAM(OutputFalseTrigger, "Tails", "The second possible output trigger, chosen with (1-Probability) chance).");
	}

	class FTriggerCoinOperator : public TExecutableOperator<FTriggerCoinOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerCoinOperator(const FOperatorSettings& InSettings, 
			const FTriggerReadRef& InTrigger, 
			const FTriggerReadRef& InTriggerReset, 
			const FInt32ReadRef& InSeed, 
			const FFloatReadRef& InProbability);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// Generate new random number
		FTriggerReadRef TriggerInput;
		// Reset seed
		FTriggerReadRef TriggerResetInput;
		// RNG Seed
		FInt32ReadRef SeedInput;
		
		// Probability that output is triggered on TriggerInput
		FFloatReadRef ProbabilityInput;
		// Trigger output
		FTriggerWriteRef TriggerTrueOutput;
		FTriggerWriteRef TriggerFalseOutput;

		// Random number generator
		FRandomStream RandomStream;
		// Internal bools to keep track of seed changes
		bool bIsDefaultSeeded = false;
		bool bIsRandomStreamInitialized = false;

		// Handle situations where the seed is changed
		void EvaluateSeedChanges();

	};

	FTriggerCoinOperator::FTriggerCoinOperator(const FOperatorSettings& InSettings,
		const FTriggerReadRef& InTrigger,
		const FTriggerReadRef& InTriggerReset,
		const FInt32ReadRef& InSeed,
		const FFloatReadRef& InProbability)
		: TriggerInput(InTrigger)
		, TriggerResetInput(InTriggerReset)
		, SeedInput(InSeed)
		, ProbabilityInput(InProbability)
		, TriggerTrueOutput(FTriggerWriteRef::CreateNew(InSettings))
		, TriggerFalseOutput(FTriggerWriteRef::CreateNew(InSettings))
		, bIsDefaultSeeded(*SeedInput == -1)
		, bIsRandomStreamInitialized(false)
	{
		EvaluateSeedChanges();
		RandomStream.Reset();
	}


	void FTriggerCoinOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerCoinVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), TriggerResetInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSeed), SeedInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputProbability), ProbabilityInput);
	}

	void FTriggerCoinOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerCoinVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTrueTrigger), TriggerTrueOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputFalseTrigger), TriggerFalseOutput);
	}

	void FTriggerCoinOperator::Execute()
	{
		TriggerTrueOutput->AdvanceBlock();
		TriggerFalseOutput->AdvanceBlock();
		
		// Reset the seed
		TriggerResetInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				EvaluateSeedChanges();
				RandomStream.Reset();
			}
		);

		// Generate new random number
		TriggerInput->ExecuteBlock(
			[](int32, int32) 
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				// Use min float so that 0 never triggers output
				const float val = RandomStream.FRandRange(TNumericLimits<float>::Min(), 1.0f);

				// If number < probability threshold, send output trigger.
				if (val <= *ProbabilityInput)
				{
					TriggerTrueOutput->TriggerFrame(StartFrame);
				}
				else
				{
					TriggerFalseOutput->TriggerFrame(StartFrame);
				}
			}
		);		

	}

	void FTriggerCoinOperator::Reset(const IOperator::FResetParams& InParams)
	{
		// Trigger output
		TriggerTrueOutput->Reset();
		TriggerFalseOutput->Reset();

		bIsDefaultSeeded = false;
		bIsRandomStreamInitialized = false;
		EvaluateSeedChanges();
		RandomStream.Reset();
	}

	TUniquePtr<IOperator> FTriggerCoinOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerCoinVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
		FTriggerReadRef TriggerResetIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
		FInt32ReadRef SeedIn = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSeed), InParams.OperatorSettings);
		FFloatReadRef ProbabilityIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputProbability), InParams.OperatorSettings);

		return MakeUnique<FTriggerCoinOperator>(InParams.OperatorSettings, TriggerIn, TriggerResetIn, SeedIn, ProbabilityIn);
	}

	const FVertexInterface& FTriggerCoinOperator::GetVertexInterface()
	{
		using namespace TriggerCoinVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputReset)),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputSeed), -1),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputProbability), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTrueTrigger)),
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(OutputFalseTrigger))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerCoinOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("Trigger Filter"), TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerCoinNodeDisplayName", "Trigger Chance");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerCoinNodeDescription", "When triggered, Heads has the given probability chance to trigger, otherwise it's Tail.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);
			Info.Keywords = {
				METASOUND_LOCTEXT("TriggerFilterRandomKeyword", "Random"),
				METASOUND_LOCTEXT("TriggerFilterProbabilityKeyword", "Probability"),
				METASOUND_LOCTEXT("TriggerFilterChanceKeyword", "Chance"),
				METASOUND_LOCTEXT("TriggerFilterCoinKeyword", "Coin"),
				METASOUND_LOCTEXT("TriggerFilterBernoulliKeyword", "Bernoulli Gate")
			};

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	void FTriggerCoinOperator::EvaluateSeedChanges()
	{
		// if we have a non-random seed
		if (*SeedInput != -1)
		{
			// If we were previously randomly-seeded OR our seed has changed
			if (bIsDefaultSeeded || !bIsRandomStreamInitialized || *SeedInput != RandomStream.GetInitialSeed())
			{
				bIsRandomStreamInitialized = true;
				bIsDefaultSeeded = false;
				RandomStream.Initialize(*SeedInput);
			}
		}
		// If we are randomly-seeded now BUT were previously not, we need to randomize our seed
		else if (!bIsDefaultSeeded || !bIsRandomStreamInitialized)
		{
			bIsRandomStreamInitialized = true;
			bIsDefaultSeeded = true;
			RandomStream.Initialize(FPlatformTime::Cycles());
		}
	}

	// Node Class
	using FTriggerCoinNode = TNodeFacade<FTriggerCoinOperator>;

	METASOUND_REGISTER_NODE(FTriggerCoinNode)
}

#undef LOCTEXT_NAMESPACE
