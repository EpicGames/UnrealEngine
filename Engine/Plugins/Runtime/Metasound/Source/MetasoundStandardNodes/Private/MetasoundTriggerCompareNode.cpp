// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundTriggerCompareNode.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	namespace MetasoundTriggerCompareNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "TriggerCompare", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Trigger },
				{
					METASOUND_LOCTEXT("EqualsKeyword", "="), METASOUND_LOCTEXT("NotEqualsKeyword", "!="),
					METASOUND_LOCTEXT("LessThanKeyword", "<"), METASOUND_LOCTEXT("GreaterThanKeyword", ">"),
					METASOUND_LOCTEXT("LessEqualsKeyword", "<="), METASOUND_LOCTEXT("GreaterEqualsKeyword", ">="),
					METASOUND_LOCTEXT("EqualsTextKeyword", "equal"), METASOUND_LOCTEXT("InequalsKeyword", "inequal"),
					METASOUND_LOCTEXT("NotKeyword", "not"),
					METASOUND_LOCTEXT("LessThanTextKeyword", "less"),
					METASOUND_LOCTEXT("GreatThanTextKeyword", "greater"),
					METASOUND_LOCTEXT("IfKeyword", "if"), METASOUND_LOCTEXT("BranchKeyword", "branch")},
				FNodeDisplayStyle()
			};

			return Metadata;
		}
	}

	DEFINE_METASOUND_ENUM_BEGIN(ETriggerComparisonType, FEnumTriggerComparisonType, "TriggerComparisonType")
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::Equals, "EqualsDescription", "Equals", "EqualsDescriptionTT", "True if A and B are equal."),
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::NotEquals, "NotEqualsDescriptioin", "Not Equals", "NotEqualsTT", "True if A and B are not equal."),
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::LessThan, "LessThanDescription", "Less Than", "LessThanTT", "True if A is less than B."),
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::GreaterThan, "GreaterThanDescription", "Greater Than", "GreaterThanTT", "True if A is greater than B."),
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::LessThanOrEquals, "LessThanOrEqualsDescription", "Less Than Or Equals", "LessThanOrEqualsTT", "True if A is less than or equal to B."),
		DEFINE_METASOUND_ENUM_ENTRY(ETriggerComparisonType::GreaterThanOrEquals, "GreaterThanOrEqualsDescription", "Greater Than Or Equals", "GreaterThanOrEqualsTT", "True if A is greater than or equal to B."),
	DEFINE_METASOUND_ENUM_END()

	using FTriggerCompareNodeInt32 = TTriggerCompareNode<int32>;
	METASOUND_REGISTER_NODE(FTriggerCompareNodeInt32)

	using FTriggerCompareNodeFloat = TTriggerCompareNode<float>;
	METASOUND_REGISTER_NODE(FTriggerCompareNodeFloat)

	using FTriggerCompareNodeBool = TTriggerCompareNode<bool>;
	METASOUND_REGISTER_NODE(FTriggerCompareNodeBool)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerDelayNode
