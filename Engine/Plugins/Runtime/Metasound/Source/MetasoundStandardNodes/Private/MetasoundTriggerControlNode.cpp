// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerControlNode"

namespace Metasound
{

	namespace TriggerControlVertexNames
	{
		METASOUND_PARAM(InputEnter, "Trigger In", "The input trigger. This will pass through if the node is open.");
		METASOUND_PARAM(InputOpen, "Open", "Once Triggered, allows triggers to pass through.");
		METASOUND_PARAM(InputClose, "Close", "Once triggered, prevents all triggers from passing through.");
		METASOUND_PARAM(InputToggle, "Toggle", "Once triggered, opens the node if closed, and closes the node if opened.");
		METASOUND_PARAM(InputStartClosed, "Start Closed", "Whether the node should be closed when the Metasound begins.");
		
		METASOUND_PARAM(OutputExit, "Trigger Out", "The output trigger.");
	}

	class FTriggerControlOperator : public TExecutableOperator<FTriggerControlOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerControlOperator(const FOperatorSettings& InSettings,
			const FTriggerReadRef& InTriggerEnter,
			const FTriggerReadRef& InTriggerOpen,
			const FTriggerReadRef& InTriggerClose,
			const FTriggerReadRef& InTriggerToggle,
			const FBoolReadRef& InStartClosed);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// Try to go through gate
		FTriggerReadRef TriggerEnterInput;
		// Open the gate
		FTriggerReadRef TriggerOpenInput;
		// Close the gate
		FTriggerReadRef TriggerCloseInput;
		// Switch between open and closed
		FTriggerReadRef TriggerToggleInput;
		// Whether the gate starts closed
		FBoolReadRef bStartClosedInput;

		// If gate is open, sends trigger
		FTriggerWriteRef TriggerExitOutput;

		// Status of the gate
		bool bIsGateOpen;
	};

	FTriggerControlOperator::FTriggerControlOperator(const FOperatorSettings& InSettings,
		const FTriggerReadRef& InTriggerEnter,
		const FTriggerReadRef& InTriggerOpen,
		const FTriggerReadRef& InTriggerClose,
		const FTriggerReadRef& InTriggerToggle,
		const FBoolReadRef& InStartClosed)
		: TriggerEnterInput(InTriggerEnter)
		, TriggerOpenInput(InTriggerOpen)
		, TriggerCloseInput(InTriggerClose)
		, TriggerToggleInput(InTriggerToggle)
		, bStartClosedInput(InStartClosed)
		, TriggerExitOutput(FTriggerWriteRef::CreateNew(InSettings))
		, bIsGateOpen(!*bStartClosedInput)
	{
	}

	void FTriggerControlOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerControlVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEnter), TriggerEnterInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOpen), TriggerOpenInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputClose), TriggerCloseInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputToggle), TriggerToggleInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartClosed), bStartClosedInput);
	}

	void FTriggerControlOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerControlVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputExit), TriggerExitOutput);
	}

	void FTriggerControlOperator::Execute()
	{
		TriggerExitOutput->AdvanceBlock();
		
		// Open Gate
		TriggerOpenInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				bIsGateOpen = true;
			}
		);
		// Close gate
		TriggerCloseInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				bIsGateOpen = false;
			}
		);
		// Toggle gate status
		TriggerToggleInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				bIsGateOpen = !bIsGateOpen;
			}
		);

		// Pass through trigger if gate is open
		TriggerEnterInput->ExecuteBlock(
			[](int32, int32) 
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				if (bIsGateOpen)
				{
					TriggerExitOutput->TriggerFrame(StartFrame);
				}
			}
		);		
	}

	void FTriggerControlOperator::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerExitOutput->Reset();
		bIsGateOpen = !(*bStartClosedInput);
	}

	TUniquePtr<IOperator> FTriggerControlOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerControlVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerEnterIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputEnter), InParams.OperatorSettings);
		FTriggerReadRef TriggerOpenIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputOpen), InParams.OperatorSettings);
		FTriggerReadRef TriggerCloseIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputClose), InParams.OperatorSettings);
		FTriggerReadRef TriggerToggleIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputToggle), InParams.OperatorSettings);
		FBoolReadRef bStartClosedIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputStartClosed), InParams.OperatorSettings);

		return MakeUnique<FTriggerControlOperator>(InParams.OperatorSettings, TriggerEnterIn, TriggerOpenIn, TriggerCloseIn, TriggerToggleIn, bStartClosedIn);
	}

	const FVertexInterface& FTriggerControlOperator::GetVertexInterface()
	{
		using namespace TriggerControlVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEnter)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOpen)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputClose)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputToggle)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartClosed), true)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputExit))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerControlOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::StandardNodes::Namespace, "Trigger Control", FName() };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerControlNodeDisplayName", "Trigger Control");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerControlNodeDescription", "Control whether input triggers are passed through.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);
			Info.Keywords = { METASOUND_LOCTEXT("TriggerControlGateKeyword", "Gate") };

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	using FTriggerControlNode = TNodeFacade<FTriggerControlOperator>;

	METASOUND_REGISTER_NODE(FTriggerControlNode)
}

#undef LOCTEXT_NAMESPACE
