// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTrigger.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CounterNode"

namespace Metasound
{
	namespace TriggerCounterVertexNames
	{
		METASOUND_PARAM(InputInTrigger, "In", "Trigger Input to count.");
		METASOUND_PARAM(InputReset, "Reset", "Resets the counter to zero and the value back to the start value.");
		METASOUND_PARAM(InputStartValue, "Start Value", "The value to set the ouput to on init and reset.");
		METASOUND_PARAM(InputStepSize, "Step Size", "The value to add to the current value for each input trigger. Can be negative.");
		METASOUND_PARAM(InputAutoResetCount, "Reset Count", "The number of input triggers to automatically set the count back to Start Count. If 0, won't auto-reset.");
		METASOUND_PARAM(OutputOnTrigger, "On Trigger", "Triggered when the input is triggered.");
		METASOUND_PARAM(OutputOnReset, "On Reset", "Triggered when the input reset trigger is triggered or if the counter automatically resets.");
		METASOUND_PARAM(OutputCount, "Count", "The current raw trigger count (i.e. number of times the input trigger has been triggered).");
		METASOUND_PARAM(OutputValue, "Value", "The current value of the node given the starting value and step size.");
	}

	class FTriggerCounterOperator : public TExecutableOperator<FTriggerCounterOperator>
	{
		public:
			static const FNodeClassMetadata& GetNodeInfo();
			static const FVertexInterface& GetVertexInterface();
			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

			FTriggerCounterOperator(const FOperatorSettings& InSettings,
				const FTriggerReadRef& InTriggerIn,
				const FTriggerReadRef& InTriggerReset,
				const FFloatReadRef& InStartValue,
				const FFloatReadRef& InStepSize,
				const FInt32ReadRef& InAutoResetCount);

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			void Execute();
			void Reset(const IOperator::FResetParams& InParams);

		private:
			FTriggerReadRef TriggerIn;
			FTriggerReadRef TriggerReset;
			FFloatReadRef StartValue;
			FFloatReadRef StepSize;
			FInt32ReadRef AutoResetCount;

			FTriggerWriteRef TriggerOut;
			FTriggerWriteRef TriggerOnReset;
			FFloatWriteRef OutValue;
			FInt32WriteRef OutCount;

			// Internal state to do the counting math
			int32 CurrentAutoResetCount = 0;
			int32 CurrentTriggerCount = 0;
			float CurrentValue = 0.0f;
			bool bIsFirstTrigger = true;
	};

	FTriggerCounterOperator::FTriggerCounterOperator(const FOperatorSettings& InSettings, 
		const FTriggerReadRef& InTriggerIn, 
		const FTriggerReadRef& InTriggerReset,
		const FFloatReadRef& InStartValue,
		const FFloatReadRef& InStepSize,
		const FInt32ReadRef& InAutoResetCount)
		: TriggerIn(InTriggerIn)
		, TriggerReset(InTriggerReset)
		, StartValue(InStartValue)
		, StepSize(InStepSize)
		, AutoResetCount(InAutoResetCount)
		, TriggerOut(FTriggerWriteRef::CreateNew(InSettings))
		, TriggerOnReset(FTriggerWriteRef::CreateNew(InSettings))
		, OutValue(FFloatWriteRef::CreateNew(*StartValue))
		, OutCount(FInt32WriteRef::CreateNew(0))
	{
		CurrentAutoResetCount = FMath::Max(0, *AutoResetCount);
	}

	void FTriggerCounterOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerCounterVertexNames;

		FDataReferenceCollection InputDataReferences;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInTrigger), TriggerIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), TriggerReset);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartValue), StartValue);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStepSize), StepSize);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAutoResetCount), AutoResetCount);
	}

	void FTriggerCounterOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerCounterVertexNames;

		FDataReferenceCollection OutputDataReferences;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnTrigger), TriggerOut);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnReset), TriggerOnReset);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputCount), OutCount);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutValue);
	}

	void FTriggerCounterOperator::Execute()
	{	
		TriggerOnReset->AdvanceBlock();
		TriggerOut->AdvanceBlock();

		// Update the current auto reset count
		CurrentAutoResetCount = FMath::Max(0, *AutoResetCount);

		TriggerReset->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				*OutCount = CurrentTriggerCount = 0;
				*OutValue = *StartValue;

				bIsFirstTrigger = true;

				TriggerOnReset->TriggerFrame(StartFrame);
			}
		);

		TriggerIn->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				++CurrentTriggerCount;

				if (bIsFirstTrigger)
				{
					CurrentTriggerCount = 1;
					*OutValue = *StartValue;
					bIsFirstTrigger = false;
				}
				else if (CurrentAutoResetCount > 0 && CurrentTriggerCount > CurrentAutoResetCount)
				{
					CurrentTriggerCount = 1;
					*OutValue = *StartValue;
					TriggerOnReset->TriggerFrame(StartFrame);
				}
				else
				{
					*OutValue += *StepSize;
				}

				*OutCount = CurrentTriggerCount;
				TriggerOut->TriggerFrame(StartFrame);
			}
		);

	}

	void FTriggerCounterOperator::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerOut->Reset();
		TriggerOnReset->Reset();
		*OutValue = *StartValue;
		*OutCount = 0;

		CurrentAutoResetCount = FMath::Max(0, *AutoResetCount);
		CurrentTriggerCount = 0;
		CurrentValue = 0.0f;
		bIsFirstTrigger = true;
	}

	TUniquePtr<IOperator> FTriggerCounterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerCounterVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputInTrigger), InParams.OperatorSettings);
		FTriggerReadRef TriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
		FFloatReadRef StartValue = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputStartValue), InParams.OperatorSettings);
		FFloatReadRef StepSize = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputStepSize), InParams.OperatorSettings);
		FInt32ReadRef AutoResetCount = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputAutoResetCount), InParams.OperatorSettings);

		return MakeUnique<FTriggerCounterOperator>(InParams.OperatorSettings, TriggerIn, TriggerReset, StartValue, StepSize, AutoResetCount);
	}

	const FVertexInterface& FTriggerCounterOperator::GetVertexInterface()
	{
		using namespace TriggerCounterVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInTrigger)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReset)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartValue), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStepSize), 1.0f),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAutoResetCount), 0)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnTrigger)),
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnReset)),
				TOutputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputCount)),
				TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerCounterOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = {StandardNodes::Namespace, TEXT("Trigger Counter"), TEXT("")};
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerCounterNodeDisplayName", "Trigger Counter");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerCounterNodeDescription", "Counts the trigger inputs. Supports a start count value, counting by a step size, and auto resetting back to the start count.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();
		return Info;
	}

	using FTriggerCounterNode = TNodeFacade<FTriggerCounterOperator>;

	METASOUND_REGISTER_NODE(FTriggerCounterNode)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerDelayNode
