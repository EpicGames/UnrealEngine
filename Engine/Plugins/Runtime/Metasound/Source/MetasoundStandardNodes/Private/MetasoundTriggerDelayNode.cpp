// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTrigger.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_DelayNode"

namespace Metasound
{
	namespace TriggerDelayVertexNames
	{
		METASOUND_PARAM(InputInTriggerDelay, "In", "Input trigger which results in a delayed trigger.");
		METASOUND_PARAM(InputResetDelay, "Reset", "Resets the trigger delay, clearing the execution task if pending.");
		METASOUND_PARAM(InputDelayTime, "Delay Time", "Time to delay and execute deferred trigger in seconds.");
		METASOUND_PARAM(OutputOnTrigger, "Out", "The delayed output trigger.");
	}

	class FTriggerDelayOperator : public TExecutableOperator<FTriggerDelayOperator>
	{
		public:
			static const FNodeClassMetadata& GetNodeInfo();
			static const FVertexInterface& GetVertexInterface();
			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

			FTriggerDelayOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerReset, const FTriggerReadRef& InTriggerIn, const FTimeReadRef& InDelay);

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			void Execute();
			void Reset(const IOperator::FResetParams& InParams);

		private:
			float SampleRate;
			FTriggerReadRef TriggerIn;
			FTriggerReadRef TriggerReset;
			FTriggerWriteRef TriggerOut;

			FTimeReadRef DelayTime;
	};

	FTriggerDelayOperator::FTriggerDelayOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerReset, const FTriggerReadRef& InTriggerIn, const FTimeReadRef& InTimeDelay)
	: SampleRate(InSettings.GetSampleRate())
	, TriggerIn(InTriggerIn)
	, TriggerReset(InTriggerReset)
	, TriggerOut(FTriggerWriteRef::CreateNew(InSettings))
	, DelayTime(InTimeDelay)
	{
	}

	void FTriggerDelayOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerDelayVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInTriggerDelay), TriggerIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetDelay), TriggerReset);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDelayTime), DelayTime);
	}

	void FTriggerDelayOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerDelayVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnTrigger), TriggerOut);
	}

	void FTriggerDelayOperator::Execute()
	{
		// Advance internal counter to get rid of old triggers.
		TriggerOut->AdvanceBlock();

		TriggerIn->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				// Static cast to int32 to ensure that values are cast to int32 
				// before going into FMath::Max(...). This protects us from overflow issues 
				// when converting between int64->int32.  On some platforms, int64->int32 
				// can result in surprising outcomes when the int64 is out of range
				// of the int32. 
				const int32 FrameToTrigger = FMath::Max(static_cast<int32>(0), static_cast<int32>(FMath::RoundToInt(DelayTime->GetSeconds() * SampleRate))) + StartFrame;
				TriggerOut->RemoveAfter(StartFrame);
				TriggerOut->TriggerFrame(FrameToTrigger);
			}
		);

		TriggerReset->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				TriggerOut->RemoveAfter(StartFrame);
			}
		);
	}

	void FTriggerDelayOperator::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerOut->Reset();
	}

	TUniquePtr<IOperator> FTriggerDelayOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerDelayVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTimeReadRef Delay = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputDelayTime), InParams.OperatorSettings);

		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputInTriggerDelay), InParams.OperatorSettings);
		FTriggerReadRef TriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputResetDelay), InParams.OperatorSettings);

		return MakeUnique<FTriggerDelayOperator>(InParams.OperatorSettings, TriggerReset, TriggerIn, Delay);
	}

	const FVertexInterface& FTriggerDelayOperator::GetVertexInterface()
	{
		using namespace TriggerDelayVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInTriggerDelay)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetDelay)),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDelayTime), 1.0f)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnTrigger))
		)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerDelayOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = {StandardNodes::Namespace, TEXT("Trigger Delay"), TEXT("")};
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerDelayNodeDisplayName", "Trigger Delay");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerDelayNodeDescription", "Executes output trigger after the given delay time from the most recent execution of the input trigger .");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();
		return Info;
	}

	using FTriggerDelayNode = TNodeFacade<FTriggerDelayOperator>;

	METASOUND_REGISTER_NODE(FTriggerDelayNode)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerDelayNode
