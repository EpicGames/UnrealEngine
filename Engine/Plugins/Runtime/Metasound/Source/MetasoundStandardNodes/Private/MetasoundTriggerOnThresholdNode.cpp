// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundTriggerOnThresholdNode.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "MetasoundStandardNodesCategories.h"
#include "Templates/Greater.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	enum class EBufferTriggerType
	{
		RisingEdge,
		FallingEdge,
		AbsThreshold,
	};

	DECLARE_METASOUND_ENUM(EBufferTriggerType, EBufferTriggerType::RisingEdge, METASOUNDSTANDARDNODES_API,
	FEnumBufferTriggerType, FEnumBufferTriggerTypeInfo, FBufferTriggerTypeReadRef, FEnumBufferTriggerTypeWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(EBufferTriggerType, FEnumBufferTriggerType, "BufferTriggerType")
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EBufferTriggerType::RisingEdge, "RisingEdgeDescription", "Rising Edge"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EBufferTriggerType::FallingEdge, "FallingEdgeDescription", "Falling Edge"),
		DEFINE_METASOUND_ENUM_ENTRY_NOTOOLTIP(EBufferTriggerType::AbsThreshold, "AbsThresholdDescription", "Abs Threshold")
	DEFINE_METASOUND_ENUM_END()

	namespace TriggerOnThresholdVertexNames
	{
		METASOUND_PARAM(OutPin, "Out", "Output");
		METASOUND_PARAM(InPin, "In", "Input");
		METASOUND_PARAM(InThresholdPin, "Threshold", "Trigger Threshold");
		METASOUND_PARAM(InTriggerType, "Type", "Trigger Threshold Type");
	}

	// This object is a set of templatized functions containing all the code that varies between int/float and Audio input in the Operator.
	// The base implementation is int/float, and there is a specialized template class for Audio below this one.
	// The reason we templatize ValueType and ThresholdType is because for Audio Buffers, they are not the same.
	// The actual operator class uses this as a more general set of functions that works for all the data types.
	template <typename ValueType, typename ThresholdType = ValueType>
	struct TTriggerOnThresholdHelper
	{
		// Generate is called in Execute, with a different PREDICATE depending on whether it's Rising or Falling Edge.
		// That PREDICATE is the only difference between the implementation of those two enum values.
		template <typename PREDICATE>
		static void Generate(const PREDICATE& ValueTester, const ValueType& InputValue, const ThresholdType& Threshold, ValueType& LastSample, FTriggerWriteRef& Out)
		{
			// block-rate only fires on the first frame
			if (!ValueTester(LastSample, Threshold) && ValueTester(InputValue, Threshold))
			{
				Out->TriggerFrame(0);
			}

			// Remember the last sample for next block.
			LastSample = InputValue;
		}

		// Absolute Threshold has different implementation--the other one does not work for gain tracking a bipolar signal like audio (ex. range [-1.0, 1.0]).
		static void GenerateAbs(const ValueType& InputValue, const ThresholdType& Threshold, bool& bTriggered, FTriggerWriteRef& Out)
		{
			const float ThresholdSqr = Threshold * Threshold;
			const float CurrentSqr = InputValue * InputValue;
			
			// Just went above the threshold, so send the trigger
			if (CurrentSqr > ThresholdSqr && !bTriggered)
			{
				bTriggered = true;
				Out->TriggerFrame(0);
			}
			// Below the threshold, so it's safe to send another trigger next time
			else if (CurrentSqr < ThresholdSqr && bTriggered)
			{
				bTriggered = false;
			}
		}

		// In DeclareVertexInterface, the only change is the first TInputDataVertex and how it's constructed.
		// The Audio version cannot have a default value, but float/int need it.
		// For this node, the value of the default doesn't matter, as it needs to receive input from a connection elsewhere to function as a TriggerOnThreshold.
		static const FVertexInterface DeclareVertexInterface(const float& DefaultThreshold)
		{
			using namespace TriggerOnThresholdVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InPin), static_cast<ValueType>(0)),
					TInputDataVertex<ThresholdType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InThresholdPin), DefaultThreshold),
					TInputDataVertex<FEnumBufferTriggerType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InTriggerType), (int32)EBufferTriggerType::RisingEdge)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutPin))
				)
			);
			return Interface;
		}

		// In CreateOperator, the only thing that varies between Audio and non-audio input is whether there is a default value for the Input. 
		static TDataReadReference<ValueType> CreateInput(const FBuildOperatorParams& InParams)
		{
			using namespace TriggerOnThresholdVertexNames;
			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			return InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InPin), Settings);
		}

	};

	// Audio Buffer specialization.
	// For details about each function, see the general template class above this one. They look mostly the same,
	// but most of the differences are difficult or impossible to replicate without specialization.
	template<>
	struct TTriggerOnThresholdHelper<FAudioBuffer, float>
	{
		template <typename PREDICATE>
		static void Generate(const PREDICATE& ValueTester, const FAudioBuffer& Input, const float& Threshold, float& LastSample, FTriggerWriteRef& Out)
		{
			const float* InputBuffer = Input.GetData();
			const int32 NumFrames = Input.Num();

			float Previous = LastSample;
			for (int32 i = 0; i < NumFrames; ++i)
			{
				const float Current = *InputBuffer;

				// If Previous didn't trigger but current value does... fire!
				if (!ValueTester(Previous, Threshold) && ValueTester(Current, Threshold))
				{
					Out->TriggerFrame(i);
				}

				Previous = Current;
				++InputBuffer;
			}

			// Remember the last sample for next Audio Block.
			LastSample = Previous;
		}

		static void GenerateAbs(const FAudioBuffer& Input, const float& Threshold, bool& bTriggered, FTriggerWriteRef& Out)
		{
			const float* InputBuffer = Input.GetData();
			const int32 NumFrames = Input.Num();
			const float ThresholdSqr = Threshold * Threshold;

			for (int32 i = 0; i < NumFrames; ++i)
			{
				const float Current = *InputBuffer;
				const float CurrentSqr = Current * Current;

				// Just went above the threshold, so send the trigger
				if (CurrentSqr > ThresholdSqr && !bTriggered)
				{
					bTriggered = true;
					Out->TriggerFrame(i);
				}
				else if (CurrentSqr < ThresholdSqr && bTriggered)
				{
					bTriggered = false;
				}

				++InputBuffer;
			}
		}

		static const FVertexInterface DeclareVertexInterface(const float& DefaultThreshold)
		{
			using namespace TriggerOnThresholdVertexNames;

			static const FVertexInterface Interface(
				FInputVertexInterface(
					TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InPin)),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InThresholdPin), DefaultThreshold),
					TInputDataVertex<FEnumBufferTriggerType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InTriggerType))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutPin))
				)
			);
			return Interface;
		}

		static FAudioBufferReadRef CreateInput(const FBuildOperatorParams& InParams)
		{
			using namespace TriggerOnThresholdVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const FOperatorSettings& Settings = InParams.OperatorSettings;

			return InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InPin), Settings);
		}
	};

	// The regular operator class. It's pretty standard from here, just templatized for all the types.
	template <typename ValueType, typename ThresholdType>
	class TTriggerOnThresholdOperator : public TExecutableOperator<TTriggerOnThresholdOperator<ValueType, ThresholdType>>
	{
	public:
		static constexpr ThresholdType DefaultThreshold = 0.85f;

		TTriggerOnThresholdOperator(const FOperatorSettings& InSettings, const TDataReadReference<ValueType>& InBuffer, const TDataReadReference<ThresholdType>& InThreshold, const FBufferTriggerTypeReadRef& InTriggerType)
			: In(InBuffer)
			, Threshold(InThreshold)
			, TriggerType(InTriggerType)
			, Out(FTriggerWriteRef::CreateNew(InSettings))
		{}

		void Execute()
		{
			Out->AdvanceBlock();

			switch (*TriggerType)
			{
			case EBufferTriggerType::RisingEdge:
			default:
				TTriggerOnThresholdHelper<ValueType, ThresholdType>::Generate(TGreater<ThresholdType> {}, *In, *Threshold, LastSample, Out);
				break;
			case EBufferTriggerType::FallingEdge:
				TTriggerOnThresholdHelper<ValueType, ThresholdType>::Generate(TLess<ThresholdType> {}, *In, *Threshold, LastSample, Out);
				break;
			case EBufferTriggerType::AbsThreshold:
				TTriggerOnThresholdHelper<ValueType, ThresholdType>::GenerateAbs(*In, *Threshold, bTriggered, Out);
				break;
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			Out->Reset();
			bTriggered = false;
			LastSample = 0;
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerOnThresholdVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InThresholdPin), Threshold);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InPin), In);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InTriggerType), TriggerType);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerOnThresholdVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutPin), Out);
		}
		static const FVertexInterface DeclareVertexInterface()
		{
			return TTriggerOnThresholdHelper<ValueType, ThresholdType>::DeclareVertexInterface(DefaultThreshold);
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerOnThreshold_DisplayNamePattern", "Trigger On Threshold ({0})", GetMetasoundDataTypeDisplayText<ValueType>());

				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TEXT("TriggerOnThreshold"), GetMetasoundDataTypeName<ValueType>() };
				Info.MajorVersion = 1;
				Info.MinorVersion = 0;
				Info.DisplayName = NodeDisplayName;
				Info.Description = METASOUND_LOCTEXT("TriggerOnThresholdNode_Description", "Trigger when input passes a given threshold.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = DeclareVertexInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

				return Info;
			};
			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace TriggerOnThresholdVertexNames;

			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FBufferTriggerTypeReadRef Type = InputData.GetOrCreateDefaultDataReadReference<FEnumBufferTriggerType>(METASOUND_GET_PARAM_NAME(InTriggerType), Settings);
			TDataReadReference<ValueType> Input = TTriggerOnThresholdHelper<ValueType>::CreateInput(InParams);
			TDataReadReference<ThresholdType> Threshold = InputData.GetOrCreateDefaultDataReadReference<ThresholdType>(METASOUND_GET_PARAM_NAME(InThresholdPin), Settings);

			return MakeUnique<TTriggerOnThresholdOperator<ValueType, ThresholdType>>(InParams.OperatorSettings, Input, Threshold, Type);
		}		

	protected:
		TDataReadReference<ValueType> In;
		TDataReadReference<ThresholdType> Threshold;
		FBufferTriggerTypeReadRef TriggerType;
		
		FTriggerWriteRef Out;
		
		bool bTriggered = false;
		ThresholdType LastSample = 0;
	};

	// Mac Clang requires linkage on constexpr
	template <>
	constexpr float TTriggerOnThresholdOperator<float, float>::DefaultThreshold = 0.85f;

	template <>
	constexpr float TTriggerOnThresholdOperator<FAudioBuffer, float>::DefaultThreshold = 0.85f;

	template <>
	constexpr int32 TTriggerOnThresholdOperator<int32, int32>::DefaultThreshold = 1;
	
	template <typename ValueType, typename ThresholdType>
	using TTriggerOnThresholdNode = TNodeFacade<TTriggerOnThresholdOperator<ValueType, ThresholdType>>;

	using FTriggerOnThresholdAudioNode = TTriggerOnThresholdNode<FAudioBuffer, float>;
	METASOUND_REGISTER_NODE(FTriggerOnThresholdAudioNode);

	using FTriggerOnThresholdFloatNode = TTriggerOnThresholdNode<float, float>;
	METASOUND_REGISTER_NODE(FTriggerOnThresholdFloatNode);

	using FTriggerOnThresholdInt32Node = TTriggerOnThresholdNode<int32, int32>;
	METASOUND_REGISTER_NODE(FTriggerOnThresholdInt32Node);
}
#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes

