// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTrigger.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerOnValueChangeNode"

namespace Metasound
{

	namespace TriggerOnValueChangeVertexNames
	{
		METASOUND_PARAM(InputValue, "Value", "The input value to watch. Whenever this changes, the output trigger is sent.");
		
		METASOUND_PARAM(OutputOnChange, "Trigger", "The output trigger.");
	}

	namespace TriggerOnValueChangePrivate
	{
		template <typename ValueType>
		static bool IsValueEqual(ValueType InValueA, ValueType InValueB)
		{
			return InValueA == InValueB;
		}

		static bool IsValueEqual(float InValueA, float InValueB)
		{
			return FMath::IsNearlyEqual(InValueA, InValueB);
		}		
	}


	template <typename ValueType>
	class TTriggerOnValueChangeOperator : public TExecutableOperator<TTriggerOnValueChangeOperator<ValueType>>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		TTriggerOnValueChangeOperator(const FOperatorSettings& InSettings, const TDataReadReference<ValueType>& InValue);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// Parameter to watch for changes
		TDataReadReference<ValueType> ValueInput;

		// If gate is open, sends trigger
		FTriggerWriteRef TriggerOnChangeOutput;

		// Status of the gate
		ValueType PrevValue;
	};

	template <typename ValueType>
	TTriggerOnValueChangeOperator<ValueType>::TTriggerOnValueChangeOperator(const FOperatorSettings& InSettings, const TDataReadReference<ValueType>& InValue)
		: ValueInput(InValue)
		, TriggerOnChangeOutput(FTriggerWriteRef::CreateNew(InSettings))
		, PrevValue(*InValue)
	{
	}

	template <typename ValueType>
	void TTriggerOnValueChangeOperator<ValueType>::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnValueChangeVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), ValueInput);
	}

	template <typename ValueType>
	void TTriggerOnValueChangeOperator<ValueType>::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnValueChangeVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnChange), TriggerOnChangeOutput);
	}

	template <typename ValueType>
	void TTriggerOnValueChangeOperator<ValueType>::Execute()
	{
		using namespace TriggerOnValueChangePrivate;

		TriggerOnChangeOutput->AdvanceBlock();

		// If value changes, call the trigger at the start of the audio block
		if (!IsValueEqual(*ValueInput, PrevValue))
		{
			PrevValue = *ValueInput;
			TriggerOnChangeOutput->TriggerFrame(0);
		}

	}
	template <typename ValueType>
	void TTriggerOnValueChangeOperator<ValueType>::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerOnChangeOutput->Reset();
		PrevValue = *ValueInput;
	}

	template <typename ValueType>
	TUniquePtr<IOperator> TTriggerOnValueChangeOperator<ValueType>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerOnValueChangeVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		TDataReadReference<ValueType> ValueIn = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputValue), InParams.OperatorSettings);

		return MakeUnique<TTriggerOnValueChangeOperator>(InParams.OperatorSettings, ValueIn);
	}

	template <typename ValueType>
	const FVertexInterface& TTriggerOnValueChangeOperator<ValueType>::GetVertexInterface()
	{
		using namespace TriggerOnValueChangeVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputValue), static_cast<ValueType>(1))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnChange))
			)
		);

		return Interface;
	}

	template <typename ValueType>
	const FNodeClassMetadata& TTriggerOnValueChangeOperator<ValueType>::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
			const FName OperatorName = TEXT("Trigger On Value Change");
			const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerOnValueChangeName", "Trigger On Value Change ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
			const FText NodeDescription = METASOUND_LOCTEXT("TriggerOnValueChangeNameDesc", "Triggers when a given value changes.");

			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::StandardNodes::Namespace, OperatorName, DataTypeName };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	template <typename ValueType>
	using TTriggerOnValueChangeNode = TNodeFacade<TTriggerOnValueChangeOperator<ValueType>>;

	using FTriggerOnInt32Change = TTriggerOnValueChangeNode<int32>;
	METASOUND_REGISTER_NODE(FTriggerOnInt32Change)

	using FTriggerOnFloatChange = TTriggerOnValueChangeNode<float>;
	METASOUND_REGISTER_NODE(FTriggerOnFloatChange)

	using FTriggerOnBoolChange = TTriggerOnValueChangeNode<bool>;
	METASOUND_REGISTER_NODE(FTriggerOnBoolChange)
}

#undef LOCTEXT_NAMESPACE
