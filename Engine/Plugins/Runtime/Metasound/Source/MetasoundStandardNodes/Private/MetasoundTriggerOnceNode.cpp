// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerOnceNode"

namespace Metasound
{

	namespace TriggerOnceVertexNames
	{
		METASOUND_PARAM(InputEnter, "Trigger In", "The input trigger. After the first time this is triggered, no subsequent triggers will pass through.");
		METASOUND_PARAM(InputReset, "Reset", "When triggered, opens the node to allow another trigger through.");
		METASOUND_PARAM(InputStartClosed, "Start Closed", "Whether the node should be closed when the Metasound begins.");
		
		METASOUND_PARAM(OutputExit, "Trigger Out", "The output trigger.");
	}

	class FTriggerOnceOperator : public TExecutableOperator<FTriggerOnceOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerOnceOperator(const FOperatorSettings& InSettings,
			const FTriggerReadRef& InTriggerEnter,
			const FTriggerReadRef& InTriggerReset,
			const FBoolReadRef& InStartClosed);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// Try to go through gate
		FTriggerReadRef TriggerEnterInput;
		// Open the gate
		FTriggerReadRef TriggerResetInput;
		// Whether the gate starts closed
		FBoolReadRef bStartClosedInput;

		// If gate is open, sends trigger
		FTriggerWriteRef TriggerExitOutput;

		// Status of the gate
		bool bIsGateOpen;
	};

	FTriggerOnceOperator::FTriggerOnceOperator(const FOperatorSettings& InSettings,
		const FTriggerReadRef& InTriggerEnter,
		const FTriggerReadRef& InTriggerReset,
		const FBoolReadRef& InStartClosed)
		: TriggerEnterInput(InTriggerEnter)
		, TriggerResetInput(InTriggerReset)
		, bStartClosedInput(InStartClosed)
		, TriggerExitOutput(FTriggerWriteRef::CreateNew(InSettings))
		, bIsGateOpen(!*bStartClosedInput)
	{
	}


	void FTriggerOnceOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnceVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEnter), TriggerEnterInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), TriggerResetInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartClosed), bStartClosedInput);
	}

	void FTriggerOnceOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerOnceVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputExit), TriggerExitOutput);
	}

	void FTriggerOnceOperator::Execute()
	{
		TriggerExitOutput->AdvanceBlock();

		// Pass through trigger if gate is open
		TriggerEnterInput->ExecuteBlock(
			[](int32, int32) 
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				if (bIsGateOpen)
				{
					bIsGateOpen = false;
					TriggerExitOutput->TriggerFrame(StartFrame);
				}
			}
		);		
		// Open Gate
		TriggerResetInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				bIsGateOpen = true;
			}
			);

	}

	void FTriggerOnceOperator::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerExitOutput->Reset();
		bIsGateOpen = !(*bStartClosedInput);
	}

	TUniquePtr<IOperator> FTriggerOnceOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerOnceVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerEnterIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputEnter), InParams.OperatorSettings);
		FTriggerReadRef TriggerResetIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
		FBoolReadRef bStartClosedIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputStartClosed), InParams.OperatorSettings);

		return MakeUnique<FTriggerOnceOperator>(InParams.OperatorSettings, TriggerEnterIn, TriggerResetIn, bStartClosedIn);
	}

	const FVertexInterface& FTriggerOnceOperator::GetVertexInterface()
	{
		using namespace TriggerOnceVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEnter)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReset)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartClosed), false)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputExit))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerOnceOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { Metasound::StandardNodes::Namespace, "Trigger Once", FName() };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerOnceNodeDisplayName", "Trigger Once");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerOnceNodeDescription", "Sends an output trigger the first time the node is triggered, and ignores all others (can be re-opened).");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	using FTriggerOnceNode = TNodeFacade<FTriggerOnceOperator>;

	METASOUND_REGISTER_NODE(FTriggerOnceNode)
}

#undef LOCTEXT_NAMESPACE
