// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerPipe"

namespace Metasound
{
	namespace TriggerPipeVertexNames
	{
		METASOUND_PARAM(InputInTrigger, "In", "Trigger to execute at a future time by the given delay amount.");
		METASOUND_PARAM(InputReset, "Reset", "Resets the trigger delay, clearing any pending execution tasks.");
		METASOUND_PARAM(InputDelayTime, "Delay Time", "Time to delay and execute deferred input trigger execution(s) in seconds.");
		METASOUND_PARAM(OutputOutTrigger, "Out", "The delayed output trigger(s).");
	}

	class FTriggerPipeOperator : public TExecutableOperator<FTriggerPipeOperator>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FTriggerPipeOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerReset, const FTriggerReadRef& InTriggerIn, const FTimeReadRef& InDelay);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		TArray<FSampleCount> SamplesUntilTrigger;

		FTriggerReadRef TriggerIn;
		FTriggerReadRef TriggerReset;
		FTriggerWriteRef TriggerOut;

		FTimeReadRef DelayTime;

		FSampleCount FramesPerBlock;
		FSampleRate SampleRate;
	};

	FTriggerPipeOperator::FTriggerPipeOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerReset, const FTriggerReadRef& InTriggerIn, const FTimeReadRef& InDelayTime)
	: TriggerIn(InTriggerIn)
	, TriggerReset(InTriggerReset)
	, TriggerOut(FTriggerWriteRef::CreateNew(InSettings))
	, DelayTime(InDelayTime)
	, FramesPerBlock(InSettings.GetNumFramesPerBlock())
	, SampleRate(InSettings.GetSampleRate())
	{
	}

	void FTriggerPipeOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerPipeVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInTrigger), TriggerIn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), TriggerReset);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDelayTime), DelayTime);
	}

	void FTriggerPipeOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerPipeVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOutTrigger), TriggerOut);
	}

	void FTriggerPipeOperator::Execute()
	{
		// Advance internal counter to get rid of old triggers.
		TriggerOut->AdvanceBlock();

		TriggerIn->ExecuteBlock(
			[](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				SamplesUntilTrigger.AddUnique(StartFrame + FMath::Max(DelayTime->GetSeconds(), 0.0) * SampleRate);
			}
		);

		TriggerReset->ExecuteBlock(
			[](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				// Iterate backward and only remove delayed triggers that occur
				// after the reset trigger's start frame.
				for (FSampleCount i = SamplesUntilTrigger.Num() - 1; i >= 0; --i)
				{
					const FSampleCount SamplesRemaining = SamplesUntilTrigger[i] - FramesPerBlock;
					if (SamplesRemaining >= StartFrame)
					{
						SamplesUntilTrigger.RemoveAtSwap(i);
					}
				}
			}
		);

		for (FSampleCount i = SamplesUntilTrigger.Num() - 1; i >= 0; --i)
		{
			const FSampleCount SamplesRemaining = SamplesUntilTrigger[i] - FramesPerBlock;
			if (SamplesRemaining >= 0)
			{
				SamplesUntilTrigger[i] -= FramesPerBlock;
			}
			else
			{
				TriggerOut->TriggerFrame(SamplesRemaining + static_cast<FSampleCount>(FramesPerBlock));
				SamplesUntilTrigger.RemoveAtSwap(i);
			}
		}
	}

	void FTriggerPipeOperator::Reset(const IOperator::FResetParams& InParams)
	{
		SamplesUntilTrigger.Reset();
		TriggerOut->Reset();
	}

	TUniquePtr<IOperator> FTriggerPipeOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerPipeVertexNames;
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputInTrigger), InParams.OperatorSettings);
		FTriggerReadRef TriggerReset = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
		FTimeReadRef DelayTime = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputDelayTime), InParams.OperatorSettings);

		return MakeUnique<FTriggerPipeOperator>(InParams.OperatorSettings, TriggerReset, TriggerIn, DelayTime);
	}

	const FVertexInterface& FTriggerPipeOperator::GetVertexInterface()
	{
		using namespace TriggerPipeVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInTrigger)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReset)),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDelayTime), 1.0f)
				),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOutTrigger))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerPipeOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = FNodeClassName(StandardNodes::Namespace, "Pipe", "");
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("PipeTriggerNode_NodeDisplayName", "Trigger Pipe");
			Info.Description = METASOUND_LOCTEXT("Metasound_DelayNodeDescription", "Delays execution of the input trigger(s) by the given delay for all input trigger executions.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();
		return Info;
	}

	using FTriggerPipeNode = TNodeFacade<FTriggerPipeOperator>;

	METASOUND_REGISTER_NODE(FTriggerPipeNode)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerPipeNode
