// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundTriggerRepeatNode.h"

#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerRepeatNode"

namespace Metasound
{
	METASOUND_REGISTER_NODE(FTriggerRepeatNode)

	namespace TriggerRepeatVertexNames
	{
		METASOUND_PARAM(InputStart, "Start", "Starts executing periodic output triggers.");
		METASOUND_PARAM(InputStop, "Stop", "Stops executing periodic output triggers.");
		METASOUND_PARAM(InputPeriod, "Period", "The period to trigger in seconds.");
		METASOUND_PARAM(InputNumRepeats, "Num Repeats", "The number of times to repeat. When set to 0, will repeat indefinitely until stopped.")
		METASOUND_PARAM(RepeatOutputOnTrigger, "RepeatOut", "The periodically generated output trigger.");
	}

	class FTriggerRepeatOperator : public TExecutableOperator<FTriggerRepeatOperator>
	{
		public:
			static const FNodeClassMetadata& GetNodeInfo();
			static const FVertexInterface& GetVertexInterface();
			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

			FTriggerRepeatOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerEnable, const FTriggerReadRef& InTriggerDisable, 
				const FTimeReadRef& InPeriod, const FInt32ReadRef& InNumRepeats);

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;

			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			void Execute();
			void Reset(const IOperator::FResetParams& InParams);

		private:
			bool bEnabled;

			FTriggerWriteRef TriggerOut;

			FTriggerReadRef TriggerEnable;
			FTriggerReadRef TriggerDisable;

			FTimeReadRef Period;

			FInt32ReadRef NumRepeats;

			FSampleCount BlockSize;
			double SampleRate;
			double SampleCounter;

			//The number of repeats the node has already performed
			int CurrentRepeats = 0;
	};

	FTriggerRepeatOperator::FTriggerRepeatOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerEnable, const FTriggerReadRef& InTriggerDisable, const FTimeReadRef& InPeriod, const FInt32ReadRef& InNumRepeats)
		: bEnabled(false)
		, TriggerOut(FTriggerWriteRef::CreateNew(InSettings))
		, TriggerEnable(InTriggerEnable)
		, TriggerDisable(InTriggerDisable)
		, Period(InPeriod)
		, NumRepeats(InNumRepeats)
		, BlockSize(InSettings.GetNumFramesPerBlock())
		, SampleRate(InSettings.GetSampleRate())
		, SampleCounter(0.0)
	{
	}


	void FTriggerRepeatOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerRepeatVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStart), TriggerEnable);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStop), TriggerDisable);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPeriod), Period);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputNumRepeats), NumRepeats);

	}

	void FTriggerRepeatOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerRepeatVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(RepeatOutputOnTrigger), TriggerOut);
	}

	void FTriggerRepeatOperator::Execute()
	{
		TriggerEnable->ExecuteBlock([](int32, int32) { },
			[this](int32 StartFrame, int32 EndFrame)
			{
				bEnabled = true;
				SampleCounter = 0.0;

				//Reset current number of repeats if we get another play command
				CurrentRepeats = 0;
			}
		);

		TriggerDisable->ExecuteBlock([](int32, int32) { },
			[this](int32 StartFrame, int32 EndFrame)
			{
				bEnabled = false;
			}
		);

		// Advance internal counter to get rid of old triggers.
		TriggerOut->AdvanceBlock();

		if (bEnabled)
		{
			double PeriodInSamples = (FTime::ToMilliseconds(*Period) / 1000.0) * SampleRate;
			double BlockSizeDouble = static_cast<double>(BlockSize);

			// Time must march on, can't stay in the now forever.
			PeriodInSamples = FMath::Max(1.0, PeriodInSamples);
			while (SampleCounter <= BlockSizeDouble)
			{
				//Whether or not the node has any repeats it should perform
				bool bShouldRepeat = (*NumRepeats == 0) || (CurrentRepeats < *NumRepeats);

				if (!bShouldRepeat)
				{
					// Disable repeat node if have satisfied the number of repeats
					bEnabled = false;
					break;
				}

				const int32 StartOffset = FMath::RoundToInt32(SampleCounter);
				TriggerOut->TriggerFrame(StartOffset);
				SampleCounter += PeriodInSamples;

				++CurrentRepeats;
			}

			SampleCounter -= BlockSizeDouble;
		}
	}

	void FTriggerRepeatOperator::Reset(const IOperator::FResetParams& InParams)
	{
		bEnabled = false;

		TriggerOut->Reset();
		SampleCounter = 0;
		CurrentRepeats = 0;
	}

	TUniquePtr<IOperator> FTriggerRepeatOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerRepeatVertexNames;

		const FTriggerRepeatNode& PeriodicTriggerNode = static_cast<const FTriggerRepeatNode&>(InParams.Node);
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerEnable = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputStart), InParams.OperatorSettings);
		FTriggerReadRef TriggerDisable = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputStop), InParams.OperatorSettings);
		FTimeReadRef Period = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputPeriod), InParams.OperatorSettings);
		FInt32ReadRef NumRepeats = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputNumRepeats), InParams.OperatorSettings);

		return MakeUnique<FTriggerRepeatOperator>(InParams.OperatorSettings, TriggerEnable, TriggerDisable, Period, NumRepeats);
	}

	const FVertexInterface& FTriggerRepeatOperator::GetVertexInterface()
	{
		using namespace TriggerRepeatVertexNames;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStart)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStop)),
				TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPeriod), 0.2f),
				TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputNumRepeats), 0)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(RepeatOutputOnTrigger))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerRepeatOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("TriggerRepeat"), TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_PeriodicTriggerNodeDisplayName", "Trigger Repeat");
			Info.Description = METASOUND_LOCTEXT("Metasound_PeriodicTriggerNodeDescription", "Emits a trigger periodically based on the period duration given.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);
			Info.Keywords.Add(METASOUND_LOCTEXT("TriggerRepeatLoopKeyword", "Loop"));

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	FTriggerRepeatNode::FTriggerRepeatNode(const FNodeInitData& InInitData)
	:	FNodeFacade(InInitData.InstanceName, InInitData.InstanceID, TFacadeOperatorClass<FTriggerRepeatOperator>())
	{
	}

	FTriggerRepeatNode::FTriggerRepeatNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	:	FNodeFacade(MoveTemp(InNodeData), MoveTemp(InMetadata), TFacadeOperatorClass<FTriggerRepeatOperator>())
	{
	}

	FNodeClassMetadata FTriggerRepeatNode::CreateNodeClassMetadata()
	{
		return FTriggerRepeatOperator::GetNodeInfo();
	}
}

#undef LOCTEXT_NAMESPACE //MetasoundPeriodicTriggerNode
