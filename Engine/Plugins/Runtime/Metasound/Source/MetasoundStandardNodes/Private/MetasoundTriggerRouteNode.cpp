// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerRouter"

#define REGISTER_TRIGGER_ROUTE_NODE(DataType, Number) \
	using FTriggerRouteNode##DataType##_##Number = TTriggerRouteNode<DataType, Number>; \
	METASOUND_REGISTER_NODE(FTriggerRouteNode##DataType##_##Number) \

namespace Metasound
{
	namespace TriggerRouteVertexNames
	{
		METASOUND_PARAM(OutputTrigger, "On Set", "Triggered when any of the input triggers are set.")
		METASOUND_PARAM(OutputValue, "Value", "The output value set by the input triggers.")

		const FVertexName GetInputTriggerName(uint32 InIndex)
		{
			return *FString::Format(TEXT("Set {0}"), { InIndex });
		}

		const FText GetInputTriggerDisplayName(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("InputTriggerDisplayName", "Set {0}", InIndex);
		}

		const FText GetInputTriggerDescription(uint32 InIndex)
		{
			if (InIndex == 0)
			{
				return METASOUND_LOCTEXT_FORMAT("TriggerRouteInputTriggerDescInit", "The input trigger {0} to cause the corresponding input value to route to the output value. This trigger is the default output.", InIndex);
			}
			return METASOUND_LOCTEXT_FORMAT("TriggerRouteInputTriggerDesc", "The input trigger {0} to cause the corresponding input value to route to the output value.", InIndex);
		}

		const FVertexName GetInputValueName(uint32 InIndex)
		{
			return *FString::Format(TEXT("Value {0}"), { InIndex });
		}

		const FText GetInputValueDescription(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("TriggerRouteValueDesc", "The input value ({0}) to route to the output when triggered by Set {0}.", InIndex);
		}

		const FText GetInputValueDisplayName(uint32 InIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("InputValueDisplayName", "Value {0}", InIndex);
		}
	}


	namespace MetasoundTriggerRouteNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName{ "TriggerRoute", InOperatorName, InDataTypeName},
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Trigger },
				{ METASOUND_LOCTEXT("TriggerRouteKeyword_Select", "select") },
				FNodeDisplayStyle()
			};

			return Metadata;
		}
	}

	template<typename ValueType, uint32 NumInputs>
	class TTriggerRouteOperator : public TExecutableOperator<TTriggerRouteOperator<ValueType, NumInputs>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace TriggerRouteVertexNames;

			auto CreateDefaultInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;
				for (uint32 i = 0; i < NumInputs; ++i)
				{
					const FDataVertexMetadata InputTriggerMetadata
					{
						  GetInputTriggerDescription(i) // description
						, GetInputTriggerDisplayName(i) // display name
					};

					const FDataVertexMetadata InputValueMetadata
					{
						  GetInputValueDescription(i) // description
						, GetInputValueDisplayName(i) // display name
					};

					InputInterface.Add(TInputDataVertex<FTrigger>(GetInputTriggerName(i), InputTriggerMetadata));
					InputInterface.Add(TInputDataVertex<ValueType>(GetInputValueName(i), InputValueMetadata));
				}

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTrigger)));
				OutputInterface.Add(TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue)));
				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				FName OperatorName = *FString::Printf(TEXT("Trigger Route (%s, %d)"), *DataTypeName.ToString(), NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerRouteDisplayNamePattern", "Trigger Route ({0}, {1})", GetMetasoundDataTypeDisplayText<ValueType>(), NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("TriggerRouteDescription", "Allows routing different values to the same output pin depending on trigger inputs.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundTriggerRouteNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace TriggerRouteVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			TArray<FTriggerReadRef> InputTriggers;
			TArray<TDataReadReference<ValueType>> InputValues;

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers.Add(InputData.GetOrCreateDefaultDataReadReference<FTrigger>(GetInputTriggerName(i), InParams.OperatorSettings));
				InputValues.Add(InputData.GetOrCreateDefaultDataReadReference<ValueType>(GetInputValueName(i), InParams.OperatorSettings));
			}

			return MakeUnique<TTriggerRouteOperator<ValueType, NumInputs>>(InParams, MoveTemp(InputTriggers), MoveTemp(InputValues));
		}

		TTriggerRouteOperator(const FBuildOperatorParams& InParams, TArray<FTriggerReadRef>&& InInputTriggers, TArray<TDataReadReference<ValueType>>&& InInputValues)
			: InputTriggers(MoveTemp(InInputTriggers))
			, InputValues(MoveTemp(InInputValues))
			, OutputTrigger(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InParams.OperatorSettings))
		{
			check(InputValues.Num() > 0);
			
			// Call Update() to initialize output value and determine initial
			// trigger index.
			Update();
		}

		virtual ~TTriggerRouteOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerRouteVertexNames;
			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InOutVertexData.BindReadVertex(GetInputTriggerName(i), InputTriggers[i]);
				InOutVertexData.BindReadVertex(GetInputValueName(i), InputValues[i]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerRouteVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTrigger), OutputTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void Execute()
		{
			OutputTrigger->AdvanceBlock();
			Update();
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			OutputTrigger->Reset();
			CurrentIndex = 0;
			Update();
		}

	private:

		void Update()
		{
			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers[i]->ExecuteBlock(
					[&](int32 StartFrame, int32 EndFrame)
					{
					},
					[this, i](int32 StartFrame, int32 EndFrame)
					{
						CurrentIndex = i;
						OutputTrigger->TriggerFrame(StartFrame);
					}
				);
			}

			*OutputValue = *InputValues[CurrentIndex];
		}

		TArray<FTriggerReadRef> InputTriggers;
		TArray<TDataReadReference<ValueType>> InputValues;

		FTriggerWriteRef OutputTrigger;
		TDataWriteReference<ValueType> OutputValue;
		int32 CurrentIndex = 0;
	};

	/** TTriggerRouteNode
	 *
	 *  Routes values from multiple input pins to a single output pin based on trigger inputs.
	 */
	template<typename ValueType, uint32 NumInputs>
	using TTriggerRouteNode = TNodeFacade<TTriggerRouteOperator<ValueType, NumInputs>>;

	REGISTER_TRIGGER_ROUTE_NODE(int32, 2)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 3)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 4)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 5)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 6)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 7)
	REGISTER_TRIGGER_ROUTE_NODE(int32, 8)

	REGISTER_TRIGGER_ROUTE_NODE(float, 2)
	REGISTER_TRIGGER_ROUTE_NODE(float, 3)
	REGISTER_TRIGGER_ROUTE_NODE(float, 4)
	REGISTER_TRIGGER_ROUTE_NODE(float, 5)
	REGISTER_TRIGGER_ROUTE_NODE(float, 6)
	REGISTER_TRIGGER_ROUTE_NODE(float, 7)
	REGISTER_TRIGGER_ROUTE_NODE(float, 8)

	REGISTER_TRIGGER_ROUTE_NODE(FTime, 2)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 3)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 4)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 5)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 6)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 7)
	REGISTER_TRIGGER_ROUTE_NODE(FTime, 8)

	REGISTER_TRIGGER_ROUTE_NODE(bool, 2)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 3)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 4)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 5)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 6)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 7)
	REGISTER_TRIGGER_ROUTE_NODE(bool, 8)

	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 2)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 3)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 4)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 5)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 6)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 7)
	REGISTER_TRIGGER_ROUTE_NODE(FAudioBuffer, 8)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerDelayNode
