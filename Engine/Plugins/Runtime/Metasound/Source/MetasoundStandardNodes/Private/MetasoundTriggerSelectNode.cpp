// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

#define REGISTER_TRIGGER_SELECT_NODE(Number) \
	using FTriggerSelectNode_##Number = TTriggerSelectNode<Number>; \
	METASOUND_REGISTER_NODE(FTriggerSelectNode_##Number) \

namespace Metasound
{

	namespace TriggerSelectVertexNames
	{
		METASOUND_PARAM(InputTrigger, "In", "Input trigger.");
		METASOUND_PARAM(InputIndex, "Index", "The output index to trigger. Values outside the range are ignored.");

		METASOUND_PARAM(OutputTrigger, "Out {0}", "Trigger Output {0}.");

	}

	template<uint32 NumOutputs>
	class TTriggerSelectOperator : public TExecutableOperator<TTriggerSelectOperator<NumOutputs>>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		TTriggerSelectOperator(const FOperatorSettings& InSettings,
			const FTriggerReadRef& InTrigger,
			const FInt32ReadRef& InIndex);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		FTriggerReadRef TriggerInput;
		FInt32ReadRef IndexInput;

		TArray<FTriggerWriteRef> TriggerOutputs;
	};

	template<uint32 NumOutputs>
	TTriggerSelectOperator<NumOutputs>::TTriggerSelectOperator(const FOperatorSettings& InSettings,
		const FTriggerReadRef& InTrigger,
		const FInt32ReadRef& InIndex)
		: TriggerInput(InTrigger)
		, IndexInput(InIndex)
	{
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs.Add(FTriggerWriteRef::CreateNew(InSettings));
		}
	}

	template<uint32 NumOutputs>
	void TTriggerSelectOperator<NumOutputs>::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerSelectVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIndex), IndexInput);
	}

	template<uint32 NumOutputs>
	void TTriggerSelectOperator<NumOutputs>::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerSelectVertexNames;
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(OutputTrigger, i), TriggerOutputs[i]);
		}
	}

	template<uint32 NumOutputs>
	void TTriggerSelectOperator<NumOutputs>::Execute()
	{
		// Advance all output blocks
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs[i]->AdvanceBlock();
		}

		// Passthrough input trigger
		TriggerInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				if (FMath::IsWithin(*IndexInput, 0, static_cast<int32>(NumOutputs)))
				{
					// Send Trigger Out
					TriggerOutputs[*IndexInput]->TriggerFrame(StartFrame);
				}
			}
			);
	}

	template<uint32 NumOutputs>
	void TTriggerSelectOperator<NumOutputs>::Reset(const IOperator::FResetParams& InParams)
	{
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs[i]->Reset();
		}
	}

	template<uint32 NumOutputs>
	TUniquePtr<IOperator> TTriggerSelectOperator<NumOutputs>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerSelectVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
		FInt32ReadRef InputIn = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputIndex), InParams.OperatorSettings);

		return MakeUnique<TTriggerSelectOperator>(InParams.OperatorSettings, TriggerIn, InputIn);
	}

	template<uint32 NumOutputs>
	const FVertexInterface& TTriggerSelectOperator<NumOutputs>::GetVertexInterface()
	{
		using namespace TriggerSelectVertexNames;

		auto CreateVertexInterface = []() -> FVertexInterface
		{
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)));
			InputInterface.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIndex), 1));

			FOutputVertexInterface OutputInterface;
			for (uint32 i = 0; i < NumOutputs; ++i)
			{
				OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(OutputTrigger, i)));
			}
			return FVertexInterface(InputInterface, OutputInterface);
		};

		static const FVertexInterface Interface = CreateVertexInterface();
		return Interface;
	}

	template<uint32 NumOutputs>
	const FNodeClassMetadata& TTriggerSelectOperator<NumOutputs>::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FName OperatorName = *FString::Printf(TEXT("Trigger Select (%d)"), NumOutputs);
			FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerSequenceDisplayNamePattern", "Trigger Select ({0})", NumOutputs);
			const FText NodeDescription = METASOUND_LOCTEXT("TriggerSequenceDescription", "Passes triggers through to the currently selected output trigger.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	template <uint32 NumOutputs>
	using TTriggerSelectNode = TNodeFacade<TTriggerSelectOperator<NumOutputs>>;

	REGISTER_TRIGGER_SELECT_NODE(2)
	REGISTER_TRIGGER_SELECT_NODE(3)
	REGISTER_TRIGGER_SELECT_NODE(4)
	REGISTER_TRIGGER_SELECT_NODE(5)
	REGISTER_TRIGGER_SELECT_NODE(6)
	REGISTER_TRIGGER_SELECT_NODE(7)
	REGISTER_TRIGGER_SELECT_NODE(8)
}

#undef LOCTEXT_NAMESPACE
