// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundSampleCounter.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_TriggerSequenceNode"

#define REGISTER_TRIGGER_SEQUENCE_NODE(Number) \
	using FTriggerSequenceNode_##Number = TTriggerSequenceNode<Number>; \
	METASOUND_REGISTER_NODE(FTriggerSequenceNode_##Number) \

namespace Metasound
{

	namespace TriggerSequenceVertexNames
	{
		METASOUND_PARAM(InputTrigger, "In", "Input trigger.");
		METASOUND_PARAM(InputReset, "Reset", "Resets the sequence. When triggered, the next trigger will be Out 0.");		
		METASOUND_PARAM(InputLoop, "Loop", "Whether the sequence will automatically loop back to Out 0 after all triggers have been called.");

		METASOUND_PARAM(OutputTrigger, "Out {0}", "Trigger Output {0} in the sequence.");

	}

	template<uint32 NumOutputs>
	class TTriggerSequenceOperator : public TExecutableOperator<TTriggerSequenceOperator<NumOutputs>>
	{
	public:
		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		TTriggerSequenceOperator(const FOperatorSettings& InSettings, 
			const FTriggerReadRef& InTrigger, 
			const FTriggerReadRef& InReset,
			const FBoolReadRef& bInLoop);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		FTriggerReadRef TriggerInput;
		FTriggerReadRef TriggerResetInput;
		FBoolReadRef bLoopInput;

		TArray<FTriggerWriteRef> TriggerOutputs;

		uint32 CurrentIndex = 0;

	};

	template<uint32 NumOutputs>
	TTriggerSequenceOperator<NumOutputs>::TTriggerSequenceOperator(const FOperatorSettings& InSettings, 
		const FTriggerReadRef& InTrigger, 
		const FTriggerReadRef& InReset,
		const FBoolReadRef& bInLoop)
		: TriggerInput(InTrigger)
		, TriggerResetInput(InReset)
		, bLoopInput(bInLoop)
	{
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs.Add(FTriggerWriteRef::CreateNew(InSettings));
		}
	}

	template<uint32 NumOutputs>
	void TTriggerSequenceOperator<NumOutputs>::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerSequenceVertexNames;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), TriggerResetInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLoop), bLoopInput);
	}

	template<uint32 NumOutputs>
	void TTriggerSequenceOperator<NumOutputs>::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerSequenceVertexNames;
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(OutputTrigger, i), TriggerOutputs[i]);
		}
	}

	template<uint32 NumOutputs>
	void TTriggerSequenceOperator<NumOutputs>::Execute()
	{
		// Advance all output blocks
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs[i]->AdvanceBlock();
		}

		// Reset the Sequence
		TriggerResetInput->ExecuteBlock(
			[](int32, int32)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				CurrentIndex = 0;
			}
		);

		// Continue sequence
		TriggerInput->ExecuteBlock(
			[](int32, int32) 
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				if (CurrentIndex < NumOutputs)
				{
					// Send Trigger Out
					TriggerOutputs[CurrentIndex]->TriggerFrame(StartFrame);

					// Update index (wrap if necessary)
					++CurrentIndex;
				}
				if (*bLoopInput && CurrentIndex >= NumOutputs)
				{
					CurrentIndex = 0;
				}
			}
		);
	}

	template<uint32 NumOutputs>
	void TTriggerSequenceOperator<NumOutputs>::Reset(const IOperator::FResetParams& InParams)
	{
		for (uint32 i = 0; i < NumOutputs; ++i)
		{
			TriggerOutputs[i]->Reset();
		}
		CurrentIndex = 0;
	}

	template<uint32 NumOutputs>
	TUniquePtr<IOperator> TTriggerSequenceOperator<NumOutputs>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerSequenceVertexNames;

		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FTriggerReadRef TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
		FTriggerReadRef TriggerResetIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
		FBoolReadRef bLoopIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputLoop), InParams.OperatorSettings);

		return MakeUnique<TTriggerSequenceOperator>(InParams.OperatorSettings, TriggerIn, TriggerResetIn, bLoopIn);
	}

	template<uint32 NumOutputs>
	const FVertexInterface& TTriggerSequenceOperator<NumOutputs>::GetVertexInterface()
	{
		using namespace TriggerSequenceVertexNames;

		auto CreateVertexInterface = []() -> FVertexInterface
		{
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)));
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReset)));
			InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLoop), true));
			
			FOutputVertexInterface OutputInterface;
			for (uint32 i = 0; i < NumOutputs; ++i)
			{
				OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(OutputTrigger, i)));
			}
			return FVertexInterface(InputInterface, OutputInterface);
		};

		static const FVertexInterface Interface = CreateVertexInterface();
		return Interface;		
	}

	template<uint32 NumOutputs>
	const FNodeClassMetadata& TTriggerSequenceOperator<NumOutputs>::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FName OperatorName = *FString::Printf(TEXT("Trigger Sequence (%d)"), NumOutputs);
			FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerSequenceDisplayNamePattern", "Trigger Sequence ({0})", NumOutputs);
			const FText NodeDescription = METASOUND_LOCTEXT("TriggerSequenceDescription", "Each time this node is triggered, it sends the next output trigger in the sequence.");

			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, OperatorName, TEXT("") };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = NodeDisplayName;
			Info.Description = NodeDescription;
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	// Node Class
	template <uint32 NumOutputs>
	using TTriggerSequenceNode = TNodeFacade<TTriggerSequenceOperator<NumOutputs>>;

	REGISTER_TRIGGER_SEQUENCE_NODE(2)
	REGISTER_TRIGGER_SEQUENCE_NODE(3)
	REGISTER_TRIGGER_SEQUENCE_NODE(4)
	REGISTER_TRIGGER_SEQUENCE_NODE(5)
	REGISTER_TRIGGER_SEQUENCE_NODE(6)
	REGISTER_TRIGGER_SEQUENCE_NODE(7)
	REGISTER_TRIGGER_SEQUENCE_NODE(8)
}

#undef LOCTEXT_NAMESPACE
