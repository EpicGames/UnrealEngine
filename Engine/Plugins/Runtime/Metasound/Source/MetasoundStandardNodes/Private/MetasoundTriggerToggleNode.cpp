// Copyright Epic Games, Inc. All Rights Reserved.
#include "MetasoundTriggerToggleNode.h"

#include "CoreMinimal.h"

#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Toggle"

namespace Metasound
{
	namespace TriggerToggle
	{
		METASOUND_PARAM(InputOnTrigger, "On", "Trigger to toggle gate output to 1.")
		METASOUND_PARAM(InputOffTrigger, "Off", "Trigger to toggle gate output to 0.")
		METASOUND_PARAM(InputInit, "Init", "Initial value of the output gate.")
		METASOUND_PARAM(OutputTrigger, "Out", "Triggers output when gate is toggled.")
		METASOUND_PARAM(OutputValue, "Value", "Current output value of the toggle.")
	}

	class FTriggerToggleOperator : public TExecutableOperator<FTriggerToggleOperator>
	{
		public:
			static const FNodeClassMetadata& GetNodeInfo();
			static const FVertexInterface& GetVertexInterface();
			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

			FTriggerToggleOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerOn, const FTriggerReadRef& InTriggerOff, const FBoolReadRef& InInitValue);

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
			virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

			void Execute();
			void Reset(const IOperator::FResetParams& InParams);

		private:

			FTriggerReadRef TriggerOn;
			FTriggerReadRef TriggerOff;
			FBoolReadRef InitValue;
			FTriggerWriteRef TriggerOutput;
			FBoolWriteRef ValueOutput;
	};

	FTriggerToggleOperator::FTriggerToggleOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InTriggerOn, const FTriggerReadRef& InTriggerOff, const FBoolReadRef& InInitValue)
	: TriggerOn(InTriggerOn)
	, TriggerOff(InTriggerOff)
	, InitValue(InInitValue)
	, TriggerOutput(FTriggerWriteRef::CreateNew(InSettings))
	, ValueOutput(FBoolWriteRef::CreateNew(*InInitValue))
	{
	}

	void FTriggerToggleOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerToggle;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOnTrigger), TriggerOn);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputOffTrigger), TriggerOff);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInit), InitValue);
	}

	void FTriggerToggleOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		using namespace TriggerToggle;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputTrigger), TriggerOutput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), ValueOutput);
	}

	void FTriggerToggleOperator::Execute()
	{
		TriggerOutput->AdvanceBlock();

		TriggerOn->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				*ValueOutput = true;
				TriggerOutput->TriggerFrame(StartFrame);
			}
		);

		TriggerOff->ExecuteBlock(
			[&](int32 StartFrame, int32 EndFrame)
			{
			},
			[this](int32 StartFrame, int32 EndFrame)
			{
				*ValueOutput = false;
				TriggerOutput->TriggerFrame(StartFrame);
			}
		);
	}

	void FTriggerToggleOperator::Reset(const IOperator::FResetParams& InParams)
	{
		TriggerOutput->Reset();
		*ValueOutput = *InitValue;
	}

	TUniquePtr<IOperator> FTriggerToggleOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace TriggerToggle;

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		FTriggerReadRef InTriggerOn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputOnTrigger), InParams.OperatorSettings);
		FTriggerReadRef InTriggerOff = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputOffTrigger), InParams.OperatorSettings);
		FBoolReadRef InInitValue = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputInit), InParams.OperatorSettings);

		return MakeUnique<FTriggerToggleOperator>(InParams.OperatorSettings, InTriggerOn, InTriggerOff, InInitValue);
	}

	const FVertexInterface& FTriggerToggleOperator::GetVertexInterface()
	{
		using namespace TriggerToggle;

		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOnTrigger)),
				TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputOffTrigger)),
				TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInit))
			),
			FOutputVertexInterface(
				TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTrigger)),
				TOutputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FTriggerToggleOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = {StandardNodes::Namespace, TEXT("Trigger Toggle"), TEXT("")};
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_TriggerToggleNodeDisplayName", "Trigger Toggle");
			Info.Description = METASOUND_LOCTEXT("Metasound_TriggerToggleNodeDescription", "Toggles a boolean value on or off.");
			Info.Author = PluginAuthor;
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();
			Info.CategoryHierarchy.Emplace(NodeCategories::Trigger);

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();
		return Info;
	}

	FTriggerToggleNode::FTriggerToggleNode(const FNodeInitData& InInitData)
	:	FNodeFacade(InInitData.InstanceName, InInitData.InstanceID, TFacadeOperatorClass<FTriggerToggleOperator>())
	{
	}

	FTriggerToggleNode::FTriggerToggleNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata)
	:	FNodeFacade(MoveTemp(InNodeData), InMetadata, TFacadeOperatorClass<FTriggerToggleOperator>())
	{
	}

	FNodeClassMetadata FTriggerToggleNode::CreateNodeClassMetadata()
	{
		return FTriggerToggleOperator::GetNodeInfo();
	}
	

	METASOUND_REGISTER_NODE(FTriggerToggleNode)
}

#undef LOCTEXT_NAMESPACE // MetasoundTriggerDelayNode
