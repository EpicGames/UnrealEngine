// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundValueNode.h"

#include "MetasoundNodeRegistrationMacro.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_Value"

namespace Metasound
{
	namespace MetasoundValueNodePrivate
	{
		FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName{ "Value", InOperatorName, InDataTypeName },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ METASOUND_LOCTEXT("ValueCategory", "Value") },
				{ },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}
	}


	using FValueNodeInt32 = TValueNode<int32>;
 	METASOUND_REGISTER_NODE(FValueNodeInt32)

	using FValueNodeFloat = TValueNode<float>;
	METASOUND_REGISTER_NODE(FValueNodeFloat)

	using FValueNodeBool = TValueNode<bool>;
	METASOUND_REGISTER_NODE(FValueNodeBool)

	using FValueNodeString = TValueNode<FString>;
	METASOUND_REGISTER_NODE(FValueNodeString)
}

#undef LOCTEXT_NAMESPACE
