// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/WaveShaper.h"
#include "Internationalization/Text.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundAudioBuffer.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_WaveShaperNode"


namespace Metasound
{
	namespace WaveShaperNode
	{
		METASOUND_PARAM(InAudioInput, "In", "Audio input.");
		METASOUND_PARAM(InWaveShapeAmount, "Amount", "The amount of wave shaping to apply.");
		METASOUND_PARAM(InWaveShapeBias, "Bias", "DC offset to apply before wave shaping.");
		METASOUND_PARAM(InOutputGain, "OutputGain", "The amount of gain to apply after processing.");
		METASOUND_PARAM(InType, "Type", "Algorithm to use to process the audio.");
		METASOUND_PARAM(OutAudio, "Out", "Audio output.");
	}

	using namespace WaveShaperNode;

	DECLARE_METASOUND_ENUM(Audio::EWaveShaperType, Audio::EWaveShaperType::ATan, METASOUNDSTANDARDNODES_API, FEnumEWaveShaperType, FEnumWaveShaperTypeInfo, FEnumWaveShaperReadRef, FEnumWaveShaperWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(Audio::EWaveShaperType, FEnumEWaveShaperType, "WaveShaperType")
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EWaveShaperType::Sin, "SinDescription", "Sine", "SinTT", "Sine WaveShaper"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EWaveShaperType::ATan, "ATanDescription", "Inverse Tangent", "ATanTT", "Inverse Tangent WaveShaper"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EWaveShaperType::Tanh, "TanHDescription", "Hyperbolic Tangent", "TanHTT", "Hyperbolic Tangent WaveShaper"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EWaveShaperType::Cubic, "CubicDescription", "Cubic Polynomial", "CubicTT", "Cubic Polynomial WaveShaper"),
		DEFINE_METASOUND_ENUM_ENTRY(Audio::EWaveShaperType::HardClip, "HarClipDescription", "Hard Clip", "HardClipTT", "Hard Clipper")
		DEFINE_METASOUND_ENUM_END()

	class FWaveShaperOperator : public TExecutableOperator<FWaveShaperOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo();
		static const FVertexInterface& GetVertexInterface();
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		FWaveShaperOperator(const FOperatorSettings& InSettings, 
			const FAudioBufferReadRef& InAudioInput, 
			const FFloatReadRef& InWaveShapeAmount,
			const FFloatReadRef& InBias,
			const FFloatReadRef& InOutputGain,
			const FEnumWaveShaperReadRef& InType);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;
		void Execute();
		void Reset(const IOperator::FResetParams& InParams);

	private:
		// The input audio buffer
		FAudioBufferReadRef AudioInput;

		// The amount that the wave is shaped
		FFloatReadRef Amount;

		// DC offset to apply before gain
		FFloatReadRef Bias;

		// The amount of gain
		FFloatReadRef OutputGain;

		// What algorithm to use
		FEnumWaveShaperReadRef Type;

		// The audio output
		FAudioBufferWriteRef AudioOutput;

		// The internal WaveShaper
		Audio::FWaveShaper WaveShaper;
	};

	FWaveShaperOperator::FWaveShaperOperator(const FOperatorSettings& InSettings, 
		const FAudioBufferReadRef& InAudioInput, 
		const FFloatReadRef& InAmount,
		const FFloatReadRef& InBias,
		const FFloatReadRef& InOutputGain,
		const FEnumWaveShaperReadRef& InType)
		: AudioInput(InAudioInput)
		, Amount(InAmount)
		, Bias(InBias)
		, OutputGain(InOutputGain)
		, Type(InType)
		, AudioOutput(FAudioBufferWriteRef::CreateNew(InSettings))
	{
		WaveShaper.Init(InSettings.GetSampleRate());
	}

	void FWaveShaperOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InAudioInput), AudioInput);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InWaveShapeAmount), Amount);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InWaveShapeBias), Bias);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InOutputGain), OutputGain);
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InType), Type);
	}

	void FWaveShaperOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData)
	{
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutAudio), AudioOutput);
	}

	void FWaveShaperOperator::Execute()
	{
		const float* InputAudio = AudioInput->GetData();

		float* OutputAudio = AudioOutput->GetData();
		int32 NumFrames = AudioInput->Num();

		WaveShaper.SetAmount(*Amount);
		WaveShaper.SetBias(*Bias);
		WaveShaper.SetOutputGainLinear(*OutputGain);
		WaveShaper.SetType(*Type);

		WaveShaper.ProcessAudioBuffer(InputAudio, OutputAudio, NumFrames);
	}

	void FWaveShaperOperator::Reset(const IOperator::FResetParams& InParams)
	{
		AudioOutput->Zero();
		WaveShaper.Init(InParams.OperatorSettings.GetSampleRate());
	}

	const FVertexInterface& FWaveShaperOperator::GetVertexInterface()
	{
		static const FVertexInterface Interface(
			FInputVertexInterface(
				TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InAudioInput)),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InWaveShapeAmount), 1.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InWaveShapeBias), 0.0f),
				TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InOutputGain), 1.0f),
				TInputDataVertex<FEnumEWaveShaperType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InType), (int32)Audio::EWaveShaperType::Sin)
			),
			FOutputVertexInterface(
				TOutputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutAudio))
			)
		);

		return Interface;
	}

	const FNodeClassMetadata& FWaveShaperOperator::GetNodeInfo()
	{
		auto InitNodeInfo = []() -> FNodeClassMetadata
		{
			FNodeClassMetadata Info;
			Info.ClassName = { StandardNodes::Namespace, TEXT("WaveShaper"), StandardNodes::AudioVariant };
			Info.MajorVersion = 1;
			Info.MinorVersion = 0;
			Info.DisplayName = METASOUND_LOCTEXT("Metasound_WaveShaperDisplayName", "WaveShaper");
			Info.Description = METASOUND_LOCTEXT("Metasound_WaveShaperNodeDescription", "Applies non-linear shaping to the audio input.");
			Info.Author = PluginAuthor;
			Info.CategoryHierarchy = { NodeCategories::Filters };
			Info.PromptIfMissing = PluginNodeMissingPrompt;
			Info.DefaultInterface = GetVertexInterface();

			return Info;
		};

		static const FNodeClassMetadata Info = InitNodeInfo();

		return Info;
	}

	TUniquePtr<IOperator> FWaveShaperOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		FAudioBufferReadRef AudioIn = InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InAudioInput), InParams.OperatorSettings);
		FFloatReadRef InAmount = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InWaveShapeAmount), InParams.OperatorSettings);
		FFloatReadRef InBias = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InWaveShapeBias), InParams.OperatorSettings);
		FFloatReadRef OutGain = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InOutputGain), InParams.OperatorSettings);
		FEnumWaveShaperReadRef InType = InputData.GetOrCreateDefaultDataReadReference<FEnumEWaveShaperType>(METASOUND_GET_PARAM_NAME(InType), InParams.OperatorSettings);

		return MakeUnique<FWaveShaperOperator>(InParams.OperatorSettings, AudioIn, InAmount, InBias, OutGain, InType);
	}

	using FWaveShaperNode = TNodeFacade<FWaveShaperOperator>;

	METASOUND_REGISTER_NODE(FWaveShaperNode)
}

#undef LOCTEXT_NAMESPACE
