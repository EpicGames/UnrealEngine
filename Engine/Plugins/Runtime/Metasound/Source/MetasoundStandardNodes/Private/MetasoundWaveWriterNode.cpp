// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/SimpleWaveWriter.h"
#include "NumberedFileCache.h"
#include "HAL/FileManager.h"
#include "MetasoundBuildError.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundVertex.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_WaveWriterNode"


namespace Metasound
{
	class FNode;
	class FBuildErrorBase;
	class IOperator;

	namespace WaveWriterVertexNames
	{
		METASOUND_PARAM(InEnabledPin, "Enabled", "If this wave writer is enabled or not.");
		METASOUND_PARAM(InFilenamePrefixPin, "Filename Prefix", "Filename Prefix of file you are writing.");
	}

	class FFileWriteError : public FBuildErrorBase
	{
	public:
		static const FName ErrorType;

		virtual ~FFileWriteError() = default;

		FFileWriteError(const FNode& InNode, const FString& InFilename)
#if WITH_EDITOR
			: FBuildErrorBase(ErrorType, METASOUND_LOCTEXT_FORMAT("MetasoundFileWriterErrorDescription", "File Writer Error while trying to write '{0}'", FText::FromString(InFilename)))
#else 
			: FBuildErrorBase(ErrorType, FText::GetEmpty())
#endif // WITH_EDITOR
		{
			AddNode(InNode);
		}
	};
	const FName FFileWriteError::ErrorType = FName(TEXT("MetasoundFileWriterError"));

	namespace WaveWriterOperatorPrivate
	{
		// Need to keep this outside the template so there's only 1
		TSharedPtr<FNumberedFileCache> GetNameCache()
		{
			static const TCHAR* WaveExt = TEXT(".wav");

			// Build cache of numbered files (do this once only).
			static TSharedPtr<FNumberedFileCache> NumberedFileCacheSP = MakeShared<FNumberedFileCache>(
				*FPaths::AudioCaptureDir(), WaveExt, IFileManager::Get());
			
			return NumberedFileCacheSP;
		}

		static const FString GetDefaultFileName()
		{
			static const FString DefaultFileName = TEXT("Output");
			return DefaultFileName;
		}
	}

	template<int32 NumInputChannels>
	class TWaveWriterOperator : public TExecutableOperator<TWaveWriterOperator<NumInputChannels>>
	{
	public:
		// Theoretical limit of .WAV files.
		static_assert(NumInputChannels > 0 && NumInputChannels <= 65535, "Num Channels > 0 and <= 65535");

		TWaveWriterOperator(const FOperatorSettings& InSettings, TArray<FAudioBufferReadRef>&& InAudioBuffers, FBoolReadRef&& InEnabled, const TSharedPtr<FNumberedFileCache, ESPMode::ThreadSafe>& InNumberedFileCache, FStringReadRef&& InFilenamePrefix)
			: AudioInputs{ MoveTemp(InAudioBuffers) }
			, Enabled{ MoveTemp(InEnabled) }
			, NumberedFileCacheSP{ InNumberedFileCache }
			, FileNamePrefix{ MoveTemp(InFilenamePrefix) }
			, SampleRate{ InSettings.GetSampleRate() }
		{
			check(AudioInputs.Num() == NumInputChannels);

			// Make an interleave buffer if we need one.
			if (NumInputChannels > 1)
			{
				InterleaveBuffer.SetNum(InSettings.GetNumFramesPerBlock() * NumInputChannels);
				AudioInputBufferPtrs.SetNum(NumInputChannels);
				for (int32 i = 0; i < NumInputChannels; ++i)
				{
					AudioInputBufferPtrs[i] = AudioInputs[i]->GetData();
				}
			}
		}


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			for (int32 i = 0; i < NumInputChannels; ++i)
			{
				InOutVertexData.BindReadVertex(GetAudioInputName(i), AudioInputs[i]);
			}

			using namespace WaveWriterVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InEnabledPin), Enabled);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InFilenamePrefixPin), FileNamePrefix);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData&) override
		{
		}

		static const FVertexInterface& DeclareVertexInterface()
		{
			auto CreateDefaultInterface = []()-> FVertexInterface
			{
				using namespace WaveWriterOperatorPrivate;
				using namespace WaveWriterVertexNames;

				// inputs
				FInputVertexInterface InputInterface(
					TInputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(InFilenamePrefixPin), GetDefaultFileName()),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InEnabledPin), true)
				);


				// For backwards compatibility with previous (mono) node, in the case of 1 channels, just provide the old interface.
				for (int32 InputIndex = 0; InputIndex < NumInputChannels; ++InputIndex)
				{
#if WITH_EDITOR
					const FDataVertexMetadata AudioInputMetadata
					{
						  GetAudioInputDescription(InputIndex) // description
						, GetAudioInputDisplayName(InputIndex) // display name
					};
#else
					const FDataVertexMetadata AudioInputMetadata;
#endif //WITH_EDITOR
					InputInterface.Add(TInputDataVertex<FAudioBuffer>(GetAudioInputName(InputIndex), AudioInputMetadata));
				}
				FOutputVertexInterface OutputInterface;

				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			// used if NumChannels == 1
			auto CreateNodeClassMetadataMono = []() -> FNodeClassMetadata
			{
				// For backwards compatibility with previous (mono) WaveWriters keep the node name the same.
				FName OperatorName = TEXT("WaveWriter");
				FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_WaveWriterNodeMonoDisplayName", "Wave Writer (1-channel, Mono)");
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_WaveWriterNodeMonoDescription", "Write a mono audio signal to disk");
				FVertexInterface NodeInterface = DeclareVertexInterface();

				return CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels == 2
			auto CreateNodeClassMetadataStereo = []() -> FNodeClassMetadata
			{
				FName OperatorName = TEXT("Wave Writer (Stereo)");
				FText NodeDisplayName = METASOUND_LOCTEXT("Metasound_WaveWriterNodeStereoDisplayName", "Wave Writer (2-channel, Stereo)");
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_WaveWriterNodeStereoDescription", "Write a stereo audio signal to disk");
				FVertexInterface NodeInterface = DeclareVertexInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			// used if NumChannels > 2
			auto CreateNodeClassMetadataMultiChan = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Wave Writer (%d-Channel)"), NumInputChannels);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_WaveWriterNodeMultiChannelDisplayName", "Wave Writer ({0}-channel)", NumInputChannels);
				const FText NodeDescription = METASOUND_LOCTEXT("Metasound_WaveWriterNodeMultiDescription", "Write a multi-channel audio signal to disk");
				FVertexInterface NodeInterface = DeclareVertexInterface();

				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = (NumInputChannels == 1) ? CreateNodeClassMetadataMono()
				: (NumInputChannels == 2) ? CreateNodeClassMetadataStereo() : CreateNodeClassMetadataMultiChan();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		void Execute()
		{
			// Enabled and wasn't before? Enable.
			if (!bIsEnabled && *Enabled)
			{
				Enable();
			}
			// Disabled but currently Enabled? Disable.
			else if (bIsEnabled && !*Enabled)
			{
				Disable();
			}

			// If we have a valid writer and enabled.
			if (Writer && *Enabled)
			{
				// Need to Interleave?
				if (NumInputChannels > 1)
				{
					InterleaveChannels(AudioInputBufferPtrs.GetData(), NumInputChannels, AudioInputs[0]->Num(), InterleaveBuffer.GetData());
					Writer->Write(MakeArrayView(InterleaveBuffer.GetData(), InterleaveBuffer.Num()));
				}
				else if (NumInputChannels == 1)
				{
					Writer->Write(MakeArrayView(AudioInputs[0]->GetData(), AudioInputs[0]->Num()));
				}
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			if (bIsEnabled)
			{
				Disable();
			}
		}

	protected:
		static const FVertexName GetAudioInputName(int32 InInputIndex)
		{
			if (NumInputChannels == 1)
			{
				// To maintain backwards compatibility with previous implementation keep the pin name the same.
				static const FName AudioInputPinName = TEXT("In");
				return AudioInputPinName;
			}
			else if (NumInputChannels == 2)
			{
				return *FString::Printf(TEXT("In %d %s"), InInputIndex, (InInputIndex == 0) ? TEXT("L") : TEXT("R"));
			}

			return *FString::Printf(TEXT("In %d"), InInputIndex);
		}

#if WITH_EDITOR 
		static const FText GetAudioInputDisplayName(int32 InInputIndex)
		{
			if (NumInputChannels == 1)
			{
				// To maintain backwards compatibility with previous implementation keep the pin name the same.
				static const FText AudioInputPinName = METASOUND_LOCTEXT("AudioInputPinNameIn", "In");
				return AudioInputPinName;
			}
			else if (NumInputChannels == 2)
			{
				if (InInputIndex == 0)
				{
					return METASOUND_LOCTEXT_FORMAT("AudioInputIn2ChannelNameL", "In {0} L", InInputIndex);
				}
				else
				{
					return METASOUND_LOCTEXT_FORMAT("AudioInputIn2ChannelNameR", "In {0} R", InInputIndex);
				}
			}
			return METASOUND_LOCTEXT_FORMAT("AudioInputInChannelName", "In {0}", InInputIndex);
		}

		static const FText GetAudioInputDescription(int32 InputIndex)
		{
			return METASOUND_LOCTEXT_FORMAT("WaveWriterAudioInputDescription", "Audio Input #: {0}", InputIndex);
		}
#endif // WITH_EDITOR

		static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { StandardNodes::Namespace, InOperatorName, StandardNodes::AudioVariant },
				1, // Major Version
				1, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Io },
				{ METASOUND_LOCTEXT("Metasound_AudioMixerKeyword", "Writer") },
				FNodeDisplayStyle{}
			};
			return Metadata;
		}

		// TODO. Move to DSP lib.
		static void InterleaveChannels(const float* RESTRICT InMonoChannelsToInterleave[], const int32 InNumChannelsToInterleave, const int32 NumSamplesPerChannel, float* RESTRICT OutInterleavedBuffer)
		{
			for (int32 Sample = 0; Sample < NumSamplesPerChannel; ++Sample)
			{
				for (int32 Channel = 0; Channel < InNumChannelsToInterleave; ++Channel)
				{
					*OutInterleavedBuffer++ = InMonoChannelsToInterleave[Channel][Sample];
				}
			}
		}

		void Enable()
		{
			if (ensure(!bIsEnabled))
			{
				bIsEnabled = true;
				FString Filename = NumberedFileCacheSP->GenerateNextNumberedFilename(*FileNamePrefix);
				TUniquePtr<FArchive> Stream{ IFileManager::Get().CreateFileWriter(*Filename, IO_WRITE) };
				if (Stream.IsValid())
				{
					Writer = MakeUnique<Audio::FSimpleWaveWriter>(MoveTemp(Stream), SampleRate, NumInputChannels, true);
				}
			}
		}
		void Disable()
		{
			if (ensure(bIsEnabled))
			{
				bIsEnabled = false;
				Writer.Reset();
			}
		}

		TArray<FAudioBufferReadRef> AudioInputs;
		TArray<const float*> AudioInputBufferPtrs;
		TArray<float> InterleaveBuffer;
		FBoolReadRef Enabled;
		TUniquePtr<Audio::FSimpleWaveWriter> Writer;
		TSharedPtr<FNumberedFileCache, ESPMode::ThreadSafe> NumberedFileCacheSP;
		FStringReadRef FileNamePrefix;
		float SampleRate = 0.f;
		bool bIsEnabled = false;
	};

	template<int32 NumInputChannels>
	TUniquePtr<Metasound::IOperator> TWaveWriterOperator<NumInputChannels>::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace WaveWriterOperatorPrivate;
		using namespace WaveWriterVertexNames;

		const FOperatorSettings& Settings = InParams.OperatorSettings;
		const FInputVertexInterfaceData& InputData = InParams.InputData;

		int32 NumConnectedAudioPins = 0;
		TArray<FAudioBufferReadRef> InputBuffers;
		for (int32 i = 0; i < NumInputChannels; ++i)
		{
			const FVertexName PinName = GetAudioInputName(i);
			if (InputData.IsVertexBound(PinName))
			{
				NumConnectedAudioPins++;
			}
			InputBuffers.Add(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(PinName, InParams.OperatorSettings));
		}
		
		// Only create a real operator if there's some connected pins.
		if (NumConnectedAudioPins > 0)
		{
			return MakeUnique<TWaveWriterOperator>(
				Settings,
				MoveTemp(InputBuffers),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InEnabledPin), Settings),
				GetNameCache(),
				InputData.GetOrCreateDefaultDataReadReference<FString>(METASOUND_GET_PARAM_NAME(InFilenamePrefixPin), Settings)
			);
		}

		// Create a no-op operator.
		return MakeUnique<FNoOpOperator>();
	}

	template<int32 NumInputChannels>
	using TWaveWriterNode = TNodeFacade<TWaveWriterOperator<NumInputChannels>>;
		
	#define REGISTER_WAVEWRITER_NODE(A) \
		using FWaveWriterNode_##A = TWaveWriterNode<A>; \
		METASOUND_REGISTER_NODE(FWaveWriterNode_##A)
	
	REGISTER_WAVEWRITER_NODE(1);
	REGISTER_WAVEWRITER_NODE(2);
	REGISTER_WAVEWRITER_NODE(3);
	REGISTER_WAVEWRITER_NODE(4);
	REGISTER_WAVEWRITER_NODE(5);
	REGISTER_WAVEWRITER_NODE(6);
	REGISTER_WAVEWRITER_NODE(7);
	REGISTER_WAVEWRITER_NODE(8);
}

#undef LOCTEXT_NAMESPACE
