// Copyright Epic Games, Inc. All Rights Reserved.
#include "NumberedFileCache.h"

namespace Metasound 
{
	const FString FNumberedFileCache::Separator{ TEXT("_") };
} //namespace Audio


