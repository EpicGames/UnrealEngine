// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundDataReference.h"
#include "MetasoundAudioBuffer.h"
#include "MetasoundDataTypeRegistrationMacro.h"

#define UE_API METASOUNDSTANDARDNODES_API

namespace Metasound
{
	/** FMultichannelAudioFormat
	 *
	 * FMultichannelAudioFormat represents deinterleaved multichannel audio which supports a constant
	 * number of channels for the lifetime of the object. 
	 *
	 * The audio buffers in FMultichannelAudioFormat are shared data references which can be accessed outside
	 * of the FMultichannelAudioFormat. All audio buffers within a FMultichannelAudioFormat object must contain the 
	 * same number of audio frames.
	 */
	class FMultichannelAudioFormat : public IAudioDataType
	{
		public:
			UE_API FMultichannelAudioFormat();

			/** FMultichannelAudioFormat Constructor.
			 *
			 * @param InNumFrames - The number of frames per an audio buffer.
			 * @param InNumChannels - The number of audio channels.
			 */
			UE_API FMultichannelAudioFormat(int32 InNumFrames, int32 InNumChannels);

			/**
			 * FMultichannelAudioFormat Constructor used by the metasound frontend.
			 *
			 * @param InSettings - Operator Settings passed in on construction.
			 * @param InNumChannels - initial number of audio channels.
			 */
			UE_API explicit FMultichannelAudioFormat(const FOperatorSettings& InSettings, int32 InNumChannels);

			/** FMultichannelAudioFormat Constructor.
			 *
			 * This constructor accepts an array of writable audio buffer references. Each 
			 * buffer in the array must contain equal number of frames. 
			 *
			 * @param InWriteRefs - An array of writable audio buffer references. 
			 */
			UE_API FMultichannelAudioFormat(TArrayView<const FAudioBufferWriteRef> InWriteRefs);

			// Enable the copy constructor.
			FMultichannelAudioFormat(const FMultichannelAudioFormat& InOther) = default;

			// Disable move constructor as incoming object should not be altered. 
			//FMultichannelAudioFormat(FMultichannelAudioFormat&& InOther) = delete;

			// Disable equal operator so channel count does not change
			FMultichannelAudioFormat& operator=(const FMultichannelAudioFormat& Other) = delete;

			// Disable move operator so channel count does not change
			//FMultichannelAudioFormat& operator=(FMultichannelAudioFormat&& Other) = delete;

			/** Return the number of audio channels. */
			int32 GetNumChannels() const { return NumChannels; }

			/** Return the maximum number of channels. Multichannel audio buffers cannot be resized after construction. */
			int32 GetMaxNumChannels() const { return NumChannels; }

			/** Return an array view of the readable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArrayView<const FAudioBufferReadRef> GetBuffers() const { return ReadableBuffers; }

			/** Return an array view of the writable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArrayView<const FAudioBufferWriteRef> GetBuffers() { return WritableBuffers; }

			/** Return an array of the readable buffer reference storage.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArray<FAudioBufferReadRef>& GetStorage() const { return ReadableBufferStorage; }

			/** Return an array of the writable buffer reference storage.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArray<FAudioBufferWriteRef>& GetStorage() { return WritableBufferStorage; }

		private:
			// Friendship with the data reference class gives it access to the protected constructor
			// for the scenario where a TDataReadReference<FMulitchannelAudioFormat> is constructed
			// with FAudioBufferReadRefs. The constructor cannot be public as it would provide writable
			// access to the passed in audio buffers, even though the passed buffers explicitly were
			// read references.
			friend class TDataReadReference<FMultichannelAudioFormat>;

			// Construct a FMultichannelAudioFormat with an array of readable buffers. 
			// 
			// This constructor should only be used when it can be assured that the constructed object
			// will not provide writable access to the passed in audio buffers. 
			UE_API FMultichannelAudioFormat(TArrayView<const FAudioBufferReadRef> InReadRefs);

			int32 NumChannels;

			TArrayView<const FAudioBufferWriteRef> WritableBuffers;
			TArrayView<const FAudioBufferReadRef> ReadableBuffers;

			TArray<FAudioBufferWriteRef> WritableBufferStorage;
			TArray<FAudioBufferReadRef> ReadableBufferStorage;
	};

	
	/** A TStaticChannelAudioFormat represents deinterleaved multichannel audio 
	 * where the number of channels is known at compile time. This is primarily 
	 * useful to define such cases as Stereo, Mono, Qaud, 5.1, etc. 
	 *
	 * The audio buffers in FMultichannelAudioFormat are shared data references 
	 * which can be accessed outside of the FMultichannelAudioFormat. All audio 
	 * buffers within a FMultichannelAudioFormat object must contain the same 
	 * number of audio frames.
	 */
	template<int32 TNumChannels>
	class TStaticChannelAudioFormat : public IAudioDataType
	{
		public:
			static constexpr int32 NumChannels = TNumChannels;

			/** TStaticChannelAudioFormat Constructor
			 *
			 * @param InNumFrames - The number of frames per an audio buffer.
			 */
			TStaticChannelAudioFormat(int32 InNumFrames)
			{
				static_assert(NumChannels > 0, "NumChannels must be greater than zero");

				InNumFrames = FMath::Max(InNumFrames, 0);

				for (int32 i = 0; i < NumChannels; i++)
				{
					FAudioBufferWriteRef Audio = FAudioBufferWriteRef::CreateNew(InNumFrames);
					Audio->Zero();

					WritableBufferStorage.Add(Audio);
					ReadableBufferStorage.Add(Audio);
				}

				WritableBuffers = WritableBufferStorage;
				ReadableBuffers = ReadableBufferStorage;
			}

			TStaticChannelAudioFormat(const FOperatorSettings& InOperatorSettings)
			:	TStaticChannelAudioFormat(InOperatorSettings.GetNumFramesPerBlock())
			{
			}


			/** Return the number of audio channels. */
			int32 GetNumChannels() const
			{
				return NumChannels;
			}

			/** Return the maximum number of channels. static channel audio buffers cannot be resized after construction. */
			int32 GetMaxNumChannels() const { return NumChannels; }

			/** Return an readable buffer reference for a specific channel. */
			template<int32 ChannelIndex>
			FAudioBufferReadRef GetBuffer() const
			{
				static_assert(ChannelIndex >= 0, "Index must be within range of channels");
				static_assert(ChannelIndex < NumChannels, "Index must be within range of channels");

				return ReadableBuffers.GetData()[ChannelIndex];
			}

			/** Return an writable buffer reference for a specific channel. */
			template<int32 ChannelIndex>
			FAudioBufferWriteRef GetBuffer() 
			{
				static_assert(ChannelIndex >= 0, "Index must be within range of channels");
				static_assert(ChannelIndex < NumChannels, "Index must be within range of channels");

				return WritableBuffers.GetData()[ChannelIndex];
			}

			/** Return an array view of the readable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArrayView<FAudioBufferReadRef> GetBuffers() const
			{
				return ReadableBuffers;
			}

			/** Return an array view of the writable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArrayView<FAudioBufferWriteRef> GetBuffers() 
			{
				return WritableBuffers;
			}

			/** Return an array of the readable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArray<FAudioBufferReadRef> GetStorage() const
			{
				return ReadableBufferStorage;
			}

			/** Return an array of the writable buffer references.
			 *
			 * This array will have GetNumChannels() elements.
			 */
			const TArray<FAudioBufferWriteRef> GetStorage()
			{
				return WritableBufferStorage;
			}

		protected:

			// TStaticChannelAudioFormat constructor with an array of writable buffers. 
			TStaticChannelAudioFormat(const FAudioBufferWriteRef(&InBuffers)[NumChannels])
			{
				int32 NumFrames = 0;
				if (NumChannels > 0)
				{
					NumFrames = InBuffers[0]->Num();
				}

				for (int32 i = 0; i < NumChannels; i++)
				{
					checkf(NumFrames == InBuffers[i]->Num(), TEXT("All buffers must have same number of frames (%d != %d)"), NumFrames, InBuffers[i]->Num());

					WritableBufferStorage.Add(InBuffers[i]);
					ReadableBufferStorage.Add(InBuffers[i]);
				}

				WritableBuffers = WritableBufferStorage;
				ReadableBuffers = ReadableBufferStorage;
			}

		private:
			TArrayView<FAudioBufferWriteRef> WritableBuffers;
			TArrayView<FAudioBufferReadRef> ReadableBuffers;

			TArray<FAudioBufferWriteRef> WritableBufferStorage;
			TArray<FAudioBufferReadRef> ReadableBufferStorage;
	};

	/** FMonoAudioFormat represents mono audio containing one channel of audio.
	 *
	 * The audio buffer is a shared data references which can be accessed 
	 * outside of the FMonoAudioFormat. 
	 */
	class FMonoAudioFormat : public TStaticChannelAudioFormat<1>
	{
		public:
			using Super = TStaticChannelAudioFormat<1>;

			// Inherit constructors of base class.
			using Super::Super;

			/** FMonoAudioFormat Construtor
			 *
			 * Construct with a single writable audio buffer reference.
			 *
			 * @param InAudio - A writable audio buffer reference.
			 */
			FMonoAudioFormat(const FAudioBufferWriteRef& InAudio)
			:	Super({InAudio})
			{
			}

			/** Return writable audio buffer reference of center channel. */
			FAudioBufferWriteRef GetCenter() { return GetBuffer<0>(); }

			/** Return readable audio buffer reference of center channel. */
			FAudioBufferReadRef GetCenter() const { return GetBuffer<0>(); }
	};
	

	/** FStereoAudioFormat represents stereo audio containing two channels of 
	 * audio.
	 *
	 * The audio buffers are shared data references which can be accessed 
	 * outside of the FStereoAudioFormat. 
	 */
	class FStereoAudioFormat : public TStaticChannelAudioFormat<2>
	{
		public:
			using Super = TStaticChannelAudioFormat<2>;

			// Inherit constructors of base class.
			using Super::Super;

			/** FStereoAudioFormat Construtor
			 *
			 * Construct with a two writable audio buffer reference.
			 *
			 * @param InLeftAudio - A writable audio buffer reference for the left channel.
			 * @param InRightAudio - A writable audio buffer reference for the right channel.
			 */
			FStereoAudioFormat(const FAudioBufferWriteRef& InLeftAudio, const FAudioBufferWriteRef& InRightAudio)
			:	Super({InLeftAudio, InRightAudio})
			{
			}

			/** Return writable audio buffer reference of left channel. */
			FAudioBufferWriteRef GetLeft() { return GetBuffer<0>(); }

			/** Return readable audio buffer reference of left channel. */
			FAudioBufferReadRef GetLeft() const { return GetBuffer<0>(); }

			/** Return writable audio buffer reference of right channel. */
			FAudioBufferWriteRef GetRight() { return GetBuffer<1>(); }

			/** Return readable audio buffer reference of right channel. */
			FAudioBufferReadRef GetRight() const { return GetBuffer<1>(); }
	};


	// TODO: currently unused. Commenting out to keep clean UX.
	//DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMultichannelAudioFormat, METASOUNDSTANDARDNODES_API, FMultichannelAudioFormatTypeInfo, FMultichannelAudioFormatReadRef, FMultichannelAudioFormatWriteRef);

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FMonoAudioFormat, METASOUNDSTANDARDNODES_API, FMonoAudioFormatTypeInfo, FMonoAudioFormatReadRef, FMonoAudioFormatWriteRef);

	DECLARE_METASOUND_DATA_REFERENCE_TYPES(FStereoAudioFormat, METASOUNDSTANDARDNODES_API, FStereoAudioFormatTypeInfo, FStereoAudioFormatReadRef, FStereoAudioFormatWriteRef);
}

#undef UE_API
