// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"

#define UE_API METASOUNDSTANDARDNODES_API

namespace Metasound
{
	/** FBPMToSecondsNode
	 *
	 *  Calculates a beat time in seconds from the given BPM, beat multiplier and divisions of a whole note. 
	 */
	class FBPMToSecondsNode : public FNodeFacade
	{
	public:
		/**
		 * Constructor used by the Metasound Frontend.
		 */
		UE_API FBPMToSecondsNode(const FNodeInitData& InitData);

		UE_API FBPMToSecondsNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
} // namespace Metasound

#undef UE_API
