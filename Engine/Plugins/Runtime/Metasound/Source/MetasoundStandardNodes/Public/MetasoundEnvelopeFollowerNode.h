// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"

#define UE_API METASOUNDSTANDARDNODES_API

namespace Metasound
{
	/** FEnvelopeFollowerNode
	 *
	 *  Delays an audio buffer by a specified amount.
	 */
	class FEnvelopeFollowerNode : public FNodeFacade
	{
	public:
		/**
		 * Constructor used by the Metasound Frontend.
		 */
		UE_API FEnvelopeFollowerNode(const FNodeInitData& InitData);

		UE_API FEnvelopeFollowerNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
} // namespace Metasound

#undef UE_API
