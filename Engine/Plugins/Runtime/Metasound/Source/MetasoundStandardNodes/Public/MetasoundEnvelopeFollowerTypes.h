// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundEnumRegistrationMacro.h"

namespace Metasound
{
	enum class EEnvelopePeakMode
	{
		Peak = 0,
		MeanSquared,
		RootMeanSquared,
	};

	DECLARE_METASOUND_ENUM(EEnvelopePeakMode, EEnvelopePeakMode::Peak, METASOUNDSTANDARDNODES_API,
	FEnumEnvelopePeakMode, FEnumEnvelopePeakModeInfo, FEnvelopePeakModeReadRef, FEnumEnvelopePeakModeWriteRef);
}