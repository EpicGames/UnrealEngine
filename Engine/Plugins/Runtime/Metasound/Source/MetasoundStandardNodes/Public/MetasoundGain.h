// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"

namespace Metasound
{
	/** Representation of a gain */
	enum class UE_DEPRECATED(5.6, "EGainRepresentation is not used and will be removed") EGainRepresentation  : uint8
	{
		/** Gain in linear scale  */
		Linear,

		/** Gain is in decibels */
		Decibels
	};

	/** FGain represents a gain value. It provides clearly defined 
	 * getters and setters as well as several convenience functions for 
	 * converting gain to decibels.
	 */
	class UE_DEPRECATED(5.6, "FGain is not used and will be removed") FGain
	{
		public:
			static constexpr const float DefaultGain = 1.0f;

			/** Construct gain of 1  */
			FGain()
			:	LinearGain(DefaultGain)
			{
			}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			/** Construct a Gain with a given value and representation.
			 *
			 * @param InValue - The value of the gain at the given 
			 * 					representation.
			 * @param InRep - The representation of the given value.
			 */
			FGain(float InValue, EGainRepresentation InRep)
			:	LinearGain(DefaultGain)
			{
				switch (InRep)
				{
					case EGainRepresentation::Decibels:
						SetDecibels(InValue);
						break;
					case EGainRepresentation::Linear:
						SetLinear(InValue);
						break;
					default:
						checkNoEntry();
						break;
				}
			}

			/**
			 * FGain constructor used to pass in float literals from the metasound frontend.
			 */
			FGain(float InValue)
				: FGain(InValue, EGainRepresentation::Linear)
			{}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
			/** Implicit operator used for math operations. */
			operator float() const
			{
				return LinearGain;
			}

			FGain& operator+=(const FGain& InOther)
			{
				LinearGain += InOther.GetLinear();
				return *this;
			}

			FGain& operator-=(const FGain& InOther)
			{
				LinearGain -= InOther.GetLinear();
				return *this;
			}

			FGain& operator*=(const FGain& InOther)
			{
				LinearGain *= InOther.GetLinear();
				return *this;
			}

			FGain& operator/=(const FGain& InOther)
			{
				LinearGain /= InOther.GetLinear();
				return *this;
			}

			/** Set the gain in decibels */
			void SetDecibels(float InDecibels)
			{
				LinearGain = Audio::ConvertToLinear(InDecibels);
			}

			/** Get the gain in decibels */
			float GetDecibels() const
			{
				return Audio::ConvertToLinear(LinearGain);
			}

			/** Set the gain in LinearScale */
			void SetLinear(float InLinearScale)
			{
				LinearGain = InLinearScale;
			}

			/** Return the gain in LinearScale. */
			float GetLinear() const
			{
				return LinearGain;
			}
		  
		private:
			float LinearGain = DefaultGain;
	};
}
