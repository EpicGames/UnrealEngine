// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundVertex.h"

#define UE_API METASOUNDSTANDARDNODES_API


namespace Metasound
{
	class FNoiseNode : public FNodeFacade
	{
	public:
		UE_API FNoiseNode(const FVertexName& InName, const FGuid& InInstanceID, int32 InDefaultSeed);
		UE_API FNoiseNode(const FNodeInitData& InInitData);
		UE_API FNoiseNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata);

		UE_DEPRECATED(5.6, "The default seed is stored on the vertex interface of the node")
		int32 GetDefaultSeed() const { return INDEX_NONE; }

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	private:
	};
}

#undef UE_API
