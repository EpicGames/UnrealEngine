// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MetasoundAudioBuffer.h"
#include "MetasoundBasicNode.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundVertex.h"
#include "MetasoundPrimitives.h"

#define UE_API METASOUNDSTANDARDNODES_API

namespace Metasound
{
	class FOscilatorNodeBase : public FBasicNode
	{
	protected:
		UE_API FOscilatorNodeBase(const FVertexName& InInstanceName, const FGuid& InInstanceID, const FNodeClassMetadata& InInfo, const TSharedRef<IOperatorFactory, ESPMode::ThreadSafe>& InFactory, float InDefaultFrequency, float InDefaultGlideTime, bool bInDefaultEnablement);
		UE_API FOscilatorNodeBase(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InClassMetadata, TSharedRef<IOperatorFactory> InFactory);

		UE_DEPRECATED(5.6, "This function should not be used")
		float GetDefaultPhaseOffset() const
		{
			return DefaultPhaseOffset; 
		}

		UE_DEPRECATED(5.6, "This function should not be used")
		float GetDefaultFrequency() const
		{
			return DefaultFrequency;
		}

		UE_DEPRECATED(5.6, "This function should not be used")
		float GetDefaultGlideFactor() const
		{
			return DefaultGlideFactor;
		}

		UE_DEPRECATED(5.6, "This function should not be used")
		bool GetDefaultEnablement() const
		{
			return bDefaultEnablement;
		}

		FOperatorFactorySharedRef GetDefaultOperatorFactory() const override
		{
			return Factory;
		}

		TSharedRef<IOperatorFactory, ESPMode::ThreadSafe> Factory;
		FVertexInterface VertexInterface;

	private:
		float DefaultPhaseOffset = 0.f;
		float DefaultFrequency = 440.f;
		float DefaultGlideFactor = 0.0f;
		bool bDefaultEnablement = true;
	};

	class FSineOscilatorNode : public FOscilatorNodeBase
	{
	public:
		class FFactory;

		UE_DEPRECATED(5.6, "Do not construct oscillators with this function")
		UE_API FSineOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefautlGlideTime, bool bInDefaultEnablement);

		UE_API FSineOscilatorNode(const FNodeInitData& InInitData);
		UE_API FSineOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
		
	class FSawOscilatorNode : public FOscilatorNodeBase
	{
	public:
		class FFactory;
		UE_DEPRECATED(5.6, "Do not construct oscillators with this function")
		UE_API FSawOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefautlGlideTime, bool bInDefaultEnablement);

		UE_API FSawOscilatorNode(const FNodeInitData& InInitData);
		UE_API FSawOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};

	class FTriangleOscilatorNode : public FOscilatorNodeBase
	{
	public:
		class FFactory;
		UE_DEPRECATED(5.6, "Do not construct oscillators with this function")
		UE_API FTriangleOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefautlGlideTime, bool bInDefaultEnablement);
		UE_API FTriangleOscilatorNode(const FNodeInitData& InInitData);
		UE_API FTriangleOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};

	class FSquareOscilatorNode : public FOscilatorNodeBase
	{
	public:
		class FFactory;
		UE_DEPRECATED(5.6, "Do not construct oscillators with this function")
		UE_API FSquareOscilatorNode(const FVertexName& InInstanceName, const FGuid& InInstanceID, float InDefaultFrequency, float InDefautlGlideTime, bool bInDefaultEnablement);
		UE_API FSquareOscilatorNode(const FNodeInitData& InInitData);
		UE_API FSquareOscilatorNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};

	class FLfoNode : public FNodeFacade
	{
	public:		
		UE_API FLfoNode(const FNodeInitData& InInitData);
		UE_API FLfoNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);
		virtual ~FLfoNode() = default;

		static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
}

#undef UE_API
