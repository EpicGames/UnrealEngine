// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes"

namespace Metasound
{
	namespace RandomNodeNames
	{
		METASOUND_PARAM(InputNextTrigger, "Next", "Trigger to generate the next random integer.")
		METASOUND_PARAM(InputResetTrigger, "Reset", "Trigger to reset the random sequence with the supplied seed. Useful to get randomized repetition.")
		METASOUND_PARAM(InputSeed, "Seed", "The seed value to use for the random node. Set to -1 to use a random seed.")
		METASOUND_PARAM(InputMin, "Min", "Min random value.")
		METASOUND_PARAM(InputMax, "Max", "Max random value.")
		
		METASOUND_PARAM(OutputOnNextTrigger, "On Next", "Triggers when next is triggered.")
		METASOUND_PARAM(OutputOnResetTrigger, "On Reset", "Triggers when reset is triggered.")
		METASOUND_PARAM(OutputValue, "Value", "The randomly generated value.")
	}

	template<typename ValueType>
	struct TRandomNodeSpecialization
	{
		bool bSupported = false;
	};

	template<>
	struct TRandomNodeSpecialization<int32>
	{
		static FName GetClassName()
		{
			return FName("RandomInt32");
		}

		static FText GetDisplayName()
		{
			return METASOUND_LOCTEXT("RandomNode_Int32RandomValueDisplayName", "Random (Int)");
		}

		static FText GetDescription()
		{
			return METASOUND_LOCTEXT("RandomNode_Int32RandomDescription", "Generates a seedable random integer in the given value range.");
		}

		static bool HasRange()
		{
			return true;
		}

		static int32 GetDefaultMin()
		{
			return 0;
		}

		static int32 GetDefaultMax()
		{
			return 100;
		}

		static int32 GetNextValue(FRandomStream& InStream, int32 InMin, int32 InMax)
		{
			return InStream.RandRange(InMin, InMax);
		}

		static TDataReadReference<int32> CreateMinValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames; 
			return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputMin), InOperatorSettings);
		}

		static TDataReadReference<int32> CreateMaxValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames; 
			return InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputMax), InOperatorSettings);
		}
	};

	template<>
	struct TRandomNodeSpecialization<float>
	{
		static FName GetClassName()
		{
			return FName("RandomFloat");
		}

		static FText GetDisplayName()
		{
			return METASOUND_LOCTEXT("RandomNode_FloatRandomValueDisplayName", "Random (Float)");
		}

		static FText GetDescription()
		{
			return METASOUND_LOCTEXT("RandomNode_FloatRandomDescription", "Generates a seedable random float in the given value range.");
		}

		static bool HasRange()
		{
			return true;
		}

		static float GetDefaultMin()
		{
			return 0.0f;
		}

		static float GetDefaultMax()
		{
			return 1.0f;
		}

		static float GetNextValue(FRandomStream& InStream, float InMin, float InMax)
		{
			return InStream.FRandRange(InMin, InMax);
		}

		static TDataReadReference<float> CreateMinValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames;
			return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputMin), InOperatorSettings);
		}

		static TDataReadReference<float> CreateMaxValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames;
			return InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputMax), InOperatorSettings);
		}
	};

	template<>
	struct TRandomNodeSpecialization<bool>
	{
		static FName GetClassName()
		{
			return FName("RandomBool");
		}

		static FText GetDisplayName()
		{
			return METASOUND_LOCTEXT("RandomNode_BoolRandomValueDisplayName", "Random (Bool)");
		}

		static FText GetDescription()
		{
			return METASOUND_LOCTEXT("RandomNode_BoolRandomDiscription", "Generates a random bool value.");
		}

		static bool HasRange()
		{
			return false;
		}

		static bool GetDefaultMin()
		{
			return false;
		}

		static bool GetDefaultMax()
		{
			return true;
		}

		static bool GetNextValue(FRandomStream& InStream, bool, bool)
		{
			return (bool)InStream.RandRange(0, 1);
		}

		static TDataReadReference<bool> CreateMinValueRef(const FInputVertexInterfaceData&, const FOperatorSettings&)
		{
			return TDataReadReference<bool>::CreateNew();
		}

		static TDataReadReference<bool> CreateMaxValueRef(const FInputVertexInterfaceData&, const FOperatorSettings&)
		{
			return TDataReadReference<bool>::CreateNew();
		}
	};

	template<>
	struct TRandomNodeSpecialization<FTime>
	{
		static FName GetClassName()
		{
			return FName("RandomTime");
		}

		static FText GetDisplayName()
		{
			return METASOUND_LOCTEXT("RandomNode_TimeRandomValueDisplayName", "Random (Time)");
		}

		static FText GetDescription()
		{
			return METASOUND_LOCTEXT("RandomNode_TimeRandomDiscription", "Generates a random time value.");
		}

		static bool HasRange()
		{
			return true;
		}

		static float GetDefaultMin()
		{
			return 0.0f;
		}

		static float GetDefaultMax()
		{
			return 1.0f;
		}


		static FTime GetNextValue(FRandomStream& InStream, FTime InMin, FTime InMax)
		{
			float Seconds = InStream.FRandRange(InMin.GetSeconds(), InMax.GetSeconds());
			return FTime(Seconds);
		}

		static TDataReadReference<FTime> CreateMinValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames; 
			return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputMin), InOperatorSettings);
		}

		static TDataReadReference<FTime> CreateMaxValueRef(const FInputVertexInterfaceData& InputData, const FOperatorSettings& InOperatorSettings)
		{
			using namespace RandomNodeNames;
			return InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputMax), InOperatorSettings);
		}
	};


	
	template<typename ValueType>
	class TRandomNodeOperator : public TExecutableOperator<TRandomNodeOperator<ValueType>>
	{
	public:
		static constexpr int32 DefaultSeed = -1;

		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace RandomNodeNames;

			if (TRandomNodeSpecialization<ValueType>::HasRange())
			{
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(
						TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputNextTrigger)),
						TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetTrigger)),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSeed), DefaultSeed),
						TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMin), TRandomNodeSpecialization<ValueType>::GetDefaultMin()),
						TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputMax), TRandomNodeSpecialization<ValueType>::GetDefaultMax())
					),
					FOutputVertexInterface(
						TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnNextTrigger)),
						TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnResetTrigger)),
						TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
					)
				);

				return DefaultInterface;
			}
			else
			{
				static const FVertexInterface DefaultInterface(
					FInputVertexInterface(
						TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputNextTrigger)),
						TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetTrigger)),
						TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSeed), DefaultSeed)
					),
					FOutputVertexInterface(
						TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnNextTrigger)),
						TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnResetTrigger)),
						TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
					)
				);

				return DefaultInterface;
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto InitNodeInfo = []() -> FNodeClassMetadata
			{
				FNodeClassMetadata Info;
				Info.ClassName = { StandardNodes::Namespace, TRandomNodeSpecialization<ValueType>::GetClassName(), "" };
				Info.MajorVersion = 1;
				Info.MinorVersion = 1;
				Info.DisplayName = TRandomNodeSpecialization<ValueType>::GetDisplayName();
				Info.Description = METASOUND_LOCTEXT("RandomNode_Description", "Generates a random value.");
				Info.Author = PluginAuthor;
				Info.PromptIfMissing = PluginNodeMissingPrompt;
				Info.DefaultInterface = GetDefaultInterface();
				Info.CategoryHierarchy.Emplace(NodeCategories::RandomUtils);

				return Info;
			};

			static const FNodeClassMetadata Info = InitNodeInfo();
			return Info;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
 			const FInputVertexInterfaceData& InputData = InParams.InputData;
 			
			using namespace RandomNodeNames;

 			FTriggerReadRef InNextTrigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputNextTrigger), InParams.OperatorSettings);
			FTriggerReadRef InResetTrigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputResetTrigger), InParams.OperatorSettings);
			FInt32ReadRef InSeedValue = InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSeed), InParams.OperatorSettings);

			// note: random bool does not have a range
			if (TRandomNodeSpecialization<ValueType>::HasRange())
			{
				TDataReadReference<ValueType> InMinValue = TRandomNodeSpecialization<ValueType>::CreateMinValueRef(InputData, InParams.OperatorSettings);
				TDataReadReference<ValueType> InMaxValue = TRandomNodeSpecialization<ValueType>::CreateMaxValueRef(InputData, InParams.OperatorSettings);

				return MakeUnique<TRandomNodeOperator<ValueType>>(InParams.OperatorSettings, InNextTrigger, InResetTrigger, InSeedValue, InMinValue, InMaxValue);
			}
			else
			{
				return MakeUnique<TRandomNodeOperator<ValueType>>(InParams.OperatorSettings, InNextTrigger, InResetTrigger, InSeedValue);
			}
		}


		TRandomNodeOperator(const FOperatorSettings& InSettings, 
			TDataReadReference<FTrigger> InNextTrigger, 
			TDataReadReference<FTrigger> InResetTrigger, 
			TDataReadReference<int32> InSeedValue, 
			TDataReadReference<ValueType> InMinValue,
			TDataReadReference<ValueType> InMaxValue)
 			: NextTrigger(InNextTrigger)
			, ResetTrigger(InResetTrigger)
			, SeedValue(InSeedValue)
 			, MinValue(InMinValue)
 			, MaxValue(InMaxValue)
			, TriggerOutOnNext(FTriggerWriteRef::CreateNew(InSettings))
			, TriggerOutOnReset(FTriggerWriteRef::CreateNew(InSettings))
 			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
			, bIsDefaultSeeded(*SeedValue == DefaultSeed)
			, bIsRandomStreamInitialized(false)
		{
			ResetInternal();
		}

		TRandomNodeOperator(const FOperatorSettings& InSettings,
			TDataReadReference<FTrigger> InNextTrigger,
			TDataReadReference<FTrigger> InResetTrigger,
			TDataReadReference<int32> InSeedValue)
			: NextTrigger(InNextTrigger)
			, ResetTrigger(InResetTrigger)
			, SeedValue(InSeedValue)
			, MinValue(TDataReadReference<ValueType>::CreateNew()) // Create stub default for no range case
			, MaxValue(TDataReadReference<ValueType>::CreateNew())
			, TriggerOutOnNext(FTriggerWriteRef::CreateNew(InSettings))
			, TriggerOutOnReset(FTriggerWriteRef::CreateNew(InSettings))
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
			, bIsDefaultSeeded(*SeedValue == DefaultSeed)
			, bIsRandomStreamInitialized(false)
		{
			ResetInternal();
		}

		virtual ~TRandomNodeOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace RandomNodeNames;

			FDataReferenceCollection Inputs;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputNextTrigger), NextTrigger);
 			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetTrigger), ResetTrigger);
 			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSeed), SeedValue);

			// If the type doesn't have a range no need for input pins to define it
			if (TRandomNodeSpecialization<ValueType>::HasRange())
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMin), MinValue);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputMax), MaxValue);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace RandomNodeNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnNextTrigger), TriggerOutOnNext);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnResetTrigger), TriggerOutOnReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		void Execute()
		{
			TriggerOutOnReset->AdvanceBlock();
			TriggerOutOnNext->AdvanceBlock();

			ResetTrigger->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					EvaluateSeedChanges();
					RandomStream.Reset();

					*OutputValue = TRandomNodeSpecialization<ValueType>::GetNextValue(RandomStream, *MinValue, *MaxValue);

					TriggerOutOnReset->TriggerFrame(StartFrame);
				}
			);

			NextTrigger->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					*OutputValue = TRandomNodeSpecialization<ValueType>::GetNextValue(RandomStream, *MinValue, *MaxValue);
					TriggerOutOnNext->TriggerFrame(StartFrame);
				}
			);
		}

		// Externally visible initialize function
		void Reset(const IOperator::FResetParams& InParams)
		{
			ResetInternal();
		}

	private:
		void ResetInternal()
		{
			TriggerOutOnNext->Reset();
			TriggerOutOnReset->Reset();
			bIsDefaultSeeded = (DefaultSeed == *SeedValue);
			bIsRandomStreamInitialized = false;

			EvaluateSeedChanges();
			RandomStream.Reset();

			// We need to initialize the output value to *something*
			*OutputValue = TRandomNodeSpecialization<ValueType>::GetNextValue(RandomStream, *MinValue, *MaxValue);
		}

		void EvaluateSeedChanges()
		{
			// if we have a non-zero seed
			if (*SeedValue != DefaultSeed)
			{
				// If we were previously zero-seeded OR our seed has changed
				if (bIsDefaultSeeded || !bIsRandomStreamInitialized || *SeedValue != RandomStream.GetInitialSeed())
				{
					bIsRandomStreamInitialized = true;
					bIsDefaultSeeded = false;
					RandomStream.Initialize(*SeedValue);
				}
			}
			// If we are zero-seeded now BUT were previously not, we need to randomize our seed
			else if (!bIsDefaultSeeded || !bIsRandomStreamInitialized)
			{
				bIsRandomStreamInitialized = true;
				bIsDefaultSeeded = true;
				RandomStream.Initialize(FPlatformTime::Cycles());
			}
		}


		FTriggerReadRef NextTrigger;
		FTriggerReadRef ResetTrigger;
		FInt32ReadRef SeedValue;
		TDataReadReference<ValueType> MinValue;
		TDataReadReference<ValueType> MaxValue;

		FTriggerWriteRef TriggerOutOnNext;
		FTriggerWriteRef TriggerOutOnReset;
		TDataWriteReference<ValueType> OutputValue;

		FRandomStream RandomStream;
		bool bIsDefaultSeeded = false;
		bool bIsRandomStreamInitialized = false;
	};

	/** TRandomNode
	 *
	 *  Generates a random float value when triggered.
	 */
	template<typename ValueType>
	using TRandomNode = TNodeFacade<TRandomNodeOperator<ValueType>>;

} // namespace Metasound

#undef LOCTEXT_NAMESPACE
