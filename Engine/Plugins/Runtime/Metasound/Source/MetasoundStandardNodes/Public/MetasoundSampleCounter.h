// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundTime.h"


namespace Metasound
{
	using FSampleCount = int64;
	static_assert(TIsIntegral<FSampleCount>::Value, "FSample must be integral type.");

	class FSampleCounter
	{
		private:
			FSampleCount NumSamples = 0;

			// Must be non-zero
			FSampleRate SampleRate = 1;

		public:
			FSampleCounter()
			{
			}

			FSampleCounter(const FSampleCounter& InSampleCounter)
				: NumSamples(InSampleCounter.NumSamples)
				, SampleRate(InSampleCounter.SampleRate)
			{
			}

			FSampleCounter(FSampleCount InNumSamples, FSampleRate InSampleRate)
				: NumSamples(InNumSamples)
				, SampleRate(InSampleRate)
			{
			}

			/** Construct a sample counter with a sample count,
			 * time resolution, and sample rate.
			 *
			 * @param InTime - The initial time value.
			 * @param InSampleRate - The sample rate associated with this time 
			 *						 object.
			 */
			static FSampleCounter FromTime(const FTime& InTime, FSampleRate InSampleRate)
			{
				return { FSampleCount(InTime.GetSeconds() * InSampleRate), InSampleRate };
			}

			/** Set the sample rate of this object without changing the number
			 * of samples. This will result in the object representing a
			 * different amount of time.
			 */
			void SetSampleRate(FSampleRate InSampleRate)
			{
				if (!ensure(InSampleRate > 0))
				{
					InSampleRate = 1;
				}

				SampleRate = InSampleRate;
			}

			/* Returns sample rate */
			FSampleRate GetSampleRate() const
			{
				return SampleRate;
			}

			FTime ToTime() const
			{
				return FTime(static_cast<FTime::TimeType>(NumSamples) / SampleRate);
			}

			/** Return the number of samples. */
			FSampleCount GetNumSamples() const
			{ 
				return NumSamples;
			}

			/** Return the number of samples which represent this time duration
			 * using a different sample rate. 
			 *
			 * @param InOtherSampleRate - Sample rate to use when calculating
			 * 							  the number of samples.
			 */
			FSampleCount GetNumSamples(FSampleRate InOtherSampleRate) const
			{
				if (InOtherSampleRate == SampleRate)
				{
					return NumSamples;
				}

				return NumSamples * InOtherSampleRate / SampleRate;
			}

			/** Set the number of samples from time value (This is functionally
			  * discretization and will result in loss of precision)
			  */
			void SetNumSamples(const FTime& InTime)
			{ 
				NumSamples = InTime.GetSeconds() * SampleRate;
			}

			/** Set the number of samples. */
			void SetNumSamples(FSampleCount InNumSamples)
			{
				NumSamples = InNumSamples;
			}

			FSampleCounter& operator=(const FSampleCounter& InRHS)
			{
				SampleRate = InRHS.SampleRate;
				NumSamples = InRHS.NumSamples;
				return *this;
			}

			friend FSampleCounter& operator+=(FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				InLHS.NumSamples += InRHS.GetNumSamples(InLHS.SampleRate);
				return InLHS;
			}

			friend FSampleCounter& operator+=(FSampleCounter& InLHS, const FTime& InRHS)
			{
				InLHS.NumSamples += InRHS * InLHS.SampleRate;
				return InLHS;
			}

			friend FSampleCounter& operator-=(FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				InLHS.NumSamples -= InRHS.GetNumSamples(InLHS.SampleRate);
				return InLHS;
			}

			friend FSampleCounter& operator-=(FSampleCounter& InLHS, const FTime& InRHS)
			{
				InLHS.NumSamples -= InRHS * InLHS.SampleRate;
				return InLHS;
			}

			friend FSampleCounter operator+(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return { InLHS.NumSamples + InRHS.NumSamples, InLHS.SampleRate };
			}

			friend FSampleCounter operator+(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return { FSampleCount(InLHS.NumSamples + (InRHS * InLHS.SampleRate)), InLHS.SampleRate };
			}

			friend FSampleCounter operator-(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return { FSampleCount(InLHS.NumSamples - InRHS.NumSamples), InLHS.SampleRate };
			}

			friend FSampleCounter operator-(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return { FSampleCount(InLHS.NumSamples - (InRHS * InLHS.SampleRate)), InLHS.SampleRate };
			}

			template<typename IntegralType>
			friend FSampleCounter operator+(const FSampleCounter& InLHS, const IntegralType& InRHS)
			{
				static_assert(TIsIntegral<IntegralType>::Value, "Right-hand addend type must be integral.");
				return FSampleCounter(InLHS.NumSamples + InRHS, InLHS.SampleRate);
			}

			template<typename IntegralType>
			friend FSampleCounter operator-(const FSampleCounter& InLHS, const IntegralType& InRHS)
			{
				static_assert(TIsIntegral<IntegralType>::Value, "Subtrahend type must be integral.");
				return FSampleCounter(InLHS.NumSamples - InRHS, InLHS.SampleRate);
			}

			template<typename ArithmeticType>
			friend FSampleCounter operator*(const FSampleCounter& InLHS, const ArithmeticType& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Multiplicand type must be arithmetic.");
				return FSampleCounter(InLHS.NumSamples * InRHS, InLHS.SampleRate);
			}

			template<typename ArithmeticType>
			friend FSampleCounter operator*(const ArithmeticType& InLHS, const FSampleCounter& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Multiplicand type must be arithmetic.");
				return FSampleCounter(InLHS * InLHS.NumSamples, InLHS.SampleRate);
			}

			template<typename ArithmeticType>
			friend FSampleCounter operator/(const FSampleCounter& InLHS, ArithmeticType InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Divisor type must be arithmetic.");
				return FSampleCounter(InLHS.NumSamples / InRHS, InLHS.SampleRate);
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator/(ArithmeticType InLHS, const FSampleCounter& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Divisor type must be arithmetic.");
				return InLHS / InRHS.NumSamples;
			}

			template<typename ArithmeticType>
			friend FSampleCounter& operator*=(FSampleCounter& InLHS, ArithmeticType InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Multiplicand type must be arithmetic.");
				InLHS.NumSamples *= InRHS;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend FSampleCounter& operator/=(FSampleCounter& InLHS, ArithmeticType InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Divisor type must be arithmetic.");
				InLHS.NumSamples /= InRHS;
				return InLHS;
			}

			template<typename ArithmeticType>
			friend ArithmeticType& operator/=(ArithmeticType& InLHS, const FSampleCounter& InRHS)
			{
				static_assert(TIsArithmetic<ArithmeticType>::Value, "Divisor type must be arithmetic.");
				InLHS.NumSamples /= InRHS;
				return InLHS;
			}

			template<typename IntegralType>
			friend FSampleCounter operator+=(FSampleCounter& InLHS, const IntegralType& InRHS)
			{
				static_assert(TIsIntegral<IntegralType>::Value, "Addend type must be integral.");
				InLHS.NumSamples += InRHS;
				return InLHS;
			}

			template<typename IntegralType>
			friend FSampleCounter& operator-=(FSampleCounter& InLHS, const IntegralType& InRHS)
			{
				static_assert(TIsIntegral<IntegralType>::Value, "Subtrahend type must be integral.");
				InLHS.NumSamples -= InRHS;
				return InLHS;
			}

			friend bool operator<(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() < InRHS;
			}

			friend bool operator<(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS < InRHS.ToTime();
			}

			friend bool operator<(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples < InRHS.NumSamples;
			}

			friend bool operator>(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() > InRHS;
			}

			friend bool operator>(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS > InRHS.ToTime();
			}

			friend bool operator>(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples > InRHS.NumSamples;
			}

			friend bool operator<=(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() <= InRHS;
			}

			friend bool operator<=(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS <= InRHS.ToTime();
			}

			friend bool operator<=(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples <= InRHS.NumSamples;
			}

			friend bool operator>=(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() >= InRHS;
			}

			friend bool operator>=(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS >= InRHS.ToTime();
			}

			friend bool operator>=(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples >= InRHS.NumSamples;
			}

			friend bool operator==(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() == InRHS;
			}

			friend bool operator==(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS == InRHS.ToTime();
			}

			friend bool operator==(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples == InRHS.GetNumSamples(InLHS.SampleRate);
			}

			friend bool operator!=(const FSampleCounter& InLHS, const FTime& InRHS)
			{
				return InLHS.ToTime() != InRHS;
			}

			friend bool operator!=(const FTime& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS != InRHS.ToTime();
			}

			friend bool operator!=(const FSampleCounter& InLHS, const FSampleCounter& InRHS)
			{
				return InLHS.NumSamples != InRHS.NumSamples;
			}
	};
}
