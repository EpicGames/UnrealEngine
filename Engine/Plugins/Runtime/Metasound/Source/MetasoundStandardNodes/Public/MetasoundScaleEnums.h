// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundEnum.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "DSP/MidiNoteQuantizer.h"
#include "Internationalization/Text.h"
#include "MetasoundDataTypeRegistrationMacro.h"

namespace Metasound
{
	// Any desired additions to this Enum/list need to first be added to the EMusicalScale enum in MidiNoteQuantizer.h
	// and defined in MidiNoteQuantizer.cpp in the TMap<EMusicalScale::Scale, ScaleDegreeSet> ScaleDegreeSetMap static init

	// Metasound enum
	DECLARE_METASOUND_ENUM(Audio::EMusicalScale::Scale, Audio::EMusicalScale::Scale::Major,
	METASOUNDSTANDARDNODES_API, FEnumEMusicalScale, FEnumMusicalScaleTypeInfo, FEnumMusicalScaleReadRef, FEnumMusicalScaleWriteRef);

} // namespace Metasound
