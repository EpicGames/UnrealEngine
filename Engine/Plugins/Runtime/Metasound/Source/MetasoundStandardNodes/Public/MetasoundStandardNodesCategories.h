// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"


namespace Metasound::NodeCategories
{
	extern const FText METASOUNDSTANDARDNODES_API Debug;
	extern const FText METASOUNDSTANDARDNODES_API Delays;
	extern const FText METASOUNDSTANDARDNODES_API Dynamics;
	extern const FText METASOUNDSTANDARDNODES_API Envelopes;
	extern const FText METASOUNDSTANDARDNODES_API Filters;
	extern const FText METASOUNDSTANDARDNODES_API Generators;
	extern const FText METASOUNDSTANDARDNODES_API Io;
	extern const FText METASOUNDSTANDARDNODES_API Math;
	extern const FText METASOUNDSTANDARDNODES_API Mix;
	extern const FText METASOUNDSTANDARDNODES_API Music;
	extern const FText METASOUNDSTANDARDNODES_API RandomUtils;
	extern const FText METASOUNDSTANDARDNODES_API Spatialization;
	extern const FText METASOUNDSTANDARDNODES_API Trigger;
	extern const FText METASOUNDSTANDARDNODES_API WaveTables;
	extern const FText METASOUNDSTANDARDNODES_API Reverbs;
} // Metasound::NodeCategories
