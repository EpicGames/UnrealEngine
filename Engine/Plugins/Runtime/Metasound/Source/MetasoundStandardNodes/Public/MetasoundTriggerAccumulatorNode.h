// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_AccumulatorNode"

namespace Metasound
{
	namespace MetasoundTriggerAccumulatorNodePrivate
	{
		METASOUNDSTANDARDNODES_API FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface);
	}

	namespace TriggerAccumulatorVertexNames
	{
		METASOUND_PARAM(InputAutoReset, "Auto Reset", "Input trigger resets the trigger accumulation count.");	
		METASOUND_PARAM(InputTrigger, "In {0}", "Trigger {0} input. All trigger inputs must be triggered before the output trigger is hit.");
		METASOUND_PARAM(AccumulateOutputOnTrigger, "Out", "Triggered when all input triggers have been triggered. Call Reset to reset the state or use \"Auto Reset\".");
	}

	template<uint32 NumInputs>
	class TTriggerAccumulatorOperator : public TExecutableOperator<TTriggerAccumulatorOperator<NumInputs>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace TriggerAccumulatorVertexNames;

			auto CreateDefaultInterface = []() -> FVertexInterface
			{
				FInputVertexInterface InputInterface;

				for (uint32 i = 0; i < NumInputs; ++i)
				{
					InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_WITH_INDEX_AND_METADATA(InputTrigger, i)));
				}

				InputInterface.Add(TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAutoReset)));

				FOutputVertexInterface OutputInterface;
				OutputInterface.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(AccumulateOutputOnTrigger)));

				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				FName OperatorName = *FString::Printf(TEXT("Trigger Accumulate (%d)") , NumInputs);
				FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerAccumulateDisplayNamePattern", "Trigger Accumulate ({0})", NumInputs);
				const FText NodeDescription = METASOUND_LOCTEXT("TriggerAccumulateDescription", "Will trigger output once all input triggers have been hit at some point in the past.");
				FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundTriggerAccumulatorNodePrivate::CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace TriggerAccumulatorVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FBoolReadRef bInAutoReset = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputAutoReset), InParams.OperatorSettings);
			TArray<FTriggerReadRef> InputTriggers;

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers.Add(InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME_WITH_INDEX(InputTrigger, i), InParams.OperatorSettings));
			}

			return MakeUnique<TTriggerAccumulatorOperator<NumInputs>>(InParams.OperatorSettings, bInAutoReset, MoveTemp(InputTriggers));
		}

		TTriggerAccumulatorOperator(const FOperatorSettings& InSettings, 
			const FBoolReadRef& bInAutoReset, 
			const TArray<FTriggerReadRef>&& InInputTriggers)
			: bAutoReset(bInAutoReset)
			, InputTriggers(InInputTriggers)
			, OutputTrigger(FTriggerWriteRef::CreateNew(InSettings))
		{
			ResetTriggerState();
		}

		virtual ~TTriggerAccumulatorOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerAccumulatorVertexNames;

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME_WITH_INDEX(InputTrigger, i), InputTriggers[i]);
			}
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAutoReset), bAutoReset);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerAccumulatorVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(AccumulateOutputOnTrigger), OutputTrigger);
		}

		void TriggerOutputIfReady(int32 InStartFrame)
		{
			for (bool WasTriggered : bInputWasTriggered)
			{
				if (!WasTriggered)
				{
					return;
				}
			}

			if (*bAutoReset)
			{
				ResetTriggerState();
			}

			if (!bOutputTriggered)
			{
				OutputTrigger->TriggerFrame(InStartFrame);
				bOutputTriggered = true;
			}
		}

		void ResetTriggerState()
		{
			bOutputTriggered = false;
			bInputWasTriggered.Reset();
			bInputWasTriggered.AddDefaulted(InputTriggers.Num());
		}

		void Execute()
		{
			OutputTrigger->AdvanceBlock();

			if (bOutputTriggered && !*bAutoReset)
			{
				return;
			}

			for (uint32 i = 0; i < NumInputs; ++i)
			{
				InputTriggers[i]->ExecuteBlock(
					[&](int32 StartFrame, int32 EndFrame)
					{
					},
					[this, i](int32 StartFrame, int32 EndFrame)
					{
						bInputWasTriggered[i] = true;
						TriggerOutputIfReady(StartFrame);
					}
				);
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			OutputTrigger->Reset();
			ResetTriggerState();
		}

	private:

		FBoolReadRef bAutoReset;
		TArray<FTriggerReadRef> InputTriggers;
		FTriggerWriteRef OutputTrigger;

		TArray<bool> bInputWasTriggered;
		bool bOutputTriggered = false;
	};

	/** TTriggerAccumulatorNode
	*
	*  Routes values from multiple input pins to a single output pin based on trigger inputs.
	*/
	template<uint32 NumInputs>
	using TTriggerAccumulatorNode = TNodeFacade<TTriggerAccumulatorOperator<NumInputs>>;

} // namespace Metasound

#undef LOCTEXT_NAMESPACE //MetasoundStandardNodes_AccumulatorNode
