// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CompareNode"

namespace Metasound
{
	namespace TriggerCompareVertexNames
	{
		METASOUND_PARAM(InputCompare, "Compare", "Trigger to compare A and B.");
		METASOUND_PARAM(InputParamA, "A", "The first value, A, to compare against.");
		METASOUND_PARAM(InputParamB, "B", "The first value, B, to compare against.");
		METASOUND_PARAM(InputCompareType, "Type", "How to compare A and B.");
		METASOUND_PARAM(OutputOnTrue, "True", "Output trigger for when the comparison is true.");
		METASOUND_PARAM(OutputOnFalse, "False", "Output trigger for when the comparison is false.");
	}

	namespace MetasoundTriggerCompareNodePrivate
	{
		METASOUNDSTANDARDNODES_API FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface);
	}

	enum class ETriggerComparisonType
	{
		Equals,
		NotEquals,
		LessThan,
		GreaterThan,
		LessThanOrEquals,
		GreaterThanOrEquals
	};

	DECLARE_METASOUND_ENUM(ETriggerComparisonType, ETriggerComparisonType::Equals, METASOUNDSTANDARDNODES_API,
	FEnumTriggerComparisonType, FEnumTriggerComparisonTypeInfo, FEnumTriggerComparisonTypeReadRef, FEnumTriggerComparisonTypeWriteRef);

	template<typename ValueType>
	class TTriggerCompareNodeOperator : public TExecutableOperator<TTriggerCompareNodeOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace TriggerCompareVertexNames;
			using namespace MetasoundTriggerCompareNodePrivate;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputCompare)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputParamA)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputParamB)),
					TInputDataVertex<FEnumTriggerComparisonType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputCompareType), (int32)ETriggerComparisonType::Equals)
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnTrue)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnFalse))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = TEXT("Clamp");
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("TriggerCompareDisplayPattern", "Trigger Compare ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("TriggerCompareDisc", "Output triggers (True or False) based on comparing inputs, A and B.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundTriggerCompareNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace TriggerCompareVertexNames;
			
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef InOnTriggerCompare = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputCompare), InParams.OperatorSettings);
			TDataReadReference<ValueType> InValueA = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputParamA), InParams.OperatorSettings);
			TDataReadReference<ValueType> InValueB = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputParamB), InParams.OperatorSettings);
			FEnumTriggerComparisonTypeReadRef InComparison = InputData.GetOrCreateDefaultDataReadReference<FEnumTriggerComparisonType>(METASOUND_GET_PARAM_NAME(InputCompareType), InParams.OperatorSettings);

			return MakeUnique<TTriggerCompareNodeOperator<ValueType>>(InParams.OperatorSettings, InOnTriggerCompare, InValueA, InValueB, InComparison);

		}


		TTriggerCompareNodeOperator(
			const FOperatorSettings& InSettings, 
			const FTriggerReadRef& InOnCompareTrigger, 
			const TDataReadReference<ValueType>& InValueA,
			const TDataReadReference<ValueType>& InValueB,
			const FEnumTriggerComparisonTypeReadRef& InTriggerComparisonType)
			: OnCompareTrigger(InOnCompareTrigger)
			, ValueA(InValueA)
			, ValueB(InValueB)
			, TriggerComparisonType(InTriggerComparisonType)
			, TriggerOutOnTrue(FTriggerWriteRef::CreateNew(InSettings))
			, TriggerOutOnFalse(FTriggerWriteRef::CreateNew(InSettings))
		{
		}

		virtual ~TTriggerCompareNodeOperator() = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerCompareVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCompare), OnCompareTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputParamA), ValueA);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputParamB), ValueB);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCompareType), TriggerComparisonType);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TriggerCompareVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnTrue), TriggerOutOnTrue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnFalse), TriggerOutOnFalse);
		}

		void Execute()
		{
			TriggerOutOnTrue->AdvanceBlock();
			TriggerOutOnFalse->AdvanceBlock();

			OnCompareTrigger->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					ValueType CurrValA = *ValueA;
					ValueType CurrValB = *ValueB;
					bool bIsTrue = false;

					switch (*TriggerComparisonType)
					{
					case ETriggerComparisonType::Equals:
						bIsTrue = (CurrValA == CurrValB);
						break;
					case ETriggerComparisonType::NotEquals:
						bIsTrue = (CurrValA != CurrValB);
						break;
					case ETriggerComparisonType::LessThan:
						bIsTrue = (CurrValA < CurrValB);
						break;
					case ETriggerComparisonType::GreaterThan:
						bIsTrue = (CurrValA > CurrValB);
						break;
					case ETriggerComparisonType::LessThanOrEquals:
						bIsTrue = (CurrValA <= CurrValB);
						break;
					case ETriggerComparisonType::GreaterThanOrEquals:
						bIsTrue = (CurrValA >= CurrValB);
						break;
					}

					if (bIsTrue)
					{
						TriggerOutOnTrue->TriggerFrame(StartFrame);
					}
					else
					{
						TriggerOutOnFalse->TriggerFrame(StartFrame);
					}
				}
				);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			TriggerOutOnTrue->Reset();
			TriggerOutOnFalse->Reset();
		}

	private:

		FTriggerReadRef OnCompareTrigger;
		TDataReadReference<ValueType> ValueA;
		TDataReadReference<ValueType> ValueB;
		FEnumTriggerComparisonTypeReadRef TriggerComparisonType;

		FTriggerWriteRef TriggerOutOnTrue;
		FTriggerWriteRef TriggerOutOnFalse;
	};

	/** TTriggerCompareNode
	 *
	 *  Compares two inputs against enumerated comparison types.
	 */
	template<typename ValueType>
	using TTriggerCompareNode = TNodeFacade<TTriggerCompareNodeOperator<ValueType>>;
}

#undef LOCTEXT_NAMESPACE
