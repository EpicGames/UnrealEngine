// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundNode.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"

#define UE_API METASOUNDSTANDARDNODES_API


namespace Metasound
{
	class FTriggerRepeatNode : public FNodeFacade
	{
		public:
			UE_API FTriggerRepeatNode(const FNodeInitData& InInitData);
			UE_API FTriggerRepeatNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

			virtual ~FTriggerRepeatNode() = default;

			static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
}

#undef UE_API
