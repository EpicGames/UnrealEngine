// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundBuilderInterface.h"
#include "MetasoundDataReferenceCollection.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNode.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundOperatorInterface.h"
#include "MetasoundTime.h"
#include "MetasoundTrigger.h"

#define UE_API METASOUNDSTANDARDNODES_API


namespace Metasound
{
	class FTriggerToggleNode : public FNodeFacade
	{
		public:
			UE_API FTriggerToggleNode(const FNodeInitData& InInitData);

			UE_API FTriggerToggleNode(FNodeData InNodeData, TSharedRef<const FNodeClassMetadata> InMetadata);

			virtual ~FTriggerToggleNode() = default;

			static UE_API FNodeClassMetadata CreateNodeClassMetadata();
	};
}

#undef UE_API
