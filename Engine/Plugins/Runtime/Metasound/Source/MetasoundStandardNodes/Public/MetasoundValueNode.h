// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MetasoundFacade.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundTrigger.h"
#include "Internationalization/Text.h"
#include "MetasoundParamHelper.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_ValueNode"

namespace Metasound
{
	namespace MetasoundValueNodePrivate
	{
		METASOUNDSTANDARDNODES_API FNodeClassMetadata CreateNodeClassMetadata(const FName& InDataTypeName, const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface);
	}

	namespace ValueVertexNames
	{
		METASOUND_PARAM(InputSetTrigger, "Set", "Trigger to write the set value to the output.");
		METASOUND_PARAM(InputResetTrigger, "Reset", "Trigger to reset the value to the initial value.");
		METASOUND_PARAM(InputInitValue, "Init Value", "Value to initialize the output value to.");
		METASOUND_PARAM(InputTargetValue, "Target Value", "Value to immediately set the output to when triggered.");
		METASOUND_PARAM(OutputOnSet, "On Set", "Triggered when the set input is triggered.");
		METASOUND_PARAM(OutputOnReset, "On Reset", "Triggered when the reset input is triggered.");
		METASOUND_PARAM(OutputValue, "Output Value", "The current output value.");
	}

	template<typename ValueType>
	class TValueOperator : public TExecutableOperator<TValueOperator<ValueType>>
	{
	public:
		static const FVertexInterface& GetDefaultInterface()
		{
			using namespace ValueVertexNames;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSetTrigger)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetTrigger)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputInitValue)),
					TInputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTargetValue))
				),
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSet)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnReset)),
					TOutputDataVertex<ValueType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputValue))
				)
			);

			return DefaultInterface;
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
			{
				const FName DataTypeName = GetMetasoundDataTypeName<ValueType>();
				const FName OperatorName = "Value";
				const FText NodeDisplayName = METASOUND_LOCTEXT_FORMAT("ValueDisplayNamePattern", "Value ({0})", GetMetasoundDataTypeDisplayText<ValueType>());
				const FText NodeDescription = METASOUND_LOCTEXT("ValueDescription", "Allows setting a value to output on trigger.");
				const FVertexInterface NodeInterface = GetDefaultInterface();

				return MetasoundValueNodePrivate::CreateNodeClassMetadata(DataTypeName, OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ValueVertexNames;

			const FInputVertexInterfaceData& InputData = InParams.InputData;
			
			FTriggerReadRef SetTrigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputSetTrigger), InParams.OperatorSettings);
			FTriggerReadRef ResetTrigger = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputResetTrigger), InParams.OperatorSettings);
			TDataReadReference<ValueType> InitValue = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputInitValue), InParams.OperatorSettings);
			TDataReadReference<ValueType> TargetValue = InputData.GetOrCreateDefaultDataReadReference<ValueType>(METASOUND_GET_PARAM_NAME(InputTargetValue), InParams.OperatorSettings);

			return MakeUnique<TValueOperator<ValueType>>(InParams.OperatorSettings, SetTrigger, ResetTrigger, InitValue, TargetValue);
		}


		TValueOperator(const FOperatorSettings& InSettings, const FTriggerReadRef& InSetTrigger, const FTriggerReadRef& InResetTrigger, const TDataReadReference<ValueType>& InInitValue, const TDataReadReference<ValueType>& InTargetValue)
			: SetTrigger(InSetTrigger)
			, ResetTrigger(InResetTrigger)
			, InitValue(InInitValue)
			, TargetValue(InTargetValue)
			, OutputValue(TDataWriteReferenceFactory<ValueType>::CreateAny(InSettings))
			, TriggerOnSet(FTriggerWriteRef::CreateNew(InSettings))
			, TriggerOnReset(FTriggerWriteRef::CreateNew(InSettings))
		{
			*OutputValue = *InitValue;
		}

		virtual ~TValueOperator() = default;


		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ValueVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSetTrigger), SetTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetTrigger), ResetTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputInitValue), InitValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTargetValue), TargetValue);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ValueVertexNames;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnSet), TriggerOnSet);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputOnReset), TriggerOnReset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(OutputValue), OutputValue);
		}

		virtual FDataReferenceCollection GetInputs() const override
		{
			// This should never be called. Bind(...) is called instead. This method
			// exists as a stop-gap until the API can be deprecated and removed.
			checkNoEntry();
			return {};
		}

		virtual FDataReferenceCollection GetOutputs() const override
		{
			// This should never be called. Bind(...) is called instead. This method
			// exists as a stop-gap until the API can be deprecated and removed.
			checkNoEntry();
			return {};
		}



		void Execute()
		{
			TriggerOnReset->AdvanceBlock();
			TriggerOnSet->AdvanceBlock();

			if (*ResetTrigger)
			{
				*OutputValue = *InitValue;
			}

			if (*SetTrigger)
			{
				*OutputValue = *TargetValue;
			}

			ResetTrigger->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					TriggerOnReset->TriggerFrame(StartFrame);
				}
			);

			SetTrigger->ExecuteBlock(
				[&](int32 StartFrame, int32 EndFrame)
				{
				},
				[this](int32 StartFrame, int32 EndFrame)
				{
					TriggerOnSet->TriggerFrame(StartFrame);
				}
			);

		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			TriggerOnSet->Reset();
			TriggerOnReset->Reset();
			*OutputValue = *InitValue;
		}

	private:

		TDataReadReference<FTrigger> SetTrigger;
		TDataReadReference<FTrigger> ResetTrigger;
		TDataReadReference<ValueType> InitValue;
		TDataReadReference<ValueType> TargetValue;
		TDataWriteReference<ValueType> OutputValue;
		TDataWriteReference<FTrigger> TriggerOnSet;
		TDataWriteReference<FTrigger> TriggerOnReset;
	};

	/** TValueNode
	 *
	 *  Generates a random float value when triggered.
	 */
	template<typename ValueType>
	using TValueNode = TNodeFacade<TValueOperator<ValueType>>;

} // namespace Metasound

#undef LOCTEXT_NAMESPACE
