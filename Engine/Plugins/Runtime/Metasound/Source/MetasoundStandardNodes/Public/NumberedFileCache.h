// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/FileManager.h"
#include "HAL/CriticalSection.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Containers/Map.h"

namespace Metasound 
{
	class FNumberedFileCache
	{
	public:
		static const METASOUNDSTANDARDNODES_API FString Separator;

		FNumberedFileCache(const FString& InRootPath, const FString& InExt, IFileManager& InFileSystem)
			: RootPath{ InRootPath }, FileExtention{ InExt }, FileSystem {InFileSystem}
		{
			CacheFilenames();
		}

		FString GenerateNextNumberedFilename(const FString& InPrefix)
		{
			FScopeLock Lock{ &Cs };
			uint32& CurrentMax = FileIndexMap.FindOrAdd(InPrefix.ToUpper());
			FString Filename{ InPrefix };
			Filename.Append(*Separator);
			Filename.AppendInt(++CurrentMax);
			Filename.Append(*FileExtention);
			return RootPath / Filename;
		}
	private:

		// Slow directory search of the root path for filenames.
		void CacheFilenames()
		{
			FScopeLock Lock{ &Cs };
					
			// Find all files, split filenames into prefix + number, saving max number we find.
			TArray<FString> Files;
			FileSystem.FindFiles(Files , *RootPath, *FileExtention);
			for (const FString& i : Files)
			{
				FString Prefix, Postfix;
				if (i.Split(Separator, &Prefix, &Postfix, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					FString NumberString = FPaths::GetBaseFilename(Postfix);
					if (FCString::IsNumeric(*NumberString))
					{
						int32 Number = FCString::Atoi(*NumberString);
						if (Number >= 0)
						{
							uint32& CurrentMax = FileIndexMap.FindOrAdd(*Prefix.ToUpper());
							if (static_cast<uint32>(Number) > CurrentMax)
							{
								CurrentMax = static_cast<uint32>(Number);
							}
						}
					}
				}
			}
		}

		FCriticalSection Cs;
		FString RootPath;
		FString FileExtention;
		TMap<FString, uint32> FileIndexMap;
		IFileManager& FileSystem;
	};

}//namespace Audio
